// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class NetCommon : ModuleRules
{
	public NetCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core"
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
