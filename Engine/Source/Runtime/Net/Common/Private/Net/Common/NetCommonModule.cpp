// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

// Force export
#include "Net/Common/Packets/PacketTraits.h"

class FNetCommonModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
};


IMPLEMENT_MODULE(FNetCommonModule, NetCommon);
