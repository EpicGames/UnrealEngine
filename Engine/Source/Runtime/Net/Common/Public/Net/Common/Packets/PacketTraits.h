// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"


// Forward declarations
namespace UE::Net
{
	struct FNetResult;
}


/**
 * Contains metadata and flags, which provide information on the traits of a packet - what it contains and how to process it.
 */
struct FOutPacketTraits
{
	/** Flags - may trigger modification of packet and traits */

	/** Whether or not the packet can/should be compressed */
	bool bAllowCompression = true;


	/** Traits */

	/** The number of ack bits in the packet - reflecting UNetConnection.NumAckBits */
	uint32 NumAckBits = 0;

	/** The number of bunch bits in the packet - reflecting UNetConnection.NumBunchBits */
	uint32 NumBunchBits = 0;

	/** Whether or not this is a keepalive packet */
	bool bIsKeepAlive = false;

	/** Whether or not the packet has been compressed */
	bool bIsCompressed = false;
};

/**
 * Contains metadata and flags, which provide information on the traits of a packet - what it contains and how to process it.
 */
struct FInPacketTraits
{
	/** Traits */

	/** This packet is not associated with a connection */
	bool bConnectionlessPacket = false;

	/** This is a connectionless packet, from a recently disconnected connection. */
	bool bFromRecentlyDisconnected = false;

	/** If there was an error processing the incoming packet, any additional error information is stored here */
	TPimplPtr<UE::Net::FNetResult, EPimplPtrMode::DeepCopy> ExtendedError;
};
