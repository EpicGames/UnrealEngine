// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Net/Common/Sockets/SocketErrors.h"
#include "Net/Common/Packets/PacketTraits.h"


// Forward declarations
class FInternetAddr;


/**
 * Represents a view of a buffer for storing packets. Buffer contents may be modified, but the allocation can not be resized.
 * Should only be stored as a local variable within functions that handle received packets.
 */
struct FPacketBufferView
{
	/** View of the packet buffer, with Num() representing allocated size. Internal buffer data can be modified, but not view/size. */
	const TArrayView<uint8>		Buffer;


	FPacketBufferView(uint8* InData, int32 MaxBufferSize)
		: Buffer(MakeArrayView<uint8>(InData, MaxBufferSize))
	{
	}
};

namespace ECountUnits
{
	enum BitType
	{
		Bits
	};

	enum ByteType
	{
		Bytes
	};
}

/**
 * Represent a restricted view of packet data, with bit-based size, allowing data to be reassigned to point elsewhere,
 * but not allowing the data being pointed to, to be modified.
 */
class FPacketDataView
{
	friend struct FReceivedPacketView;

private:
	/** View of the raw packet data */
	TArrayView<const uint8> Data;

	/** The size of the data in bits */
	int32 CountBits;

public:
	/**
	 * Constructs a view of packet data with bit-based size.
	 *
	 * @param InData		Pointer to the packet data
	 * @param InCountBits	The packet size, in bits
	 * @param CountUnits	Whether or not the size is specified in bits (ECountUnits::Bits) or bytes (ECountUnits::Bytes)
	 */
	FPacketDataView(const uint8* InData, int32 InCountBits, ECountUnits::BitType)
		: Data(MakeArrayView(InData, FMath::DivideAndRoundUp(InCountBits, 8)))
		, CountBits(InCountBits)
	{
	}

	/**
	 * Constructs a view of packet data with byte-based size.
	 *
	 * @param InData		Pointer to the packet data
	 * @param InCountBytes	The packet size, in bytes
	 * @param CountUnits	Whether or not the size is specified in bits (ECountUnits::Bits) or bytes (ECountUnits::Bytes)
	 */
	FPacketDataView(const uint8* InData, int32 InCountBytes, ECountUnits::ByteType)
		: Data(MakeArrayView(InData, InCountBytes))
		, CountBits(InCountBytes * 8)
	{
	}

	const uint8* GetData() const
	{
		return Data.GetData();
	}

	// @todo: Remove this when the PacketHandler code is updated
	uint8* GetMutableData() const
	{
		return const_cast<uint8*>(Data.GetData());
	}

	int32 NumBits() const
	{
		return CountBits;
	}

	int32 NumBytes() const
	{
		return Data.Num();
	}
};


/**
 * Represents a view of a received packet, which may be modified to update Data it points to and Data size, as a packet is processed.
 * Should only be stored as a local variable within functions that handle received packets.
 */
struct FReceivedPacketView
{
	/** View of packet data - can reassign to point elsewhere, but don't use to modify packet data */
	FPacketDataView						DataView = {nullptr, 0, ECountUnits::Bits};

	// @todo: When removing deprecation, remove FReceivedPacketView's friend access to FPacketDataView
	UE_DEPRECATED(4.26, "Data is deprecated, use DataView instead")
	TArrayView<const uint8>&			Data = DataView.Data;

	/** Receive address for the packet */
	TSharedPtr<const FInternetAddr>		Address;

	/** Error if receiving a packet failed */
	ESocketErrors						Error;

	/** Metadata and flags for the received packet, indicating what it contains and how to process it */
	FInPacketTraits						Traits;
};

/**
 * Stores a platform-specific timestamp for a packet. Can be translated for local use by ISocketSubsystem::TranslatePacketTimestamp.
 */
struct FPacketTimestamp
{
	/** The internal platform specific timestamp (does NOT correspond to FPlatformTime::Seconds, may use a different clock source). */
	FTimespan	Timestamp;
};

