// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// @todo: Unfortunately, this had to be moved back as well. Convert to enum class eventually

/** All supported error types by the engine, mapped from platform specific values */
enum ESocketErrors
{
	SE_NO_ERROR,
	SE_EINTR,
	SE_EBADF,
	SE_EACCES,
	SE_EFAULT,
	SE_EINVAL,
	SE_EMFILE,
	SE_EWOULDBLOCK,
	SE_EINPROGRESS,
	SE_EALREADY,
	SE_ENOTSOCK,
	SE_EDESTADDRREQ,
	SE_EMSGSIZE,
	SE_EPROTOTYPE,
	SE_ENOPROTOOPT,
	SE_EPROTONOSUPPORT,
	SE_ESOCKTNOSUPPORT,
	SE_EOPNOTSUPP,
	SE_EPFNOSUPPORT,
	SE_EAFNOSUPPORT,
	SE_EADDRINUSE,
	SE_EADDRNOTAVAIL,
	SE_ENETDOWN,
	SE_ENETUNREACH,
	SE_ENETRESET,
	SE_ECONNABORTED,
	SE_ECONNRESET,
	SE_ENOBUFS,
	SE_EISCONN,
	SE_ENOTCONN,
	SE_ESHUTDOWN,
	SE_ETOOMANYREFS,
	SE_ETIMEDOUT,
	SE_ECONNREFUSED,
	SE_ELOOP,
	SE_ENAMETOOLONG,
	SE_EHOSTDOWN,
	SE_EHOSTUNREACH,
	SE_ENOTEMPTY,
	SE_EPROCLIM,
	SE_EUSERS,
	SE_EDQUOT,
	SE_ESTALE,
	SE_EREMOTE,
	SE_EDISCON,
	SE_SYSNOTREADY,
	SE_VERNOTSUPPORTED,
	SE_NOTINITIALISED,
	SE_HOST_NOT_FOUND,
	SE_TRY_AGAIN,
	SE_NO_RECOVERY,
	SE_NO_DATA,
	SE_UDP_ERR_PORT_UNREACH,
	SE_ADDRFAMILY,
	SE_SYSTEM,
	SE_NODEV,

	// this is a special error which means to lookup the most recent error (via GetLastErrorCode())
	SE_GET_LAST_ERROR_CODE,
};
