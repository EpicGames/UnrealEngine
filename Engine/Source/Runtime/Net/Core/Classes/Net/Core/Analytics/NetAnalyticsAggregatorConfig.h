// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "NetAnalyticsAggregatorConfig.generated.h"


/**
 * Configuration for FNetAnalyticsData - enabling/disabling analytics data, based on DataName
 */
USTRUCT()
struct FNetAnalyticsDataConfig
{
	GENERATED_USTRUCT_BODY()

	/** The name of the analytics data type (should match analytics stat name) */
	UPROPERTY(config)
	FName DataName;

	/** Whether or not the specified analytics data type, is enabled */
	UPROPERTY(config)
	bool bEnabled = false;
};

/**
 * Configuration for FNetAnalyticsAggregator - loaded PerObjectConfig, for each NetDriverName
 */
UCLASS(config=Engine, PerObjectConfig)
class UNetAnalyticsAggregatorConfig : public UObject
{
	GENERATED_UCLASS_BODY()

protected:
	void OverridePerObjectConfigSection(FString& SectionName) override;

public:
	/** Registers FNetAnalyticsData data holders, by DataName - and specifies whether they are enabled or disabled */
	UPROPERTY(config)
	TArray<FNetAnalyticsDataConfig> NetAnalyticsData;
};


