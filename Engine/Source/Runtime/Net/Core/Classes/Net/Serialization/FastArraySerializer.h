// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NetworkSerialization.h: 
	Contains custom network serialization functionality.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Serialization/BitReader.h"
#include "Misc/NetworkGuid.h"
#include "UObject/CoreNet.h"
#include "Containers/ArrayView.h"
#include "Net/Core/Misc/GuidReferences.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/Core/NetCoreModule.h"
#include "HAL/IConsoleManager.h"
#include "Templates/EnableIf.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include "FastArraySerializer.generated.h"

class Error;
struct FFastArraySerializer;
struct FFastArraySerializerItem;

// Override this to define the compiled log level of LogNetFastTArray and reduce the amount of logs generated with each FastArray implementation
#ifndef UE_FAST_ARRAY_COMPILE_LOG_LEVEL
	#define UE_FAST_ARRAY_COMPILE_LOG_LEVEL All
#endif

// Runtime log level must be <= compiled log level
#ifndef UE_FAST_ARRAY_RUNTIME_LOG_LEVEL
	#define UE_FAST_ARRAY_RUNTIME_LOG_LEVEL Warning
#endif

NETCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogNetFastTArray, UE_FAST_ARRAY_RUNTIME_LOG_LEVEL, UE_FAST_ARRAY_COMPILE_LOG_LEVEL);

DECLARE_CYCLE_STAT_EXTERN(TEXT("NetSerializeFast Array"), STAT_NetSerializeFastArray, STATGROUP_ServerCPU, NETCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("NetSerializeFast Array BuildMap"), STAT_NetSerializeFastArray_BuildMap, STATGROUP_ServerCPU, NETCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("NetSerializeFast Array Delta Struct"), STAT_NetSerializeFastArray_DeltaStruct, STATGROUP_ServerCPU, NETCORE_API);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(NETCORE_API, Networking);

/**
 *	===================== Fast TArray Replication ===================== 
 *
 *	Fast TArray Replication is a custom implementation of NetDeltaSerialize that is suitable for TArrays of UStructs. It offers performance
 *	improvements for large data sets, it serializes removals from anywhere in the array optimally, and allows events to be called on clients
 *	for adds and removals. The downside is that you will need to have game code mark items in the array as dirty, and well as the *order* of the list
 *	is not guaranteed to be identical between client and server in all cases.
 *
 *	Using FTR is more complicated, but this is the code you need:
 *
 */
#if 0
	
/** Step 1: Make your struct inherit from FFastArraySerializerItem */
USTRUCT()
struct FExampleItemEntry : public FFastArraySerializerItem
{
	GENERATED_USTRUCT_BODY()

	// Your data:
	UPROPERTY()
	int32		ExampleIntProperty;	

	UPROPERTY()
	float		ExampleFloatProperty;


	/** 
	 * Optional functions you can implement for client side notification of changes to items; 
	 * Parameter type can match the type passed as the 2nd template parameter in associated call to FastArrayDeltaSerialize
	 * 
	 * NOTE: It is not safe to modify the contents of the array serializer within these functions, nor to rely on the contents of the array 
	 * being entirely up-to-date as these functions are called on items individually as they are updated, and so may be called in the middle of a mass update.
	 */
	void PreReplicatedRemove(const struct FExampleArray& InArraySerializer);
	void PostReplicatedAdd(const struct FExampleArray& InArraySerializer);
	void PostReplicatedChange(const struct FExampleArray& InArraySerializer);

	// Optional: debug string used with LogNetFastTArray logging
	FString GetDebugString();

};

/** Step 2: You MUST wrap your TArray in another struct that inherits from FFastArraySerializer */
USTRUCT()
struct FExampleArray: public FFastArraySerializer
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FExampleItemEntry>	Items;	/** Step 3: You MUST have a TArray named Items of the struct you made in step 1. */

	/** Step 4: Copy this, replace example with your names */
	bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms)
	{
	   return FFastArraySerializer::FastArrayDeltaSerialize<FExampleItemEntry, FExampleArray>( Items, DeltaParms, *this );
	}
};

/** Step 5: Copy and paste this struct trait, replacing FExampleArray with your Step 2 struct. */
template<>
struct TStructOpsTypeTraits< FExampleArray > : public TStructOpsTypeTraitsBase2< FExampleArray >
{
       enum 
       {
			WithNetDeltaSerializer = true,
       };
};

#endif

/** Step 6 and beyond: 
 *		-Declare a UPROPERTY of your FExampleArray (step 2) type.
 *		-You MUST call MarkItemDirty on the FExampleArray when you change an item in the array. You pass in a reference to the item you dirtied. 
 *			See FFastArraySerializer::MarkItemDirty.
 *		-You MUST call MarkArrayDirty on the FExampleArray if you remove something from the array.
 *		-In your classes GetLifetimeReplicatedProps, use DOREPLIFETIME(YourClass, YourArrayStructPropertyName);
 *
 *		You can provide the following functions in your structure (step 1) to get notifies before add/deletes/removes:
 *			-void PreReplicatedRemove(const FFastArraySerializer& Serializer)
 *			-void PostReplicatedAdd(const FFastArraySerializer& Serializer)
 *			-void PostReplicatedChange(const FFastArraySerializer& Serializer)
 *			-void PostReplicatedReceive(const FFastArraySerializer::FPostReplicatedReceiveParameters& Parameters)
 *
 *		Thats it!
 */ 




/**
 *	
 *	===================== An Overview of Net Serialization and how this all works =====================
 *
 *		Everything originates in UNetDriver::ServerReplicateActors.
 *		Actors are chosen to replicate, create actor channels, and UActorChannel::ReplicateActor is called.
 *		ReplicateActor is ultimately responsible for deciding what properties have changed, and constructing an FOutBunch to send to clients.
 *
 *	The UActorChannel has 2 ways to decide what properties need to be sent.
 *		The traditional way, which is a flat TArray<uint8> buffer: UActorChannel::Recent. This represents a flat block of the actor properties.
 *		This block literally can be cast to an AActor* and property values can be looked up if you know the FProperty offset.
 *		The Recent buffer represents the values that the client using this actor channel has. We use recent to compare to current, and decide what to send.
 *
 *		This works great for 'atomic' properties; ints, floats, object*, etc.
 *		It does not work for 'dynamic' properties such as TArrays, which store values Num/Max but also a pointer to their array data,
 *		The array data has no where to fit in the flat ::Recent buffer. (Dynamic is probably a bad name for these properties)
 *
 *		To get around this, UActorChannel also has a TMap for 'dynamic' state. UActorChannel::RecentDynamicState. This map allows us to look up
 *		a 'base state' for a property given a property's RepIndex.
 *
 *	NetSerialize & NetDeltaSerialize
 *		Properties that fit into the flat Recent buffer can be serialized entirely with NetSerialize. NetSerialize just reads or writes to an FArchive.
 *		Since the replication can just look at the Recent[] buffer and do a direct comparison, it can tell what properties are dirty. NetSerialize just
 *		reads or writes.
 *
 *		Dynamic properties can only be serialized with NetDeltaSerialize. NetDeltaSerialize is serialization from a given base state, and produces
 *		both a 'delta' state (which gets sent to the client) and a 'full' state (which is saved to be used as the base state in future delta serializes).
 *		NetDeltaSerialize essentially does the diffing as well as the serialization. It must do the diffing so it can know what parts of the property it must
 *		send.
 *	
 *	Base States and dynamic properties replication.
 *		As far as the replication system / UActorChannel is concerned, a base state can be anything. The base state only deals with INetDeltaBaseState*.
 *
 *		UActorChannel::ReplicateActor will ultimately decide whether to call FProperty::NetSerializeItem or FProperty::NetDeltaSerializeItem.
 *
 *		As mentioned above NetDeltaSerialize takes in an extra base state and produces a diff state and a full state. The full state produced is used
 *		as the base state for future delta serialization. NetDeltaSerialize uses the base state and the current values of the actor to determine what parts
 *		it needs to send.
 *		
 *		The INetDeltaBaseStates are created within the NetDeltaSerialize functions. The replication system / UActorChannel does not know about the details.
 *
 *		Right now, there are 2 forms of delta serialization: Generic Replication and Fast Array Replication.
 *
 *	
 *	Generic Delta Replication
 *		Generic Delta Replication is implemented by FStructProperty::NetDeltaSerializeItem, FArrayProperty::NetDeltaSerializeItem, FProperty::NetDeltaSerializeItem.
 *		It works by first NetSerializing the current state of the object (the 'full' state) and using memcmp to compare it to previous base state. FProperty
 *		is what actually implements the comparison, writing the current state to the diff state if it has changed, and always writing to the full state otherwise.
 *		The FStructProperty and FArrayProperty functions work by iterating their fields or array elements and calling the FProperty function, while also embedding
 *		meta data. 
 *
 *		For example FArrayProperty basically writes: 
 *			"Array has X elements now" -> "Here is element Y" -> Output from FProperty::NetDeltaSerialize -> "Here is element Z" -> etc
 *
 *		Generic Data Replication is the 'default' way of handling FArrayProperty and FStructProperty serialization. This will work for any array or struct with any 
 *		sub properties as long as those properties can NetSerialize.
 *
 *	Custom Net Delta Serialiation
 *		Custom Net Delta Serialiation works by using the struct trait system. If a struct has the WithNetDeltaSerializer trait, then its native NetDeltaSerialize
 *		function will be called instead of going through the Generic Delta Replication code path in FStructProperty::NetDeltaSerializeItem.
 *
 *	Fast TArray Replication
 *		Fast TArray Replication is implemented through custom net delta serialization. Instead of a flat TArray buffer to represent states, it only is concerned
 *		with a TMap of IDs and ReplicationKeys. The IDs map to items in the array, which all have a ReplicationID field defined in FFastArraySerializerItem.
 *		FFastArraySerializerItem also has a ReplicationKey field. When items are marked dirty with MarkItemDirty, they are given a new ReplicationKey, and assigned
 *		a new ReplicationID if they don't have one.
 *
 *		FastArrayDeltaSerialize (defined below)
 *		During server serialization (writing), we compare the old base state (e.g, the old ID<->Key map) with the current state of the array. If items are missing
 *		we write them out as deletes in the bunch. If they are new or changed, they are written out as changed along with their state, serialized via a NetSerialize call.
 *
 *		For example, what actually is written may look like:
 *			"Array has X changed elements, Y deleted elements" -> "element A changed" -> Output from NetSerialize on rest of the struct item -> "Element B was deleted" -> etc
 *
 *		Note that the ReplicationID is replicated and in sync between client and server. The indices are not.
 *
 *		During client serialization (reading), the client reads in the number of changed and number of deleted elements. It also builds a mapping of ReplicationID -> local index of the current array.
 *		As it deserializes IDs, it looks up the element and then does what it needs to (create if necessary, serialize in the current state, or delete).
 *
 *		Delta Serialization for inner structs is now enabled by default. That means that when a ReplicationKey changes, we will compare the current state of the
 *		struct to the last sent state, tracking changelists and only sending properties that changed exactly like the standard replication path.
 *		If this causes issues with a specific FastArray type, it can be disabled by calling FFastArraySerializer::SetDeltaSerializationEnabled(false) in the constructor.
 *		The feature can be completely disabled by setting the "net.SupportFastArrayDelta" CVar to 0.
 *
 *		ReplicationID and ReplicationKeys are set by the MarkItemDirty function on FFastArraySerializer. These are just int32s that are assigned in order as things change.
 *		There is nothing special about them other than being unique.
 */

/**
* Helper to get auto-deduced FastArrayItemType from FastArraySerializer
*/
template <typename FastArrayType>
class TFastArrayTypeHelper
{
private:
	struct CGetFastArrayItemTypeFuncable
	{
		template <typename InFastArrayType, typename...>
		auto Requires(InFastArrayType* FastArray) -> decltype(FastArray->GetFastArrayItemTypePtr());
	};

	// Helper to always return a Type even if GetFastArrayItemTypePtr is not defined
	static constexpr auto FastArrayTypePtr = []
	{
		if constexpr (TModels_V<CGetFastArrayItemTypeFuncable, FastArrayType>)
		{
			return FastArrayType::GetFastArrayItemTypePtr();
		}
		else
		{
			return static_cast<int*>(nullptr);
		}
	}();

public:
	using FastArrayItemType = typename TRemovePointer<typename std::decay<decltype(FastArrayTypePtr)>::type>::Type;

	/**
	 * Returns true if auto detected FastArrayItemType is a valid FastArrayItemType
	 */
	static constexpr bool HasValidFastArrayItemType() { return TIsDerivedFrom<FastArrayItemType, FFastArraySerializerItem>::IsDerived; }
};

/** Custom INetDeltaBaseState used by Fast Array Serialization */
class FNetFastTArrayBaseState : public INetDeltaBaseState
{
public:

	FNetFastTArrayBaseState()
	: ArrayReplicationKey(INDEX_NONE)
	{}

	virtual bool IsStateEqual(INetDeltaBaseState* OtherState)
	{
		FNetFastTArrayBaseState * Other = static_cast<FNetFastTArrayBaseState*>(OtherState);
		for (auto It = IDToCLMap.CreateIterator(); It; ++It)
		{
			auto Ptr = Other->IDToCLMap.Find(It.Key());
			if (!Ptr || *Ptr != It.Value())
			{
				return false;
			}
		}
		return true;
	}

	virtual void CountBytes(FArchive& Ar) const override
	{
		IDToCLMap.CountBytes(Ar);
	}

	/** Maps an element's Replication ID to Index. */
	TMap<int32, int32> IDToCLMap;

	int32 ArrayReplicationKey;
};

/** Base struct for items using Fast TArray Replication */
USTRUCT()
struct FFastArraySerializerItem
{
	GENERATED_USTRUCT_BODY()

	FFastArraySerializerItem()
		: ReplicationID(INDEX_NONE), ReplicationKey(INDEX_NONE), MostRecentArrayReplicationKey(INDEX_NONE)
	{

	}

	FFastArraySerializerItem(const FFastArraySerializerItem &InItem)
		: ReplicationID(INDEX_NONE), ReplicationKey(INDEX_NONE), MostRecentArrayReplicationKey(INDEX_NONE)
	{

	}

	FFastArraySerializerItem& operator=(const FFastArraySerializerItem& In)
	{
		if (&In != this)
		{
			ReplicationID = INDEX_NONE;
			ReplicationKey = INDEX_NONE;
			MostRecentArrayReplicationKey = INDEX_NONE;
		}
		return *this;
	}

	UPROPERTY(NotReplicated)
	int32 ReplicationID;

	UPROPERTY(NotReplicated)
	int32 ReplicationKey;

	UPROPERTY(NotReplicated)
	int32 MostRecentArrayReplicationKey;
	
	/**
	 * Called right before deleting element during replication.
	 * 
	 * @param InArraySerializer	Array serializer that owns the item and has triggered the replication call
	 * 
	 * NOTE: intentionally not virtual; invoked via templated code, @see FExampleItemEntry
	 */
	inline void PreReplicatedRemove(const struct FFastArraySerializer& InArraySerializer) { }
	/**
	 * Called after adding and serializing a new element
	 *
	 * @param InArraySerializer	Array serializer that owns the item and has triggered the replication call
	 * 
	 * NOTE: intentionally not virtual; invoked via templated code, @see FExampleItemEntry
	 */
	inline void PostReplicatedAdd(const struct FFastArraySerializer& InArraySerializer) { }
	/**
	 * Called after updating an existing element with new data
	 *
	 * @param InArraySerializer	Array serializer that owns the item and has triggered the replication call
	 * NOTE: intentionally not virtual; invoked via templated code, @see FExampleItemEntry
	 */
	inline void PostReplicatedChange(const struct FFastArraySerializer& InArraySerializer) { }

	/**
	 * Called when logging LogNetFastTArray (log or lower verbosity)
	 *
	 * @param InArraySerializer	Array serializer that owns the item and has triggered the replication call
	 * NOTE: intentionally not virtual; invoked via templated code, @see FExampleItemEntry
	 */
	inline FString GetDebugString() { return FString(); }
};

/** Struct for holding guid references */
struct FFastArraySerializerGuidReferences
{
	/** List of guids that were unmapped so we can quickly check */
	TSet<FNetworkGUID> UnmappedGUIDs;

	/** List of guids that were mapped so we can move them to unmapped when necessary (i.e. actor channel closes) */
	TSet<FNetworkGUID> MappedDynamicGUIDs;

	/** Buffer of data to re-serialize when the guids are mapped */
	TArray<uint8> Buffer;

	/** Number of bits in the buffer */
	int32 NumBufferBits;
};

/**
 * Struct used only in FFastArraySerializer::FastArrayDeltaSerialize, however, declaring it within the templated function
 * causes crashes on some clang compilers
 */
struct FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair
{
	FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair(int32 _idx, int32 _id) : Idx(_idx), ID(_id) { }
	int32	Idx;
	int32	ID;
};

UENUM()
enum class EFastArraySerializerDeltaFlags : uint8
{
	None,								//! No flags.
	HasBeenSerialized = 1 << 0,			//! Set when serialization at least once (i.e., this struct has been written or read).
	HasDeltaBeenRequested = 1 << 1,		//! Set if users requested Delta Serialization for this struct.
	IsUsingDeltaSerialization = 1 << 2,	//! This will remain unset until we've serialized at least once.
										//! At that point, this will be set if delta serialization was requested and
										//! we support it.
};
ENUM_CLASS_FLAGS(EFastArraySerializerDeltaFlags);

/** Base struct for wrapping the array used in Fast TArray Replication */
USTRUCT()
struct FFastArraySerializer
{
	GENERATED_USTRUCT_BODY()

	NETCORE_API FFastArraySerializer();
	NETCORE_API ~FFastArraySerializer();

	/** Maps Element ReplicationID to Array Index.*/
	TMap<int32, int32> ItemMap;

	/** Counter used to assign IDs to new elements. */
	UPROPERTY(NotReplicated, Transient)
	int32 IDCounter;

	/** Counter used to track array replication. */
	UPROPERTY(NotReplicated)
	int32 ArrayReplicationKey;

	/** List of items that need to be re-serialized when the referenced objects are mapped */
	TMap<int32, FFastArraySerializerGuidReferences> GuidReferencesMap;

	/** List of items that need to be re-serialized when the referenced objects are mapped.*/
	TMap<int32, FGuidReferencesMap> GuidReferencesMap_StructDelta;

#if WITH_PUSH_MODEL
	// Object that is replicating this fast array
	UObject* OwningObject;

	// Property index of this array in the owning object's replication layout
	int32 RepIndex;
#endif // WITH_PUSH_MODEL

	/** This must be called if you add or change an item in the array */
	void MarkItemDirty(FFastArraySerializerItem & Item)
	{
		if (Item.ReplicationID == INDEX_NONE)
		{
			Item.ReplicationID = ++IDCounter;
			if (IDCounter == INDEX_NONE)
			{
				IDCounter++;
			}
		}

		Item.ReplicationKey++;
		MarkArrayDirty();
	}

	/** This must be called if you just remove something from the array */
	void MarkArrayDirty()
	{
		ItemMap.Reset();		// This allows to clients to add predictive elements to arrays without affecting replication.
		IncrementArrayReplicationKey();

		// Invalidate the cached item counts so that they're recomputed during the next write
		CachedNumItems = INDEX_NONE;
		CachedNumItemsToConsiderForWriting = INDEX_NONE;
	}

	void IncrementArrayReplicationKey()
	{
		ArrayReplicationKey++;
		if (ArrayReplicationKey == INDEX_NONE)
		{
			ArrayReplicationKey++;
		}

#if WITH_PUSH_MODEL
		if (OwningObject != nullptr && RepIndex != INDEX_NONE)
		{
			MARK_PROPERTY_DIRTY_UNSAFE(OwningObject, RepIndex);
		}
#endif // WITH_PUSH_MODEL
	}

	/**
	 * Performs "standard" delta serialization on the items in the FastArraySerializer.
	 * This method relies more on the INetSerializeCB interface and custom logic and sends all properties
	 * that aren't marked as SkipRep, regardless of whether or not they've changed.
	 * This will be less CPU intensive, but require more bandwidth.
	 *
	 * @param Items				Array of items owned by ArraySerializer.
	 * @param Parms				Set of parms that dictate what serialization will do / return.
	 * @param ArraySerializer	The typed subclass of FFastArraySerializer that we're serializing.
	 */
	template<typename Type, typename SerializerType>
	static bool FastArrayDeltaSerialize(TArray<Type>& Items, FNetDeltaSerializeInfo& Parms, SerializerType& ArraySerializer);

	/**
	 * Called before removing elements and after the elements themselves are notified.  The indices are valid for this function call only!
	 *
	 * NOTE: intentionally not virtual; invoked via templated code, @see FExampleItemEntry
	 */
	inline void PreReplicatedRemove(const TArrayView<int32>& RemovedIndices, int32 FinalSize) { }

	/**
	 * Called after adding all new elements and after the elements themselves are notified.  The indices are valid for this function call only!
	 *
	 * NOTE: intentionally not virtual; invoked via templated code, @see FExampleItemEntry
	 */
	inline void PostReplicatedAdd(const TArrayView<int32>& AddedIndices, int32 FinalSize) { }

	/**
	 * Called after updating all existing elements with new data and after the elements themselves are notified. The indices are valid for this function call only!
	 *
	 * NOTE: intentionally not virtual; invoked via templated code, @see FExampleItemEntry
	 */
	inline void PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize) { }

	/**
	 * If a function with the signature void PostReplicatedReceive(const FFastArraySerializer::FPostReplicatedReceiveParameters& Parameters) is defined in the derived struct
	 * It will be called after each call to NetDeltaSerialize on the receiving end, including if we have mapped some unmapped objects
	 */
	struct FPostReplicatedReceiveParameters
	{
		// This is the size that the array had before the receive.
		int32 OldArraySize;
		UE_DEPRECATED(5.4, "This is unsafe to use and will be removed.")
		uint32 bHasMoreUnmappedReferences : 1U;
	};

	// Intentionally commented out, as we do not want to call this method unless it is defined by the derived type
	// void PostReplicatedReceive(const FPostReplicatedReceiveParameters& Parameters)

	// Concept used to detect if PostReplicatedReceive is defined or not
	struct CPostReplicatedReceiveFuncable
	{
		template <typename InFastArrayType, typename...>
		auto Requires(InFastArrayType* FastArray, const FFastArraySerializer::FPostReplicatedReceiveParameters& Parameters) -> decltype(FastArray->PostReplicatedReceive(Parameters));
	};

	/**
	* Helper function for FastArrayDeltaSerialize to consolidate the logic of whether to consider writing an item in a fast TArray during network serialization.
	* For client replay recording, we don't want to write any items that have been added to the array predictively.
	*/
	template<typename Type, typename SerializerType>
	bool ShouldWriteFastArrayItem(const Type& Item, const bool bIsWritingOnClient) const
	{
		return !bIsWritingOnClient || Item.ReplicationID != INDEX_NONE;
	}

	void SetDeltaSerializationEnabled(const bool bEnabled)
	{
		if (!EnumHasAnyFlags(DeltaFlags, EFastArraySerializerDeltaFlags::HasBeenSerialized))
		{
			if (bEnabled)
			{
				DeltaFlags |= EFastArraySerializerDeltaFlags::HasDeltaBeenRequested;
			}
			else
			{
				DeltaFlags &= ~EFastArraySerializerDeltaFlags::HasDeltaBeenRequested;
			}
		}
		else
		{
			UE_LOGF(LogNetFastTArray, Log, "FFastArraySerializer::SetDeltaSerializationEnabled - Called after array has been serialized. Ignoring");
		}
	}

	const EFastArraySerializerDeltaFlags GetDeltaSerializationFlags() const
	{
		return DeltaFlags;
	}

	static const int32 GetMaxNumberOfAllowedChangesPerUpdate()
	{
		return MaxNumberOfAllowedChangesPerUpdate;
	}

	static const int32 GetMaxNumberOfAllowedDeletionsPerUpdate()
	{
		return MaxNumberOfAllowedDeletionsPerUpdate;
	}

#if WITH_PUSH_MODEL
	void CachePushModelState(UObject* Object, const uint16 PropertyRepIndex)
	{
		if (OwningObject == nullptr && RepIndex == INDEX_NONE)
		{
			OwningObject = Object;
			RepIndex = PropertyRepIndex;
		}
	}
#endif // WITH_PUSH_MODEL

private:

	NETCORE_API static int32 MaxNumberOfAllowedChangesPerUpdate;
	NETCORE_API static int32 MaxNumberOfAllowedDeletionsPerUpdate;
	NETCORE_API static FAutoConsoleVariableRef CVarMaxNumberOfAllowedChangesPerUpdate;
	NETCORE_API static FAutoConsoleVariableRef CVarMaxNumberOfAllowedDeletionsPerUpdate;

	/** Struct containing common header data that is written / read when serializing Fast Arrays. */
	struct FFastArraySerializerHeader
	{
		/** The current ArrayReplicationKey. */
		int32 ArrayReplicationKey;

		/** The previous ArrayReplicationKey. */
		int32 BaseReplicationKey;

		/** The number of changed elements (adds or removes). */
		int32 NumChanged;

		/**
		 * The list of deleted elements.
		 * When writing, this will be treated as IDs that are translated to indices prior to serialization.
		 * When reading, this will be actual indices.
		 */
		TArray<int32, TInlineAllocator<8>> DeletedIndices;
	};

	/**
	 * Helper struct that contains common methods / logic for standard Fast Array serialization and
	 * Delta Struct Fast Array serialization.
	 */
	template<typename Type, typename SerializerType>
	struct TFastArraySerializeHelper
	{
		/** Array element type struct. */
		UScriptStruct* Struct;

		/** Set of array elements we're serializing. */
		TArray<Type>& Items;

		/** The actual FFastArraySerializer struct we're serializing. */
		SerializerType& ArraySerializer;

		/** Cached DeltaSerialize params. */
		FNetDeltaSerializeInfo& Parms;

		/**
		 * Conditionally rebuilds the ID to Index map for items.
		 * This is generally only necessary on first serialization, or if we receive deletes and
		 * can no longer trust our ordering is correct.
		 */
		void ConditionalRebuildItemMap();

		/** Calculates the number of Items that actually need to be written. */
		int32 CalcNumItemsForConsideration() const;

		/** Conditionally logs the important state of the serializer. For debug purposes only. */
		void ConditionalLogSerializerState(const TMap<int32, int32>* OldIDToKeyMap) const;

		/**
		 * Checks to see if the ArrayReplicationKey has changed, and if so creates a new DeltaState that
		 * is passed out to the caller.
		 * Note, this state may just be a copy of a previous state, or a brand new state.
		 *
		 * @return	True if the keys were different and a state was created.
		 *			False if the keys were the same, and we can skip serialization.
		 */
		bool ConditionalCreateNewDeltaState(const TMap<int32, int32>& OldIDToKeyMap, const int32 BaseReplicationKey);

		/**
		 * Iterates over the current set of properties, comparing their keys with our old state, to figure
		 * out which have changed and need to be serialized. Also populates a list of elements that are
		 * no longer in our list (by ID).
		 */
		void BuildChangedAndDeletedBuffers(
			TMap<int32, int32>& NewIDToKeyMap,
			const TMap<int32, int32>* OldIDToKeyMap,
			TArray<FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair, TInlineAllocator<8>>& ChangedElements,
			TArray<int32, TInlineAllocator<8>>& DeletedElements);

		/**
		 * Variant of BuildChangedAndDeletedBuffersFromDefault used when we initialize from the default state which we are not suppose to modify
		 */
		void BuildChangedAndDeletedBuffersFromDefault(
			TMap<int32, int32>& NewIDToKeyMap,
			TArray<FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair, TInlineAllocator<8>>& ChangedElements) const;

		/** Writes out a FFastArraySerializerHeader */
		void WriteDeltaHeader(FFastArraySerializerHeader& Header) const;

		/** Reads in a FFastArraySerializerHeader */
		bool ReadDeltaHeader(FFastArraySerializerHeader& Header) const;

		/**
		 * Manages any cleanup work that needs to be done after receiving elements,
		 * such as looking for items that were implicitly deleted, removing all deleted items,
		 * firing off any PostReceive / PostDeleted events, etc.
		 */
		template<typename GuidMapType>
		void PostReceiveCleanup(
			FFastArraySerializerHeader& Header,
			TArray<int32, TInlineAllocator<8>>& ChangedIndices,
			TArray<int32, TInlineAllocator<8>>& AddedIndices,
			GuidMapType& GuidMap);
	
		/** Conditionally invoke PostReplicatedReceive method depending on if is defined or not */
		template<typename FastArrayType = SerializerType>
		inline typename TEnableIf<TModels_V<CPostReplicatedReceiveFuncable, FastArrayType, const FFastArraySerializer::FPostReplicatedReceiveParameters>, void>::Type CallPostReplicatedReceiveOrNot(int32 OldArraySize)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FFastArraySerializer::FPostReplicatedReceiveParameters PostReceivedParameters = { OldArraySize, Parms.bOutHasMoreUnmapped };
			ArraySerializer.PostReplicatedReceive(PostReceivedParameters);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		template<typename FastArrayType = SerializerType>
		inline typename TEnableIf<!TModels_V<CPostReplicatedReceiveFuncable, FastArrayType, const FFastArraySerializer::FPostReplicatedReceiveParameters>, void>::Type CallPostReplicatedReceiveOrNot(int32) {}

		// Validate that deduced FastArrayItemType is valid and that it is the same as the specified one		
		static_assert(std::is_same_v<typename TFastArrayTypeHelper<SerializerType>::FastArrayItemType, Type>, "Auto deduced FastArrayItemType is invalid or differs from the specified type. Make sure that the FastArraySerializer has a single replicated array property.");
	};

	/**
	 * Performs "struct delta" serialization on the items in the FastArraySerializer.
	 * This method relies more directly on FRepLayout for management, and will only send properties
	 * that have changed since the last update.
	 * This is potentially more CPU intensive since we'll be doing comparisons, but should require less bandwidth.
	 *
	 * For this method to work, the following **must** be true:
	 *	- Your array of FFastArraySerializerItems must be a top level UPROPERTY within your FFastArraySerializer.
	 *	- Your array of FFastArraySerializerItems must **not** be marked RepSkip.
	 *	- Your array of FFastArraySerializerItems must be the **only** replicated array of FFastArraySerializerItems within the FastArraySerializer.
	 *		Note, it's OK to have multiple arrays of items, as long as only one is replicated (all others **must** be marked RepSkip).
	 *	- Your FFastArraySerializer must not be nested in a static array.
	 *	- Your array of FFastArraySerializerItem must not be nested in a static array.
	 *
	 * @param Items				Array of items owned by ArraySerializer.
	 * @param Parms				Set of parms that dictate what serialization will do / return.
	 * @param ArraySerializer	The typed subclass of FFastArraySerializer that we're serializing.
	 */
	template<typename Type, typename SerializerType>
	static bool FastArrayDeltaSerialize_DeltaSerializeStructs(TArray<Type>& Items, FNetDeltaSerializeInfo& Parms, SerializerType& ArraySerializer);

private:

	// Cached item counts, used for fast sanity checking when writing.
	int32 CachedNumItems;
	int32 CachedNumItemsToConsiderForWriting;

	UPROPERTY(NotReplicated, Transient)
	EFastArraySerializerDeltaFlags DeltaFlags;
};

template<typename Type, typename SerializerType>
void FFastArraySerializer::TFastArraySerializeHelper<Type, SerializerType>::ConditionalRebuildItemMap()
{
	if ((Parms.bUpdateUnmappedObjects || Parms.Writer == NULL) && ArraySerializer.ItemMap.Num() != Items.Num())
	{
		SCOPE_CYCLE_COUNTER(STAT_NetSerializeFastArray_BuildMap);
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize: Recreating Items map. Struct: %ls, Items.Num: %d Map.Num: %d", *Struct->GetOwnerStruct()->GetName(), Items.Num(), ArraySerializer.ItemMap.Num());
		ArraySerializer.ItemMap.Empty();
		for (int32 i = 0; i < Items.Num(); ++i)
		{
			if (Items[i].ReplicationID == INDEX_NONE)
			{
				if (Parms.Writer)
				{
					UE_LOGF(LogNetFastTArray, Warning, "FastArrayDeltaSerialize: Item with uninitialized ReplicationID. Struct: %ls, ItemIndex: %i", *Struct->GetOwnerStruct()->GetName(), i);
				}
				else
				{
					// This is benign for clients, they may add things to their local array without assigning a ReplicationID
					continue;
				}
			}

			ArraySerializer.ItemMap.Add(Items[i].ReplicationID, i);
		}
	}
}

template<typename Type, typename SerializerType>
int32 FFastArraySerializer::TFastArraySerializeHelper<Type, SerializerType>::CalcNumItemsForConsideration() const
{
	int32 Count = 0;

	// Count the number of items in the current array that may be written. On clients, items that were predicted will be skipped.
	for (const Type& Item : Items)
	{
		if (ArraySerializer.template ShouldWriteFastArrayItem<Type, SerializerType>(Item, Parms.bIsWritingOnClient))
		{
			Count++;
		}
	}

	return Count;
};

template<typename Type, typename SerializerType>
void FFastArraySerializer::TFastArraySerializeHelper<Type, SerializerType>::ConditionalLogSerializerState(const TMap<int32, int32>* OldIDToKeyMap) const
{
	// Log out entire state of current/base state
	if (UE_LOG_ACTIVE(LogNetFastTArray, Log))
	{
		FString CurrentState = FString::Printf(TEXT("Current: %d "), ArraySerializer.ArrayReplicationKey);
		for (const Type& Item : Items)
		{
			CurrentState += FString::Printf(TEXT("[%d/%d], "), Item.ReplicationID, Item.ReplicationKey);
		}
		UE_LOGF(LogNetFastTArray, Log, "%ls", *CurrentState);


		FString ClientStateStr = FString::Printf(TEXT("Client: %d "), Parms.OldState ? ((FNetFastTArrayBaseState*)Parms.OldState)->ArrayReplicationKey : 0);
		if (OldIDToKeyMap)
		{
			for (const TPair<int32, int32>& KeyValuePair : *OldIDToKeyMap)
			{
				ClientStateStr += FString::Printf(TEXT("[%d/%d], "), KeyValuePair.Key, KeyValuePair.Value);
			}
		}
		UE_LOGF(LogNetFastTArray, Log, "%ls", *ClientStateStr);
	}
}

template<typename Type, typename SerializerType>
bool FFastArraySerializer::TFastArraySerializeHelper<Type, SerializerType>::ConditionalCreateNewDeltaState(const TMap<int32, int32>& OldIDToKeyMap, const int32 BaseReplicationKey)
{
	if (ArraySerializer.ArrayReplicationKey == BaseReplicationKey)
	{
		// If the keys didn't change, only update the item count caches if necessary.
		if (ArraySerializer.CachedNumItems == INDEX_NONE ||
			ArraySerializer.CachedNumItems != Items.Num() ||
			ArraySerializer.CachedNumItemsToConsiderForWriting == INDEX_NONE)
		{
			ArraySerializer.CachedNumItems = Items.Num();
			ArraySerializer.CachedNumItemsToConsiderForWriting = CalcNumItemsForConsideration();
		}

		if (UNLIKELY(OldIDToKeyMap.Num() != ArraySerializer.CachedNumItemsToConsiderForWriting))
		{
			UE_LOGF(LogNetFastTArray, Warning, "OldMap size (%d) does not match item count (%d) for struct (%ls), missing a MarkArrayDirty on element add/remove?", OldIDToKeyMap.Num(), ArraySerializer.CachedNumItemsToConsiderForWriting, *Struct->GetOwnerStruct()->GetName());
		}

		if (Parms.OldState)
		{
			// Nothing changed and we had a valid old state, so just use/share the existing state. No need to create a new one.
			*Parms.NewState = Parms.OldState->AsShared();
		}
		else
		{
			// Nothing changed but we don't have an existing state of our own yet so we need to make one here.
			FNetFastTArrayBaseState* NewState = new FNetFastTArrayBaseState();
			*Parms.NewState = MakeShareable(NewState);
			NewState->ArrayReplicationKey = ArraySerializer.ArrayReplicationKey;
		}
		
		return false;
	}

	return true;
}

template<typename Type, typename SerializerType>
void FFastArraySerializer::TFastArraySerializeHelper<Type, SerializerType>::BuildChangedAndDeletedBuffersFromDefault(
	TMap<int32, int32>& NewIDToKeyMap,
	TArray<FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair, TInlineAllocator<8>>& ChangedElements) const
{
	const int32 NumConsideredItems = CalcNumItemsForConsideration();

	// Verify assumptions, we never expect the default state to have replicated and thus IDCounter and ArrayReplicationKey should be at the defaults
	check(Parms.bIsInitializingBaseFromDefault);

	//-----------------------------------------------------------------
	// When initializing from default we assume that all items are new
	//-----------------------------------------------------------------
	for (int32 i = 0; i < Items.Num(); ++i)
	{
		Type& Item = Items[i];

		UE_LOGF(LogNetFastTArray, Log, "    Array[%d] - ID %d. CL %d.", i, Item.ReplicationID, Item.ReplicationKey);
		if (!ArraySerializer.template ShouldWriteFastArrayItem<Type, SerializerType>(Item, Parms.bIsWritingOnClient))
		{
			// On clients, this will skip items that were added predictively.
			continue;
		}

		// When initializing from the CDO or archetype we do not allow the state to be modified as this will potentially lead to mismatch in ID assignment
		if (Item.ReplicationID == INDEX_NONE)
		{
			UE_LOGF(LogNetFastTArray, Log, "    FastArraySerializer::BuildChangedAndDeletedBuffersFromDefault Item with uninitialized ReplicationID detected in. Struct: %ls, ItemIndex: %i", *Parms.DebugName, i);
			continue;
		}

		NewIDToKeyMap.Add(Item.ReplicationID, Item.ReplicationKey);

		UE_LOGF(LogNetFastTArray, Log, "       New! Element ID: %d. %ls", Item.ReplicationID, *Item.GetDebugString());
		// New
		ChangedElements.Add(FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair(i, Item.ReplicationID));
	}
}

template<typename Type, typename SerializerType>
void FFastArraySerializer::TFastArraySerializeHelper<Type, SerializerType>::BuildChangedAndDeletedBuffers(
	TMap<int32, int32>& NewIDToKeyMap,
	const TMap<int32, int32>* OldIDToKeyMap,
	TArray<FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair, TInlineAllocator<8>>& ChangedElements,
	TArray<int32, TInlineAllocator<8>>& DeletedElements)
{
	ConditionalLogSerializerState(OldIDToKeyMap);

	const int32 NumConsideredItems = CalcNumItemsForConsideration();

	int32 DeleteCount = (OldIDToKeyMap ? OldIDToKeyMap->Num() : 0) - NumConsideredItems; // Note: this is incremented when we add new items below.
	UE_LOGF(LogNetFastTArray, Log, "NetSerializeItemDeltaFast: %ls. DeleteCount: %d", *Parms.DebugName, DeleteCount);

	//--------------------------------------------
	// Find out what is new or what has changed
	//--------------------------------------------
	for (int32 i = 0; i < Items.Num(); ++i)
	{
		Type& Item = Items[i];

		UE_LOGF(LogNetFastTArray, Log, "    Array[%d] - ID %d. CL %d.", i, Item.ReplicationID, Item.ReplicationKey);
		if (!ArraySerializer.template ShouldWriteFastArrayItem<Type, SerializerType>(Item, Parms.bIsWritingOnClient))
		{
			// On clients, this will skip items that were added predictively.
			continue;
		}

		// The item really should have a valid ReplicationID but in the case of loading from a save game, or initializing from a default state/archetype with existing data
		// items may not have been marked dirty individually. Its ok to just assign them one here.
		if (Item.ReplicationID == INDEX_NONE)
		{
			ArraySerializer.MarkItemDirty(Item);
		}

		NewIDToKeyMap.Add(Item.ReplicationID, Item.ReplicationKey);

		const int32* OldValuePtr = OldIDToKeyMap ? OldIDToKeyMap->Find(Item.ReplicationID) : NULL;
		if (OldValuePtr)
		{
			if (*OldValuePtr == Item.ReplicationKey)
			{
				UE_LOGF(LogNetFastTArray, Log, "       Stayed The Same - Skipping");

				// Stayed the same, it might have moved but we dont care
				continue;
			}
			else
			{
				UE_LOGF(LogNetFastTArray, Log, "       Changed! Was: %d. Element ID: %d. %ls", *OldValuePtr, Item.ReplicationID, *Item.GetDebugString());

				// Changed
				ChangedElements.Add(FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair(i, Item.ReplicationID));
			}
		}
		else
		{
			UE_LOGF(LogNetFastTArray, Log, "       New! Element ID: %d. %ls", Item.ReplicationID, *Item.GetDebugString());

			// New
			ChangedElements.Add(FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair(i, Item.ReplicationID));
			++DeleteCount; // We added something new, so our initial DeleteCount value must be incremented.
		}
	}

	// Find out what was deleted
	if (DeleteCount > 0 && OldIDToKeyMap)
	{
		for (auto It = OldIDToKeyMap->CreateConstIterator(); It; ++It)
		{
			if (!NewIDToKeyMap.Contains(It.Key()))
			{
				UE_LOGF(LogNetFastTArray, Log, "   Deleting ID: %d", It.Key());

				DeletedElements.Add(It.Key());
				if (--DeleteCount <= 0)
				{
					break;
				}
			}
		}
	}
}

template<typename Type, typename SerializerType>
void FFastArraySerializer::TFastArraySerializeHelper<Type, SerializerType>::WriteDeltaHeader(FFastArraySerializerHeader& Header) const
{
	FBitWriter& Writer = *Parms.Writer;

	Writer << Header.ArrayReplicationKey;
	Writer << Header.BaseReplicationKey;

	int32 NumDeletes = Header.DeletedIndices.Num();
	Writer << NumDeletes;

	Writer << Header.NumChanged;

	UE_LOGF(LogNetFastTArray, Log, "   Writing Bunch. NumChange: %d. NumDel: %d [%d/%d]",
		Header.NumChanged, Header.DeletedIndices.Num(), Header.ArrayReplicationKey, Header.BaseReplicationKey);

	const int32 MaxNumDeleted = FFastArraySerializer::GetMaxNumberOfAllowedDeletionsPerUpdate();
	const int32 MaxNumChanged = FFastArraySerializer::GetMaxNumberOfAllowedChangesPerUpdate();
	
	// TODO: We should consider propagating this error in the same way we handle
	// array overflows in RepLayout SendProperties / CompareProperties.
	UE_CLOGF(NumDeletes > MaxNumDeleted, LogNetFastTArray, Warning, "NumDeletes > GetMaxNumberOfAllowedDeletionsPerUpdate: %d > %d. (Write)", NumDeletes, MaxNumDeleted);
	UE_CLOGF(Header.NumChanged > MaxNumChanged, LogNetFastTArray, Warning, "NumChanged > GetMaxNumberOfAllowedChangesPerUpdate: %d > %d. (Write)", Header.NumChanged, MaxNumChanged);

	// Serialize deleted items, just by their ID
	for (auto It = Header.DeletedIndices.CreateIterator(); It; ++It)
	{
		int32 ID = *It;
		Writer << ID;
		UE_LOGF(LogNetFastTArray, Log, "   Deleted ElementID: %d", ID);
	}
}

template<typename Type, typename SerializerType>
bool FFastArraySerializer::TFastArraySerializeHelper<Type, SerializerType>::ReadDeltaHeader(FFastArraySerializerHeader& Header) const
{
	FBitReader& Reader = *Parms.Reader;

	//---------------
	// Read header
	//---------------

	Reader << Header.ArrayReplicationKey;
	Reader << Header.BaseReplicationKey;

	int32 NumDeletes = 0;
	Reader << NumDeletes;

	UE_LOGF(LogNetFastTArray, Log, "Received [%d/%d].", Header.ArrayReplicationKey, Header.BaseReplicationKey);

	const int32 MaxNumDeleted = FFastArraySerializer::GetMaxNumberOfAllowedDeletionsPerUpdate();
	if (NumDeletes > MaxNumDeleted)
	{
		UE_LOGF(LogNetFastTArray, Warning, "NumDeletes > GetMaxNumberOfAllowedDeletionsPerUpdate: %d > %d. (Read)", NumDeletes, MaxNumDeleted);
		Reader.SetError();
		return false;
	}

	Reader << Header.NumChanged;

	const int32 MaxNumChanged = FFastArraySerializer::GetMaxNumberOfAllowedChangesPerUpdate();
	if (Header.NumChanged > MaxNumChanged)
	{
		UE_LOGF(LogNetFastTArray, Warning, "NumChanged > GetMaxNumberOfAllowedChangesPerUpdate: %d > %d. (Read)", Header.NumChanged, MaxNumChanged);
		Reader.SetError();
		return false;
	}

	UE_LOGF(LogNetFastTArray, Log, "Read NumChanged: %d NumDeletes: %d.", Header.NumChanged, NumDeletes);

	//---------------
	// Read deleted elements
	//---------------
	if (NumDeletes > 0)
	{
		for (int32 i = 0; i < NumDeletes; ++i)
		{
			int32 ElementID = 0;
			Reader << ElementID;

			int32* ElementIndexPtr = ArraySerializer.ItemMap.Find(ElementID);
			if (ElementIndexPtr)
			{
				int32 DeleteIndex = *ElementIndexPtr;
				Header.DeletedIndices.Add(DeleteIndex);
				UE_LOGF(LogNetFastTArray, Log, "   Adding ElementID: %d for deletion", ElementID);
			}
			else
			{
				UE_LOGF(LogNetFastTArray, Log, "   Couldn't find ElementID: %d for deletion!", ElementID);
			}
		}
	}

	return true;
}

template<typename Type, typename SerializerType>
template<typename GuidMapType>
void FFastArraySerializer::TFastArraySerializeHelper<Type, SerializerType>::PostReceiveCleanup(
	FFastArraySerializerHeader& Header,
	TArray<int32, TInlineAllocator<8>>& ChangedIndices,
	TArray<int32, TInlineAllocator<8>>& AddedIndices,
	GuidMapType& GuidMap)
{
	CSV_SCOPED_TIMING_STAT(Networking, FastArray_Apply);

	// ---------------------------------------------------------
	// Look for implicit deletes that would happen due to Naks
	// ---------------------------------------------------------

	// If we're sending data completely reliably, there's no need to do this.
	if (!Parms.bInternalAck)
	{
		for (int32 idx = 0; idx < Items.Num(); ++idx)
		{
			Type& Item = Items[idx];
			if (Item.MostRecentArrayReplicationKey < Header.ArrayReplicationKey && Item.MostRecentArrayReplicationKey > Header.BaseReplicationKey)
			{
				// Make sure this wasn't an explicit delete in this bunch (otherwise we end up deleting an extra element!)
				if (!Header.DeletedIndices.Contains(idx))
				{
					// This will happen in normal conditions in network replays.
					UE_LOGF(LogNetFastTArray, Log, "Adding implicit delete for ElementID: %d. MostRecentArrayReplicationKey: %d. Current Payload: [%d/%d]",
						Item.ReplicationID, Item.MostRecentArrayReplicationKey, Header.ArrayReplicationKey, Header.BaseReplicationKey);

					Header.DeletedIndices.Add(idx);
				}
			}
		}
	}

	// Increment keys so that a client can re-serialize the array if needed, such as for client replay recording.
	// Must check the size of DeleteIndices instead of NumDeletes to handle implicit deletes.
	if (Header.DeletedIndices.Num() > 0 || Header.NumChanged > 0)
	{
		ArraySerializer.IncrementArrayReplicationKey();
	}

	// ---------------------------------------------------------
	// Invoke all callbacks: removed -> added -> changed
	// ---------------------------------------------------------

	int32 PreRemoveSize = Items.Num();
	int32 FinalSize = PreRemoveSize - Header.DeletedIndices.Num();
	for (int32 idx : Header.DeletedIndices)
	{
		if (Items.IsValidIndex(idx))
		{
			// Remove the deleted element's tracked GUID references
			if (GuidMap.Remove(Items[idx].ReplicationID) > 0)
			{
				Parms.bGuidListsChanged = true;
			}

			// Call the delete callbacks now, actually remove them at the end
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
			if (Parms.bCallFastArrayClientCallbacks)
#endif
			{
				Items[idx].PreReplicatedRemove(ArraySerializer);
			}
		}
	}

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	if (Parms.bCallFastArrayClientCallbacks)
#endif
	{
		ArraySerializer.PreReplicatedRemove(Header.DeletedIndices, FinalSize);
	}

	if (PreRemoveSize != Items.Num())
	{
		UE_LOGF(LogNetFastTArray, Error, "Item size changed after PreReplicatedRemove! PremoveSize: %d  Item.Num: %d",
			PreRemoveSize, Items.Num());
	}

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	if (Parms.bCallFastArrayClientCallbacks)
#endif
	{
		for (int32 idx : AddedIndices)
		{
			Items[idx].PostReplicatedAdd(ArraySerializer);
		}
		ArraySerializer.PostReplicatedAdd(AddedIndices, FinalSize);
	}

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	if (Parms.bCallFastArrayClientCallbacks)
#endif
	{
		for (int32 idx : ChangedIndices)
		{
			Items[idx].PostReplicatedChange(ArraySerializer);
		}
		ArraySerializer.PostReplicatedChange(ChangedIndices, FinalSize);
	}

	if (PreRemoveSize != Items.Num())
	{
		UE_LOGF(LogNetFastTArray, Error, "Item size changed after PostReplicatedAdd/PostReplicatedChange! PremoveSize: %d  Item.Num: %d",
			PreRemoveSize, Items.Num());
	}

	if (Header.DeletedIndices.Num() > 0)
	{
		Header.DeletedIndices.Sort();
		for (int32 i = Header.DeletedIndices.Num() - 1; i >= 0; --i)
		{
			int32 DeleteIndex = Header.DeletedIndices[i];
			if (Items.IsValidIndex(DeleteIndex))
			{
				Items.RemoveAtSwap(DeleteIndex, EAllowShrinking::No);

				UE_LOGF(LogNetFastTArray, Log, "   Deleting: %d", DeleteIndex);
			}
		}

		// Clear the map now that the indices are all shifted around. This kind of sucks, we could use slightly better data structures here I think.
		// This will force the ItemMap to be rebuilt for the current Items array
		ArraySerializer.ItemMap.Empty();
	}
}

/** The function that implements Fast TArray Replication  */
template<typename Type, typename SerializerType >
bool FFastArraySerializer::FastArrayDeltaSerialize(TArray<Type> &Items, FNetDeltaSerializeInfo& Parms, SerializerType& ArraySerializer)
{
	// It's possible that we end up calling this method on clients before we've actually received
	// anything from the server (Net Conditions, Static Actors, etc.)
	// That should be fine though, because none of the GUID Tracking work should actually do anything
	// until after we've received.
	if (EnumHasAllFlags(ArraySerializer.DeltaFlags, EFastArraySerializerDeltaFlags::IsUsingDeltaSerialization))
	{
		return FastArrayDeltaSerialize_DeltaSerializeStructs(Items, Parms, ArraySerializer);
	}

	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_NetSerializeFastArray, GUseDetailedScopeCounters);
	class UScriptStruct* InnerStruct = Type::StaticStruct();

	UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize for %ls. %ls. %ls", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName(), Parms.Reader ? TEXT("Reading") : TEXT("Writing"));

	TFastArraySerializeHelper<Type, SerializerType> Helper{
		InnerStruct,
		Items,
		ArraySerializer,
		Parms
	};

	//---------------
	// Build ItemMap if necessary. This maps ReplicationID to our local index into the Items array.
	//---------------
	Helper.ConditionalRebuildItemMap();

	if ( Parms.GatherGuidReferences )
	{
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize for %ls. %ls. GatherGuidReferences", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName());

		// Loop over all tracked guids, and return what we have
		for ( const auto& GuidReferencesPair : ArraySerializer.GuidReferencesMap )
		{
			const FFastArraySerializerGuidReferences& GuidReferences = GuidReferencesPair.Value;

			Parms.GatherGuidReferences->Append( GuidReferences.UnmappedGUIDs );
			Parms.GatherGuidReferences->Append( GuidReferences.MappedDynamicGUIDs );

			if ( Parms.TrackedGuidMemoryBytes )
			{
				*Parms.TrackedGuidMemoryBytes += GuidReferences.Buffer.Num();
			}
		}

		return true;
	}

	if ( Parms.MoveGuidToUnmapped )
	{
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize for %ls. %ls. MovedGuidToUnmapped", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName());

		bool bFound = false;

		const FNetworkGUID GUID = *Parms.MoveGuidToUnmapped;

		// Try to find the guid in the list, and make sure it's on the unmapped lists now
		for ( auto& GuidReferencesPair : ArraySerializer.GuidReferencesMap )
		{
			FFastArraySerializerGuidReferences& GuidReferences = GuidReferencesPair.Value;

			if ( GuidReferences.MappedDynamicGUIDs.Contains( GUID ) )
			{
				GuidReferences.MappedDynamicGUIDs.Remove( GUID );
				GuidReferences.UnmappedGUIDs.Add( GUID );
				bFound = true;
			}
		}

		return bFound;
	}

	if ( Parms.bUpdateUnmappedObjects )
	{
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize for %ls. %ls. UpdateUnmappedObjects", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName());

		TArray<int32, TInlineAllocator<8>> ChangedIndices;

		// Loop over each item that has unmapped objects
		for ( auto It = ArraySerializer.GuidReferencesMap.CreateIterator(); It; ++It )
		{
			// Get the element id
			const int32 ElementID = It.Key();
			
			// Get a reference to the unmapped item itself
			FFastArraySerializerGuidReferences& GuidReferences = It.Value();

			if ( ( GuidReferences.UnmappedGUIDs.Num() == 0 && GuidReferences.MappedDynamicGUIDs.Num() == 0 ) || ArraySerializer.ItemMap.Find( ElementID ) == NULL )
			{
				// If for some reason the item is gone (or all guids were removed), we don't need to track guids for this item anymore
				It.RemoveCurrent();
				continue;		// We're done with this unmapped item
			}

			// Loop over all the guids, and check to see if any of them are loaded yet
			bool bMappedSomeGUIDs = false;

			for ( auto UnmappedIt = GuidReferences.UnmappedGUIDs.CreateIterator(); UnmappedIt; ++UnmappedIt )
			{
				const FNetworkGUID& GUID = *UnmappedIt;

				if ( Parms.Map->IsGUIDBroken( GUID, false ) )
				{
					// Stop trying to load broken guids
					UE_LOGF( LogNetFastTArray, Warning, "FastArrayDeltaSerialize: Broken GUID. NetGuid: %ls", *GUID.ToString() );
					UnmappedIt.RemoveCurrent();
					continue;
				}

				UObject* Object = Parms.Map->GetObjectFromNetGUID( GUID, false );

				if ( Object != NULL )
				{
					// This guid loaded!
					if ( GUID.IsDynamic() )
					{
						GuidReferences.MappedDynamicGUIDs.Add( GUID );		// Move back to mapped list
					}
					UnmappedIt.RemoveCurrent();
					bMappedSomeGUIDs = true;
				}
			}

			// Check to see if we loaded any guids. If we did, we can serialize the element again which will load it this time
			if ( bMappedSomeGUIDs )
			{
				Parms.bOutSomeObjectsWereMapped = true;

				if ( !Parms.bCalledPreNetReceive )
				{
					// Call PreNetReceive if we are going to change a value (some game code will need to think this is an actual replicated value)
					Parms.Object->PreNetReceive();
					Parms.bCalledPreNetReceive = true;
				}

				const int32 ElementIndex = ArraySerializer.ItemMap.FindChecked(ElementID);
				Type* ThisElement = &Items[ElementIndex];

				ChangedIndices.Add(ElementIndex);

				// Initialize the reader with the stored buffer that we need to read from
				FNetBitReader Reader( Parms.Map, GuidReferences.Buffer.GetData(), GuidReferences.NumBufferBits );

				// Read the property (which should serialize any newly mapped objects as well)
				Parms.Struct = InnerStruct;
				Parms.Data = ThisElement;
				Parms.Reader = &Reader;
				Parms.NetSerializeCB->NetSerializeStruct(Parms);

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
				if (Parms.bCallFastArrayClientCallbacks)
#endif
				{
					// Let the element know it changed
					ThisElement->PostReplicatedChange(ArraySerializer);
				}
			}

			// If we have no more guids, we can remove this item for good
			if ( GuidReferences.UnmappedGUIDs.Num() == 0 && GuidReferences.MappedDynamicGUIDs.Num() == 0 )
			{
				It.RemoveCurrent();
			}
		}

		// If we still have unmapped items, then communicate this to the outside
		if ( ArraySerializer.GuidReferencesMap.Num() > 0 )
		{
			Parms.bOutHasMoreUnmapped = true;
		}

		if (Parms.bOutSomeObjectsWereMapped)
		{
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
			if (Parms.bCallFastArrayClientCallbacks)
#endif
			{
				ArraySerializer.PostReplicatedChange(ChangedIndices, Items.Num());
				Helper.CallPostReplicatedReceiveOrNot(Items.Num());
			}
		}
		return true;
	}

	// If we've made it this far, it means that we're going to be serializing something.
	// So, it should be safe for us to update our cached state.
	// Also, make sure that we hit the right path if we need to.
	if (!EnumHasAnyFlags(ArraySerializer.DeltaFlags, EFastArraySerializerDeltaFlags::HasBeenSerialized))
	{
		ArraySerializer.DeltaFlags |= EFastArraySerializerDeltaFlags::HasBeenSerialized;
		if (Parms.bSupportsFastArrayDeltaStructSerialization && EnumHasAnyFlags(ArraySerializer.DeltaFlags, EFastArraySerializerDeltaFlags::HasDeltaBeenRequested))
		{
			ArraySerializer.DeltaFlags |= EFastArraySerializerDeltaFlags::IsUsingDeltaSerialization;
			return FastArrayDeltaSerialize_DeltaSerializeStructs(Items, Parms, ArraySerializer);
		}
	}

	if (Parms.Writer)
	{
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize for %ls. %ls. Writing", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName());

		//-----------------------------
		// Saving
		//-----------------------------	
		check(Parms.Struct);
		FBitWriter& Writer = *Parms.Writer;

		// Get the old map if its there
		TMap<int32, int32> * OldMap = nullptr;
		int32 BaseReplicationKey = INDEX_NONE;

		// See if the array changed at all. If the ArrayReplicationKey matches we can skip checking individual items
		if (Parms.OldState)
		{
			OldMap = &((FNetFastTArrayBaseState*)Parms.OldState)->IDToCLMap;
			BaseReplicationKey = ((FNetFastTArrayBaseState*)Parms.OldState)->ArrayReplicationKey;

			// If we didn't create a new delta state, that implies nothing changed,
			// so we're done.
			if (!Helper.ConditionalCreateNewDeltaState(*OldMap, BaseReplicationKey))
			{
				return false;
			}
		}

		// Create a new map from the current state of the array		
		FNetFastTArrayBaseState * NewState = new FNetFastTArrayBaseState();

		check(Parms.NewState);
		*Parms.NewState = MakeShareable( NewState );
		TMap<int32, int32>& NewMap = NewState->IDToCLMap;
		NewState->ArrayReplicationKey = ArraySerializer.ArrayReplicationKey;

		FFastArraySerializerHeader Header{
			ArraySerializer.ArrayReplicationKey,
			BaseReplicationKey,
			0
		};

		TArray<FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair, TInlineAllocator<8>> ChangedElements;

		// When we are initializing from default we use a simplified variant of BuildChangedAndDeletedBuffers that does not modify the source state
		if (Parms.bIsInitializingBaseFromDefault)
		{
			Helper.BuildChangedAndDeletedBuffersFromDefault(NewMap, ChangedElements);
		}
		else
		{
			Helper.BuildChangedAndDeletedBuffers(NewMap, OldMap, ChangedElements, Header.DeletedIndices);

			// The array replication key may have changed while adding new elements (in the call to BuildChangedAndDeletedBuffers above)
			NewState->ArrayReplicationKey = ArraySerializer.ArrayReplicationKey;
		}

		// Note: we used to early return false here if nothing had changed, but we still need to send
		// a bunch with the array key / base key, so that clients can look for implicit deletes.
		
		//----------------------
		// Write it out.
		//----------------------

		Header.NumChanged = ChangedElements.Num();
		Helper.WriteDeltaHeader(Header);

		// Serialized new elements with their payload
		for (auto It = ChangedElements.CreateIterator(); It; ++It)
		{
			void* ThisElement = &Items[It->Idx];

			// Dont pack this, want property to be byte aligned
			uint32 ID = It->ID;
			Writer << ID;

			UE_LOGF(LogNetFastTArray, Log, "   Changed ElementID: %d", ID);

			Parms.Struct = InnerStruct;
			Parms.Data = ThisElement;
			Parms.NetSerializeCB->NetSerializeStruct(Parms);
		}
	}
	else
	{
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize for %ls. %ls. Reading", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName());

		//-----------------------------
		// Loading
		//-----------------------------	
		check(Parms.Reader);
		FBitReader& Reader = *Parms.Reader;

		FFastArraySerializerHeader Header;
		if (!Helper.ReadDeltaHeader(Header))
		{
			return false;
		}

		const int32 OldNumItems = Items.Num();
		TArray<int32, TInlineAllocator<8>> ChangedIndices;
		TArray<int32, TInlineAllocator<8>> AddedIndices;

		//---------------
		// Read Changed/New elements
		//---------------
		for(int32 i = 0; i < Header.NumChanged; ++i)
		{
			int32 ElementID = 0;
			Reader << ElementID;

			int32* ElementIndexPtr = ArraySerializer.ItemMap.Find(ElementID);
			int32 ElementIndex = 0;
			Type* ThisElement = nullptr;
		
			if (!ElementIndexPtr)
			{
				UE_LOGF(LogNetFastTArray, Log, "   New. ID: %d. New Element!", ElementID);

				ThisElement = &Items.AddDefaulted_GetRef();
				ThisElement->ReplicationID = ElementID;

				ElementIndex = Items.Num()-1;
				ArraySerializer.ItemMap.Add(ElementID, ElementIndex);
				
				AddedIndices.Add(ElementIndex);
			}
			else
			{
				UE_LOGF(LogNetFastTArray, Log, "   Changed. ID: %d -> Idx: %d", ElementID, *ElementIndexPtr);
				ElementIndex = *ElementIndexPtr;
				ThisElement = &Items[ElementIndex];
				ChangedIndices.Add(ElementIndex);
			}

			// Update this element's most recent array replication key
			ThisElement->MostRecentArrayReplicationKey = Header.ArrayReplicationKey;

			// Update this element's replication key so that a client can re-serialize the array for client replay recording
			ThisElement->ReplicationKey++;

			// Let package map know we want to track and know about any guids that are unmapped during the serialize call
			Parms.Map->ResetTrackedGuids( true );

			// Remember where we started reading from, so that if we have unmapped properties, we can re-deserialize from this data later
			FBitReaderMark Mark( Reader );

			Parms.Struct = InnerStruct;
			Parms.Data = ThisElement;
			Parms.NetSerializeCB->NetSerializeStruct(Parms);

			if ( !Reader.IsError() )
			{
				// Track unmapped guids
				const TSet< FNetworkGUID >& TrackedUnmappedGuids = Parms.Map->GetTrackedUnmappedGuids();
				const TSet< FNetworkGUID >& TrackedMappedDynamicGuids = Parms.Map->GetTrackedDynamicMappedGuids();

				if ( TrackedUnmappedGuids.Num() || TrackedMappedDynamicGuids.Num() )
				{	
					FFastArraySerializerGuidReferences& GuidReferences = ArraySerializer.GuidReferencesMap.FindOrAdd( ElementID );

					// If guid lists are different, make note of that, and copy respective list
					if ( !NetworkGuidSetsAreSame( GuidReferences.UnmappedGUIDs, TrackedUnmappedGuids ) )
					{
						// Copy the unmapped guid list to this unmapped item
						GuidReferences.UnmappedGUIDs = TrackedUnmappedGuids;
						Parms.bGuidListsChanged = true;
					}

					if ( !NetworkGuidSetsAreSame( GuidReferences.MappedDynamicGUIDs, TrackedMappedDynamicGuids ) )
					{ 
						// Copy the mapped guid list
						GuidReferences.MappedDynamicGUIDs = TrackedMappedDynamicGuids;
						Parms.bGuidListsChanged = true;
					}

					GuidReferences.Buffer.Empty();

					// Remember the number of bits in the buffer
					check(Reader.GetPosBits() - Mark.GetPos() <= TNumericLimits<int32>::Max());
					GuidReferences.NumBufferBits = int32(Reader.GetPosBits() - Mark.GetPos());
					
					// Copy the buffer itself
					Mark.Copy( Reader, GuidReferences.Buffer );

					// Hijack this property to communicate that we need to be tracked since we have some unmapped guids
					if ( TrackedUnmappedGuids.Num() )
					{
						Parms.bOutHasMoreUnmapped = true;
					}
				}
				else
				{
					// If we don't have any unmapped objects, make sure we're no longer tracking this item in the unmapped lists
					ArraySerializer.GuidReferencesMap.Remove( ElementID );
				}
			}

			// Stop tracking unmapped objects
			Parms.Map->ResetTrackedGuids( false );

			if ( Reader.IsError() )
			{
				UE_LOGF( LogNetFastTArray, Warning, "Parms.NetSerializeCB->NetSerializeStruct: Reader.IsError() == true" );
				return false;
			}
		}

		Helper.template PostReceiveCleanup<>(Header, ChangedIndices, AddedIndices, ArraySerializer.GuidReferencesMap);

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
		if (Parms.bCallFastArrayClientCallbacks)
#endif
		{
			Helper.CallPostReplicatedReceiveOrNot(OldNumItems);
		}
	}

	return true;
}

struct FFastArrayDeltaSerializeParams
{
	FNetDeltaSerializeInfo& DeltaSerializeInfo;
	FFastArraySerializer& ArraySerializer;

	const TFunction<void(void*, const FFastArrayDeltaSerializeParams&)> PreReplicatedRemove;
	const TFunction<void(void*, const FFastArrayDeltaSerializeParams&)> PostReplicatedAdd;
	const TFunction<void(void*, const FFastArrayDeltaSerializeParams&)> PostReplicatedChange;
	const TFunction<void(void*, const FFastArrayDeltaSerializeParams&, const uint32)> ReceivedItem;

	TArray<FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair, TInlineAllocator<8>>* WriteChangedElements = nullptr;
	FNetFastTArrayBaseState* WriteBaseState = nullptr;
	TArray<int32, TInlineAllocator<8>>* ReadChangedElements = nullptr;
	TArray<int32, TInlineAllocator<8>>* ReadAddedElements = nullptr;
	int32 ReadNumChanged = INDEX_NONE;
	int32 ReadArrayReplicationKey = INDEX_NONE;
};

template<typename Type, typename SerializerType>
bool FFastArraySerializer::FastArrayDeltaSerialize_DeltaSerializeStructs(TArray<Type>& Items, FNetDeltaSerializeInfo& Parms, SerializerType& ArraySerializer)
{
	/**
	 * These methods are exposed on FastArraySerializerItems, but they aren't virtual.
	 * Further, we may not know the exact type when we want to call them, and won't
	 * safely be able to cast to the type in non-templated code.
	 *
	 * Maybe this defeats the purpose of having them not virtual in the first place.
	 * However, for now ReceivedItem and PostReplicatedChanged are the only ones that will actually
	 * be called in this way, whereas PostReplicatedAdd and PreReplicatedRemove will still be called
	 * from templated code.
	 */
	struct FFastArrayItemCallbackHelper
	{
		static void PreReplicatedRemove(void* FastArrayItem, const struct FFastArrayDeltaSerializeParams& Params)
		{
			reinterpret_cast<Type*>(FastArrayItem)->PreReplicatedRemove(static_cast<SerializerType&>(Params.ArraySerializer));
		}

		static void PostReplicatedAdd(void* FastArrayItem, const struct FFastArrayDeltaSerializeParams& Params)
		{
			reinterpret_cast<Type*>(FastArrayItem)->PostReplicatedAdd(static_cast<SerializerType&>(Params.ArraySerializer));
		}

		static void PostReplicatedChange(void* FastArrayItem, const struct FFastArrayDeltaSerializeParams& Params)
		{
			reinterpret_cast<Type*>(FastArrayItem)->PostReplicatedChange(static_cast<SerializerType&>(Params.ArraySerializer));
		}

		static void ReceivedItem(void* FastArrayItem, const FFastArrayDeltaSerializeParams& Params, const uint32 ReplicationID)
		{
			Type* Item = reinterpret_cast<Type*>(FastArrayItem);
			Item->ReplicationID = ReplicationID;
			Item->MostRecentArrayReplicationKey = Params.ReadArrayReplicationKey;
			Item->ReplicationKey++;
		}
	};

	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_NetSerializeFastArray_DeltaStruct, GUseDetailedScopeCounters);

	class UScriptStruct* InnerStruct = Type::StaticStruct();

	TFastArraySerializeHelper<Type, SerializerType> Helper{
		InnerStruct,
		Items,
		ArraySerializer,
		Parms
	};

	FFastArrayDeltaSerializeParams DeltaSerializeParams{
		Parms,
		ArraySerializer,
		FFastArrayItemCallbackHelper::PreReplicatedRemove,
		FFastArrayItemCallbackHelper::PostReplicatedAdd,
		FFastArrayItemCallbackHelper::PostReplicatedChange,
		FFastArrayItemCallbackHelper::ReceivedItem,
	};

	//---------------
	// Build ItemMap if necessary. This maps ReplicationID to our local index into the Items array.
	//---------------
	Helper.ConditionalRebuildItemMap();

	if (Parms.GatherGuidReferences)
	{
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize_DeltaSerializeStruct for %ls. %ls. GatherGuidReferences", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName());
		Parms.NetSerializeCB->GatherGuidReferencesForFastArray(DeltaSerializeParams);
		return true;
	}
	else if (Parms.MoveGuidToUnmapped)
	{
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize_DeltaSerializeStruct for %ls. %ls. MoveGuidToUnmapped", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName());
		return Parms.NetSerializeCB->MoveGuidToUnmappedForFastArray(DeltaSerializeParams);
	}
	else if (Parms.bUpdateUnmappedObjects)
	{
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize_DeltaSerializeStruct for %ls. %ls. UpdateUnmappedObjects", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName());

		TArray<int32, TInlineAllocator<8>> ChangedIndices;

		DeltaSerializeParams.ReadChangedElements = &ChangedIndices;

		Parms.NetSerializeCB->UpdateUnmappedGuidsForFastArray(DeltaSerializeParams);

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
		if (Parms.bCallFastArrayClientCallbacks)
#endif
		{
			if (Parms.bOutSomeObjectsWereMapped)
			{
				ArraySerializer.PostReplicatedChange(ChangedIndices, Items.Num());
				Helper.CallPostReplicatedReceiveOrNot(Items.Num());
			}
		}

		return true;
	}
	else if (Parms.Writer)
	{
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize_DeltaSerializeStruct for %ls. %ls. Writing", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName());

		//-----------------------------
		// Saving
		//-----------------------------	
		check(Parms.Struct);
		FBitWriter& Writer = *Parms.Writer;

		// Get the old map if its there
		TMap<int32, int32>* OldItemMap = nullptr;
		int32 BaseReplicationKey = INDEX_NONE;

		// We set ChangelistHistory and LastAckedHistory to 0, as that's the default RepLayout expects.
		// Setting them to anything else may cause issues with initial sends.
		uint32 OldChangelistHistory = 0;
		uint32 OldLastAckedHistory = 0;

		// See if the array changed at all. If the ArrayReplicationKey matches we can skip checking individual items
		if (Parms.OldState)
		{
			OldItemMap = &((FNetFastTArrayBaseState*)Parms.OldState)->IDToCLMap;
			BaseReplicationKey = ((FNetFastTArrayBaseState*)Parms.OldState)->ArrayReplicationKey;
			OldChangelistHistory = Parms.OldState->GetChangelistHistory();
			OldLastAckedHistory = Parms.OldState->GetLastAckedHistory();

			if (!Helper.ConditionalCreateNewDeltaState(*OldItemMap, BaseReplicationKey))
			{
				return false;
			}
		}

		// Create a new map from the current state of the array		
		FNetFastTArrayBaseState* NewState = new FNetFastTArrayBaseState();

		check(Parms.NewState);

		*Parms.NewState = MakeShareable(NewState);
		NewState->ArrayReplicationKey = ArraySerializer.ArrayReplicationKey;

		TMap<int32, int32>& NewItemMap = NewState->IDToCLMap;

		FFastArraySerializerHeader Header{
			ArraySerializer.ArrayReplicationKey,
			BaseReplicationKey,
			0
		};

		TArray<FFastArraySerializer_FastArrayDeltaSerialize_FIdxIDPair, TInlineAllocator<8>> ChangedElements;

		// When we are initializing from default we use a simplified variant of BuildChangedAndDeletedBuffers that does not modify the source state
		if (Parms.bIsInitializingBaseFromDefault)
		{
			Helper.BuildChangedAndDeletedBuffersFromDefault(NewItemMap, ChangedElements);
		}
		else
		{
			Helper.BuildChangedAndDeletedBuffers(NewItemMap, OldItemMap, ChangedElements, Header.DeletedIndices);

			// The array replication key may have changed while adding new elemnts (in the call to MarkItemDirty above)
			NewState->ArrayReplicationKey = ArraySerializer.ArrayReplicationKey;
		}

		// Note: we used to early return false here if nothing had changed, but we still need to send
		// a bunch with the array key / base key, so that clients can look for implicit deletes.

		//----------------------
		// Write it out.
		//----------------------

		Header.NumChanged = ChangedElements.Num();
		Helper.WriteDeltaHeader(Header);

		NewState->SetChangelistHistory(OldChangelistHistory);
		NewState->SetLastAckedHistory(OldLastAckedHistory);

		DeltaSerializeParams.WriteChangedElements = &ChangedElements;
		DeltaSerializeParams.WriteBaseState = NewState;

		return Parms.NetSerializeCB->NetDeltaSerializeForFastArray(DeltaSerializeParams);
	}
	else
	{
		UE_LOGF(LogNetFastTArray, Log, "FastArrayDeltaSerialize_DeltaSerializeStruct for %ls. %ls. Reading", *InnerStruct->GetName(), *InnerStruct->GetOwnerStruct()->GetName());

		//-----------------------------
		// Loading
		//-----------------------------	
		check(Parms.Reader);
		FBitReader& Reader = *Parms.Reader;

		FFastArraySerializerHeader Header;
		if (!Helper.ReadDeltaHeader(Header))
		{
			return false;
		}

		const int32 OldNumItems = Items.Num();
		TArray<int32, TInlineAllocator<8>> ChangedIndices;
		TArray<int32, TInlineAllocator<8>> AddedIndices;

		DeltaSerializeParams.ReadAddedElements = &AddedIndices;
		DeltaSerializeParams.ReadChangedElements = &ChangedIndices;
		DeltaSerializeParams.ReadNumChanged = Header.NumChanged;
		DeltaSerializeParams.ReadArrayReplicationKey = Header.ArrayReplicationKey;

		if (!Parms.NetSerializeCB->NetDeltaSerializeForFastArray(DeltaSerializeParams))
		{
			return false;
		}

		//---------------
		// Read Changed/New elements
		//---------------

		Helper.template PostReceiveCleanup<>(Header, ChangedIndices, AddedIndices, ArraySerializer.GuidReferencesMap_StructDelta);

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
		if (Parms.bCallFastArrayClientCallbacks)
#endif
		{
			Helper.CallPostReplicatedReceiveOrNot(OldNumItems);
		}
	}

	return true;
}


/**
 * Macro injected from UHT to facilitate automatic registration of FastArraySerializers when using iris replication
 */
#if UE_WITH_IRIS
#define UE_NET_DECLARE_FASTARRAY(FastArrayType, FastArrayItemArrayMemberName, Api) \
	static constexpr auto GetFastArrayItemTypePtr() { return static_cast<decltype(FastArrayType::FastArrayItemArrayMemberName)::ElementType*>(nullptr); }; \
	Api static UE::Net::CreateAndRegisterReplicationFragmentFunc GetFastArrayCreateReplicationFragmentFunction();
#else
#define UE_NET_DECLARE_FASTARRAY(FastArrayType, FastArrayItemArrayMemberName, Api) 	static constexpr auto GetFastArrayItemTypePtr() { return static_cast<decltype(FastArrayType::FastArrayItemArrayMemberName)::ElementType*>(nullptr); };;
#endif

