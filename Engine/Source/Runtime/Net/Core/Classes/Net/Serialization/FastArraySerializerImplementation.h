// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Bad include. This header can't compile standalone

#if UE_WITH_IRIS
#include "Iris/ReplicationSystem/FastArrayReplicationFragment.h"

#define UE_NET_IMPLEMENT_FASTARRAY(FastArrayType) \
UE::Net::CreateAndRegisterReplicationFragmentFunc FastArrayType::GetFastArrayCreateReplicationFragmentFunction() \
{ \
	auto CreateFastArrayReplicationFragmentFunction = [](UObject* InLocalOwner, const UE::Net::FReplicationStateDescriptor* InLocalDescriptor, UE::Net::FFragmentRegistrationContext& InLocalContext) \
	{ \
		return UE::Net::Private::CreateAndRegisterFragment<FastArrayType>(InLocalOwner, InLocalDescriptor, InLocalContext); \
	}; \
	return CreateFastArrayReplicationFragmentFunction; \
}

#define UE_NET_IMPLEMENT_FASTARRAY_STUB(FastArrayType) \
UE::Net::CreateAndRegisterReplicationFragmentFunc FastArrayType::GetFastArrayCreateReplicationFragmentFunction() \
{ \
	return nullptr; \
}

#else
#define UE_NET_IMPLEMENT_FASTARRAY(...)
#define UE_NET_IMPLEMENT_FASTARRAY_STUB(...)
#endif
