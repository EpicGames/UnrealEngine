// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class NetCore : ModuleRules
{
	public NetCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
				"CoreUObject",
				"TraceLog",
				"NetCommon"
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		//Iris Parallel support requires Oodle to be threadsafe on the server
		//So enable multithreading if not Client/Program and not disabled on the build command line 
		if (!Target.bExcludeParallelIrisSupport && !(Target.Type == TargetType.Client || Target.Type == TargetType.Program))
		{
			PublicDefinitions.Add("NET_ANALYTICS_MULTITHREADING=1");
		}
	}
}
