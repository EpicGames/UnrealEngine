// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/Core/Analytics/NetAnalytics.h"
#include "Net/Core/Analytics/NetAnalyticsAggregatorConfig.h"
#include "Net/Core/Misc/NetCoreLog.h"


// Globals

TAtomic<uint8> GNetAnalyticsCounter(0);


/**
 * FNetAnalytics
 */

FNetAnalyticsData::FNetAnalyticsData()
	: Aggregator(nullptr)
{
}

FNetAnalyticsData::~FNetAnalyticsData() = default;

void FNetAnalyticsData::InternalSendAnalytics()
{
	SendAnalytics();
}

#if NET_ANALYTICS_MULTITHREADING
/**
 * FThreadedNetAnalyticsData
 */

FThreadedNetAnalyticsData::FThreadedNetAnalyticsData()
	: FNetAnalyticsData()
	, bReadyToSend(false)
{
}

void FThreadedNetAnalyticsData::InternalSendAnalytics()
{
	bReadyToSend = true;
}

void FThreadedNetAnalyticsData::NotifyFinalRelease()
{
	if (bReadyToSend)
	{
		SendAnalytics();
	}
}
#endif


/**
 * FNetAnalyticsAggregator
 */

FNetAnalyticsAggregator::FNetAnalyticsAggregator(TSharedPtr<IAnalyticsProvider> InProvider, FName InNetDriverName)
	: AnalyticsProvider(InProvider)
	, NetDriverName(InNetDriverName)
	, AnalyticsDataMap()
	, AnalyticsDataTypeMap()
	, AnalyticsDataConfigMap()
	, bSentAnalytics(false)
{
}

void FNetAnalyticsAggregator::Init()
{
	GNetAnalyticsCounter++;

	InitConfig();
}

void FNetAnalyticsAggregator::InitConfig()
{
	UClass* ClassRef = UNetAnalyticsAggregatorConfig::StaticClass();
	UNetAnalyticsAggregatorConfig* CurConfig = FindObject<UNetAnalyticsAggregatorConfig>(ClassRef, *NetDriverName.ToString());

	if (CurConfig == nullptr)
	{
		CurConfig = NewObject<UNetAnalyticsAggregatorConfig>(ClassRef, NetDriverName);
	}

	AnalyticsDataConfigMap.Empty();

	// If the config is hotfixed, make sure no data holders are currently active, as they can't be selectively hotfixed if loaded
	// (this does seem to happen, frequently - so limits the hotfixability of Net Analytics)
	UE_CLOGF(AnalyticsDataMap.Num() > 0, LogNetCore, Warning,
			"Net Analytics hotfixed while already active. Analytics hotfix changes may not be applied correctly.");

	if (CurConfig != nullptr)
	{
		for (const FNetAnalyticsDataConfig& CurEntry : CurConfig->NetAnalyticsData)
		{
			AnalyticsDataConfigMap.Add(CurEntry.DataName, CurEntry.bEnabled);

			UE_LOGF(LogNetCore, Log, "Adding NetAnalyticsData: %ls, bEnabled: %i", *CurEntry.DataName.ToString(),
					CurEntry.bEnabled);
		}
	}
}

TNetAnalyticsDataPtr<> FNetAnalyticsAggregator::RegisterAnalyticsData_Internal(TNetAnalyticsDataRef<> InData, const FName& InDataName,
																					FString InTypeName)
{
	TNetAnalyticsDataPtr<> Result;
	bool* bEnabled = AnalyticsDataConfigMap.Find(InDataName);

	if (bEnabled != nullptr && *bEnabled)
	{
		FString* TypeVal = AnalyticsDataTypeMap.Find(InDataName);

		if (TypeVal == nullptr)
		{
			AnalyticsDataTypeMap.Add(InDataName, InTypeName);
		}
		else
		{
			check(*TypeVal == InTypeName);
		}

		TNetAnalyticsDataRef<>* Found = AnalyticsDataMap.Find(InDataName);

		Result = Found != nullptr ? *Found : AnalyticsDataMap.Add(InDataName, InData);

		InData->Aggregator = this;
	}
	else if (bEnabled == nullptr)
	{
		UE_LOGF(LogNetCore, Error, "NetAnalyticsData type '%ls' must be added to NetAnalyticsAggregatorConfig, for NetDriverName: %ls.",
				*InDataName.ToString(), *NetDriverName.ToString());
	}

	return Result;
}

void FNetAnalyticsAggregator::SendAnalytics()
{
	if (!bSentAnalytics)
	{
		for (TMap<FName, TNetAnalyticsDataRef<>>::TConstIterator It(AnalyticsDataMap); It; ++It)
		{
			It.Value()->InternalSendAnalytics();
		}

		bSentAnalytics = true;
	}
}

void FNetAnalyticsAggregator::SetAnalyticsAppender(FAnalyticsAppender AppenderFunction)
{
	AnalyticsAppender = AppenderFunction;
}

void FNetAnalyticsAggregator::AppendGameInstanceAttributes(TArray<FAnalyticsEventAttribute>& OutAttributes)
{
	if (AnalyticsAppender)
	{
		AnalyticsAppender(OutAttributes);
	}
}