// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/Analytics/NetAnalyticsAggregatorConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetAnalyticsAggregatorConfig)


/**
 * UNetAnalyticsAggregatorConfig
 */
UNetAnalyticsAggregatorConfig::UNetAnalyticsAggregatorConfig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNetAnalyticsAggregatorConfig::OverridePerObjectConfigSection(FString& SectionName)
{
	SectionName = GetName() + TEXT(" ") + GetClass()->GetName();
}

