// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/Core/Analytics/NetStatsUtils.h"

namespace UE
{
namespace Net
{

/**
 * FBinnedMovingValueBase
 */

FBinnedMovingValueBase::FBinnedMovingValueBase(const TArrayView<FBin> InBins, double InTimePerBin, EBinnedValueMode InMode)
	: BinsView(InBins)
	, TimePerBin(InTimePerBin)
	, BinRange(InBins.Num() * InTimePerBin)
	, SampleMode(InMode)
{
}

void FBinnedMovingValueBase::Reset()
{
	for (FBin& CurBin : BinsView)
	{
		CurBin = {};
	}

	FirstBinIndex = INDEX_NONE;
	LastWrittenBinIndex = INDEX_NONE;
	TotalSum = 0.0;
	TotalCount = 0;
}

void FBinnedMovingValueBase::AddMeasurement_Implementation(double TimeVal, double Value, TOptional<double>& OutSample)
{
	const double BinnedTimeVal = TimeVal - (FMath::FloorToDouble(TimeVal / BinRange) * BinRange);
	const int32 BinIdx = FMath::Min(static_cast<int32>(FMath::FloorToDouble(BinnedTimeVal / TimePerBin)), BinsView.Num()-1);

	if (!BinsView.IsValidIndex(BinIdx))
	{
		return;
	}

	// If moving into a new bin, reset it and any skipped-over bins - and output a sample if bins reach the start again
	if (BinIdx != LastWrittenBinIndex)
	{
		ResetNewAndSkippedBins(BinIdx, OutSample);
	}

	FBin& CurBin = BinsView[BinIdx];

	CurBin.Sum += Value;
	CurBin.Count++;

	TotalSum += Value;
	TotalCount++;
}

double FBinnedMovingValueBase::GetSample() const
{
	return (SampleMode == EBinnedValueMode::MovingSum ? TotalSum : (TotalSum / (double)TotalCount));
}

void FBinnedMovingValueBase::ResetNewAndSkippedBins(int32 BinIdx, TOptional<double>& OutSample)
{
	if (UNLIKELY(LastWrittenBinIndex == INDEX_NONE))
	{
		FirstBinIndex = BinIdx;
	}
	else
	{
		auto BinDecrement = [&](int32 BinDecIndex) -> int32
		{
			return (BinDecIndex == 0 ? BinsView.Num() : BinDecIndex) - 1;
		};

		// Wipe skipped over bins, that received no write
		for (int32 ResetIdx=BinDecrement(BinIdx); ResetIdx!=LastWrittenBinIndex; ResetIdx=BinDecrement(ResetIdx))
		{
			if (ResetIdx == FirstBinIndex)
			{
				OutSample = GetSample();
			}

			FBin& ResetBin = BinsView[ResetIdx];

			TotalSum -= ResetBin.Sum;
			TotalCount -= ResetBin.Count;
			ResetBin = {};
		}

		if (BinIdx == FirstBinIndex)
		{
			OutSample = GetSample();
		}
	}

	FBin& CurBin = BinsView[BinIdx];

	TotalSum -= CurBin.Sum;
	TotalCount -= CurBin.Count;
	CurBin = {};

	LastWrittenBinIndex = BinIdx;
}


/**
 * FSampleMinMaxAvg
 */

void FSampleMinMaxAvg::AddSample_Internal(double Value)
{
	CurrentValue = Value;
	TotalSum += Value;
	TotalCount++;
}

void FSampleMinMaxAvg::Reset()
{
	MinValue = std::numeric_limits<double>::max();
	MaxValue = std::numeric_limits<double>::min();
	CurrentValue = 0.0;
	TotalSum = 0.0;
	TotalCount = 0;
}

}
}


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBinnedMovingValueTest, "System.Core.Networking.BinnedMovingValue",
									EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter

);

bool FBinnedMovingValueTest::RunTest(const FString& Parameters)
{
	using namespace UE::Net;

	TSampleMinMaxAvg<EMinMaxValueMode::PerSample> DeltaStats;
	TDeltaBinnedMovingSum<decltype(DeltaStats), TBinParms::NumBins(6)> DeltaTest{TBinParms::TimePerBin(10.0)};

	// 250 per minute, with 6 bins
	const double MarginOfError = 250.0 / 6.0;

	DeltaTest.SetConsumer(&DeltaStats);

	for (double TimeVal = 60.0, TotalVal = 0.0; TimeVal <= 360.0; TimeVal += 3.0, TotalVal += 12.5)
	{
		DeltaTest.AddMeasurement(TimeVal, TotalVal);
	}

	TestTrue(TEXT("Moving sum test of 250 delta per minute failed"), (250.0 - DeltaTest.GetSample()) < MarginOfError);
	TestTrue(TEXT("DeltaStats output from moving sum test of 250 delta per minute failed"), (250.0 - DeltaStats.GetAvg()) < MarginOfError);

	return true;
}
#endif
