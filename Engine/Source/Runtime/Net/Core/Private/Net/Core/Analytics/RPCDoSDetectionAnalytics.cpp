// Copyright Epic Games, Inc. All Rights Reserved.

// Includes

#include "Net/Core/Analytics/RPCDoSDetectionAnalytics.h"
#include "HAL/IConsoleManager.h"


// Globals

FModifyRPCDoSAnalytics GModifyRPCDoSAnalytics;
FModifyRPCDoSAnalytics GModifyRPCDoSEscalationAnalytics;


/**
 * CVars
 */

TAutoConsoleVariable<int32> CVarRPCDoSAnalyticsMaxRPCs(
	TEXT("net.RPCDoSAnalyticsMaxRPCs"), 20,
	TEXT("The top 'x' number of RPC's to include in RPC DoS analytics, ranked by RPC rate per Second."));


/**
 * FRPCDoSAnalyticsVars
 */

FRPCDoSAnalyticsVars::FRPCDoSAnalyticsVars()
	: RPCTrackingAnalytics()
	, MaxRPCAnalytics(CVarRPCDoSAnalyticsMaxRPCs.GetValueOnAnyThread())
{
}

bool FRPCDoSAnalyticsVars::operator == (const FRPCDoSAnalyticsVars& A) const
{
	return PlayerIP == A.PlayerIP && PlayerUID == A.PlayerUID && MaxSeverityIndex == A.MaxSeverityIndex &&
			MaxSeverityCategory == A.MaxSeverityCategory && MaxAnalyticsSeverityIndex == A.MaxAnalyticsSeverityIndex &&
			MaxAnalyticsSeverityCategory == A.MaxAnalyticsSeverityCategory && RPCTrackingAnalytics == A.RPCTrackingAnalytics &&
			MaxPlayerSeverity == A.MaxPlayerSeverity;
}

void FRPCDoSAnalyticsVars::CommitAnalytics(FRPCDoSAnalyticsVars& AggregatedData)
{
	if (MaxSeverityIndex > AggregatedData.MaxSeverityIndex)
	{
		AggregatedData.MaxSeverityIndex = MaxSeverityIndex;
		AggregatedData.MaxSeverityCategory = MaxSeverityCategory;
	}

	if (MaxAnalyticsSeverityIndex > AggregatedData.MaxAnalyticsSeverityIndex)
	{
		AggregatedData.MaxAnalyticsSeverityIndex = MaxAnalyticsSeverityIndex;
		AggregatedData.MaxAnalyticsSeverityCategory = MaxAnalyticsSeverityCategory;
	}


	TArray<TSharedPtr<FRPCAnalytics>>& AggRPCTrackingAnalytics = AggregatedData.RPCTrackingAnalytics;
	AggRPCTrackingAnalytics.Append(RPCTrackingAnalytics);

	AggRPCTrackingAnalytics.Sort(
		[](const TSharedPtr<FRPCAnalytics>& A, const TSharedPtr<FRPCAnalytics>& B)
		{
			return A->MaxTimePerSec > B->MaxTimePerSec;
		});

	const int32 MaxSize = AggregatedData.MaxRPCAnalytics;

	if (AggRPCTrackingAnalytics.Num() > MaxSize)
	{
		AggRPCTrackingAnalytics.SetNum(MaxSize);
	}


	if (MaxAnalyticsSeverityIndex != 0)
	{
		TArray<FMaxRPCDoSEscalation>& AggMaxPlayerSeverity = AggregatedData.MaxPlayerSeverity;
		int32 SeverityInsertIdx = 0;

		for (; SeverityInsertIdx<AggMaxPlayerSeverity.Num(); SeverityInsertIdx++)
		{
			if (AggMaxPlayerSeverity[SeverityInsertIdx].MaxAnalyticsSeverityIndex < MaxAnalyticsSeverityIndex)
			{
				break;
			}
		}

		AggMaxPlayerSeverity.Insert({PlayerIP, PlayerUID, MaxSeverityIndex, MaxSeverityCategory, MaxAnalyticsSeverityIndex,
										MaxAnalyticsSeverityCategory},
										SeverityInsertIdx);
	}
}


/**
 * FRPCDoSAnalyticsData
 */

void FRPCDoSAnalyticsData::SetAnalyticsSender(TSharedPtr<FRPCDoSAnalyticsSender> InSender)
{
	AnalyticsSender = InSender;
}

FRPCDoSAnalyticsVars* FRPCDoSAnalyticsData::GetVars()
{
	return this;
}

void FRPCDoSAnalyticsData::SendAnalytics()
{
	if (AnalyticsSender.IsValid())
	{
		AnalyticsSender->SendAnalytics(Aggregator, this);
	}
}

void FRPCDoSAnalyticsData::FireEvent_ServerRPCDoSEscalation(int32 SeverityIndex, const FString& SeverityCategory, int32 WorstCountPerSec,
															double WorstTimePerSec, const FString& InPlayerIP, const FString& InPlayerUID,
															const TArray<FName>& InRPCGroup, double InRPCGroupTime/*=0.0*/)
{
	if (AnalyticsSender.IsValid())
	{
		AnalyticsSender->FireEvent_ServerRPCDoSEscalation(Aggregator, this, SeverityIndex, SeverityCategory, WorstCountPerSec,
															WorstTimePerSec, InPlayerIP, InPlayerUID, InRPCGroup, InRPCGroupTime);
	}
}

