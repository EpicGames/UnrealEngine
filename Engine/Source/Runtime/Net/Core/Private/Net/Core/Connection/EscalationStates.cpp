// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/Core/Connection/EscalationStates.h"
#include "Net/Core/Misc/NetCoreLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EscalationStates)


/**
 * FEscalationState
 */

void FEscalationState::ValidateConfigInternal()
{
	Super::ValidateConfigInternal();

	if (AutoEscalateTime > 0 && AutoEscalateTime < CooloffTime)
	{
		UE_LOGF(LogNetCore, Warning, "FEscalationState: AutoEscalateTime must be larger than CooloffTime.");
	}
}

void FEscalationState::ValidateTimePeriod(int8& Value, const TCHAR* PropertyName, EValidateTime Requirement/*=EValidateTime::Optional*/)
{
	const int8 BaseValue = (Requirement == EValidateTime::MustBeSet) ? 1 : 0;

	if (Value < BaseValue || Value > 16)
	{
		UE_LOGF(LogNetCore, Warning, "FEscalationState: %ls '%i' must be between %i and 16. Clamping.",
				PropertyName, Value, BaseValue);

		constexpr int8 MaxValue = 16;
		Value = FMath::Clamp(Value, BaseValue, MaxValue);
	}
}

bool FEscalationState::IsDormant() const
{
	return bDormant;
}

int8 FEscalationState::GetHighestTimePeriod() const
{
	return HighestTimePeriod;
}

const TArray<int8>& FEscalationState::GetAllTimePeriods() const
{
	return AllTimePeriods;
}


namespace UE
{
namespace Net
{

/**
 * EEscalateReason
 */

/**
 * Convert EEscalateReason enum values, to a string.
 *
 * @param Reason	The enum value to convert.
 * @return			The string name for the enum value.
 */
const TCHAR* LexToString(EEscalateReason Reason)
{
	switch (Reason)
	{
	case EEscalateReason::QuotaLimit:
		return TEXT("QuotaLimit");

	case EEscalateReason::AutoEscalate:
		return TEXT("AutoEscalate");

	case EEscalateReason::Deescalate:
		return TEXT("Deescalate");

	default:
		return TEXT("Unknown");
	}
};


/**
 * FEscalationManager
 */

FEscalationManager::FEscalationManager(FEscalationManagerParms Parms)
	: NumCounters(Parms.NumCounters)
	, State(Parms.StateStruct, Parms.StateMemory)
	, RegisteredCounters(Parms.RegisteredCountersCache)
{
	// If the array sizes change, the elements in braced initialization need to be altered, too
	static_assert(UE_ARRAY_COUNT(CountersPerPeriodHistory) == 16,
					"Constructor brace-enclosed initializer must be updated for CountersPerPeriodHistory");

	check(NumCounters > 0);
	check(State.IsValid());
}

void FEscalationManager::InitParms(FEscalationManagerInitParms Parms)
{
	FrameCounters = Parms.FrameCounters;
	SecondCounters = Parms.SecondCounters;

	for (int32 PeriodIdx=0; PeriodIdx<UE_ARRAY_COUNT(CountersPerPeriodHistory); PeriodIdx++)
	{
		CountersPerPeriodHistory[PeriodIdx] = MakeArrayView(&Parms.CountersPerPeriodHistoryAlloc[PeriodIdx * NumCounters], NumCounters);
	}
}

void FEscalationManager::InitConfig(FStateConfigParms ConfigParms)
{
	check(FCString::Strlen(ConfigParms.ConfigSection) > 0);

	BaseConfig = CastChecked<UEscalationManagerConfig>(UStatePerObjectConfig::Get(ConfigParms));

	bEnabled = BaseConfig->bEnabled;


	if (bEnabled)
	{
		const TArray<TStructOnScope<FEscalationState>>& EscalationSeverityState = BaseConfig->EscalationSeverityState;
		const int32 HighestHistoryRequirement = GetHighestHistoryRequirement();

		if (EscalationSeverityState.Num() > 0)
		{
			UStatePerObjectConfig::ApplyState(EscalationSeverityState[ActiveState], State.Get());

			CountersPerSecHistoryAlloc.SetNum(HighestHistoryRequirement * NumCounters);
			CountersPerSecHistory.SetNum(HighestHistoryRequirement);

			for (int32 CurHistoryIdx=0; CurHistoryIdx<HighestHistoryRequirement; CurHistoryIdx++)
			{
				FEscalationCounter* CurAllocPtr = &CountersPerSecHistoryAlloc[CurHistoryIdx * NumCounters];

				CountersPerSecHistory[CurHistoryIdx] = MakeArrayView(CurAllocPtr, NumCounters);
			}

			if (NotifySeverityUpdate)
			{
				NotifySeverityUpdate(*State.Get(), *State.Get(), ESeverityUpdate::Escalate);
			}
		}
		else
		{
			UE_LOGF(LogNetCore, Warning, "Escalation Manager enabled, but no EscalationSeverity states specified! Disabling.");

			bEnabled = false;
		}
	}
}

EEscalateResult FEscalationManager::UpdateSeverity(ESeverityUpdate Update, EEscalateReason Reason, FString ReasonContext/*=TEXT("")*/)
{
	bool bEscalate = Update == ESeverityUpdate::Escalate || Update == ESeverityUpdate::AutoEscalate;
	EEscalateResult ReturnVal = bEscalate ? EEscalateResult::Escalated : EEscalateResult::Deescalated;
	const TArray<TStructOnScope<FEscalationState>>& EscalationSeverity = BaseConfig->EscalationSeverityState;
	
	int8 NewStateIdx = static_cast<int8>(FMath::Clamp(ActiveState + (bEscalate ? 1 : -1), 0, EscalationSeverity.Num()-1));

	if (NewStateIdx != ActiveState)
	{
		const double CurTime = FPlatformTime::Seconds();

		if (bEscalate)
		{
			LastMetEscalationConditions = CurTime;
		}
		else
		{
			// De-escalate to the lowest state which hasn't cooled off, and estimate the timestamp for when the cooloff was last reset
			// (due to estimating, there is slight inaccuracy in the cooloff time).
			// Will not de-escalate if the active state still exceeds quota limits.
			bool bCooloffReached = true;

			NewStateIdx = ActiveState;

			if (NewStateIdx > 0)
			{
				TArray<FEscalationCounter, TInlineAllocator<16 * 8>> WorkingPerPeriodAlloc;
				TArrayView<FEscalationCounter> WorkingPerPeriodHistory[16];

				WorkingPerPeriodAlloc.SetNumUninitialized(NumCounters * UE_ARRAY_COUNT(WorkingPerPeriodHistory));

				for (int32 CounterIdx=0; CounterIdx<UE_ARRAY_COUNT(WorkingPerPeriodHistory); CounterIdx++)
				{
					WorkingPerPeriodHistory[CounterIdx] = MakeArrayView(&WorkingPerPeriodAlloc[CounterIdx * NumCounters], NumCounters);
				}

				while (bCooloffReached && NewStateIdx > 0)
				{
					const TStructOnScope<FEscalationState>& PrevState = EscalationSeverity[NewStateIdx-1];
					const TStructOnScope<FEscalationState>& CurState = EscalationSeverity[NewStateIdx];
					int32 CurStateCooloffTime = CurState->CooloffTime;

					check(CountersPerSecHistory.Num() >= CurStateCooloffTime);

					for (FEscalationCounter& CurCounterAlloc : WorkingPerPeriodAlloc)
					{
						CurCounterAlloc.ResetCounters();
					}

					// Count backwards through every second of CountersPerSecHistory, whose starting index wraps around like a circular buffer
					for (int32 SecondsDelta=0; SecondsDelta<CurStateCooloffTime; SecondsDelta++)
					{
						const TArray<int8>& PrevStateTimePeriods = PrevState->GetAllTimePeriods();
						int32 StartIdx = LastCountersPerSecHistoryIdx - SecondsDelta;

						StartIdx = (StartIdx < 0 ? CountersPerSecHistory.Num() + StartIdx : StartIdx);

						check(StartIdx >= 0 && StartIdx < CountersPerSecHistory.Num());

						RecalculatePeriodHistory(PrevStateTimePeriods, WorkingPerPeriodHistory, StartIdx);

						// Determine if any time period quota's for the current SecondsDelta-offset CounterPerSecHistory were breached
						if (PrevState->HasHitAnyQuota({RegisteredCounters, WorkingPerPeriodHistory, MakeArrayView(SecondCounters),
														MakeArrayView(FrameCounters), EQuotaType::DeescalateQuota}))
						{
							// The state we're transitioning down into, would have last had its cooloff reset around this time
							if (NewStateIdx != ActiveState)
							{
								LastMetEscalationConditions = CurTime - (double)SecondsDelta;
							}

							bCooloffReached = false;
							break;
						}
					}

					if (bCooloffReached)
					{
						NewStateIdx--;
					}
				}
			}
		}
	}

	if (NewStateIdx != ActiveState)
	{
		const TStructOnScope<FEscalationState>& OldState = EscalationSeverity[ActiveState];

		while (true)
		{
			const TStructOnScope<FEscalationState>& CurState = EscalationSeverity[NewStateIdx];

			ActiveState = NewStateIdx;

			UStatePerObjectConfig::ApplyState(CurState, State.Get());

			const TArray<int8>& NeededTimePeriods = State->GetAllTimePeriods();

			RecalculatePeriodHistory(NeededTimePeriods, CountersPerPeriodHistory);

			// When entering dormancy, reset all counters
			if (!bEscalate && State->IsDormant())
			{
				ResetAllCounters();
			}

			// If escalating, keep escalating until the quota checks fail
			if (bEscalate && State->HasHitAnyQuota({RegisteredCounters, CountersPerPeriodHistory, MakeArrayView(SecondCounters),
													MakeArrayView(FrameCounters), EQuotaType::EscalateQuota}))
			{
				NewStateIdx = static_cast<int8>(FMath::Clamp(ActiveState + 1, 0, EscalationSeverity.Num()-1));

				if (NewStateIdx == ActiveState)
				{
					break;
				}
			}
			else
			{
				break;
			}
		}

		if (NotifySeverityUpdate)
		{
			NotifySeverityUpdate(*OldState.Get(), *State.Get(), Update);
		}

		if (State->bLogEscalate)
		{
			const TStructOnScope<FEscalationState>& NewState = EscalationSeverity[NewStateIdx];

			UE_LOGF(LogNetCore, Warning, "Updated '%ls' severity state for '%ls' from '%ls' to '%ls' (Reason: %ls%ls)",
					ToCStr(NewState.GetStruct()->GetName()), ToCStr(ManagerContext), ToCStr(OldState->GetStateName()),
					ToCStr(NewState->GetStateName()), LexToString(Reason),
					ToCStr(!ReasonContext.IsEmpty() ? FString::Printf(TEXT(", %s"), *ReasonContext) : TEXT("")));
		}
	}
	else
	{
		ReturnVal = EEscalateResult::NoChange;
	}

	return ReturnVal;
}

void FEscalationManager::TickRealtime(double TimeSeconds)
{
	if (bEnabled)
	{
		// Accumulate from previous Tick
		for (int32 CounterIdx=0; CounterIdx<FrameCounters.Num(); CounterIdx++)
		{
			FEscalationCounter& CurFrameCounter = FrameCounters[CounterIdx];

			SecondCounters[CounterIdx].AccumulateCounter(CurFrameCounter);

			CurFrameCounter.ResetCounters();
		}


		if ((TimeSeconds - LastPerSecQuotaBegin) > 1.0)
		{
			// NOTE: It's important that these UpdateSeverity checks are limited to once per second,
			//			as in some circumstances they can be called every Tick otherwise (expensive)
			if (ActiveState > 0)
			{
				double ActiveStateTime = TimeSeconds - LastMetEscalationConditions;
				const double& CooloffTime = State->CooloffTime;
				const double& AutoEscalateTime = State->AutoEscalateTime;
				bool bUpdatedState = false;

				if (CooloffTime > 0 && ActiveStateTime > CooloffTime)
				{
					bUpdatedState = UpdateSeverity(ESeverityUpdate::Deescalate, EEscalateReason::Deescalate) != EEscalateResult::NoChange;
				}

				if (!bUpdatedState && AutoEscalateTime > 0 && ActiveStateTime > AutoEscalateTime)
				{
					UpdateSeverity(ESeverityUpdate::AutoEscalate, EEscalateReason::AutoEscalate);
				}
			}

			// NOTE: This timing is only approximate, and e.g. if there is a 10 second hitch during the previous Tick,
			//			data from those 10 seconds will be stored as 1 second.
			//			This shouldn't affect the accuracy of most packet-receive based network escalation triggers.

			// Record the last quota
			check(CountersPerSecHistory.Num() > 0);

			LastCountersPerSecHistoryIdx++;
			LastCountersPerSecHistoryIdx = (LastCountersPerSecHistoryIdx >= CountersPerSecHistory.Num()) ? 0 : LastCountersPerSecHistoryIdx;

			{
				int32 SecondCounterIdx = 0;

				for (FEscalationCounter& CurCounter : CountersPerSecHistory[LastCountersPerSecHistoryIdx])
				{
					CurCounter = SecondCounters[SecondCounterIdx];

					SecondCounterIdx++;
				}
			}

			LastPerSecQuotaBegin = TimeSeconds;


			for (FEscalationCounter& CurCounter : SecondCounters)
			{
				CurCounter.ResetCounters();
			}

			const TArray<int8>& NeededTimePeriods = State->GetAllTimePeriods();

			RecalculatePeriodHistory(NeededTimePeriods, CountersPerPeriodHistory);
		}
	}
}

void FEscalationManager::RecalculatePeriodHistory(const TArray<int8>& InTimePeriods, TArrayView<FEscalationCounter>(&OutPerPeriodHistory)[16],
													int32 StartPerSecHistoryIdx/*=INDEX_NONE*/)
{
	if (StartPerSecHistoryIdx == INDEX_NONE)
	{
		StartPerSecHistoryIdx = LastCountersPerSecHistoryIdx;
	}

	for (int8 PeriodSeconds : InTimePeriods)
	{
		const TArrayView<FEscalationCounter>& CurPeriod = OutPerPeriodHistory[PeriodSeconds-1];

		for (FEscalationCounter& CurCounter : CurPeriod)
		{
			CurCounter.ResetCounters();
		}

		for (int32 SecIdx=0; SecIdx<PeriodSeconds; SecIdx++)
		{
			int32 PerSecHistoryIdx = StartPerSecHistoryIdx - SecIdx;

			PerSecHistoryIdx = (PerSecHistoryIdx < 0 ? CountersPerSecHistory.Num() + PerSecHistoryIdx : PerSecHistoryIdx);

			check(PerSecHistoryIdx >= 0 && PerSecHistoryIdx < CountersPerSecHistory.Num());

			for (int32 CounterIdx=0; CounterIdx<CurPeriod.Num(); CounterIdx++)
			{
				FEscalationCounter& CurCounter = CurPeriod[CounterIdx];

				CurCounter.AccumulateCounter(CountersPerSecHistory[PerSecHistoryIdx][CounterIdx]);
			}
		}
	}
}

int32 FEscalationManager::GetHighestHistoryRequirement() const
{
	int32 ReturnVal = 0;

	if (BaseConfig != nullptr)
	{
		const TArray<TStructOnScope<FEscalationState>>& EscalationSeverityState = BaseConfig->EscalationSeverityState;

		for (const TStructOnScope<FEscalationState>& CurSeverityConfig : EscalationSeverityState)
		{
			const int8 HighestTimePeriod = CurSeverityConfig->GetHighestTimePeriod();

			ReturnVal = FMath::Max(ReturnVal, (int32)(CurSeverityConfig->CooloffTime + HighestTimePeriod));
		}
	}

	return ReturnVal;
}

void FEscalationManager::ResetAllCounters()
{
	for (FEscalationCounter& CurCounter : FrameCounters)
	{
		CurCounter.ResetCounters();
	}

	for (FEscalationCounter& CurCounter : SecondCounters)
	{
		CurCounter.ResetCounters();
	}

	for (const TArrayView<FEscalationCounter>& CurHistory : CountersPerSecHistory)
	{
		for (FEscalationCounter& CurCounter : CurHistory)
		{
			CurCounter.ResetCounters();
		}
	}

	for (const TArrayView<FEscalationCounter>& CurHistory : CountersPerPeriodHistory)
	{
		for (FEscalationCounter& CurCounter : CurHistory)
		{
			CurCounter.ResetCounters();
		}
	}
}

int32 FEscalationManager::AddNewCounter_Internal(int32 Count, const TArrayView<FEscalationCounter>& CountersPerPeriodAlloc)
{
	const int32 OldNumCounters = NumCounters;
	const int32 NewNumCounters = OldNumCounters + Count;

	for (int32 PeriodIdx=UE_ARRAY_COUNT(CountersPerPeriodHistory)-1; PeriodIdx >= 0; PeriodIdx--)
	{
		TArrayView<FEscalationCounter>& CurView = CountersPerPeriodHistory[PeriodIdx];
		TArrayView<FEscalationCounter> OldView = CurView;

		CurView = MakeArrayView(&CountersPerPeriodAlloc[PeriodIdx * NewNumCounters], NewNumCounters);

		// Moves up all of CountersPerPeriodAlloc
		for (int32 ElementIdx=0; ElementIdx<OldNumCounters; ElementIdx++)
		{
			CurView[ElementIdx] = OldView[ElementIdx];
		}
	}


	const int32 HighestHistoryRequirement = GetHighestHistoryRequirement();

	CountersPerSecHistoryAlloc.Reserve(CountersPerSecHistoryAlloc.Num() + (Count * HighestHistoryRequirement));

	for (int32 CurHistoryIdx=0; CurHistoryIdx<HighestHistoryRequirement; CurHistoryIdx++)
	{
		const int32 BaseIdx = CurHistoryIdx * NewNumCounters;
		const int32 InsertIdx = BaseIdx + OldNumCounters;

		CountersPerSecHistoryAlloc.InsertDefaulted(InsertIdx, Count);

		CountersPerSecHistory[CurHistoryIdx] = MakeArrayView(&CountersPerSecHistoryAlloc[BaseIdx], NewNumCounters);
	}

	NumCounters += Count;

	return OldNumCounters;
}


void FEscalationManager::CheckQuotas()
{
	if (State.IsValid() && State->HasHitAnyQuota({RegisteredCounters, CountersPerPeriodHistory, MakeArrayView(SecondCounters),
													MakeArrayView(FrameCounters), EQuotaType::EscalateQuota}))
	{
		UpdateSeverity(ESeverityUpdate::Escalate, EEscalateReason::QuotaLimit);
	}
}

bool FEscalationManager::IsDormant() const
{
	return State.IsValid() && State->IsDormant();
}

const UEscalationManagerConfig* FEscalationManager::GetBaseConfig() const
{
	return BaseConfig;
}

void FEscalationManager::SetManagerContext(FString InManagerContext)
{
	ManagerContext = InManagerContext;
}

void FEscalationManager::SetNotifySeverityUpdate(FNotifySeverityUpdate&& InNotifySeverityUpdate)
{
	NotifySeverityUpdate = MoveTemp(InNotifySeverityUpdate);
}

}
}


/**
 * UEscalationManagerConfig
 */

void UEscalationManagerConfig::LoadStateConfig()
{
	RegisterStateConfig(EscalationSeverity, EscalationSeverityState);
}



