// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/Core/Connection/NetCloseResult.h"
#include "Net/Core/Connection/NetEnums.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetCloseResult)


/**
 * ENetCloseResult
 */

#ifndef CASE_ENUM_TO_TEXT_RET
#define CASE_ENUM_TO_TEXT_RET(txt) case txt: ReturnVal = TEXT(#txt); break;
#endif

const TCHAR* LexToString(ENetCloseResult Enum)
{
	const TCHAR* ReturnVal = TEXT("::Invalid");

	switch (Enum)
	{
		FOREACH_ENUM_ENETCLOSERESULT(CASE_ENUM_TO_TEXT_RET)
	}

	while (*ReturnVal != ':')
	{
		ReturnVal++;
	}

	ReturnVal += 2;

	return ReturnVal;
}

ENetCloseResult FromNetworkFailure(ENetworkFailure::Type Val)
{
	const UEnum* NetFailEnum = StaticEnum<ENetworkFailure::Type>();
	const uint32 RawVal = (uint32)Val;

	if (NetFailEnum != nullptr && RawVal < (uint32)NetFailEnum->GetMaxEnumValue())
	{
		return (ENetCloseResult)RawVal;
	}

	return ENetCloseResult::Unknown;
}

ENetworkFailure::Type ToNetworkFailure(ENetCloseResult Val)
{
	const UEnum* NetFailEnum = StaticEnum<ENetworkFailure::Type>();
	const uint32 RawVal = (uint32)Val;

	if (NetFailEnum != nullptr && RawVal < (uint32)NetFailEnum->GetMaxEnumValue())
	{
		return (ENetworkFailure::Type)RawVal;
	}

	return ENetworkFailure::Type::ConnectionLost;
}

ENetCloseResult FromSecurityEvent(ESecurityEvent::Type Val)
{
	if (const UEnum* NetFailEnum = StaticEnum<ENetworkFailure::Type>())
	{
		const uint32 FirstSecurityEvent = (uint32)NetFailEnum->GetMaxEnumValue();
		const uint32 ConvertedVal = FirstSecurityEvent + (uint32)Val;

		if (ConvertedVal < (uint32)ENetCloseResult::Unknown)
		{
			return (ENetCloseResult)(ConvertedVal);
		}
	}

	return ENetCloseResult::Unknown;
}



#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNetCloseResultEnumTest, "System.Core.Networking.FNetCloseResult.EnumTest",
									EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter);

bool FNetCloseResultEnumTest::RunTest(const FString& Parameters)
{
	using namespace UE::Net;

	// Search by name due to remapping being required for old enums that have been moved
	const UEnum* NetFailEnum = StaticEnum<ENetworkFailure::Type>();
	const UEnum* NetCloseResultEnum = StaticEnum<ENetCloseResult>();
	int64 NetFailEnumLast = 0;

	// Until ENetworkFailure is deprecated
	if (TestTrue(TEXT("ENetworkFailure must exist"), NetFailEnum != nullptr) && NetFailEnum != nullptr)
	{
		// If a new element is added to the end of ENetworkFailure, update this
		const ENetworkFailure::Type LastNetworkFailureEntry = ENetworkFailure::NetChecksumMismatch;
		const ENetCloseResult LastNetworkFailureDuplicate = ENetCloseResult::NetChecksumMismatch;

		NetFailEnumLast = NetFailEnum->GetMaxEnumValue() - 1;

		TestTrue(TEXT("ENetCloseResult must contain (start with) all ENetworkFailure elements"),
					(NetFailEnumLast == (int64)LastNetworkFailureEntry) &&
					((int64)LastNetworkFailureEntry == (int64)LastNetworkFailureDuplicate));


		if (NetCloseResultEnum != nullptr)
		{
			bool bConversionMismatch = false;

			for (int64 EnumIdx=0; EnumIdx<=NetFailEnumLast && !bConversionMismatch; EnumIdx++)
			{
				bConversionMismatch = NetCloseResultEnum->GetNameStringByValue((int64)FromNetworkFailure((ENetworkFailure::Type)EnumIdx)) !=
										NetFailEnum->GetNameStringByValue(EnumIdx);
			}

			TestFalse(TEXT("Start of ENetCloseResult entries must match ENetworkFailure entries"), bConversionMismatch);
		}
	}

	// ESecurityEvent (to be deprecated eventually)
	if (NetFailEnum != nullptr)
	{
		const int64 LastSecurityEvent = (int64)ESecurityEvent::Closed;

		TestTrue(TEXT("Tests must cover all ESecurityEvent entries"),
					FCString::Strlen(ESecurityEvent::ToString((ESecurityEvent::Type)(LastSecurityEvent + 1))) == 0);

		if (NetCloseResultEnum != nullptr)
		{
			const int64 FirstSecurityEventDuplicate = NetFailEnumLast + 1;
			bool bFirstMismatch = false;
			bool bListMismatch = false;

			auto ConvertSecurityEnumName =
				[](ESecurityEvent::Type InVal) -> FString
				{
					TStringBuilder<256> ConvertedElement;

					ConvertedElement.Append(TEXT("Security"));
					ConvertedElement.Append(ToCStr(FString(ESecurityEvent::ToString((ESecurityEvent::Type)InVal)).Replace(TEXT("_"), TEXT(""))));

					return ConvertedElement.ToString();
				};

			for (int64 EnumIdx=0; EnumIdx<=LastSecurityEvent; EnumIdx++)
			{
				bListMismatch = bListMismatch || NetCloseResultEnum->GetNameStringByValue(FirstSecurityEventDuplicate + EnumIdx) !=
													ConvertSecurityEnumName((ESecurityEvent::Type)EnumIdx);

				if (EnumIdx == 0)
				{
					bFirstMismatch = bListMismatch;
				}
			}

			TestFalse(TEXT("ENetCloseResult must contain ESecurityEvent entries, after ENetworkFailure entries"), bFirstMismatch);
			TestFalse(TEXT("ENetCloseResult must contain all ESecurityEvent entries"), bListMismatch);

			bool bConversionMismatch = false;

			for (int64 EnumIdx=0; EnumIdx<=LastSecurityEvent && !bConversionMismatch; EnumIdx++)
			{
				bConversionMismatch = NetCloseResultEnum->GetNameStringByValue((int64)FromSecurityEvent((ESecurityEvent::Type)EnumIdx)) !=
										ConvertSecurityEnumName((ESecurityEvent::Type)EnumIdx);
			}

			TestFalse(TEXT("Start of ENetCloseResult entries must match ESecurityEvent entries"), bConversionMismatch);
		}
	}

	return true;
}
#endif

