// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/Core/Connection/NetConnectionFaultRecoveryBase.h"
#include "Net/Core/Misc/NetCoreLog.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetConnectionFaultRecoveryBase)


// CVars
#if !UE_BUILD_SHIPPING
static int32 GNetFaultRecoveryLogQuotaChecks = false;

FAutoConsoleVariableRef GNetFaultRecoveryLogQuotaChecksCVar(
	TEXT("net.NetFaultRecoveryLogQuotaChecks"),
	GNetFaultRecoveryLogQuotaChecks,
	TEXT("Whether or not to enable debug logging for quota checks (useful for debugging new net faults used with 'RegisterCounterCategory')"));
#endif


/**
 * FNetFaultState
 */

const TCHAR* FNetFaultState::GetConfigSection()
{
	return TEXT("NetFault");
}

UClass* FNetFaultState::GetBaseConfigClass()
{
	return FindObject<UClass>(nullptr, TEXT("/Script/Engine.NetFaultConfig"));
}


EInitStateDefaultsResult FNetFaultState::InitConfigDefaultsInternal()
{
	EInitStateDefaultsResult ReturnVal = EInitStateDefaultsResult::Initialized;

	FString SeverityCategory = GetStateName();

	if (SeverityCategory == TEXT("Normal"))
	{
		bDormant = true;
		EscalateQuotaFaultsPerPeriod = 1;
		EscalateQuotaTimePeriod = 1;
	}
	else if (SeverityCategory == TEXT("Fault"))
	{
		EscalateQuotaFaultsPerPeriod = 8;
		DescalateQuotaFaultsPerPeriod = 6;
		EscalateQuotaTimePeriod = 8;
		CooloffTime = 10;
		AutoEscalateTime = 11;
	}
	else if (SeverityCategory == TEXT("PersistentFault"))
	{
		EscalateQuotaFaultsPerPeriod = 64;
		EscalateQuotaFaultPercentPerPeriod = 50;
		DescalateQuotaFaultsPerPeriod = 48;
		DescalateQuotaFaultPercentPerPeriod = 40;
		EscalateQuotaTimePeriod = 8;
		CooloffTime = 10;
		AutoEscalateTime = 20;
		bLogEscalate = true;
	}
	else if (SeverityCategory == TEXT("DisconnectCountdown"))
	{
		EscalateQuotaFaultPercentPerPeriod = 70;
		DescalateQuotaFaultPercentPerPeriod = 56;
		EscalateQuotaTimePeriod = 16;
		CooloffTime = 10;
		AutoEscalateTime = 60;
		bLogEscalate = true;
	}
	else if (SeverityCategory == TEXT("Disconnect"))
	{
		bCloseConnection = true;
		bLogEscalate = true;
		EscalateQuotaTimePeriod = 0;
	}
	else
	{
		ReturnVal = EInitStateDefaultsResult::NotInitialized;
	}

	return ReturnVal;
}

void FNetFaultState::ApplyImpliedValuesInternal()
{
	Super::ApplyImpliedValuesInternal();

	// Make sure that both escalate and de-escalate quota's are set

	if ((EscalateQuotaFaultsPerPeriod > 0) != (DescalateQuotaFaultsPerPeriod > 0))
	{
		const int16 QuotaFaultsPerPeriod = FMath::Max(EscalateQuotaFaultsPerPeriod, DescalateQuotaFaultsPerPeriod);

		EscalateQuotaFaultsPerPeriod = QuotaFaultsPerPeriod;
		DescalateQuotaFaultsPerPeriod = QuotaFaultsPerPeriod;
	}

	if ((EscalateQuotaFaultPercentPerPeriod > 0) != (DescalateQuotaFaultPercentPerPeriod > 0))
	{
		const int8 QuotaFaultPercentPerPeriod = FMath::Max(EscalateQuotaFaultPercentPerPeriod, DescalateQuotaFaultPercentPerPeriod);

		EscalateQuotaFaultPercentPerPeriod = QuotaFaultPercentPerPeriod;
		DescalateQuotaFaultPercentPerPeriod = QuotaFaultPercentPerPeriod;
	}


	if (EscalateQuotaTimePeriod > 0)
	{
		AllTimePeriods.AddUnique(EscalateQuotaTimePeriod);
	}

	HighestTimePeriod = FMath::Max(AllTimePeriods);
}

void FNetFaultState::ValidateConfigInternal()
{
	using EValidateTime = FEscalationState::EValidateTime;

	Super::ValidateConfigInternal();

	ValidateTimePeriod(EscalateQuotaTimePeriod, TEXT("EscalateQuotaTimePeriod"),
		((EscalateQuotaFaultsPerPeriod > 0 || EscalateQuotaFaultPercentPerPeriod > 0) ? EValidateTime::MustBeSet : EValidateTime::Optional));
}


bool FNetFaultState::HasHitAnyQuota(FHasHitAnyQuotaParms Parms) const
{
	using namespace UE::Net;

	bool bReturnVal = false;
	const uint8 NetworkCorruptionCategory = ToInt(ENetFaultCounterCategory::NetworkCorruption);

	if (Parms.RegisteredCounters.Num() > NetworkCorruptionCategory)
	{
		const TArrayView<int32>& NetCorruptionCounters = Parms.RegisteredCounters[NetworkCorruptionCategory];

		if (NetCorruptionCounters.Num() > 0 && (EscalateQuotaFaultsPerPeriod > 0 || EscalateQuotaFaultPercentPerPeriod > 0))
		{
			const TArrayView<FEscalationCounter>& SecondCounters = Parms.SecondCounters;
			const TArrayView<FEscalationCounter>& FrameCounters = Parms.FrameCounters;
			const TArrayView<FEscalationCounter>& CurHistory = Parms.PerPeriodHistory[EscalateQuotaTimePeriod - 1];
			int32 HighestFaultCount = 0;
			const uint8 PacketCountIdx = ToInt(ENetFaultCounters::PacketCount);
			const int32 FullPacketCount = CurHistory[PacketCountIdx].Counter + SecondCounters[PacketCountIdx].Counter +
											FrameCounters[PacketCountIdx].Counter;

			for (const int32 CounterIdx : NetCorruptionCounters)
			{
				const int32 FullFaultCount = CurHistory[CounterIdx].Counter + SecondCounters[CounterIdx].Counter + FrameCounters[CounterIdx].Counter;

				HighestFaultCount = FMath::Max(FullFaultCount, HighestFaultCount);
			}

			const bool bEscalateQuota = Parms.QuotaType == EQuotaType::EscalateQuota;
			const int16 QuotaFaultsPerPeriod = (bEscalateQuota ? EscalateQuotaFaultsPerPeriod : DescalateQuotaFaultsPerPeriod);
			const int16 QuotaFaultPercentPerPeriod = (bEscalateQuota ? EscalateQuotaFaultPercentPerPeriod : DescalateQuotaFaultPercentPerPeriod);

			if (QuotaFaultsPerPeriod > 0)
			{
				bReturnVal = HighestFaultCount >= QuotaFaultsPerPeriod;
			}

			if (QuotaFaultPercentPerPeriod > 0)
			{
				const double HighestFaultPercentage = (FullPacketCount > 0 ? (double)HighestFaultCount / (double)FullPacketCount : 0.0) * 100.0;

				bReturnVal = bReturnVal || (HighestFaultPercentage > QuotaFaultPercentPerPeriod);
			}

#if !UE_BUILD_SHIPPING
			if (GNetFaultRecoveryLogQuotaChecks)
			{
				const double HighestFaultPercentage = (FullPacketCount > 0 ? (double)HighestFaultCount / (double)FullPacketCount : 0.0) * 100.0;

				UE_LOGF(LogNetCore, Log, "FNetFaultState::HasHitAnyQuota: bReturnVal: %i, bEscalateQuota: %i, FullPacketCount: %i, "
					"HighestFaultCount: %i, QuotaFaultsPerPeriod: %i, QuotaFaultPercentPerPeriod: %i, HighestFaultPercentage: %f",
					(int32)bReturnVal, (int32)bEscalateQuota, FullPacketCount, HighestFaultCount, QuotaFaultsPerPeriod,
					QuotaFaultPercentPerPeriod, HighestFaultPercentage);
			}
#endif
		}
	}

	return bReturnVal;
}


namespace UE
{
namespace Net
{

EHandleNetResult FNetConnectionFaultRecoveryBase::HandleNetResult(FNetCloseResult&& InResult)
{
	return FaultManager.HandleNetResult(static_cast<FNetResult&&>(MoveTemp(InResult)));
}

EHandleNetResult FNetConnectionFaultRecoveryBase::NotifyHandledFault(FNetResult&& InResult)
{
	EHandleNetResult ReturnVal = EHandleNetResult::Handled;

	if (NetFaultEscalationManager.IsValid() && NetFaultEscalationManager->IsDormant())
	{
		TrackedFaults.Reset();
		TrackedFaultEnumHashes.Empty();
	}

	const uint32 ResultHash = GetTypeHash(InResult);

	if (!TrackedFaultEnumHashes.Contains(ResultHash))
	{
		AddToChainResultPtr(TrackedFaults, MoveTemp(InResult));
		TrackedFaultEnumHashes.Add(ResultHash);
	}

	if (NetFaultEscalationManager.IsValid())
	{
		NetFaultEscalationManager->CheckQuotas();

		if (bDisconnected)
		{
			ReturnVal = EHandleNetResult::Closed;
		}
	}

	return ReturnVal;
}

int32 FNetConnectionFaultRecoveryBase::AddNewCounter(int32 Count/*=1*/)
{
	int32 ReturnVal = INDEX_NONE;

	if (NetFaultEscalationManager.IsValid())
	{
		ReturnVal = NetFaultEscalationManager->AddNewCounter(Count);
	}
	else
	{
		ReturnVal = LastCounterIndex;

		LastCounterIndex += Count;
	}

	return ReturnVal;
}

FEscalationCounter& FNetConnectionFaultRecoveryBase::GetFrameCounter(int32 CounterIndex)
{
	if (!NetFaultEscalationManager.IsValid())
	{
		InitEscalationManager();
	}

	return NetFaultEscalationManager->GetFrameCounter(CounterIndex);
}

void FNetConnectionFaultRecoveryBase::RegisterCounterCategory(ENetFaultCounterCategory Category, int32 CounterIndex)
{
	const uint8 CastedCategory = ToInt(Category);

	if (NetFaultEscalationManager.IsValid())
	{
		NetFaultEscalationManager->RegisterCounterCategory(CastedCategory, CounterIndex);
	}
	else
	{
		PendingCategories.AddUnique({CastedCategory, CounterIndex});
	}
}

}
}

