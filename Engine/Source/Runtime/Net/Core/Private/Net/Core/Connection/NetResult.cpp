// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/Core/Connection/NetResult.h"
#include "Misc/StringBuilder.h"


namespace UE
{
namespace Net
{

/**
 * FNetResult
 */

FString FNetResult::DynamicToString(ENetResultString ConversionType/*=ENetResultString::WithoutChain*/) const
{
	using namespace UE::Net::Private;

	FString ReturnVal;
	const bool bIncludeChain = ConversionType == ENetResultString::WithChain;
	const UEnum* EnumObj = ResultEnumObj.Get();

	if (EnumObj != nullptr)
	{
		for (FConstIterator It(*this); It; ++It)
		{
			FString ResultLex = EnumObj->GetNameStringByValue(It->Result);

			if (ConversionType == ENetResultString::ResultEnumOnly)
			{
				ReturnVal = ResultLex;
			}
			else
			{
				NetResultToString(ReturnVal, *ResultLex, It->ErrorContext, ConversionType);
			}

			if (!bIncludeChain)
			{
				break;
			}
		}
	}
	else
	{
		ReturnVal = TEXT("BadEnum");
	}

	return ReturnVal;
}

namespace Private
{

void NetResultToString(FString& OutResultStr, const TCHAR* LexResult, const FString& ErrorContext, ENetResultString ConversionType)
{
	TStringBuilder<2048> CurResultStr;
	const bool bIncludeChain = ConversionType == ENetResultString::WithChain;

	if (bIncludeChain)
	{
		if (!OutResultStr.IsEmpty())
		{
			CurResultStr.Append(TEXT(", "));
		}

		CurResultStr.AppendChar(TEXT('('));
	}

	CurResultStr.Append(TEXT("Result="));
	CurResultStr.Append(LexResult);
	CurResultStr.Append(TEXT(", ErrorContext=\""));
	CurResultStr.Append(ToCStr(ErrorContext));
	CurResultStr.AppendChar(TEXT('\"'));

	if (bIncludeChain)
	{
		CurResultStr.AppendChar(TEXT(')'));
	}

	OutResultStr += CurResultStr.ToString();
}

}
}
}
