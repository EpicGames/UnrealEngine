// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/Core/Connection/NetResultManager.h"


namespace UE
{
namespace Net
{

/**
 * FNetResultManager
 */

void FNetResultManager::AddResultHandler(TUniquePtr<FNetResultHandler>&& InResultHandler,
											EAddResultHandlerPos Position/*=EAddResultHandlerPos::Last*/)
{
	TUniquePtr<FNetResultHandler>& NewHandler = OwnedResultHandlers.Add_GetRef(MoveTemp(InResultHandler));
	const int32 InsertPos = (Position == EAddResultHandlerPos::First ? 0 : ResultHandlers.Num());

	ResultHandlers.Insert(NewHandler.Get(), InsertPos);

	NewHandler->ResultManager = this;
	NewHandler->Init();
}

void FNetResultManager::AddResultHandlerPtr(FNetResultHandler* InResultHandler, EAddResultHandlerPos Position/*=EAddResultHandlerPos::Last*/)
{
	const int32 InsertPos = (Position == EAddResultHandlerPos::First ? 0 : ResultHandlers.Num());

	ResultHandlers.Insert(InResultHandler, InsertPos);

	InResultHandler->ResultManager = this;
	InResultHandler->Init();
}

EHandleNetResult FNetResultManager::HandleNetResult(FNetResult&& InResult)
{
	for (FNetResultHandler* CurHandler : ResultHandlers)
	{
		EHandleNetResult CurResult = CurHandler->HandleNetResult(MoveTemp(InResult));

		if (CurResult != EHandleNetResult::NotHandled)
		{
			return CurResult;
		}
	}

	if (UnhandledResultCallback)
	{
		return UnhandledResultCallback(MoveTemp(InResult));
	}

	return EHandleNetResult::NotHandled;
}

void FNetResultManager::SetUnhandledResultCallback(FUnhandledResultFunc InCallback)
{
	UnhandledResultCallback = MoveTemp(InCallback);
}

}
}
