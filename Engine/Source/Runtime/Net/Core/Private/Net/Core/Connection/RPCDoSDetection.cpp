// Copyright Epic Games, Inc. All Rights Reserved.

// Includes

#include "Net/Core/Connection/RPCDoSDetection.h"
#include "Net/Core/Connection/RPCDoSDetectionConfig.h"
#include "Net/Core/Misc/NetCoreLog.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RPCDoSDetection)


/**
 * RPC DoS Detection
 *
 * DoS attacks against game servers through expensive or spammed server RPC calls are relatively rare compared to DDoS attacks,
 * and the players responsible much easier to identify due to requiring a full post-handshake connection to the game server
 * (which means that their IP address has been verified, unlike DDoS attacks).
 *
 * There are many cases where there can be false positives during normal or competitive gameplay which result in RPC spam,
 * so RPC DoS Detection should lean towards being less strict - with analytics audited to watch for false positives,
 * and performance optimization of expensive RPC's used to get rid of those false positives (with the bonus of optimizing server perf).
 *
 * The default configuration of RPC DoS states should be loose enough to avoid blocking of RPC's or kicking players,
 * outside of extreme circumstances - while still protecting servers from expensive RPC DoS attacks.
 *
 *
 * Implementation
 *
 * RPC DoS Detection is implemented in a similar but improved way to the DDoS Detection, using thresholds with escalating states of severity.
 *
 * RPC DoS attacks are detected by setting configurable thresholds for the number of RPC's received and time spent executing them,
 * per-frame or per configurable time period (e.g. over 1 second, 4 seconds etc.).
 *
 * To minimize false positives and be resilient to temporary bursts of RPC's, several escalating states of different thresholds are used,
 * first being based on using the same counter/timer for all RPC's, then more expensive tracking of individual RPC counts/execution-time,
 * eventually escalating into blocking particularly spammy or expensive RPC's, and in rare/extreme cases kicking the player.
 *
 * Once tracking of RPC's is enabled, all RPC's above a certain timing threshold will be collected for analytics,
 * which makes the analytics useful for tracking general worst-case RPC performance.
 *
 * Importantly, RPC timing is approximate (using cached 'close enough' time values), so while it will give a good indication of performance,
 * it's not a substitute for proper profiling - it is still useful for analytics and performance optimization though.
 */


/**
 * CVars
 */

TAutoConsoleVariable<FString> CVarRPCDoSDetectionOverride(
	TEXT("net.RPCDoSDetectionOverride"), TEXT(""),
	TEXT("Overrides whether or not RPC DoS Detection is enabled per-NetDriver. 0 = disabled, 1 = enabled. ")
	TEXT("Example: net.RPCDoSDetectionOverride=GameNetDriver=1,BeaconNetDriver=0"));

int32 GAllowRPCDoSDetectionBlocking = 1;

FAutoConsoleVariableRef CVarAllowRPCDoSDetectionBlocking(
	TEXT("net.AllowRPCDoSDetectionBlocking"),
	GAllowRPCDoSDetectionBlocking,
	TEXT("Overrides whether or not RPC DoS Detection RPC blocking is allowed. 0 = disabled, 1 = enabled."));

TAutoConsoleVariable<int32> CVarAllowRPCDoSDetectionKicking(
	TEXT("net.AllowRPCDoSDetectionKicking"), 1,
	TEXT("Overrides whether or not RPC DoS Detection kicking is enabled. 0 = disabled, 1 = enabled."));

int32 GMaxBlockedRPCLogs = 16;

FAutoConsoleVariableRef CVarMaxBlockedRPCLogs(
	TEXT("net.MaxBlockedRPCLogs"),
	GMaxBlockedRPCLogs,
	TEXT("The maximum number of individual blocked RPC log messages per tick, before switching to aggregated per-tick logging."));

TAutoConsoleVariable<FString> CVarRPCDoSForcedRPCTracking(
	TEXT("net.RPCDoSForcedRPCTracking"), TEXT(""),
	TEXT("Sets a single RPC that, when encountered, forcibly enables RPC tracking (limited to one RPC for performance). ")
	TEXT("Can also specify a random chance, between 0.0 and 1.0, for when encountering the RPC enables tracking, ")
	TEXT("and a length of time for leaving tracking enabled (disables the next tick, otherwise).")
	TEXT("Example (50% chance for 10 seconds): net.RPCDoSForcedRPCTracking=ServerAdmin,0.5,10"));

#if RPC_DOS_SCOPE_DEBUG
namespace UE::Net
{
	int32 GRPCDoSScopeDebugging = 0;

	FAutoConsoleVariableRef CVarRPCDoSScopeDebugging(
		TEXT("net.RPCDoSScopeDebugging"),
		GRPCDoSScopeDebugging,
		TEXT("Sets whether or not debugging/ensures for RPC DoS Tick/Packet scopes should be enabled."));
}
#endif


/**
 * ERPCDoSEscalateReason
 */

/**
 * Convert ERPCDoSEscalateReason enum values, to a string.
 *
 * @param Reason	The enum value to convert.
 * @return			The string name for the enum value.
 */
const TCHAR* LexToString(ERPCDoSEscalateReason Reason)
{
	switch (Reason)
	{
	case ERPCDoSEscalateReason::CountLimit:
		return TEXT("CountLimit");

	case ERPCDoSEscalateReason::TimeLimit:
		return TEXT("TimeLimit");

	case ERPCDoSEscalateReason::AutoEscalate:
		return TEXT("AutoEscalate");

	case ERPCDoSEscalateReason::Deescalate:
		return TEXT("Deescalate");

	default:
		return TEXT("Unknown");
	}
};


/**
 * FRPCDoSState
 */

void FRPCDoSState::ApplyImpliedValues()
{
	EscalateQuotaTimePeriod = (EscalateQuotaRPCsPerPeriod != -1 || EscalateTimeQuotaMSPerPeriod != -1) ? EscalateQuotaTimePeriod : -1;
	RPCRepeatLimitTimePeriod = (RPCRepeatLimitPerPeriod != -1 || RPCRepeatLimitMSPerPeriod != -1) ? RPCRepeatLimitTimePeriod : -1;

	bTrackRecentRPCs = bTrackRecentRPCs || (RPCRepeatLimitPerPeriod != -1 && RPCRepeatLimitTimePeriod != -1);
	EscalateTimeQuotaSecsPerFrame = (EscalateTimeQuotaMSPerFrame != -1 ? (EscalateTimeQuotaMSPerFrame / 1000.0) : 0.0);
	EscalateTimeQuotaSecsPerPeriod = (EscalateTimeQuotaMSPerPeriod != -1 ? (EscalateTimeQuotaMSPerPeriod / 1000.0) : 0.0);
	RPCRepeatLimitSecsPerPeriod = (RPCRepeatLimitMSPerPeriod != -1 ? (RPCRepeatLimitMSPerPeriod / 1000.0) : 0.0);
	EscalationTimeToleranceSeconds = (EscalationTimeToleranceMS != -1 ? (EscalationTimeToleranceMS / 1000.0) : 0.0);
}

bool FRPCDoSState::HasHitQuota_Count(const FRPCDoSCounters(&PerPeriodHistory)[16], FRPCDoSCounters& InFrameCounter) const
{
	bool bReturnVal = EscalateQuotaRPCsPerFrame > 0 && InFrameCounter.RPCCounter >= EscalateQuotaRPCsPerFrame;

#if RPC_QUOTA_DEBUG
	UE_CLOGF(bReturnVal, LogNetCore, Log, "HasHitQuota_Count: Hit Frame Quota: RPCCounter: %i, Limit: %i",
			InFrameCounter.RPCCounter, EscalateQuotaRPCsPerFrame);
#endif

	if (EscalateQuotaRPCsPerPeriod > 0 && !bReturnVal)
	{
		const FRPCDoSCounters& PeriodCounter = PerPeriodHistory[EscalateQuotaTimePeriod - 1];

		bReturnVal = PeriodCounter.RPCCounter + InFrameCounter.RPCCounter >= EscalateQuotaRPCsPerPeriod;

#if RPC_QUOTA_DEBUG
		UE_CLOGF(bReturnVal, LogNetCore, Log, "HasHitQuota_Count: Hit Period Quota: RPCsPerPeriodCounter: %i, Limit: %i",
				(PeriodCounter.RPCCounter + InFrameCounter.RPCCounter), EscalateQuotaRPCsPerPeriod);
#endif
	}

	return bReturnVal;
}

bool FRPCDoSState::HasHitQuota_Time(const FRPCDoSCounters(&PerPeriodHistory)[16], FRPCDoSCounters& InFrameCounter) const
{
	bool bReturnVal = EscalateTimeQuotaMSPerFrame > 0 && InFrameCounter.AccumRPCTime >= EscalateTimeQuotaSecsPerFrame;

#if RPC_QUOTA_DEBUG
	bool bDebugReturnVal = EscalateTimeQuotaMSPerFrame > 0 && InFrameCounter.DebugAccumRPCTime >= EscalateTimeQuotaSecsPerFrame;

	UE_CLOGF(bReturnVal, LogNetCore, Log,
			"HasHitQuota_Time: Hit Frame Quota: AccumRPCTime: %f, DebugAccumRPCTime: %f, Limit: %f (%i ms)",
			InFrameCounter.AccumRPCTime, InFrameCounter.DebugAccumRPCTime, EscalateTimeQuotaSecsPerFrame,
			EscalateTimeQuotaMSPerFrame);

	UE_CLOGF(bReturnVal ^ bDebugReturnVal, LogNet, Warning,
			"HasHitQuota_Time: Approximate vs Debug Quota mismatch: AccumRPCTime: %f, DebugAccumRPCTime: %f, Limit: %f (%i ms)",
			InFrameCounter.AccumRPCTime, InFrameCounter.DebugAccumRPCTime, EscalateTimeQuotaSecsPerFrame,
			EscalateTimeQuotaMSPerFrame);
#endif

	if (EscalateTimeQuotaMSPerPeriod > 0 && !bReturnVal)
	{
		const FRPCDoSCounters& PeriodCounter = PerPeriodHistory[EscalateQuotaTimePeriod - 1];

		bReturnVal = (PeriodCounter.AccumRPCTime + InFrameCounter.AccumRPCTime) >= EscalateTimeQuotaSecsPerPeriod;

#if RPC_QUOTA_DEBUG
		bDebugReturnVal = (PeriodCounter.DebugAccumRPCTime + InFrameCounter.DebugAccumRPCTime) >=
							EscalateTimeQuotaSecsPerPeriod;

		UE_CLOGF(bReturnVal, LogNetCore, Log,
				"HasHitQuota_Time: Hit Period Quota: AccumPerPeriodRPCTime: %f, DebugAccumPerPeriodRPCTime: %f, Limit: %f (%i ms)",
				(PeriodCounter.AccumRPCTime + InFrameCounter.AccumRPCTime),
				(PeriodCounter.DebugAccumRPCTime + InFrameCounter.DebugAccumRPCTime),
				EscalateTimeQuotaSecsPerPeriod, EscalateTimeQuotaMSPerPeriod);

		UE_CLOGF(bReturnVal ^ bDebugReturnVal, LogNetCore, Warning,
				"HasHitQuota_Time: Approximate vs Debug Period Quota mismatch: " "AccumPerPeriodRPCTime: %f, DebugAccumPerPeriodRPCTime: %f, Limit: %f (%i ms)",
				(PeriodCounter.AccumRPCTime + InFrameCounter.AccumRPCTime),
				(PeriodCounter.DebugAccumRPCTime + InFrameCounter.DebugAccumRPCTime),
				EscalateTimeQuotaSecsPerPeriod, EscalateTimeQuotaMSPerPeriod);
#endif
	}

	return bReturnVal;
}


/**
 * FRPCDoSStateConfig
 */

void FRPCDoSStateConfig::ApplyImpliedValues()
{
	FRPCDoSState::ApplyImpliedValues();

	if (EscalateQuotaTimePeriod > 0)
	{
		AllTimePeriods.AddUnique(EscalateQuotaTimePeriod);
	}

	if (RPCRepeatLimitTimePeriod > 0)
	{
		AllTimePeriods.AddUnique(RPCRepeatLimitTimePeriod);
	}

	HighestTimePeriod = FMath::Max(AllTimePeriods);
}

bool FRPCDoSStateConfig::LoadStructConfig(const TCHAR* SectionName, const TCHAR* InFilename/*=nullptr*/)
{
	bool bSuccess = false;
	FString ConfigFile = (InFilename != nullptr ? InFilename : GEngineIni);
	TArray<FString> StructVars;
	bool bFoundSection = GConfig->GetSection(SectionName, StructVars, ConfigFile);

	if (bFoundSection)
	{
		bSuccess = true;

		for (const FString& CurVar : StructVars)
		{
			FString Var;
			FString Value;

			if (CurVar.Split(TEXT("="), &Var, &Value))
			{
				FProperty* CurProp = FRPCDoSStateConfig::StaticStruct()->FindPropertyByName(*Var);

				if (CurProp != nullptr)
				{
					if (CurProp->HasAllPropertyFlags(CPF_Config))
					{
						CurProp->ImportText_InContainer(*Value, this, nullptr, 0);
					}
					else
					{
						UE_LOGF(LogNetCore, Error, "FRPCDoSStateConfig: Ini property '%ls' is not a config property.", *Var);
					}
				}
				else
				{
					UE_LOGF(LogNetCore, Error, "FRPCDoSStateConfig: Ini property '%ls' not found.", *Var);
				}
			}
		}

		ApplyImpliedValues();
	}

	return bSuccess;
}

void FRPCDoSStateConfig::ValidateConfig()
{
	auto ValidateTimePeriod = [](int8& Value, const TCHAR* PropertyName, bool bMustBeSet)
		{
			int32 BaseValue = bMustBeSet ? 1 : 0;

			if (Value < BaseValue || Value > 16)
			{
				UE_LOGF(LogNetCore, Warning, "FRPCDoSStateConfig: %ls '%i' must be between %i and 16. Clamping.",
						PropertyName, Value, BaseValue);

				Value = static_cast<int8>(FMath::Clamp((int32)Value, BaseValue, 16));
			}
		};

	ValidateTimePeriod(EscalateQuotaTimePeriod, TEXT("EscalateQuotaTimePeriod"),
						(EscalateQuotaRPCsPerPeriod > 0 || EscalateTimeQuotaMSPerPeriod > 0));

	ValidateTimePeriod(RPCRepeatLimitTimePeriod, TEXT("RPCRepeatLimitTimePeriod"),
						(RPCRepeatLimitPerPeriod > 0 || RPCRepeatLimitMSPerPeriod > 0));

	if (AutoEscalateTime > 0 && AutoEscalateTime < CooloffTime)
	{
		UE_LOGF(LogNetCore, Warning, "FRPCDoSStateConfig: AutoEscalateTime must be larger than CooloffTime.");
	}
}

int8 FRPCDoSStateConfig::GetHighestTimePeriod() const
{
	return HighestTimePeriod;
}

const TArray<int8>& FRPCDoSStateConfig::GetAllTimePeriods() const
{
	return AllTimePeriods;
}

void FRPCDoSStateConfig::ApplyState(FRPCDoSState& Target)
{
	Target.bLogEscalate						= bLogEscalate;
	Target.bSendEscalateAnalytics			= bSendEscalateAnalytics;
	Target.bKickPlayer						= bKickPlayer;
	Target.bTrackRecentRPCs					= bTrackRecentRPCs;
	Target.EscalateQuotaRPCsPerFrame		= EscalateQuotaRPCsPerFrame;
	Target.EscalateTimeQuotaMSPerFrame		= EscalateTimeQuotaMSPerFrame;
	Target.EscalateQuotaRPCsPerPeriod		= EscalateQuotaRPCsPerPeriod;
	Target.EscalateTimeQuotaMSPerPeriod		= EscalateTimeQuotaMSPerPeriod;
	Target.EscalateQuotaTimePeriod			= EscalateQuotaTimePeriod;
	Target.RPCRepeatLimitPerPeriod			= RPCRepeatLimitPerPeriod;
	Target.RPCRepeatLimitMSPerPeriod		= RPCRepeatLimitMSPerPeriod;
	Target.RPCRepeatLimitTimePeriod			= RPCRepeatLimitTimePeriod;
	Target.CooloffTime						= CooloffTime;
	Target.AutoEscalateTime					= AutoEscalateTime;

	Target.ApplyImpliedValues();
}


/**
 * RPC DoS Detection
 */

void FRPCDoSDetection::Init(FName NetDriverName, TSharedPtr<FNetAnalyticsAggregator>& AnalyticsAggregator,
							TSharedPtr<FRPCDoSAnalyticsSender> AnalyticsSender, FGetRPCDoSAddress&& InAddressFunc,
							FGetRPCDoSPlayerUID&& InPlayerUIDFunc, FRPCDoSKickPlayer&& InKickPlayerFunc)
{
	AddressFunc = MoveTemp(InAddressFunc);
	PlayerUIDFunc = MoveTemp(InPlayerUIDFunc);
	KickPlayerFunc = MoveTemp(InKickPlayerFunc);

	InitConfig(NetDriverName);

	if (bRPCDoSDetection && bRPCDoSAnalytics && AnalyticsAggregator.IsValid())
	{
		RPCDoSAnalyticsData = REGISTER_NET_ANALYTICS(AnalyticsAggregator, FRPCDoSAnalyticsData, TEXT("Core.ServerRPCDoS"));

		if (RPCDoSAnalyticsData.IsValid())
		{
			RPCDoSAnalyticsData->SetAnalyticsSender(AnalyticsSender);
		}
	}

	double CurSeconds = FPlatformTime::Seconds();

	InitState(CurSeconds);

	if (bRPCDoSAnalytics)
	{
		FPlatformTime::AutoUpdateGameThreadCPUTime(0.25);
	}

	// NetConnection's are created mid-TickDispatch, when this is initialized.
	PreTickDispatch(CurSeconds);
}


void FRPCDoSDetection::InitConfig(FName NetDriverName)
{
	URPCDoSDetectionConfig* CurConfigObj = URPCDoSDetectionConfig::Get(NetDriverName);

	if (CurConfigObj != nullptr)
	{
		bRPCDoSDetection = CurConfigObj->bRPCDoSDetection;
		bRPCDoSAnalytics = CurConfigObj->bRPCDoSAnalytics;
		HitchTimeQuotaMS = CurConfigObj->HitchTimeQuotaMS;
		HitchSuspendDetectionTimeMS = CurConfigObj->HitchSuspendDetectionTimeMS;

		if (NextTimeQuotaCheck == 0.0 && CurConfigObj->InitialConnectToleranceMS > 0)
		{
			NextTimeQuotaCheck = FPlatformTime::Seconds() + (CurConfigObj->InitialConnectToleranceMS / 1000.0);
		}

		RPCBlockAllowList = CurConfigObj->RPCBlockAllowlist;
	}


	FString RPCDoSDetectionOverride = CVarRPCDoSDetectionOverride.GetValueOnAnyThread();

	if (RPCDoSDetectionOverride.Contains(TEXT(",")) || RPCDoSDetectionOverride.Contains(TEXT("=")))
	{
		TArray<FString> NetDriverOverrides;

		RPCDoSDetectionOverride.ParseIntoArray(NetDriverOverrides, TEXT(","));

		for (const FString& CurOverride : NetDriverOverrides)
		{
			TArray<FString> KeyVal;

			CurOverride.ParseIntoArray(KeyVal, TEXT("="));

			if (KeyVal.Num() > 1 && KeyVal[0] == NetDriverName.ToString() && !KeyVal[1].IsEmpty())
			{
				bRPCDoSDetection = (FCString::Atoi(*KeyVal[1]) == 0) ? false : true;
			}
		}
	}
	else if (!RPCDoSDetectionOverride.IsEmpty())
	{
		const bool bOverrideVal = (FCString::Atoi(*RPCDoSDetectionOverride) == 0) ? false : true;

		if (bOverrideVal)
		{
			// Only allow a global enable for GameNetDriver
			if (NetDriverName == NAME_GameNetDriver)
			{
				bRPCDoSDetection = bOverrideVal;
			}
		}
		else
		{
			bRPCDoSDetection = bOverrideVal;
		}
	}


	DetectionSeverity.Empty();

	if (bRPCDoSDetection)
	{
		int32 HighestHistoryRequirment = 0;

		if (CurConfigObj != nullptr)
		{
			TArray<FString>& SeverityCategories = CurConfigObj->DetectionSeverity;
			const TCHAR* RPCDoSSectionName = URPCDoSDetectionConfig::GetConfigSectionName();

			for (const FString& CurCategory : SeverityCategories)
			{
				FString CurSection = FString(RPCDoSSectionName) + TEXT(".") + CurCategory;

				if (GConfig->DoesSectionExist(*CurSection, GEngineIni))
				{
					FRPCDoSStateConfig& CurState = DetectionSeverity.AddDefaulted_GetRef();

					CurState.SeverityCategory = CurCategory;

					bool bSuccess = CurState.LoadStructConfig(*CurSection, *GEngineIni);

					if (bSuccess)
					{
						const int8 HighestTimePeriod = CurState.GetHighestTimePeriod();

						HighestHistoryRequirment = FMath::Max(HighestHistoryRequirment, (int32)(CurState.CooloffTime + HighestTimePeriod));
						MaxRPCTrackingPeriod = FMath::Max(MaxRPCTrackingPeriod, HighestTimePeriod);
					}
					else
					{
						UE_LOGF(LogNetCore, Warning, "RPC DoS Detection failed to load ini section: %ls", *CurSection);
					}
				}
				else
				{
					UE_LOGF(LogNetCore, Warning, "RPC DoS detection could not find ini section: %ls", *CurSection);
				}
			}
		}

		if (DetectionSeverity.Num() > 0)
		{
			DetectionSeverity[ActiveState].ApplyState(*this);

			CounterPerSecHistory.SetNum(HighestHistoryRequirment);
		}
		else
		{
			UE_LOGF(LogNetCore, Warning, "RPC DoS detection enabled, but no DetectionSeverity states specified! Disabling.");

			bRPCDoSDetection = false;
		}
	}


	if (bRPCDoSDetection)
	{
		if (ActiveRPCTracking.Num() == 0)
		{
			ActiveRPCTracking.Empty(DefaultActiveRPCTrackingSize);
		}

		FString RPCTrackingStr = CVarRPCDoSForcedRPCTracking.GetValueOnAnyThread();

		if (!RPCTrackingStr.IsEmpty())
		{
			TArray<FString> RPCTrackingParms;

			RPCTrackingStr.ParseIntoArray(RPCTrackingParms, TEXT(","));

			ForcedRPCTracking = FName(*RPCTrackingParms[0]);
			ForcedRPCTrackingChance = RPCTrackingParms.Num() > 1 ? FCString::Atof(*RPCTrackingParms[1]) : 1.0;
			ForcedRPCTrackingLength = RPCTrackingParms.Num() > 2 ? FCString::Atof(*RPCTrackingParms[2]) : 0.0;
		}
	}
}

void FRPCDoSDetection::UpdateSeverity_Private(ERPCDoSSeverityUpdate Update, ERPCDoSEscalateReason Reason)
{
	bool bEscalate = Update == ERPCDoSSeverityUpdate::Escalate || Update == ERPCDoSSeverityUpdate::AutoEscalate;
	int32 NewStateIdx = FMath::Clamp(ActiveState + (bEscalate ? 1 : -1), 0, DetectionSeverity.Num()-1);

	// If kicking is disabled, and the new state is the kick state, exclude that state (otherwise RPC DoS Detection will become stuck)
	const bool bPreventKick = CVarAllowRPCDoSDetectionKicking.GetValueOnAnyThread() == 0;

	if (DetectionSeverity[NewStateIdx].bKickPlayer && bPreventKick)
	{
		NewStateIdx = ActiveState;
	}

	if (NewStateIdx != ActiveState)
	{
		double CurTime = FPlatformTime::Seconds();

		// Cache these values - Init code is too early, NotifyClose may be too late
		CachePlayerAddress();
		CachePlayerUID();

		FTickScope& TickScope = GetTickScope();

		if (bEscalate)
		{
			LastMetEscalationConditions = CurTime;
		}
		else
		{
			// De-escalate to the lowest state which hasn't cooled off, and estimate the timestamp for when the cooloff was last reset
			// (due to estimating, there is slight inaccuracy in the cooloff time)
			bool bCooloffReached = true;

			while (bCooloffReached && NewStateIdx > 0)
			{
				FRPCDoSStateConfig& PrevState = DetectionSeverity[NewStateIdx-1];
				FRPCDoSStateConfig& CurState = DetectionSeverity[NewStateIdx];
				int32 CurStateCooloffTime = CurState.CooloffTime;
				const TArray<int8>& CurStateTimePeriods = CurState.GetAllTimePeriods();
				FRPCDoSCounters CurPerPeriodHistory[16] = {};

				check(CounterPerSecHistory.Num() >= CurStateCooloffTime);

				// Count backwards through every second of CounterPerSecHistory, whose starting index wraps around like a circular buffer
				for (int32 SecondsDelta=0; SecondsDelta<CurStateCooloffTime; SecondsDelta++)
				{
					int32 StartIdx = LastCounterPerSecHistoryIdx - SecondsDelta;

					StartIdx = (StartIdx < 0 ? CounterPerSecHistory.Num() + StartIdx : StartIdx);

					check(StartIdx >= 0 && StartIdx < CounterPerSecHistory.Num());

					RecalculatePeriodHistory(CurStateTimePeriods, CurPerPeriodHistory, StartIdx);

					// Determine if any time period quota's for the current SecondsDelta-offset CounterPerSecHistory were breached
					if (PrevState.HasHitQuota_Count(CurPerPeriodHistory, TickScope.FrameCounter) ||
						PrevState.HasHitQuota_Time(CurPerPeriodHistory, TickScope.FrameCounter))
					{
						// The state we're transitioning down into, would have last had its cooloff reset around this time
						LastMetEscalationConditions = CurTime - (double)SecondsDelta;

						bCooloffReached = false;
						break;
					}
				}

				if (bCooloffReached)
				{
					NewStateIdx--;
				}
			}
		}


		FRPCDoSStateConfig& OldState = DetectionSeverity[ActiveState];

		while (true)
		{
			FRPCDoSStateConfig& CurState = DetectionSeverity[NewStateIdx];

			ActiveState = static_cast<int8>(NewStateIdx);

			CurState.ApplyState(*this);

			const TArray<int8>& NeededTimePeriods = DetectionSeverity[ActiveState].GetAllTimePeriods();

			RecalculatePeriodHistory(NeededTimePeriods, CounterPerPeriodHistory);


			// If escalating, keep escalating until the quota checks fail
			if (bEscalate && (HasHitQuota_Count(CounterPerPeriodHistory, TickScope.FrameCounter) ||
								HasHitQuota_Time(CounterPerPeriodHistory, TickScope.FrameCounter)))
			{
				NewStateIdx = FMath::Clamp(ActiveState + 1, 0, DetectionSeverity.Num()-1);

				if (DetectionSeverity[NewStateIdx].bKickPlayer && bPreventKick)
				{
					NewStateIdx = ActiveState;
					break;
				}

				if (NewStateIdx == ActiveState)
				{
					break;
				}
			}
			else
			{
				break;
			}
		}


		FRPCDoSStateConfig& NewState = DetectionSeverity[NewStateIdx];

		if (bLogEscalate)
		{
			UE_LOGF(LogNetCore, Warning, "Updated RPC DoS detection severity for '%ls' from '%ls' to '%ls' (Reason: %ls)",
					*GetPlayerAddress(), *OldState.SeverityCategory, *NewState.SeverityCategory, LexToString(Reason));
		}


		InitState(CurTime);


		if (bEscalate)
		{
			NewState.EscalationCount++;

			if (ActiveState > WorstActiveState)
			{
				WorstActiveState = ActiveState;

				AnalyticsVars.MaxSeverityIndex = ActiveState;
				AnalyticsVars.MaxSeverityCategory = NewState.SeverityCategory;
			}

			if (ActiveState > WorstAnalyticsState && !NewState.bEscalationConfirmed)
			{
				NewState.bEscalationConfirmed = NewState.EscalationCount >= NewState.EscalationCountTolerance ||
					(NewState.EscalationTimeToleranceMS != -1 && TickScope.FrameCounter.AccumRPCTime >= NewState.EscalationTimeToleranceSeconds);

				if (NewState.bEscalationConfirmed)
				{
					TArray<FName> UntimedRPCs;
					double RPCGroupTime = 0.0;
					const int8 OldWorstState = WorstAnalyticsState;

					WorstAnalyticsState = ActiveState;

					if (bRPCDoSAnalytics && bSendEscalateAnalytics && RPCDoSAnalyticsData.IsValid() &&
						RPCDoSAnalyticsData->WorstAnalyticsState < WorstAnalyticsState)
					{
						const FString& PlayerIP = GetPlayerAddress();
						const FString& PlayerUID = GetPlayerUID();

						// Estimate the worst per second RPC execution time/count, that may have triggered this escalation, for ranking
						int32 WorstCountPerSec = 0;
						double WorstTimePerSec = 0.0;

						for (int32 SecIdx=0; SecIdx<OldState.EscalateQuotaTimePeriod; SecIdx++)
						{
							int32 PerSecHistoryIdx = LastCounterPerSecHistoryIdx - SecIdx;

							PerSecHistoryIdx = (PerSecHistoryIdx < 0 ? CounterPerSecHistory.Num() + PerSecHistoryIdx : PerSecHistoryIdx);

							WorstCountPerSec = FMath::Max(CounterPerSecHistory[PerSecHistoryIdx].RPCCounter, WorstCountPerSec);
							WorstTimePerSec = FMath::Max(CounterPerSecHistory[PerSecHistoryIdx].AccumRPCTime, WorstTimePerSec);
						}


						// If RPC tracking was enabled during this escalation, but lightweight RPC tracking has not been reset,
						// that means the RPC responsible for the escalation could not be identified,
						// and analytics should list all potential RPC's
						if (ShouldMonitorReceivedRPC() && PacketScopePrivate.IsActive())
						{
							FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();
							FLightweightRPCTracking& LightweightRPCTracking = SequentialRPCScope.LightweightRPCTracking;

							if (SequentialRPCScope.LightweightRPCTracking.Count > 0)
							{
								for (int32 LWIdx=0; LWIdx<LightweightRPCTracking.Count; LWIdx++)
								{
									UntimedRPCs.AddUnique(LightweightRPCTracking.RPC[LWIdx].Name);
								}

								UntimedRPCs.Sort(
									[](const FName& A, const FName& B)
									{
										return A.ToString() < B.ToString();
									});

								LightweightRPCTracking.Count = 0;

								FPacketScope& PacketScope = GetPacketScope();

								RPCGroupTime = FPlatformTime::Seconds() - PacketScope.ReceivedPacketStartTime;
							}
						}


						RPCDoSAnalyticsData->FireEvent_ServerRPCDoSEscalation(NewStateIdx, NewState.SeverityCategory, WorstCountPerSec,
																				WorstTimePerSec, PlayerIP, PlayerUID, UntimedRPCs,
																				RPCGroupTime);

						RPCDoSAnalyticsData->WorstAnalyticsState = WorstAnalyticsState;
					}

					if (bLogEscalate)
					{
						UE_LOGF(LogNetCore, Warning, "Updated analytics RPC DoS detection severity from '%ls' to '%ls'.",
								*DetectionSeverity[OldWorstState].SeverityCategory, *NewState.SeverityCategory);

						// Log RPC's which could not be timed individually (as they won't display in analytics)
						if (UntimedRPCs.Num() > 0)
						{
							TStringBuilder<2048> RPCGroupStr;

							for (const FName& CurRPC : UntimedRPCs)
							{
								if (RPCGroupStr.Len() > 0)
								{
									RPCGroupStr.Append(TEXT(", "));
								}

								RPCGroupStr.Append(*CurRPC.ToString());
							}

							UE_LOGF(LogNetCore, Log, "RPC DoS escalation Untimed RPC's: Time: %f, RPC's: %ls",
									RPCGroupTime, ToCStr(RPCGroupStr.ToString()))
						}
					}

					AnalyticsVars.MaxAnalyticsSeverityIndex = ActiveState;
					AnalyticsVars.MaxAnalyticsSeverityCategory = NewState.SeverityCategory;
				}
			}

			if (bKickPlayer && KickPlayerFunc)
			{
				UE_LOGF(LogNetCore, Warning, "RPC DoS detection kicking player IP '%ls', UID '%ls'.", *GetPlayerAddress(),
						*GetPlayerUID());

				KickPlayerFunc();
			}
		}
	}
}

void FRPCDoSDetection::PreTickDispatch(double TimeSeconds)
{
#if RPC_DOS_DEV_STATS
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RPCDoS_Checks);
#endif

	TickScopePrivate.SetActive(true);

	if (bRPCDoSDetection)
	{
		NextTimeQuotaCheck = FMath::Max(TimeSeconds + TimeQuotaCheckInterval, NextTimeQuotaCheck);

		if (HitchTimeQuotaMS > 0 && ReceivedPacketEndTime != 0.0)
		{
			// Timing is approximate to reduce timestamp retrieval, and packets aren't received each frame
			if (LastPreTickDispatchTime > ReceivedPacketEndTime)
			{
				ReceivedPacketEndTime = LastPreTickDispatchTime;
			}

			double EstHitchTimeMS = (TimeSeconds - ReceivedPacketEndTime) * 1000.0;

			if ((int32)EstHitchTimeMS >= HitchTimeQuotaMS)
			{
				bHitchSuspendDetection = true;
				HitchSuspendDetectionStartTime = TimeSeconds;
			}
			else if (bHitchSuspendDetection && (int32)((TimeSeconds - HitchSuspendDetectionStartTime) * 1000.0) > HitchSuspendDetectionTimeMS)
			{
				bHitchSuspendDetection = false;
			}
		}

		if (ActiveState > 0)
		{
			FTickScope& TickScope = GetTickScope();
			double ActiveStateTime = TimeSeconds - LastMetEscalationConditions;

			if (CooloffTime > 0 && ActiveStateTime > CooloffTime)
			{
				TickScope.UpdateSeverity(*this, ERPCDoSSeverityUpdate::Deescalate, ERPCDoSEscalateReason::Deescalate);
			}
			else if (AutoEscalateTime > 0 && ActiveStateTime > AutoEscalateTime)
			{
				TickScope.UpdateSeverity(*this, ERPCDoSSeverityUpdate::AutoEscalate, ERPCDoSEscalateReason::AutoEscalate);
			}
		}

		if (ForcedRPCTrackingEndTime != 0.0 && (TimeSeconds - ForcedRPCTrackingEndTime) > 0.0)
		{
			// Only disable tracking if the active state does not normally enable it
			if (!bTrackRecentRPCs)
			{
				DisableRPCTracking(TimeSeconds);
			}

			ForcedRPCTrackingEndTime = 0.0;
		}


		// NOTE: This timing is only approximate, and e.g. if there is a 10 second hitch during the previous Tick,
		//			data from those 10 seconds will be stored as 1 second. This shouldn't affect the accuracy of RPC DoS detection,
		//			however (it will be the NEXT Tick, which may contain an excess of RPC packets in the socket buffer).
		if ((TimeSeconds - LastPerSecQuotaBegin) > 1.0)
		{
			// Record the last quota
			check(CounterPerSecHistory.Num() > 0);

			LastCounterPerSecHistoryIdx++;
			LastCounterPerSecHistoryIdx = (LastCounterPerSecHistoryIdx >= CounterPerSecHistory.Num()) ? 0 : LastCounterPerSecHistoryIdx;

			CounterPerSecHistory[LastCounterPerSecHistoryIdx] = SecondCounter;

			LastPerSecQuotaBegin = TimeSeconds;


			SecondCounter.ResetRPCCounters();

			const TArray<int8>& NeededTimePeriods = DetectionSeverity[ActiveState].GetAllTimePeriods();

			RecalculatePeriodHistory(NeededTimePeriods, CounterPerPeriodHistory);


			SecondsIncrementer++;


			// After a new per second quota period, check if RPC tracking needs a cleanout (NOTE: bRPCTrackingEnabled may be false, here)
			if (ActiveRPCTracking.Num() > 0 && (TimeSeconds - LastRPCTrackingClean) > (30.0 + FMath::FRandRange(0.0, 5.0)))
			{
				ClearStaleRPCTracking(TimeSeconds);
			}

			// Recalculate RPC minimum time thresholds
			if (bRPCTrackingEnabled && AnalyticsVars.RPCTrackingAnalytics.Num() == AnalyticsVars.MaxRPCAnalytics &&
				(TimeSeconds - LastAnalyticsThresholdRecalc) > (5.0 + FMath::FRandRange(0.0, 5.0)))
			{
				LastAnalyticsThresholdRecalc = TimeSeconds;
				AnalyticsMinTimePerSecThreshold = -1.0;

				for (const TSharedPtr<FRPCAnalytics>& CurAnalytics : AnalyticsVars.RPCTrackingAnalytics)
				{
					double CurMaxTimePerSec = CurAnalytics->MaxTimePerSec;

					if (AnalyticsMinTimePerSecThreshold < 0.0 || CurMaxTimePerSec < AnalyticsMinTimePerSecThreshold)
					{
						AnalyticsMinTimePerSecThreshold = CurMaxTimePerSec;
					}
				}
			}
		}
	}

	LastPreTickDispatchTime = TimeSeconds;
}

void FRPCDoSDetection::PostSequentialRPC(EPostSequentialRPCType SequenceType, double TimeSeconds, FRPCDoSCounters* RPCCounter,
											FRPCTrackingInfo* RPCTrackingInfo)
{
	using namespace UE::Net;

	FPacketScope& PacketScope = GetPacketScope();
	FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();
	const double StartTime = SequentialRPCScope.LastReceivedRPCTimeCache == 0.0 ? PacketScope.ReceivedPacketStartTime :
								SequentialRPCScope.LastReceivedRPCTimeCache;

	RPCCounter->AccumRPCTime += (TimeSeconds - StartTime);

	if (bRPCDoSAnalytics)
	{
		TArray<TSharedPtr<FRPCAnalytics>>& RPCTrackingAnalytics = AnalyticsVars.RPCTrackingAnalytics;
		const bool bTrackingFull = RPCTrackingAnalytics.Num() == AnalyticsVars.MaxRPCAnalytics;
		const bool bAnalyticsThreshold = !bTrackingFull || RPCCounter->AccumRPCTime > AnalyticsMinTimePerSecThreshold;
		TSharedPtr<FRPCAnalytics>& RPCTrackingAnalyticsEntry = RPCTrackingInfo->RPCTrackingAnalyticsEntry;

		// Setup analytics tracking for this RPC. When tracking is full, existing analytics tracking entries need rechecking,
		// in case they were removed from the 'top x' RPCTrackingAnalytics list.
		if (bAnalyticsThreshold && (!RPCTrackingAnalyticsEntry.IsValid() || bTrackingFull))
		{
			// If the value is unique, it has been detached from the RPCTrackingAnalytics array - recreate
			if (RPCTrackingAnalyticsEntry.IsUnique())
			{
				RPCTrackingAnalyticsEntry.Reset();
			}

			if (!RPCTrackingAnalyticsEntry.IsValid())
			{
				int32 InsertIdx = INDEX_NONE;

				for (int32 AnalyticsIdx=0; AnalyticsIdx<RPCTrackingAnalytics.Num(); AnalyticsIdx++)
				{
					const TSharedPtr<FRPCAnalytics>& CurAnalytics = RPCTrackingAnalytics[AnalyticsIdx];

					if (CurAnalytics->RPCName == SequentialRPCScope.PostReceivedRPCName)
					{
						RPCTrackingAnalyticsEntry = CurAnalytics;
						break;
					}

					if (InsertIdx == INDEX_NONE && CurAnalytics->MaxTimePerSec < RPCCounter->AccumRPCTime)
					{
						InsertIdx = AnalyticsIdx;
					}
				}

				if (!RPCTrackingAnalyticsEntry.IsValid())
				{
					// If new entry is not prioritized higher than an existing entry, and tracking is full, then it can't be added
					if (InsertIdx == INDEX_NONE)
					{
						if (!bTrackingFull)
						{
							InsertIdx = RPCTrackingAnalytics.Num();
						}
					}
					else if (bTrackingFull)
					{
						const int32 EndIdx = FMath::Max(RPCTrackingAnalytics.Num() - 1, 0);

						RPCTrackingAnalytics.RemoveAt(EndIdx, EAllowShrinking::No);
					}

					if (InsertIdx != INDEX_NONE)
					{
						TSharedPtr<FRPCAnalytics> NewRPCAnalytics = MakeShared<FRPCAnalytics>();

						RPCTrackingAnalyticsEntry = RPCTrackingAnalytics.Insert_GetRef(MoveTemp(NewRPCAnalytics), InsertIdx);

						// Setup initial/persistent analytics values
						RPCTrackingAnalyticsEntry->RPCName = SequentialRPCScope.PostReceivedRPCName;
						RPCTrackingAnalyticsEntry->PlayerIP = GetPlayerAddress();
						RPCTrackingAnalyticsEntry->PlayerUID = GetPlayerUID();
					}
				}
			}
		}


		FRPCAnalytics* RawAnalyticsEntry = RPCTrackingAnalyticsEntry.Get();

		if (RawAnalyticsEntry != nullptr)
		{
			RawAnalyticsEntry->MaxCountPerSec = FMath::Max(RawAnalyticsEntry->MaxCountPerSec, RPCCounter->RPCCounter);

			if (RPCCounter->AccumRPCTime > RawAnalyticsEntry->MaxTimePerSec)
			{
				RawAnalyticsEntry->MaxTimePerSec = RPCCounter->AccumRPCTime;
				RawAnalyticsEntry->MaxTimeGameThreadCPU = static_cast<uint8>(FPlatformTime::GetThreadCPUTime().CPUTimePctRelative);
			}

			if (RPCTrackingInfo->BlockState == ERPCBlockState::Blocked)
			{
				RawAnalyticsEntry->BlockedCount += SequentialRPCScope.PostReceivedRPCBlockCount;
			}

			if (SequenceType == EPostSequentialRPCType::PostPacket && SequentialRPCScope.bReceivedPacketRPCUnique)
			{
				const double PacketProcessTime = TimeSeconds - PacketScope.ReceivedPacketStartTime;

				if (PacketProcessTime > RawAnalyticsEntry->MaxSinglePacketRPCTime)
				{
					RawAnalyticsEntry->MaxSinglePacketRPCTime = PacketProcessTime;
					RawAnalyticsEntry->SinglePacketRPCCount = SequentialRPCScope.ReceivedPacketRPCCount;
					RawAnalyticsEntry->SinglePacketGameThreadCPU = static_cast<uint8>(FPlatformTime::GetThreadCPUTime().CPUTimePctRelative);
				}
			}
		}
	}
}

void FRPCDoSDetection::PostReceivedRPCPacket(double TimeSeconds)
{
	using namespace UE::Net;

	FTickScope& TickScope = GetTickScope();
	FPacketScope& PacketScope = GetPacketScope();
	FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();
	const double PacketProcessTime = TimeSeconds - PacketScope.ReceivedPacketStartTime;

	TickScope.FrameCounter.AccumRPCTime += PacketProcessTime;

	if (SequentialRPCScope.PostReceivedRPCCounter != nullptr)
	{
#if RPC_DOS_SCOPE_DEBUG
		// If this ensure fails, there is the potential for a crash - seen very rarely, and the Tick/Packet/SequentialRPC scoping should eliminate it
		ensure(GRPCDoSScopeDebugging == 0 || (RPCTracking.Num() > 0 && ActiveRPCTracking.Num() > 0));
#endif

		if (RPCTracking.Num() > 0 && ActiveRPCTracking.Num() > 0)
		{
			PostSequentialRPC(EPostSequentialRPCType::PostPacket, TimeSeconds, SequentialRPCScope.PostReceivedRPCCounter,
								SequentialRPCScope.PostReceivedRPCTracking);

#if RPC_QUOTA_DEBUG
			PostReceivedRPCCounter->DebugAccumRPCTime += (DebugReceivedRPCEndTime - DebugReceivedRPCStartTime);
#endif
		}
	}

	TickScope.CondCheckTimeQuota(*this, TimeSeconds);

#if RPC_DOS_SCOPE_DEBUG
	ensure(GRPCDoSScopeDebugging == 0 || SequentialRPCScopePrivate.IsActive());
	ensure(GRPCDoSScopeDebugging == 0 || PacketScopePrivate.IsActive());
	ensure(GRPCDoSScopeDebugging == 0 || TickScopePrivate.IsActive());
#endif

	SequentialRPCScopePrivate.SetActive(false);
}

void FRPCDoSDetection::TrackBlockedRPC(FName RPCName)
{
	if (RPCName != CachedBlockedRPCName)
	{
		CachedBlockedRPCName = RPCName;
		CachedBlockedRPCCount = &BlockedRPCCounts.FindOrAdd(RPCName, 0);
	}

	(*CachedBlockedRPCCount)++;

	if (BlockedRPCLogCounter < GMaxBlockedRPCLogs)
	{
		BlockedRPCLogCounter++;

		UE_LOGF(LogNetCore, Log, "Blocked RPC: %ls", ToCStr(RPCName.ToString()));
	}
}

void FRPCDoSDetection::FlushBlockedRPCLogs()
{
	if (BlockedRPCLogCounter >= GMaxBlockedRPCLogs && BlockedRPCCounts.Num() > 0)
	{
		TStringBuilder<2048> AggregateStr;

		AggregateStr.Append(TEXT("Blocked RPCs: "));

		bool bFirst = true;

		for (TPair<FName, int32>& Pair : BlockedRPCCounts)
		{
			if (Pair.Value > 0)
			{
				if (!bFirst)
				{
					AggregateStr.Append(TEXT(", "));
				}

				AggregateStr.Append(ToCStr(Pair.Key.ToString()));
				AggregateStr.Appendf(TEXT(" (%i)"), Pair.Value);

				Pair.Value = 0;

				bFirst = false;
			}
		}

		UE_LOGF(LogNetCore, Log, "%ls", ToCStr(AggregateStr.ToString()));
	}

	BlockedRPCLogCounter = 0;
}

void FRPCDoSDetection::ResetBlockedRPCTracking()
{
	BlockedRPCCounts.Empty();
	CachedBlockedRPCName = NAME_None;
	CachedBlockedRPCCount = nullptr;
	BlockedRPCLogCounter = 0;
}

void FRPCDoSDetection::PostTickDispatch()
{
#if RPC_DOS_DEV_STATS
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RPCDoS_Checks);
#endif

	FTickScope& TickScope = GetTickScope();
	FRPCDoSCounters& FrameCounter = TickScope.FrameCounter;

	if (FrameCounter.RPCCounter > 0)
	{
		SecondCounter.AccumulateCounter(FrameCounter);
	}

	FlushBlockedRPCLogs();

	TickScopePrivate.SetActive(false);
}

void FRPCDoSDetection::NotifyClose()
{
	if (RPCDoSAnalyticsData.IsValid())
	{
		CachePlayerAddress();
		CachePlayerUID();

		AnalyticsVars.PlayerIP = GetPlayerAddress();
		AnalyticsVars.PlayerUID = GetPlayerUID();

		RPCDoSAnalyticsData->CommitAnalytics(AnalyticsVars);
	}
}

void FRPCDoSDetection::InitState(double TimeSeconds)
{
	if (bRPCTrackingEnabled != bTrackRecentRPCs)
	{
		if (bTrackRecentRPCs)
		{
			EnableRPCTracking(TimeSeconds);
		}
		else
		{
			DisableRPCTracking(TimeSeconds);
		}
	}
}

ERPCNotifyResult FRPCDoSDetection::CheckRPCTracking(const UFunction* Function, FName FunctionName)
{
	ERPCNotifyResult Result = ERPCNotifyResult::ExecuteRPC;
	bool bNewTracking = false;

	if (!bHitchSuspendDetection)
	{
		FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();
		FRPCTrackingInfo& Tracking = SequentialRPCScope.FindOrAddRPCTracking(*this, Function, bNewTracking);
		const uint8 OldTrackedSecondIncrement = Tracking.LastTrackedSecondIncrement;

		Tracking.LastTrackedSecondIncrement = SecondsIncrementer;

		FRPCDoSCounters& CurActiveCounter = Tracking.PerPeriodHistory[Tracking.GetCurrentHistoryIdx()];

		if (bNewTracking)
		{
			Tracking.HistoryCount = 1;
			Tracking.BlockState = RPCBlockAllowList.Contains(FunctionName) ? ERPCBlockState::OnAllowList : ERPCBlockState::NotBlocked;
		}
		else if (OldTrackedSecondIncrement != SecondsIncrementer)
		{
			const uint8 TimeDiff = (SecondsIncrementer > OldTrackedSecondIncrement) ?
									(SecondsIncrementer - OldTrackedSecondIncrement) :
									((MAX_uint8 - OldTrackedSecondIncrement) + SecondsIncrementer);

			// Carry over history that is still in date
			if (TimeDiff < MaxRPCTrackingPeriod)
			{
				const uint8 NumHistoryCarriedOver = FMath::Min((uint8)(MaxRPCTrackingPeriod - TimeDiff), Tracking.HistoryCount);

				if (NumHistoryCarriedOver > 1)
				{
					const uint8 OldHistoryIdx = OldTrackedSecondIncrement & 0xF;
					FRPCDoSCounters& OldActiveCounter = Tracking.PerPeriodHistory[OldHistoryIdx];
					const uint8 StopIdx = OldHistoryIdx - NumHistoryCarriedOver;

					// Recalculate all of the old counters, to include the last active counter.
					// The loop index deliberately wraps-around, so must use an inequality check instead of less-than
					for (uint8 CurAccumIdx=OldHistoryIdx-1; CurAccumIdx!=StopIdx; CurAccumIdx--)
					{
						const uint8 CorrectedAccumIdx = CurAccumIdx & 0xF;

						Tracking.PerPeriodHistory[CorrectedAccumIdx].AccumulateCounter(OldActiveCounter);
					}
				}

				Tracking.HistoryCount = NumHistoryCarriedOver + 1;
			}
			else
			{
				Tracking.HistoryCount = 1;
			}

			CurActiveCounter = {};
		}

		CurActiveCounter.RPCCounter++;

		if (SequentialRPCScope.bReceivedPacketRPCUnique && SequentialRPCScope.PostReceivedRPCName != NAME_None
			&& SequentialRPCScope.PostReceivedRPCName != FunctionName)
		{
			SequentialRPCScope.bReceivedPacketRPCUnique = false;
		}


		// To minimize timestamp generation, use the 'free' pre/post packet timestamps as much as possible,
		// and group sequential calls to the same RPC together instead of timing for each one
		if (SequentialRPCScope.PostReceivedRPCCounter == nullptr)
		{
			SequentialRPCScope.PostReceivedRPCName = FunctionName;
			SequentialRPCScope.PostReceivedRPCTracking = &Tracking;
			SequentialRPCScope.PostReceivedRPCCounter = &CurActiveCounter;
		}
		else if (SequentialRPCScope.PostReceivedRPCCounter != &CurActiveCounter)
		{
			const double CurTimeSeconds = FPlatformTime::Seconds();

			PostSequentialRPC(EPostSequentialRPCType::MidPacket, CurTimeSeconds, SequentialRPCScope.PostReceivedRPCCounter,
								SequentialRPCScope.PostReceivedRPCTracking);

			SequentialRPCScope.LastReceivedRPCTimeCache = CurTimeSeconds;

#if RPC_QUOTA_DEBUG
			SequentialRPCScope.PostReceivedRPCCounter->DebugAccumRPCTime +=
				(SequentialRPCScope.DebugReceivedRPCEndTime - SequentialRPCScope.DebugReceivedRPCStartTime);
#endif

			SequentialRPCScope.PostReceivedRPCName = FunctionName;
			SequentialRPCScope.PostReceivedRPCTracking = &Tracking;
			SequentialRPCScope.PostReceivedRPCCounter = &CurActiveCounter;
			SequentialRPCScope.PostReceivedRPCBlockCount = 0;
		}

		// When tracking is enabled, the quota's are checked every RPC call until blocked (and analytics in the PostSequential call)
		if (!bNewTracking && SequentialRPCScope.PostReceivedRPCBlockCount == 0 && Tracking.BlockState != ERPCBlockState::OnAllowList)
		{
			auto HasHitQuota = [&Tracking](int32 TimePeriod, int32 CountPerPeriod, double SecsPerPeriod)
				{
					const uint8 TargetPeriodOffset = FMath::Min((uint8)TimePeriod, Tracking.HistoryCount);
					const uint8 TargetPeriodIdx = (Tracking.GetCurrentHistoryIdx() - (TargetPeriodOffset - 1)) & 0xF;
					const FRPCDoSCounters& TargetPeriod = Tracking.PerPeriodHistory[TargetPeriodIdx];

					return (CountPerPeriod != -1 && TargetPeriod.RPCCounter >= CountPerPeriod) ||
							(SecsPerPeriod != 0.0 && TargetPeriod.AccumRPCTime >= SecsPerPeriod);
				};

			const bool bBlockRPC = GAllowRPCDoSDetectionBlocking == 1 && RPCRepeatLimitTimePeriod != -1 &&
									HasHitQuota(RPCRepeatLimitTimePeriod, RPCRepeatLimitPerPeriod, RPCRepeatLimitSecsPerPeriod);

			Tracking.BlockState = bBlockRPC ? ERPCBlockState::Blocked : ERPCBlockState::NotBlocked;
		}

		if (Tracking.BlockState == ERPCBlockState::Blocked)
		{
			SequentialRPCScope.PostReceivedRPCBlockCount++;

			Result = ERPCNotifyResult::BlockRPC;
		}
	}
	else
	{
		TSharedPtr<FRPCTrackingInfo>* TrackingPtr = RPCTracking.Find(Function);

		if (TrackingPtr != nullptr && TrackingPtr->IsValid() && TrackingPtr->Get()->BlockState == ERPCBlockState::Blocked)
		{
			FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();

			SequentialRPCScope.PostReceivedRPCBlockCount++;

			Result = ERPCNotifyResult::BlockRPC;
		}
	}

	return Result;
}

void FRPCDoSDetection::RecalculatePeriodHistory(const TArray<int8>& InTimePeriods, FRPCDoSCounters(&OutPerPeriodHistory)[16],
												int32 StartPerSecHistoryIdx/*=INDEX_NONE*/)
{
	if (StartPerSecHistoryIdx == INDEX_NONE)
	{
		StartPerSecHistoryIdx = LastCounterPerSecHistoryIdx;
	}

	for (int8 PeriodSeconds : InTimePeriods)
	{
		FRPCDoSCounters& CurPeriod = OutPerPeriodHistory[PeriodSeconds-1];

		CurPeriod.ResetRPCCounters();

		for (int32 SecIdx=0; SecIdx<PeriodSeconds; SecIdx++)
		{
			int32 PerSecHistoryIdx = StartPerSecHistoryIdx - SecIdx;

			PerSecHistoryIdx = (PerSecHistoryIdx < 0 ? CounterPerSecHistory.Num() + PerSecHistoryIdx : PerSecHistoryIdx);

			check(PerSecHistoryIdx >= 0 && PerSecHistoryIdx < CounterPerSecHistory.Num());

			CurPeriod.AccumulateCounter(CounterPerSecHistory[PerSecHistoryIdx]);
		}
	}
}

void FRPCDoSDetection::EnableRPCTracking(double TimeSeconds)
{
	bRPCTrackingEnabled = true;

	if (LastRPCTrackingClean == 0.0)
	{
		LastRPCTrackingClean = TimeSeconds;
	}

	// Do checks that should occur in the middle of receiving a packet
	if (SequentialRPCScopePrivate.IsActive())
	{
		FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();
		FLightweightRPCTracking& LightweightRPCTracking = SequentialRPCScope.LightweightRPCTracking;

		// If lightweight tracking has entries, see if we can retroactively add this to full RPC tracking (only possible with single RPC)
		// NOTE: Lightweight tracking UFunction's are valid, here.
		bool bUniqueLightweightRPC = LightweightRPCTracking.Count > 0;
		const UFunction* FirstFunc = LightweightRPCTracking.RPC[0].Function;

		for (int32 LWIdx=1; LWIdx<LightweightRPCTracking.Count; LWIdx++)
		{
			if (LightweightRPCTracking.RPC[LWIdx].Function != FirstFunc)
			{
				bUniqueLightweightRPC = false;
				break;
			}
		}

		if (bUniqueLightweightRPC)
		{
			// Add and time the lightweight-tracked RPC, from the start of packet receive
			SequentialRPCScope.CheckRPCTracking(*this, FirstFunc, FirstFunc->GetFName());
		}
		else
		{
			// Ensure tracking knows the packet contains more than one RPC type
			SequentialRPCScope.bReceivedPacketRPCUnique = false;

			// Very important to update the cached time for last received RPC, to avoid mistiming RPC's when tracking is enabled mid-receive
			if (SequentialRPCScope.LastReceivedRPCTimeCache == 0.0)
			{
				SequentialRPCScope.LastReceivedRPCTimeCache = TimeSeconds;
			}
		}
	}
}

void FRPCDoSDetection::EnableForcedRPCTracking(const UFunction* Function, FName FunctionName, double TimeSeconds)
{
	ForcedRPCTrackingEndTime = TimeSeconds + ForcedRPCTrackingLength;

	EnableRPCTracking(TimeSeconds);

	// If lightweight RPC tracking was not made up of one unique RPC, this RPC will not have been added to tracking - add it now
	FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();

	if (!SequentialRPCScope.bReceivedPacketRPCUnique)
	{
		CheckRPCTracking(Function, FunctionName);
	}
}

void FRPCDoSDetection::DisableRPCTracking(double TimeSeconds)
{
	// NOTE: Stale RPC tracking cleaning will still occur after RPC tracking is disabled, until tracking info empties out
	bRPCTrackingEnabled = false;

	ResetBlockedRPCTracking();
}

void FRPCDoSDetection::ClearStaleRPCTracking(double TimeSeconds)
{
	int32 EndRemoveIdx = INDEX_NONE;

	auto BulkClear = [&EndRemoveIdx, &ActiveRPCTracking = ActiveRPCTracking](int32 StartRemoveIdx)
		{
			const int32 NumToRemove = (EndRemoveIdx - StartRemoveIdx) + 1;

			ActiveRPCTracking.RemoveAt(StartRemoveIdx, NumToRemove, EAllowShrinking::No);

			EndRemoveIdx = INDEX_NONE;
		};

	const int32 OrigNum = ActiveRPCTracking.Num();

	for (int32 TrackIdx=ActiveRPCTracking.Num()-1; TrackIdx>=0; TrackIdx--)
	{
		const FActiveRPCTrackingInfo& CurInfo = ActiveRPCTracking[TrackIdx];
		const uint8 CurSecondIncrement = CurInfo.TrackingInfo->LastTrackedSecondIncrement;
		const uint8 TimeDiff = (SecondsIncrementer > CurSecondIncrement) ?
								(SecondsIncrementer - CurSecondIncrement) : ((MAX_uint8 - CurSecondIncrement) + SecondsIncrementer);

		if (TimeDiff > UE_ARRAY_COUNT(FRPCTrackingInfo::PerPeriodHistory))
		{
			if (EndRemoveIdx == INDEX_NONE)
			{
				EndRemoveIdx = TrackIdx;
			}

			TSharedPtr<FRPCTrackingInfo>& TrackingInfoVal = RPCTracking.FindChecked(CurInfo.Key);

			TrackingInfoVal.Reset();
		}
		else
		{
			if (EndRemoveIdx != INDEX_NONE)
			{
				BulkClear(TrackIdx + 1);
			}
		}
	}

	if (EndRemoveIdx != INDEX_NONE)
	{
		BulkClear(0);
	}

	if (ActiveRPCTracking.Num() != OrigNum)
	{
		ActiveRPCTracking.Reserve(FMath::Max(ActiveRPCTracking.Num(), DefaultActiveRPCTrackingSize));
	}

	LastRPCTrackingClean = TimeSeconds;
}

void FRPCDoSDetection::CachePlayerAddress()
{
	if (CachedAddress.IsEmpty() && AddressFunc)
	{
		CachedAddress = AddressFunc();
	}
}

void FRPCDoSDetection::CachePlayerUID()
{
	if (CachedPlayerUID.IsEmpty() && PlayerUIDFunc)
	{
		CachedPlayerUID = PlayerUIDFunc();
	}
}

void FRPCDoSDetection::SetAddressFunc(FGetRPCDoSAddress&& InAddressFunc)
{
	AddressFunc = MoveTemp(InAddressFunc);

	if (!CachedAddress.IsEmpty())
	{
		CachedAddress = TEXT("");

		CachePlayerAddress();
	}
}

void FRPCDoSDetection::SetPlayerUIDFunc(FGetRPCDoSPlayerUID&& InPlayerUIDFunc)
{
	PlayerUIDFunc = MoveTemp(InPlayerUIDFunc);

	if (!CachedPlayerUID.IsEmpty())
	{
		CachedPlayerUID = TEXT("");

		CachePlayerUID();
	}
}

void FRPCDoSDetection::SetKickPlayerFunc(FRPCDoSKickPlayer&& InKickPlayerFunc)
{
	KickPlayerFunc = MoveTemp(InKickPlayerFunc);
}


