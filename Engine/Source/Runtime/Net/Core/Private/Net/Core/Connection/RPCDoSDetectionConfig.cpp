// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/Connection/RPCDoSDetectionConfig.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RPCDoSDetectionConfig)


/**
 * URPCDoSDetectionConfig
 */
URPCDoSDetectionConfig::URPCDoSDetectionConfig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void URPCDoSDetectionConfig::OverridePerObjectConfigSection(FString& SectionName)
{
	SectionName = GetName() + TEXT(" ") + GetConfigSectionName();
}

URPCDoSDetectionConfig* URPCDoSDetectionConfig::Get(FName NetDriverName)
{
	UClass* ClassRef = URPCDoSDetectionConfig::StaticClass();
	URPCDoSDetectionConfig* ReturnVal = FindObject<URPCDoSDetectionConfig>(ClassRef, *NetDriverName.ToString());

	if (ReturnVal == nullptr)
	{
		ReturnVal = NewObject<URPCDoSDetectionConfig>(ClassRef, NetDriverName);
	}

	return ReturnVal;
}

const TCHAR* URPCDoSDetectionConfig::GetConfigSectionName()
{
	return TEXT("RPCDoSDetection");
}

