// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "RPCDoSDetectionConfig.generated.h"


/**
 * Specifies time or count thresholds for when an RPC should be included in analytics
 */
USTRUCT()
struct FRPCAnalyticsThreshold
{
	GENERATED_BODY()

	/** The RPC name */
	UPROPERTY()
	FName RPC;

	/** The number of calls to an RPC's per second, before including in analytics */
	UPROPERTY()
	int32 CountPerSec = -1;

	/** Time spent executing an RPC per second, before including in analytics (can specify more than 1 second, for long running RPC's) */
	UPROPERTY()
	double TimePerSec = 0.0;
};


/**
 * Configuration for FRPCDoSDetection - using PerObjectConfig, to hack a single hardcoded section name
 */
UCLASS(config=Engine, PerObjectConfig)
class URPCDoSDetectionConfig : public UObject
{
	GENERATED_UCLASS_BODY()

private:
	void OverridePerObjectConfigSection(FString& SectionName) override;

public:
	/**
	 * Singleton for getting a per-NetDriver instance of this config section/object
	 *
	 * @param NetDriverName		The NetDriver name used for getting the correct config section
	 */
	static NETCORE_API URPCDoSDetectionConfig* Get(FName NetDriverName);

	/**
	 * Returns the section name used by this class in the config file
	 */
	static const TCHAR* GetConfigSectionName();

public:
	/** Whether or not RPC DoS detection is presently enabled */
	UPROPERTY(config)
	bool bRPCDoSDetection;

	/** Whether or not analytics for RPC DoS detection is enabled */
	UPROPERTY(config)
	bool bRPCDoSAnalytics;

	/** The amount of time since the previous frame, for detecting hitches, to prevent false positives from built-up packets */
	UPROPERTY(config)
	int32 HitchTimeQuotaMS;

	/** The amount of time to suspend RPC DoS Detection, once a hitch is encountered, prevent false positives from built-up packets */
	UPROPERTY(config)
	int32 HitchSuspendDetectionTimeMS;

	/** Names of the different RPC DoS detection states, for escalating severity, depending on the amount of RPC spam */
	UPROPERTY(config)
	TArray<FString> DetectionSeverity;

	/** The amount of time since the client connected, before time-based checks should become active (to reduce false positives) */
	UPROPERTY(config)
	int32 InitialConnectToleranceMS;

	/** List of RPC's which should never be blocked */
	UPROPERTY(config)
	TArray<FName> RPCBlockAllowlist;

	/** Custom thresholds for when specific RPC's should be included in analytics */
	UPROPERTY(config)
	TArray<FRPCAnalyticsThreshold> RPCAnalyticsThresholds;

	/** Specifies a random chance, between 0.0 and 1.0, for when RPCAnalyticsThresholds should be overridden (adds to separate analytics) */
	UPROPERTY(config)
	double RPCAnalyticsOverrideChance;
};
