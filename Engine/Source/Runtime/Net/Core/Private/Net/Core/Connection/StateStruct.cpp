// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/Core/Connection/StateStruct.h"
#include "Misc/ConfigCacheIni.h"
#include "Net/Core/Misc/NetCoreLog.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateStruct)


/**
 * FStateStruct
 */

FString FStateStruct::GetStateName() const
{
	return StateName;
}


/**
 * UStatePerObjectConfig
 */

const UStatePerObjectConfig* UStatePerObjectConfig::Get(FStateConfigParms ConfigParms)
{
	FString& ConfigContext = ConfigParms.ConfigContext;
	const TCHAR* ConfigSection = ConfigParms.ConfigSection;
	UClass* ConfigClass = ConfigParms.ConfigClass;

	check(ConfigClass->IsChildOf(UStatePerObjectConfig::StaticClass()));


	TStringBuilder<1024> FormattedName;

	FormattedName.Append(ToCStr(ConfigContext));
	FormattedName.AppendChar(TEXT('_'));
	FormattedName.Append(ToCStr(ConfigSection));
	FormattedName.AppendChar(TEXT('_'));
	FormattedName.Append(ToCStr(ConfigClass->GetFName().ToString()));

	const TCHAR* FormattedNameStr = FormattedName.ToString();
	UStatePerObjectConfig* ReturnVal = FindObject<UStatePerObjectConfig>(ConfigClass, FormattedNameStr);

	if (ReturnVal == nullptr)
	{
		TStringBuilder<1024> FullSection;

		if (!ConfigContext.IsEmpty())
		{
			FullSection.Append(ToCStr(ConfigContext));
			FullSection.AppendChar(TEXT(' '));
		}

		FullSection.Append(ToCStr(ConfigSection));


		UStatePerObjectConfig* ArchetypeObj = GetArchetype(ConfigParms, FullSection.ToString(), FormattedNameStr);

		ReturnVal = NewObject<UStatePerObjectConfig>(GetTransientPackage(), ConfigClass, FName(FormattedNameStr), RF_NoFlags, ArchetypeObj);

		ReturnVal->ConfigParms = ConfigParms;

		ReturnVal->LoadStateConfig();
		ReturnVal->AddToRoot();
	}

	return ReturnVal;
}

UStatePerObjectConfig* UStatePerObjectConfig::GetArchetype(FStateConfigParms ConfigParms, FString FullSection, FString FormattedName)
{
	UClass* ConfigClass = ConfigParms.ConfigClass;

	FormattedName += TEXT("_Archetype");

	UStatePerObjectConfig* ReturnVal = FindObject<UStatePerObjectConfig>(ConfigClass, *FormattedName);

	if (ReturnVal == nullptr)
	{
		ReturnVal = NewObject<UStatePerObjectConfig>(GetTransientPackage(), ConfigClass, FName(FormattedName), RF_ArchetypeObject);

		ReturnVal->PerObjectConfigSection = FullSection;

		ReturnVal->InitConfigDefaults();
	}

	return ReturnVal;
}

void UStatePerObjectConfig::ApplyState(const FStructOnScope& ConfigState, FStateStruct* TargetState)
{
	const UScriptStruct* BaseScriptStruct = ::Cast<const UScriptStruct>(ConfigState.GetStruct());

	check(BaseScriptStruct != nullptr);

	if (BaseScriptStruct != nullptr && TargetState != nullptr)
	{
		const FStateStruct* StructCast = reinterpret_cast<const FStateStruct*>(ConfigState.GetStructMemory());

		check(StructCast != nullptr);

		BaseScriptStruct->CopyScriptStruct(TargetState, StructCast);
	}
}

void UStatePerObjectConfig::RegisterStateConfig(const TArray<FString>& StateNames, TArray<TStructOnScope<FStateStruct>>& OutStates)
{
	OutStates.Empty();

	if (bEnabled)
	{
		for (const FString& CurStateName : StateNames)
		{
			TStringBuilder<1024> CurSection;

			CurSection.Append(ToCStr(ConfigParms.ConfigSection));
			CurSection.AppendChar(TEXT('.'));
			CurSection.Append(ToCStr(CurStateName));

			const TCHAR* CurSectionStr = CurSection.ToString();
			TStructOnScope<FStateStruct>& CurState = OutStates.AddDefaulted_GetRef();

			CurState.InitializeFromChecked(FStructOnScope(ConfigParms.StateStruct));

			// Copy this value early, so InitConfigDefaults can check the state
			CurState->StateName = CurStateName;

			bool bHasDefaults = CurState->InitConfigDefaults() == EInitStateDefaultsResult::Initialized;
			bool bHasConfig = GConfig->DoesSectionExist(CurSectionStr, GEngineIni);

			if (bHasDefaults || bHasConfig)
			{
				bool bConfigSuccess = !bHasConfig || LoadStructConfig(CurState, CurSectionStr, *GEngineIni);

				if (bConfigSuccess)
				{
					CurState->ApplyImpliedValues();
					CurState->ValidateConfig();
				}
				else
				{
					UE_LOGF(LogNetCore, Warning, "StatePerObjectConfig failed to load ini section: %ls", CurSectionStr);
				}
			}
			else
			{
				OutStates.RemoveAt(OutStates.Num()-1, EAllowShrinking::No);

				UE_LOGF(LogNetCore, Warning, "StatePerObjectConfig could not load defaults or ini section, removing: %ls", CurSectionStr);
			}
		}

#if !UE_BUILD_SHIPPING
		RegisteredStateConfigs.Add({StateNames, OutStates});
#endif
	}
}

void UStatePerObjectConfig::OverridePerObjectConfigSection(FString& SectionName)
{
	if (!PerObjectConfigSection.IsEmpty())
	{
		SectionName = PerObjectConfigSection;
	}
}

bool UStatePerObjectConfig::LoadStructConfig(FStructOnScope& OutStruct, const TCHAR* SectionName, const TCHAR* InFilename/*=nullptr*/)
{
	bool bSuccess = false;

	if (OutStruct.IsValid())
	{
		FString ConfigFile = (InFilename != nullptr ? InFilename : GEngineIni);
		TArray<FString> StructVars;
		bool bFoundSection = GConfig->GetSection(SectionName, StructVars, ConfigFile);

		if (bFoundSection)
		{
			bSuccess = true;

			for (const FString& CurVar : StructVars)
			{
				FString Var;
				FString Value;

				if (CurVar.Split(TEXT("="), &Var, &Value))
				{
					FProperty* CurProp = OutStruct.GetStruct()->FindPropertyByName(*Var);

					if (CurProp != nullptr)
					{
						if (CurProp->HasAllPropertyFlags(CPF_Config))
						{
							CurProp->ImportText_InContainer(*Value, OutStruct.GetStructMemory(), nullptr, 0);
						}
						else
						{
							UE_LOGF(LogNetCore, Error, "LoadStructConfig: Ini property '%ls' is not a config property.", *Var);
						}
					}
					else
					{
						UE_LOGF(LogNetCore, Error, "LoadStructConfig: Ini property '%ls' not found.", *Var);
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogNetCore, Error, "LoadStructConfig: OutStruct not set");
	}

	return bSuccess;
}

#if !UE_BUILD_SHIPPING
void UStatePerObjectConfig::DebugDump() const
{
	UStruct* StateStruct = ConfigParms.StateStruct;

	UE_LOGF(LogNetCore, Log, "UStatePerObjectConfig::DebugDump: %ls", ToCStr(GetFullName()));
	UE_LOGF(LogNetCore, Log, " - ConfigContext: %ls", ToCStr(ConfigParms.ConfigContext));
	UE_LOGF(LogNetCore, Log, " - ConfigSection: %ls", ToCStr(ConfigParms.ConfigSection != nullptr ? ConfigParms.ConfigSection : TEXT("")));
	UE_LOGF(LogNetCore, Log, " - ConfigClass: %ls", ToCStr(ConfigParms.ConfigClass != nullptr ? ConfigParms.ConfigClass->GetFullName() : TEXT("")));
	UE_LOGF(LogNetCore, Log, " - StateStruct: %ls", ToCStr(StateStruct != nullptr ? StateStruct->GetFullName() : TEXT("")));
	UE_LOGF(LogNetCore, Log, " - bEnabled: %ls", (bEnabled ? TEXT("True") : TEXT("False")));

	if (StateStruct != nullptr)
	{
		for (TFieldIterator<FProperty> It(GetClass()); It; ++It)
		{
			FString TextValue;
			It->ExportTextItem_InContainer(TextValue, this, nullptr, nullptr, PPF_DebugDump, nullptr);

			UE_LOGF(LogNetCore, Log, " - %ls: %ls", ToCStr(It->GetName()), ToCStr(TextValue));
		}
	}

	for (const FStateConfigRegister& CurStateRegister : RegisteredStateConfigs)
	{
		FString StateNamesVar = "UnknownStateNames";

		// Indirectly figure out the variable name for these states
		for (TFieldIterator<FProperty> It(GetClass()); It; ++It)
		{
			const uint8* PropAddr = It->ContainerPtrToValuePtr<uint8>(this);

			if (PropAddr == (uint8*)&CurStateRegister.StateNames)
			{
				StateNamesVar = It->GetName();
				break;
			}
		}


		UE_LOGF(LogNetCore, Log, " - %ls:", ToCStr(StateNamesVar));

		for (TStructOnScope<FStateStruct>& CurStateConfig : CurStateRegister.States)
		{
			if (CurStateConfig.IsValid())
			{
				UE_LOGF(LogNetCore, Log, "  - %ls:", ToCStr(CurStateConfig->GetStateName()));

				for (TFieldIterator<FProperty> It(CurStateConfig.GetStruct()); It; ++It)
				{
					if (It->GetName() != TEXT("StateName"))
					{
						FString TextValue;
						It->ExportTextItem_InContainer(TextValue, CurStateConfig.Get(), nullptr, nullptr, PPF_DebugDump, nullptr);

						UE_LOGF(LogNetCore, Log, "   - %ls: %ls", ToCStr(It->GetName()), ToCStr(TextValue));
					}
				}
			}
			else
			{
				UE_LOGF(LogNetCore, Log, "  - *INVALID*");
			}
		}
	}
}
#endif


