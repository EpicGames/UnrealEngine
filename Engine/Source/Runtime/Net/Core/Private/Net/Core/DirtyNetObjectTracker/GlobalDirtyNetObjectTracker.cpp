// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/DirtyNetObjectTracker/GlobalDirtyNetObjectTracker.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "Misc/CoreDelegates.h"
#include "HAL/IConsoleManager.h"

namespace UE::Net
{

namespace Private::CVars
{

bool bUsePerPropertyDirtyTracking = true;
static FAutoConsoleVariableRef CVarUsePerPropertyDirtyTracking(
	TEXT("net.Iris.UsePerPropertyDirtyTracking"),
	bUsePerPropertyDirtyTracking,
	TEXT("When true we use per property dirty tracking.")
);

}

class FGlobalDirtyNetObjectTracker::FPimpl
{
private:
	friend FGlobalDirtyNetObjectTracker;

	static TSet<FNetHandle> EmptyDirtyObjects;
	static FDirtyHandleAndPropertyMap EmptyDirtyObjectsAndProperties;

private:

	struct FPollerStatus
	{
		FPreResetDelegate PreResetDelegate;

		// Is this status tied to an active registered poller
		uint8 bIsActive:1;

		// Does this poller need to read the dirty list this frame.
		uint8 bNeedsGather:1;

		FPollerStatus() : bIsActive(false), bNeedsGather(true) {}

		void ClearStatus()
		{
			*this = FPollerStatus();
		}
	};

private:

	TSet<FNetHandle> DirtyObjects;

	FDirtyHandleAndPropertyMap DirtyObjectsAndProperties;

	FNetBitArray AssignedHandleIndices;
	FNetBitArray Pollers;

	TArray<FPollerStatus> PollerStatuses;

	uint32 PollerCount = 0;

	/** When true detect and prevent illegal changes to the dirty object list. */
	bool bLockDirtyList = false;
};

TSet<FNetHandle> FGlobalDirtyNetObjectTracker::FPimpl::EmptyDirtyObjects;
FGlobalDirtyNetObjectTracker::FDirtyHandleAndPropertyMap FGlobalDirtyNetObjectTracker::FPimpl::EmptyDirtyObjectsAndProperties;

FGlobalDirtyNetObjectTracker::FPimpl* FGlobalDirtyNetObjectTracker::Instance = nullptr;

void FGlobalDirtyNetObjectTracker::MarkNetObjectStateDirty(FNetHandle NetHandle, const int32 StartRepIndex, const int32 EndRepIndex)
{
	if (Instance && Instance->PollerCount > 0)
	{
		if (ensureMsgf(!Instance->bLockDirtyList, TEXT("MarkNetObjectStateDirty was called while the dirty list was set to read-only.")))
		{
			if (Private::CVars::bUsePerPropertyDirtyTracking)
			{
				FDirtyPropertyStorage& Entry = Instance->DirtyObjectsAndProperties.FindOrAdd(NetHandle);
				if (EndRepIndex != INDEX_NONE)
				{
					const int32 WordCount = (int32)FNetBitArrayView::CalculateRequiredWordCount(EndRepIndex + 1);
					if (WordCount > Entry.Num())
					{
						Entry.SetNumZeroed(WordCount);
					}
					FNetBitArrayView BitMask(Entry.GetData(), Entry.Num() * 32U, FNetBitArrayView::NoResetNoValidate);
					BitMask.SetBits(StartRepIndex, EndRepIndex - StartRepIndex + 1U);
				}
			}
			else
			{
				Instance->DirtyObjects.Add(NetHandle);
			}

			// If there are multiple pollers, any poller that has already gathered this frame will need to gather again
			// to know about this newly dirtied object, since it may have happened in another replication system's PreUpdate.
			// Setting bNeedsGather = true will cause it to gather during the last-chance delegate in ResetDirtyNetObjects.
			if (Instance->PollerCount > 1)
			{
				Instance->Pollers.ForAllSetBits([](uint32 BitIndex)
					{
						Instance->PollerStatuses[BitIndex].bNeedsGather = true;
					});
			}
		}
	}
}

FGlobalDirtyNetObjectTracker::FPollHandle FGlobalDirtyNetObjectTracker::CreatePoller(FPreResetDelegate InPreResetDelegate)
{
	if (Instance)
	{
		if (Instance->PollerCount >= Instance->AssignedHandleIndices.GetNumBits())
		{
			Instance->AssignedHandleIndices.SetNumBits(Instance->PollerCount + 1U);
			Instance->Pollers.SetNumBits(Instance->PollerCount + 1U);
		}

		const uint32 HandleIndex = Instance->AssignedHandleIndices.FindFirstZero();
		if (!ensure(HandleIndex != FNetBitArrayBase::InvalidIndex))
		{
			return FPollHandle();
		}

		Instance->AssignedHandleIndices.SetBit(HandleIndex);
		++Instance->PollerCount;

		Instance->PollerStatuses.SetNum(Instance->PollerCount, EAllowShrinking::No);
		Instance->PollerStatuses[HandleIndex].bIsActive = true;
		Instance->PollerStatuses[HandleIndex].PreResetDelegate = InPreResetDelegate;

		return FPollHandle(HandleIndex);
	}

	return FPollHandle();
}

void FGlobalDirtyNetObjectTracker::DestroyPoller(uint32 HandleIndex)
{
	if (HandleIndex == FPollHandle::InvalidIndex)
	{
		return;
	}

	if (ensureMsgf((HandleIndex < Instance->AssignedHandleIndices.GetNumBits()) && Instance->AssignedHandleIndices.GetBit(HandleIndex), TEXT("Destroying unknown poller with handle index %u"), HandleIndex))
	{
		Instance->AssignedHandleIndices.ClearBit(HandleIndex);

		const uint32 PollerCalled = Instance->Pollers.GetBit(HandleIndex);
		ensureMsgf(PollerCalled == 0U, TEXT("Destroying poller that called GetDirtyNetObjects() but not ResetDirtyNetObjects()"));
		Instance->Pollers.ClearBit(HandleIndex);

		Instance->PollerStatuses[HandleIndex].ClearStatus();

		--Instance->PollerCount;
		if (Instance->PollerCount <= 0)
		{
			Instance->DirtyObjects.Reset();
			Instance->DirtyObjectsAndProperties.Reset();
			Instance->bLockDirtyList = false;
		}
	}
}

const TSet<FNetHandle>& FGlobalDirtyNetObjectTracker::GetDirtyNetObjects(const FPollHandle& Handle)
{
	if (Instance && Handle.IsValid())
	{
		check(Instance->PollerStatuses[Handle.Index].bIsActive);

		Instance->Pollers.SetBit(Handle.Index);
		Instance->PollerStatuses[Handle.Index].bNeedsGather = false;
		return Instance->DirtyObjects;
	}

	return FPimpl::EmptyDirtyObjects;
}

const FGlobalDirtyNetObjectTracker::FDirtyHandleAndPropertyMap& FGlobalDirtyNetObjectTracker::GetDirtyNetObjectsAndProperties(const FPollHandle& Handle)
{
	if (Instance && Handle.IsValid())
	{
		check(Instance->PollerStatuses[Handle.Index].bIsActive);

		Instance->Pollers.SetBit(Handle.Index);
		Instance->PollerStatuses[Handle.Index].bNeedsGather = false;
		return Instance->DirtyObjectsAndProperties;
	}

	return FPimpl::EmptyDirtyObjectsAndProperties;
}

void FGlobalDirtyNetObjectTracker::LockDirtyListUntilReset(const FPollHandle& Handle)
{
	if (Instance && Handle.IsValid())
	{
		check(Instance->PollerStatuses[Handle.Index].bIsActive);

		// From here prevent new dirty objects until the list is reset.
		Instance->bLockDirtyList = true;
	}
}

void FGlobalDirtyNetObjectTracker::ResetDirtyNetObjects(const FPollHandle& Handle)
{
	if (Instance && Handle.IsValid())
	{
		check(Instance->PollerStatuses[Handle.Index].bIsActive);

		Instance->Pollers.ClearBit(Handle.Index);

		if (Instance->Pollers.IsNoBitSet())
		{
			for (uint32 PollerIndex = 0; PollerIndex < Instance->PollerCount; ++PollerIndex)
			{
				FPimpl::FPollerStatus& PollerStatus = Instance->PollerStatuses[PollerIndex];

				if (PollerStatus.bIsActive && PollerStatus.bNeedsGather)
				{
					PollerStatus.PreResetDelegate.ExecuteIfBound();
					Instance->Pollers.ClearBit(PollerIndex);
				}

				// Poller will need to call GetDirtyNetObjects next frame
				PollerStatus.bNeedsGather = true;
			}

			Instance->DirtyObjects.Reset();
			Instance->DirtyObjectsAndProperties.Reset();
		}

		Instance->bLockDirtyList = false;
	}
}

bool FGlobalDirtyNetObjectTracker::ResetDirtyNetObjectsIfSinglePoller(const FPollHandle& Handle)
{
	if (Instance && Handle.IsValid())
	{
		check(Instance->PollerStatuses[Handle.Index].bIsActive);

		Instance->bLockDirtyList = false;

		// Are we the only poller registered to read and reset the list
		if (Instance->AssignedHandleIndices.CountSetBits() == 1)
		{
			check(Instance->Pollers.IsBitSet(Handle.Index));

			Instance->DirtyObjects.Reset();
			Instance->DirtyObjectsAndProperties.Reset();
			Instance->Pollers.ClearBit(Handle.Index);

			return true;
		}
	}

	return false;
}

bool FGlobalDirtyNetObjectTracker::IsUsingPerPropertyDirtyTracking()
{
	return UE::Net::Private::CVars::bUsePerPropertyDirtyTracking;
}

void FGlobalDirtyNetObjectTracker::Init()
{
	checkf(Instance == nullptr, TEXT("%s"), TEXT("Only one FGlobalDirtyNetObjectTracker instance may exist."));
	Instance = new FGlobalDirtyNetObjectTracker::FPimpl();
}

void FGlobalDirtyNetObjectTracker::Deinit()
{
	delete Instance;
	Instance = nullptr;
}

}
