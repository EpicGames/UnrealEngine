// Copyright Epic Games, Inc. All Rights Reserved.

// Includes

#include "Net/Core/Misc/DDoSDetection.h"
#include "Net/Core/Misc/NetCoreLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Templates/UnrealTemplate.h"

/**
 * DDoS Detection
 *
 * DDoS (Distributed Denial of Service) attacks typically hinder game servers by flooding them with so many packets,
 * that they are unable to process all of the packets without locking up and/or drowning out other players packets,
 * causing players to time out or to suffer severe packet loss which hinders gameplay.
 *
 * Typically these attacks use spoofed UDP packets, where the source IP is unverifiable,
 * and so IP banning is usually not an effective or advisable means of blocking such attacks.
 *
 * This DDoS detection focuses specifically on this situation, detecting/mitigating DDoS attacks based on spoofed UDP packets,
 * which do not originate from an existing NetConnection. Flooding attacks coming from an existing NetConnection are a separate issue,
 * as (due to the stateless handshake required before creating a NetConnection) the IP will be verified,
 * and so such attacks should be dealt with through IP banning - this and other types of DoS attacks are not dealt with by this code.
 *
 *
 * Implementation:
 *
 * DDoS attacks are detected by setting configurable thresholds for the number of packets per second,
 * and milliseconds per frame spent processing packets, beyond which the DDoS detection will escalate to a higher severity state.
 *
 * Each severity state has a different set of thresholds before it will escalate to the next state,
 * and can also place a limit on the number of packets processed per second, and/or milliseconds spent processing.
 *
 * The stronger the DDoS attack, the higher the severity state will escalate (based on the thresholds),
 * and the stronger the limitations places on incoming packets will be, in order to try and maintain good server performance.
 *
 *
 * Limitations:
 *
 * Heavy DDoS:
 * While the code can withstand a heavy, locally hosted, multithreaded DDoS,
 * past a certain point network hardware and bandwidth capacity will become a limit, and even with strong enough hardware,
 * the OS kernel calls for receiving packets, will become a limit (for Linux, recvmmsg may be used to alleviate this later).
 *
 * So this code just deals with as much of the DDoS as it can, at an application level - if you're getting hit with a bad enough DDoS,
 * then you're going to have to look at measures at the network infrastructure level - for example,
 * IP filtering at the edge of your network, communicating with the game server to only allow packets from existing NetConnection IP's.
 *
 * Tuning thresholds per-Game:
 * You will need to manually tune the packet thresholds specifically for your game, even for each different gametype within your game,
 * and maybe even community server admins will need to retune, if hosting a server with mods etc..
 *
 * Blocking new connections:
 * If a DDoS is expensive enough, that you choose to drop non-NetConnection packets after a threshold (a wise move, for performance...),
 * then new players will be blocked from entering the server.
 */

// @todo #JohnB: The code deliberately counts the time spent processing NetConnection RPC's, not just merely receiving packets.
//					Make sure this doesn't introduce problems or false positives (or at least, that they're tolerable, if it does).

/**
 * FDDoSDetection
 */

FDDoSDetection::FDDoSDetection()
	: bDDoSDetection(false)
	, bDDoSAnalytics(false)
	, bHitFrameNonConnLimit(false)
	, bHitFrameNetConnLimit(false)
	, DetectionSeverity()
	, ActiveState(0)
	, WorstActiveState(0)
	, LastMetEscalationConditions(0.0)
	, bMetEscalationConditionsThisFrame(false)
	, bDDoSLogRestrictions(false)
	, DDoSLogSpamLimit(0)
	, LogHitCounter(0)
	, HitchTimeQuotaMS(-1)
	, HitchFrameTolerance(-1)
	, HitchFrameCount(0)
	, LastPerSecQuotaBegin(0.0)
	, CounterPerSecHistory()
	, LastCounterPerSecHistoryIdx(0)
	, StartFrameRecvTimestamp(0.0)
	, EndFrameRecvTimestamp(0.0)
	, StartFramePacketCount(0)
	, ExpectedFrameTime(0.0)
	, FrameAdjustment(0.f)
	, NotifySeverityEscalation()
{
}

void FDDoSDetection::Init(int32 MaxTickRate)
{
	SetMaxTickRate(MaxTickRate);

	InitConfig();
}

void FDDoSDetection::InitConfig()
{
	const TCHAR* DDoSSection = TEXT("DDoSDetection");
	int32 HitchFrameTolerance32 = -1;

	GConfig->GetBool(DDoSSection, TEXT("bDDoSDetection"), bDDoSDetection, GEngineIni);
	GConfig->GetBool(DDoSSection, TEXT("bDDoSAnalytics"), bDDoSAnalytics, GEngineIni);
	GConfig->GetInt(DDoSSection, TEXT("DDoSLogSpamLimit"), DDoSLogSpamLimit, GEngineIni);
	GConfig->GetInt(DDoSSection, TEXT("HitchTimeQuotaMS"), HitchTimeQuotaMS, GEngineIni);
	GConfig->GetInt(DDoSSection, TEXT("HitchFrameTolerance"), HitchFrameTolerance32, GEngineIni);

	HitchFrameTolerance = IntCastChecked<int8>(HitchFrameTolerance32);
	DDoSLogSpamLimit = DDoSLogSpamLimit > 0 ? DDoSLogSpamLimit : 64;

	DetectionSeverity.Empty();

	UE_LOGF(LogNetCore, Log, "DDoS detection status: detection enabled: %d analytics enabled: %d", bDDoSDetection, bDDoSAnalytics);

	if (bDDoSDetection)
	{
		TArray<FString> SeverityCatagories;
		int32 HighestCooloffTime = 0;

		GConfig->GetArray(DDoSSection, TEXT("DetectionSeverity"), SeverityCatagories, GEngineIni);

		for (const FString& CurCategory : SeverityCatagories)
		{
			FString CurSection = FString(DDoSSection) + TEXT(".") + CurCategory;

			if (GConfig->DoesSectionExist(*CurSection, GEngineIni))
			{
				FDDoSStateConfig& CurState = DetectionSeverity.AddDefaulted_GetRef();
				int32 EscalateTime32 = 0;

				CurState.SeverityCategory = CurCategory;

				GConfig->GetBool(*CurSection, TEXT("bSendEscalateAnalytics"), CurState.bSendEscalateAnalytics, GEngineIni);
				GConfig->GetInt(*CurSection, TEXT("EscalateQuotaPacketsPerSec"), CurState.EscalateQuotaPacketsPerSec, GEngineIni);
				GConfig->GetInt(*CurSection, TEXT("EscalateQuotaDisconnPacketsPerSec"), CurState.EscalateQuotaDisconnPacketsPerSec, GEngineIni);
				GConfig->GetInt(*CurSection, TEXT("EscalateQuotaBadPacketsPerSec"), CurState.EscalateQuotaBadPacketsPerSec, GEngineIni);
				GConfig->GetInt(*CurSection, TEXT("PacketLimitPerFrame"), CurState.PacketLimitPerFrame, GEngineIni);
				GConfig->GetInt(*CurSection, TEXT("PacketTimeLimitMSPerFrame"), CurState.PacketTimeLimitMSPerFrame, GEngineIni);
				GConfig->GetInt(*CurSection, TEXT("NetConnPacketTimeLimitMSPerFrame"), CurState.NetConnPacketTimeLimitMSPerFrame, GEngineIni);
				GConfig->GetInt(*CurSection, TEXT("CooloffTime"), CurState.CooloffTime, GEngineIni);

				if (GConfig->GetInt(*CurSection, TEXT("EscalateTimeQuotaMSPerFrame"), EscalateTime32, GEngineIni))
				{
					CurState.EscalateTimeQuotaMSPerFrame = IntCastChecked<int16>(EscalateTime32);
				}

				HighestCooloffTime = FMath::Max(HighestCooloffTime, CurState.CooloffTime);
			}
			else
			{
				UE_LOGF(LogNetCore, Warning, "DDoS detection could not find ini section: %ls", *CurSection);
			}
		}

		if (DetectionSeverity.Num() > 0)
		{
			DetectionSeverity[ActiveState].ApplyState(*this);

			CounterPerSecHistory.SetNum(HighestCooloffTime);
		}
		else
		{
			UE_LOGF(LogNetCore, Warning, "DDoS detection enabled, but no DetectionSeverity states specified! Disabling.");

			bDDoSDetection = false;
		}
	}
}

void FDDoSDetection::SetMaxTickRate(int32 MaxTickRate)
{
	ExpectedFrameTime = 1.0 / (MaxTickRate > 0.0 ? MaxTickRate : 30.0);
}

void FDDoSDetection::UpdateSeverity(bool bEscalate)
{
	int8 NewState = static_cast<int8>(FMath::Clamp(ActiveState + (bEscalate ? 1 : -1), 0, DetectionSeverity.Num()-1));

	if (NewState != ActiveState)
	{
		double CurTime = FPlatformTime::Seconds();

		if (bEscalate)
		{
			LastMetEscalationConditions = CurTime;
		}
		else
		{
			// De-escalate to the lowest state which hasn't cooled off, and estimate the timestamp for when the cooloff was last reset
			// (due to estimating, there is slight inaccuracy in the cooloff time)
			bool bCooloffReached = true;

			while (bCooloffReached && NewState > 0)
			{
				FDDoSStateConfig& PrevState = DetectionSeverity[NewState-1];
				int32 CurStateCooloffTime = DetectionSeverity[NewState].CooloffTime;

				check(CounterPerSecHistory.Num() >= CurStateCooloffTime);

				for (int32 SecondsDelta=0; SecondsDelta<CurStateCooloffTime; SecondsDelta++)
				{
					int32 CurIdx = LastCounterPerSecHistoryIdx - SecondsDelta;

					CurIdx = (CurIdx < 0 ? CounterPerSecHistory.Num() + CurIdx : CurIdx);

					check(CurIdx >= 0 && CurIdx < CounterPerSecHistory.Num());

					FDDoSPacketCounters& CurPerSecHistory = CounterPerSecHistory[CurIdx];

					if (PrevState.HasHitQuota(CurPerSecHistory, CurPerSecHistory.WorstFrameReceiveTimeMS))
					{
						// The state we're transitioning down into, would have last had its cooloff reset around this time
						LastMetEscalationConditions = CurTime - (double)SecondsDelta;

						bCooloffReached = false;
						break;
					}
				}

				if (bCooloffReached)
				{
					NewState--;
				}
			}
		}


		FDDoSStateConfig& OldState = DetectionSeverity[ActiveState];
		FDDoSStateConfig& CurState = DetectionSeverity[NewState];

		// If we're at anything other than the base state, then disable all unnecessary logs
		bDDoSLogRestrictions = NewState > 0;
		ActiveState = NewState;
		bMetEscalationConditionsThisFrame = false;

		CurState.ApplyState(*this);

		if (FrameAdjustment > 0.f)
		{
			CurState.ApplyAdjustedState(*this, FrameAdjustment);
		}


		UE_LOGF(LogNetCore, Warning, "Updated DDoS detection severity from '%ls' to '%ls'",
				*OldState.SeverityCategory, *CurState.SeverityCategory);

		if (bEscalate && ActiveState > WorstActiveState)
		{
			if (bDDoSAnalytics && CurState.bSendEscalateAnalytics)
			{
				NotifySeverityEscalation.ExecuteIfBound(CurState.SeverityCategory);
			}

			WorstActiveState = ActiveState;
		}
	}
}

void FDDoSDetection::PreFrameReceive(float DeltaTime)
{
	if (bDDoSDetection)
	{
		StartFrameRecvTimestamp = FPlatformTime::Seconds();
		bMetEscalationConditionsThisFrame = false;

		if (HitchTimeQuotaMS > 0 && EndFrameRecvTimestamp != 0.0)
		{
			const double HitchTimeMS = (StartFrameRecvTimestamp - EndFrameRecvTimestamp) * 1000.0;

			if ((((int32)HitchTimeMS) - HitchTimeQuotaMS) > 0)
			{
				HitchFrameCount++;

				UE_LOGF(LogNetCore, Verbose, "Detected '%i' successive hitches outside NetDriver Tick. Last Hitch: %fms (Max: %ims)",
						HitchFrameCount, HitchTimeMS, HitchTimeQuotaMS);
			}
			else
			{
				HitchFrameCount = 0;
			}
		}

		// At the start of every frame, adjust the DDoS detection based upon DeltaTime - unless there is excessive hitching
		FrameAdjustment = static_cast<float>((HitchFrameCount > 0 && HitchFrameCount > HitchFrameTolerance) ? 1.0 : (double)DeltaTime / ExpectedFrameTime);

		if (ActiveState > 0 && CooloffTime > 0 && (float)(StartFrameRecvTimestamp - LastMetEscalationConditions) > (float)CooloffTime)
		{
			UpdateSeverity(false);
		}

		DetectionSeverity[ActiveState].ApplyAdjustedState(*this, FMath::Max(0.25f, FrameAdjustment));

		if (((StartFrameRecvTimestamp - LastPerSecQuotaBegin) - 1.0) > 0.0)
		{
			UE_CLOGF(DroppedPacketCounter > 0, LogNetCore, Warning,
				"DDoS Detection dropped '%i' packets during last second (bHitFrameNonConnLimit: %i, bHitFrameNetConnLimit: %i, " "DetectionSeverity: %ls).",
				DroppedPacketCounter, (int32)bHitFrameNonConnLimit, (int32)bHitFrameNetConnLimit,
				*DetectionSeverity[ActiveState].SeverityCategory);


			// Record the last quota
			check(CounterPerSecHistory.Num() > 0);

			LastCounterPerSecHistoryIdx++;
			LastCounterPerSecHistoryIdx = (LastCounterPerSecHistoryIdx >= CounterPerSecHistory.Num()) ? 0 : LastCounterPerSecHistoryIdx;

			CounterPerSecHistory[LastCounterPerSecHistoryIdx] = *this;


			LastPerSecQuotaBegin = StartFrameRecvTimestamp;
			NonConnPacketCounter = 0;
			NetConnPacketCounter = 0;
			BadPacketCounter = 0;
			ErrorPacketCounter = 0;
			DroppedPacketCounter = 0;
			WorstFrameReceiveTimeMS = 0;
		}

		StartFramePacketCount = NonConnPacketCounter;

		if (LogHitCounter >= DDoSLogSpamLimit)
		{
			UE_LOGF(LogNetCore, Warning, "Previous frame hit DDoS LogHitCounter limit - hit count: %i (Max: %i)", LogHitCounter,
					DDoSLogSpamLimit);
		}

		LogHitCounter = 0;
		bHitFrameNonConnLimit = false;
		bHitFrameNetConnLimit = false;
	}
}

void FDDoSDetection::PostFrameReceive()
{
	if (bDDoSDetection)
	{
		// Some packet counters require an end-frame check for DDoS detection
		CheckNonConnQuotasAndLimits();


		EndFrameRecvTimestamp = FPlatformTime::Seconds();

		int32 FrameReceiveTimeMS = (int32)((EndFrameRecvTimestamp - StartFrameRecvTimestamp) * 1000.0);

		WorstFrameReceiveTimeMS = FMath::Max(FrameReceiveTimeMS, WorstFrameReceiveTimeMS);
	}
}

bool FDDoSDetection::CheckNonConnQuotasAndLimits()
{
	bool bReturnVal = false;
	double CurTime = FPlatformTime::Seconds();
	int32 TimePassedMS = (int32)((CurTime - StartFrameRecvTimestamp) * 1000.0);

	if (HasHitQuota(*this, TimePassedMS))
	{
		UpdateSeverity(true);
	}
	// Check if we're still at the conditions which led to the current escalated state
	else if (!bMetEscalationConditionsThisFrame && ActiveState > 0)
	{
		const int32 PrevState = ActiveState - 1;

		if (DetectionSeverity[PrevState].HasHitQuota(*this, TimePassedMS))
		{
			LastMetEscalationConditions = CurTime;
			bMetEscalationConditionsThisFrame = true;
		}
	}


	// NOTE: PacketLimitPerFrame == 0 is a valid value, and blocks all non-NetConnection packets
	bReturnVal = PacketLimitPerFrame == 0 || (PacketLimitPerFrame > 0 && (NonConnPacketCounter - StartFramePacketCount) >= PacketLimitPerFrame);
	bReturnVal = bReturnVal || (PacketTimeLimitMSPerFrame > 0 && TimePassedMS > PacketTimeLimitMSPerFrame);

	return bReturnVal;
}
