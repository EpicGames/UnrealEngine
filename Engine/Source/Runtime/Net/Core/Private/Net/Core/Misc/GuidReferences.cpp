// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/Misc/GuidReferences.h"
#include "HAL/IConsoleManager.h"
#include "Net/Core/Misc/NetCoreLog.h"
#include "Net/Core/PushModel/PushModel.h"
#include "UObject/CoreNet.h"
#include "UObject/UnrealType.h"

namespace UE::Net::Private
{
	bool bRemapStableSubobjects = true;
	static FAutoConsoleVariableRef CVarNetRemapStableSubobjects(TEXT("net.RemapStableSubobjects"), bRemapStableSubobjects, TEXT("If enabled, attempts to remap stable subobjects when net.OptimizedRemapping is also enabled."));
}

FGuidReferences::FGuidReferences(
	FBitReader&					InReader,
	FBitReaderMark&				InMark,
	const TSet<FNetworkGUID>&	InUnmappedGUIDs,
	const TSet<FNetworkGUID>&	InMappedDynamicGUIDs,
	const int32					InParentIndex,
	const int32					InCmdIndex,
	UPackageMap*				InPackageMap)
	: ParentIndex(InParentIndex)
	, CmdIndex(InCmdIndex)
	, UnmappedGUIDs(InUnmappedGUIDs)
	, MappedDynamicGUIDs(InMappedDynamicGUIDs)
	, Array(nullptr)
	, PackageMap(InPackageMap)
{
	NumBufferBits = UE_PTRDIFF_TO_INT32(InReader.GetPosBits() - InMark.GetPos());
	InMark.Copy(InReader, Buffer);

	TrackAllUnmappedGUIDs();
}

FGuidReferences::FGuidReferences(const FGuidReferences& Other)
	: ParentIndex(Other.ParentIndex)
	, CmdIndex(Other.CmdIndex)
	, NumBufferBits(Other.NumBufferBits)
	, UnmappedGUIDs(Other.UnmappedGUIDs)
	, MappedDynamicGUIDs(Other.MappedDynamicGUIDs)
	, Buffer(Other.Buffer)
	, Array(nullptr)
	, PackageMap(Other.PackageMap)
{
	if (Other.Array)
	{
		Array = new FGuidReferencesMap(*Other.Array);
	}

	TrackAllUnmappedGUIDs();
}

FGuidReferences& FGuidReferences::operator=(FGuidReferences&& Other)
{
	if (this != &Other)
	{
		delete Array;

		ParentIndex = Other.ParentIndex;
		CmdIndex = Other.CmdIndex;
		NumBufferBits = Other.NumBufferBits;
		UnmappedGUIDs = MoveTemp(Other.UnmappedGUIDs);
		MappedDynamicGUIDs = MoveTemp(Other.MappedDynamicGUIDs);
		Buffer = MoveTemp(Other.Buffer);
		Array = Other.Array;
		PackageMap = Other.PackageMap;

		Other.Array = nullptr;
		Other.NumBufferBits = 0;
		Other.PackageMap.Reset();
	}

	return *this;
}

FGuidReferences::FGuidReferences(FGuidReferences&& Other) 
	: ParentIndex(Other.ParentIndex)
	, CmdIndex(Other.CmdIndex)
	, NumBufferBits(Other.NumBufferBits)
	, UnmappedGUIDs(MoveTemp(Other.UnmappedGUIDs))
	, MappedDynamicGUIDs(MoveTemp(Other.MappedDynamicGUIDs))
	, Buffer(MoveTemp(Other.Buffer))
	, Array(Other.Array)
	, PackageMap(Other.PackageMap)
{
	Other.Array = nullptr;
	Other.NumBufferBits = 0;
	Other.PackageMap.Reset();
}

FGuidReferences::~FGuidReferences()
{
	delete Array;

	if (UE::Net::Private::bRemapStableSubobjects)
	{
		if (UPackageMap* LocalPackageMap = PackageMap.Get())
		{
			for (const FNetworkGUID NetGUID : UnmappedGUIDs)
			{
				LocalPackageMap->RemoveUnmappedNetGUIDReference(NetGUID);
			}
		}
	}
}

void FGuidReferences::AddUnmappedGUID(FNetworkGUID InGUID)
{
	bool bIsAlreadyInSet = false;
	UnmappedGUIDs.Add(InGUID, &bIsAlreadyInSet);

	if (!bIsAlreadyInSet && UE::Net::Private::bRemapStableSubobjects)
	{
		if (UPackageMap* LocalPackageMap = PackageMap.Get())
		{
			PackageMap->AddUnmappedNetGUIDReference(InGUID);
		}
	}
}

void FGuidReferences::RemoveUnmappedGUID(FNetworkGUID InGUID)
{
	int32 NumRemoved = UnmappedGUIDs.Remove(InGUID);

	if (NumRemoved > 0 && UE::Net::Private::bRemapStableSubobjects)
	{
		ensureMsgf(NumRemoved == 1, TEXT("FGuidReferences::RemoveUnmappedGUID: removed more than one GUID from UnmappedGUIDs set."));

		if (UPackageMap* LocalPackageMap = PackageMap.Get())
		{
			PackageMap->RemoveUnmappedNetGUIDReference(InGUID);
		}
	}
}

bool FGuidReferences::UpdateUnmappedGUIDs(UPackageMap* InPackageMap, UObject* OriginalObject, const FProperty* Property, int32 AbsOffset)
{
	checkf(InPackageMap != nullptr, TEXT("FGuidReferences::UpdateUnmappedGUIDs: invalid PackageMap"));

	bool bMappedSomeGUIDs = false;

	for (auto UnmappedIt = UnmappedGUIDs.CreateIterator(); UnmappedIt; ++UnmappedIt)
	{
		const FNetworkGUID& GUID = *UnmappedIt;

		if (InPackageMap->IsGUIDBroken(GUID, false))
		{
			UE_LOGF(LogNetCore, Warning, "FGuidReferences::UpdateUnmappedGUIDs: Broken GUID. NetGuid: %ls", *GUID.ToString());
			InPackageMap->RemoveUnmappedNetGUIDReference(GUID);
			UnmappedIt.RemoveCurrent();
			continue;
		}

		UObject* Object = InPackageMap->GetObjectFromNetGUID(GUID, false);

		if (Object != nullptr)
		{
			UE_LOGF(LogNetCore, VeryVerbose, "FGuidReferences::UpdateUnmappedGUIDs: REMOVED unmapped property: Offset: %i, Guid: %ls, PropName: %ls, ObjName: %ls", AbsOffset, *GUID.ToString(), *Property->GetName(), *Object->GetName());

			if (GUID.IsDynamic())
			{
				// If this guid is dynamic, move it to the dynamic guids list
				MappedDynamicGUIDs.Add(GUID);
			}

			// Remove from unmapped guids list
			InPackageMap->RemoveUnmappedNetGUIDReference(GUID);
			UnmappedIt.RemoveCurrent();
			bMappedSomeGUIDs = true;

#if WITH_PUSH_MODEL
			MARK_PROPERTY_DIRTY_UNSAFE(OriginalObject, ParentIndex);
#endif
		}
	}

	return bMappedSomeGUIDs;
}

void FGuidReferences::TrackAllUnmappedGUIDs() const
{
	if (UE::Net::Private::bRemapStableSubobjects)
	{
		if (UPackageMap* LocalPackageMap = PackageMap.Get())
		{
			for (const FNetworkGUID UnmappedGUID : UnmappedGUIDs)
			{
				LocalPackageMap->AddUnmappedNetGUIDReference(UnmappedGUID);
			}
		}
	}
}