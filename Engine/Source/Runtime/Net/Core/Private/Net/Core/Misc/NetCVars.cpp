// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/Misc/NetCVars.h"
#include "HAL/IConsoleManager.h"

namespace UE::Net
{

int32 CVar_ForceConnectionViewerPriority = 1;
// Deprecated RepGraph specific name.
static FAutoConsoleVariableRef CVarRepGraphForceConnectionViewerPriority(
	TEXT("Net.RepGraph.ForceConnectionViewerPriority"), 
	CVar_ForceConnectionViewerPriority, 
	TEXT("Force the connection's player controller and viewing pawn as topmost priority. Same as Net.ForceConnectionViewerPriority.")
);

// New name without RepGraph as this is used by Iris as well.
static FAutoConsoleVariableRef CVarForceConnectionViewerPriority(
	TEXT("Net.ForceConnectionViewerPriority"), 
	CVar_ForceConnectionViewerPriority, 
	TEXT("Force the connection's player controller and viewing pawn as topmost priority.")
);

bool bAutoRegisterReplicatedProperties = true;
static FAutoConsoleVariableRef CVarAutoRegisterReplicatedProperties(
	TEXT("Net.AutoRegisterReplicatedProperties"),
	bAutoRegisterReplicatedProperties,
	TEXT("Automatically register replicated variables if they are not registered by the class in GetLifetimeReplicatedProps."), 
	ECVF_Default);

bool bEnsureForMissingProperties = false;
static FAutoConsoleVariableRef CVarEnsureForMissingProperties(
	TEXT("Net.EnsureOnMissingReplicatedPropertiesRegister"),
	bEnsureForMissingProperties,
	TEXT("Ensure when we detect a missing replicated property in GetLifetimeReplicatedProps of the class."),
	ECVF_Default);

/*
 * FastArrays and other custom delta properties may have order dependencies due to callbacks being fired during serialization at which time other custom delta properties have not yet received their state.
 * This cvar toggles the behavior between using the RepIndex of the property or the order of appearance in the lifetime property array filled during a GetLifetimeReplicatedProps() call.
 * Default is false to keep the legacy behavior of using the GetLifetimeReplicatedProps() order for the custom delta properties.
 * The cvar is used in ReplicationStateDescriptorBuilder as well. Search for the cvar name in the code base before removing it.
*/
bool bReplicateCustomDeltaPropertiesInRepIndexOrder = false;
static FAutoConsoleVariableRef CVarReplicateCustomDeltaPropertiesInRepIndexOrder(
	TEXT("net.ReplicateCustomDeltaPropertiesInRepIndexOrder"),
	bReplicateCustomDeltaPropertiesInRepIndexOrder,
	TEXT("If false (default) custom delta properties will replicate in the same order as they're added to the lifetime property array during the call to GetLifetimeReplicatedProps. If true custom delta properties will be replicated in the property RepIndex order, which is typically in increasing property offset order. Note that custom delta properties are always serialized after regular properties.")
);

}
