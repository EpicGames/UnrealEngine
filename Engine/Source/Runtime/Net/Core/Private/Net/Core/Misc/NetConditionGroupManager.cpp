// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/Misc/NetConditionGroupManager.h"
#include "UObject/Object.h"

extern const FName UE::Net::NetGroupOwner(TEXT("NetGroupOwner"));
extern const FName UE::Net::NetGroupReplay(TEXT("NetGroupReplay"));

namespace UE::Net
{

bool IsSpecialNetConditionGroup(FName NetConditionGroup)
{
	return NetConditionGroup == NetGroupOwner || NetConditionGroup == NetGroupReplay;
}

void FNetConditionGroupManager::RegisterSubObjectInGroup(UObject* SubObject, FName NetConditionGroup)
{
	check(IsValid(SubObject));
	checkf(!NetConditionGroup.IsNone(), TEXT("%s was registered with invalid netcondition group NONE"), *GetNameSafe(SubObject));

	FSubObjectGroupInfo& SubObjectInfo = SubObjectGroupMap.FindOrAdd(SubObject);
	SubObjectInfo.NetGroups.AddUnique(NetConditionGroup);
}

void FNetConditionGroupManager::RegisterSubObjectInMultipleGroups(UObject* SubObject, TArrayView<FName> NetConditionGroups)
{
	check(IsValid(SubObject));

	FSubObjectGroupInfo& SubObjectInfo = SubObjectGroupMap.FindOrAdd(SubObject);

	for (FName GroupName : NetConditionGroups)
	{
		checkf(!GroupName.IsNone(), TEXT("%s was registered with invalid netcondition group NONE"), *GetNameSafe(SubObject));
		SubObjectInfo.NetGroups.AddUnique(GroupName);
	}
}

void FNetConditionGroupManager::UnregisterSubObjectFromGroup(UObject* SubObject, FName NetConditionGroup)
{
	check(SubObject);
	const FObjectKey SubObjectKey = SubObject;

	if (FSubObjectGroupInfo* SubObjectInfo = SubObjectGroupMap.Find(SubObjectKey))
	{
		SubObjectInfo->NetGroups.RemoveSingleSwap(NetConditionGroup);

		if (SubObjectInfo->NetGroups.Num() == 0)
		{
			// The object is not in any group anymore. Delete the entry from the TMap
			SubObjectGroupMap.Remove(SubObjectKey);
		}
	}
}

void FNetConditionGroupManager::UnregisterSubObjectFromMultipleGroups(UObject* SubObject, TArrayView<FName> NetConditionGroups)
{
	check(SubObject);
	const FObjectKey SubObjectKey = SubObject;

	if (FSubObjectGroupInfo* SubObjectInfo = SubObjectGroupMap.Find(SubObjectKey))
	{
		for (FName GroupName : NetConditionGroups)
		{
			SubObjectInfo->NetGroups.RemoveSingleSwap(GroupName);
		}

		if (SubObjectInfo->NetGroups.Num() == 0)
		{
			// The object is not in any group anymore. Delete the entry from the TMap
			SubObjectGroupMap.Remove(SubObjectKey);
		}
	}
}

void FNetConditionGroupManager::UnregisterSubObjectFromAllGroups(UObject* SubObject)
{
	check(SubObject);
	SubObjectGroupMap.Remove(SubObject);
}

void FNetConditionGroupManager::CountBytes(FArchive& Ar) const
{
	SubObjectGroupMap.CountBytes(Ar);
}

} // end namespace UE::Net