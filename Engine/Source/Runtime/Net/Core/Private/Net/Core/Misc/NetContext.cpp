// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/Misc/NetContext.h"
#include "Misc/AssertionMacros.h"

namespace UE::Net
{

bool FNetContext::bIsInRPCStack = false;

FScopedNetContextRPC::FScopedNetContextRPC()
{
	FNetContext::bIsInRPCStack = true;
}

FScopedNetContextRPC::~FScopedNetContextRPC()
{
	checkSlow(FNetContext::bIsInRPCStack); // Cannot have two remote rpc stacked on top of each other
	FNetContext::bIsInRPCStack = false;
}

} // end namespace UE::Net

