// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/Misc/NetSubObjectRegistry.h"

namespace UE::Net 
{

// ----------------------------------------------------------------
//	FSubObjectRegistry
// ----------------------------------------------------------------
FSubObjectRegistry::EResult FSubObjectRegistry::AddSubObjectUnique(UObject* InSubObject, ELifetimeCondition InNetCondition)
{
	check(InSubObject);

	const FEntry SubObjectEntry(InSubObject, InNetCondition);

	const int32 CurrentIndex = Registry.Find(SubObjectEntry);
	if (CurrentIndex == INDEX_NONE)
	{
		Registry.Add(SubObjectEntry);
		return EResult::NewEntry;
	}
	else
	{
		// Check if the net condition is different.
		const FEntry& ActiveEntry = Registry[CurrentIndex];
		return (ActiveEntry.NetCondition == InNetCondition) ? EResult::AlreadyRegistered : EResult::NetConditionConflict;
	}
}

ELifetimeCondition FSubObjectRegistry::GetNetCondition(UObject* InSubObject) const
{
	const int32 CurrentIndex = Registry.IndexOfByKey(InSubObject);
	return CurrentIndex == INDEX_NONE ? COND_Max : Registry[CurrentIndex].NetCondition;
}

bool FSubObjectRegistry::RemoveSubObject(UObject* InSubObject)
{
	int32 Index = Registry.IndexOfByKey(InSubObject);
	if (Index != INDEX_NONE)
	{
		Registry.RemoveAt(Index);
		return true;
	}
	return false;
}

bool FSubObjectRegistry::IsSubObjectInRegistry(const UObject* SubObject) const
{
	return Registry.FindByKey(SubObject) != nullptr;
}

void FSubObjectRegistry::CleanRegistryIndexes(const TArrayView<int32>& IndexesToClean)
{
#if !UE_BUILD_SHIPPING
	// Test to make sure the array is sorted 
	int32 LastRemovedIndex = INT32_MAX;
#endif
	
	for (int32 i=IndexesToClean.Num()-1; i >= 0; --i)
	{
		const int32 IndexToRemove = IndexesToClean[i];

#if !UE_BUILD_SHIPPING
		const bool bIsSorted = IndexToRemove < LastRemovedIndex;
		ensureMsgf(bIsSorted, TEXT("CleanRegistryIndexes did not receive a sorted array. Index value %d (position %d) was >= then Index value %d (position %d)"), LastRemovedIndex, i + 1, IndexToRemove, i);
		LastRemovedIndex = IndexToRemove;
		if (!bIsSorted)
		{
			continue;
		}
#endif

		Registry.RemoveAtSwap(IndexToRemove);
	}
}

} //namespace UE::Net