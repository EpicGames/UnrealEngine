// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Includes
 */
#include "Net/Core/Misc/PacketAudit.h"
#include "Net/Core/Misc/NetCoreLog.h"

#include "Misc/CommandLine.h"
#include "Serialization/BitReader.h"
#include "Serialization/MemoryArchive.h"


#if !UE_BUILD_SHIPPING
/**
 * Defines
 */

/** The general name used/adapted for mutexes and shared memory */
#define AUDIT_MUTEX_NAME	TEXT("UEPacketAudit")

/** Hardcoded amount of shared memory reserved for both the send/receive maps */
#define AUDIT_MAPPING_SIZE	(1024 * 1024 * 32)

/** Maximum allowed number of packets in the send/receive maps, to detect leaks */
#define AUDIT_MAX_PACKETS	2048


/**
 * Globals
 */
FPacketAudit* GPacketAuditor = nullptr;


/**
 * Stored data for a single packet stage
 */
struct FPacketStageData
{
	/** The size of the packet, in bits */
	uint32 SizeBits;

	/** The CRC of this packet stage */
	uint32 StageCRC;


	friend FArchive& operator << (FArchive& Ar, FPacketStageData& StageData)
	{
		return Ar << StageData.SizeBits << StageData.StageCRC;
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("(SizeBits: %i, StageCRC: %08X)"), SizeBits, StageCRC);
	}
};

/**
 * Stored data for an entire packet
 */
struct FPacketAuditData
{
	/** The size of the packet, in bits */
	uint32								SizeBits;

	/** Map of the stage name, and the packet data for this stage */
	TMap<FString, FPacketStageData>		StageMap;


	friend FArchive& operator << (FArchive& Ar, FPacketAuditData& AuditData)
	{
		return Ar << AuditData.SizeBits << AuditData.StageMap;
	}

	FString ToString(int32 BaseIndent) const
	{
		FString ReturnVal;
		FString IndentStr;

		for (int32 i=0; i<BaseIndent; i++)
		{
			IndentStr +=  TEXT("\t");
		}

		ReturnVal += IndentStr + TEXT("(\r\n");
		ReturnVal += IndentStr + FString::Printf(TEXT("\tSizeBits: %i,\r\n"), SizeBits);
		ReturnVal += IndentStr + TEXT("\tStageMap:\r\n");
		ReturnVal += IndentStr + TEXT("\t(\r\n");

		for (TMap<FString, FPacketStageData>::TConstIterator It(StageMap); It; ++It)
		{
			ReturnVal += IndentStr + FString::Printf(TEXT("\t\t(StageName: %s, StageData: %s),\r\n"), *It.Key(),
														*It.Value().ToString());
		}

		ReturnVal += IndentStr + TEXT("\t)\r\n");
		ReturnVal += IndentStr + TEXT(")");

		return ReturnVal;
	}
};

/**
 * Tracks a pending packet send
 */
struct FPendingPacket
{
	/** The FBitWriter associated with the packet */
	FBitWriter*							Writer;

	/** The stage data being collected for the packet */
	TMap<FString, FPacketStageData>		StageMap;
};


/** Map of low-level packet CRC's, and the audit data associated with those packets, for packets being sent. */
static TMap<uint32, FPacketAuditData>		GSendPacketAudits;

/** As above, except for received packets */
static TMap<uint32, FPacketAuditData>		GReceivePacketAudits;

/** Collects stage data, for the current packet being sent (local only) */
static FPendingPacket						GPendingSendPacket;

/** Stores the CRC for the current packet being received */
static uint32								GCurrentReceivePacket;

/** Stores the CRC for the packet that was last removed from GReceivePacketAudits (reset to 0 when receiving a new packet) */
static uint32								GLastReceivePacket = 0;


/**
 * FSharedMemoryReader - based on FMemoryReader (just takes a pointer instead of array)
 *
 * NOTE: Hardcoded to AUDIT_MAPPING_SIZE
 */
class FSharedMemoryReader : public FMemoryArchive
{
public:
	/**
	 * Default constructor
	 */
	FSharedMemoryReader(uint8* InData)
	{
		MemSizeData = (uint32*)InData;
		MemPayloadData = InData + sizeof(uint32);

		check(*MemSizeData <= (AUDIT_MAPPING_SIZE - sizeof(uint32)));

		this->SetIsLoading(true);
	}

	virtual int64 TotalSize() override
	{
		return *MemSizeData;
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		if (Num > 0 && !IsError())
		{
			if (Offset + Num <= TotalSize())
			{
				FMemory::Memcpy(Data, MemPayloadData + Offset, Num);
				Offset += Num;
			}
			else
			{
				SetError();
			}
		}
	}

private:
	/** The memory location storing the size of the archive */
	uint32*		MemSizeData;

	/** The memory location storing the archive payload */
	uint8*		MemPayloadData;
};

/**
 * FSharedMemoryWriter	- as above, same as FMemoryWriter except takes a pointer
 *
 * NOTE: Hardcoded to AUDIT_MAPPING_SIZE
 * NOTE: Immediately invalidates existing data upon construction
 */
class FSharedMemoryWriter : public FMemoryArchive
{
public:
	/**
	 * Default constructor
	 */
	FSharedMemoryWriter(uint8* InData)
	{
		MemSizeData = (uint32*)InData;
		MemPayloadData = InData + sizeof(uint32);

		this->SetIsSaving(true);

		*MemSizeData = 0;
	}

	virtual int64 TotalSize() override
	{
		return *MemSizeData;
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		check(Offset + Num < (AUDIT_MAPPING_SIZE - sizeof(uint32)));

		if (Num > 0)
		{
			FMemory::Memcpy(MemPayloadData + Offset, Data, Num);

			Offset += Num;

			*MemSizeData = IntCastChecked<uint32>(Offset);
		}
	}

private:
	/** The memory location storing the size of the archive */
	uint32*		MemSizeData;

	/** The memory location storing the archive payload */
	uint8*		MemPayloadData;
};
#endif


/**
 * FPacketAudit
 */

FPacketAudit::FPacketAudit(FPlatformProcess::FSemaphore* InGameMutex)
	: GameMutex(InGameMutex)
{
#if !UE_BUILD_SHIPPING
	SharedMutex = FPlatformProcess::NewInterprocessSynchObject(AUDIT_MUTEX_NAME, true);

	FString SendMemName = FString(AUDIT_MUTEX_NAME) + (GIsServer ? TEXT("Server") : TEXT("Client")) + TEXT("Send");
	FString ReceiveMemName = FString(AUDIT_MUTEX_NAME) + (GIsServer ? TEXT("Client") : TEXT("Server")) + TEXT("Send");

	SendSharedMemory = FPlatformMemory::MapNamedSharedMemoryRegion(SendMemName, true,
					(FPlatformMemory::ESharedMemoryAccess::Read | FPlatformMemory::ESharedMemoryAccess::Write), AUDIT_MAPPING_SIZE);

	ReceiveSharedMemory = FPlatformMemory::MapNamedSharedMemoryRegion(ReceiveMemName, true,
					(FPlatformMemory::ESharedMemoryAccess::Read | FPlatformMemory::ESharedMemoryAccess::Write), AUDIT_MAPPING_SIZE);


	// @todo #JohnB: There is probably a way to fix the shared memory platform code, so this works without admin. Low priority though.
	if (SendSharedMemory == nullptr || ReceiveSharedMemory == nullptr)
	{
		LowLevelFatalError(TEXT("You need to run UnrealEngine as administrator, for packet auditing to work."));
		return;
	}

	FSharedMemoryWriter WipeSendSharedMemory((uint8*)SendSharedMemory->GetAddress());
	FSharedMemoryWriter WipeReceiveSharedMemory((uint8*)ReceiveSharedMemory->GetAddress());
#endif
}

FPacketAudit::~FPacketAudit()
{
#if !UE_BUILD_SHIPPING
	check(GameMutex != nullptr);
	check(SharedMutex != nullptr);
	check(SendSharedMemory != nullptr);
	check(ReceiveSharedMemory != nullptr);

	GameMutex->Unlock();

	FPlatformProcess::DeleteInterprocessSynchObject(GameMutex);
	FPlatformProcess::DeleteInterprocessSynchObject(SharedMutex);

	GameMutex = nullptr;
	SharedMutex = nullptr;

	FPlatformMemory::UnmapNamedSharedMemoryRegion(SendSharedMemory);
	FPlatformMemory::UnmapNamedSharedMemoryRegion(ReceiveSharedMemory);
#endif
}

void FPacketAudit::Init()
{
#if !UE_BUILD_SHIPPING
	if (GPacketAuditor == nullptr && FParse::Param(FCommandLine::Get(), TEXT("PacketAudit")))
	{
		FString MutexName = FString(AUDIT_MUTEX_NAME) + (GIsServer ? TEXT("ServerProcess") : TEXT("ClientProcess"));
		FPlatformProcess::FSemaphore* CurGameMutex = FPlatformProcess::NewInterprocessSynchObject(MutexName, true);

		if (CurGameMutex->TryLock(1))
		{
			GPacketAuditor = new FPacketAudit(CurGameMutex);
		}
		else
		{
			UE_LOGF(LogNetCore, Log, "%ls", *(FString("Packet auditor already active for a game ") +
					(GIsServer ? TEXT("server") : TEXT("client")) + TEXT(", can't start multiple instances.")));

			FPlatformProcess::DeleteInterprocessSynchObject(CurGameMutex);
		}
	}
#endif
}

void FPacketAudit::Destruct()
{
#if !UE_BUILD_SHIPPING
	delete GPacketAuditor;
	GPacketAuditor = nullptr;
#endif
}

void FPacketAudit::AddStage_Internal(FString StageName, FBitWriter& OutPacket, bool bByteAligned)
{
#if !UE_BUILD_SHIPPING
	if (GPendingSendPacket.Writer == nullptr)
	{
		GPendingSendPacket.Writer = &OutPacket;
	}

	check(GPendingSendPacket.Writer == &OutPacket);

	FPacketStageData& StageData = GPendingSendPacket.StageMap.Add(StageName);

	StageData.SizeBits = IntCastChecked<uint32>(bByteAligned ? (OutPacket.GetNumBytes() * 8) : OutPacket.GetNumBits());
	StageData.StageCRC = PacketCRC(OutPacket.GetData(), StageData.SizeBits);
#endif
}

void FPacketAudit::CheckStage_Internal(FString StageName, FBitReader& InPacket, bool bByteAligned)
{
#if !UE_BUILD_SHIPPING
	{
		FScopedAuditAccess AuditScope(*this);
		FPacketAuditData* AuditData = GReceivePacketAudits.Find(GCurrentReceivePacket);

		if (AuditData != nullptr)
		{
			FPacketStageData* StageData = AuditData->StageMap.Find(StageName);

			if (StageData != nullptr)
			{
				uint32 BitsLeft = IntCastChecked<uint8>(bByteAligned ? ((InPacket.GetNumBytes() * 8) - InPacket.GetPosBits()) : InPacket.GetBitsLeft());

				if (BitsLeft == StageData->SizeBits)
				{
					uint32 PacketStageCRC = PacketCRC(InPacket.GetData(), BitsLeft);

					if (PacketStageCRC == StageData->StageCRC)
					{
						AuditData->StageMap.Remove(StageName);

						if (AuditData->StageMap.Num() == 0)
						{
							GReceivePacketAudits.Remove(GCurrentReceivePacket);

							GLastReceivePacket = GCurrentReceivePacket;
						}
					}
					else
					{
						DumpAuditData(AuditScope, GCurrentReceivePacket);
						LowLevelFatalError(TEXT("%s: Expected Stage CRC '%08X', got Stage CRC '%08X'."), *StageName,
											StageData->StageCRC, PacketStageCRC);
					}
				}
				else
				{
					DumpAuditData(AuditScope, GCurrentReceivePacket);
					LowLevelFatalError(TEXT("%s: Expected '%i' bits left, got '%i'."), *StageName, StageData->SizeBits, BitsLeft);
				}
			}
			else
			{
				DumpAuditData(AuditScope, GCurrentReceivePacket);
				LowLevelFatalError(TEXT("%s: Could not find StageName."), *StageName);
			}
		}
		else
		{
			DumpAuditData(AuditScope, GCurrentReceivePacket);

			if (GLastReceivePacket == GCurrentReceivePacket)
			{
				LowLevelFatalError(TEXT("%s: Already finished proccesing all stages from packet."), *StageName);
			}
			else
			{
				LowLevelFatalError(TEXT("%s: Failed to find packet CRC: %08X. Duplicate/resent packet?"), *StageName,
									GCurrentReceivePacket);
			}
		}
	}
#endif
}

void FPacketAudit::NotifyLowLevelSend_Internal(FBitWriter& OutPacket)
{
#if !UE_BUILD_SHIPPING
	if (GPendingSendPacket.Writer != nullptr)
	{
		// @todo #JohnB: Restore when you have the whole packet pipeline unified
		//check(GPendingSendPacket.Writer == &OutPacket);

		uint32 OutSizeBits = IntCastChecked<uint32>(OutPacket.GetNumBits());

		if (OutSizeBits > 0)
		{
			uint32 OutPacketCRC = PacketCRC(OutPacket.GetData(), OutSizeBits);

			{
				FScopedAuditAccess AuditScope(*this);

				// @todo: Most often this will be due to reliability handling. Collisions may be possible though.
				check(!GSendPacketAudits.Contains(OutPacketCRC));


				check(GSendPacketAudits.Num() < AUDIT_MAX_PACKETS);

				FPacketAuditData& AuditData = GSendPacketAudits.Add(OutPacketCRC);

				AuditData.SizeBits = OutSizeBits;

				AuditData.StageMap = GPendingSendPacket.StageMap;
			}
		}

		GPendingSendPacket.Writer = nullptr;
		GPendingSendPacket.StageMap.Empty();
	}
#endif
}

void FPacketAudit::NotifyLowLevelReceive_Internal(FBitReader& InPacket)
{
#if !UE_BUILD_SHIPPING
	GCurrentReceivePacket = PacketCRC(InPacket.GetData(), IntCastChecked<uint32>(InPacket.GetNumBits()));
	GLastReceivePacket = 0;
#endif
}

uint32 FPacketAudit::PacketCRC(uint8* Data, uint32 DataLenBits)
{
	uint32 ReturnVal = 0;

	if (DataLenBits > 0)
	{
		uint32 PacketByteSize = (DataLenBits + 7) >> 3;
		uint8* PacketData = new uint8[PacketByteSize];

		PacketData[PacketByteSize-1] = 0;

		appBitsCpy(PacketData, 0, Data, 0, DataLenBits);

		ReturnVal = FCrc::MemCrc32(PacketData, PacketByteSize);

		delete[] PacketData;
	}

	return ReturnVal;
}

#if !UE_BUILD_SHIPPING
void FPacketAudit::DumpAuditData(FScopedAuditAccess& AuditLock, uint32 InPacketCRC/*=0*/)
{
	FString FinalLogStr = (InPacketCRC != 0 ? FString::Printf(TEXT("PacketCRC: %08X,\r\n"), InPacketCRC) : TEXT(""));
	
	FinalLogStr += TEXT("GSendPacketAudits:\r\n(\r\n");

	for (TMap<uint32, FPacketAuditData>::TConstIterator It(GSendPacketAudits); It; ++It)
	{
		FinalLogStr += FString::Printf(TEXT("\tPacketCRC: %08X,\r\n"), It.Key());
		FinalLogStr += TEXT("\tAuditData:\r\n");
		FinalLogStr += It.Value().ToString(1) + TEXT("\r\n");
	}

	FinalLogStr += TEXT("),\r\nGReceivePacketAudits:\r\n(\r\n");

	for (TMap<uint32, FPacketAuditData>::TConstIterator It(GReceivePacketAudits); It; ++It)
	{
		FinalLogStr += FString::Printf(TEXT("\tPacketCRC: %08X,\r\n"), It.Key());
		FinalLogStr += TEXT("\tAuditData:\r\n");
		FinalLogStr += It.Value().ToString(1) + TEXT("\r\n");
	}

	FinalLogStr += TEXT(")");

	UE_LOGF(LogNetCore, Log, "%ls", *FinalLogStr);
}


FPacketAudit::FScopedAuditAccess::FScopedAuditAccess(FPacketAudit& InAuditor)
	: Auditor(InAuditor)
{
	Auditor.SharedMutex->Lock();

	{
		FSharedMemoryReader SendReader((uint8*)Auditor.SendSharedMemory->GetAddress());
		FSharedMemoryReader ReceiveReader((uint8*)Auditor.ReceiveSharedMemory->GetAddress());

		SendReader << GSendPacketAudits;
		ReceiveReader << GReceivePacketAudits;
	}
}

FPacketAudit::FScopedAuditAccess::~FScopedAuditAccess()
{
	{
		FSharedMemoryWriter SendWriter((uint8*)Auditor.SendSharedMemory->GetAddress());
		FSharedMemoryWriter ReceiveWriter((uint8*)Auditor.ReceiveSharedMemory->GetAddress());

		SendWriter << GSendPacketAudits;
		ReceiveWriter << GReceivePacketAudits;
	}

	Auditor.SharedMutex->Unlock();
}
#endif

