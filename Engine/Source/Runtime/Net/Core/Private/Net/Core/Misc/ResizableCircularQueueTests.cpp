// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Net/Core/Misc/ResizableCircularQueue.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FResizableCircularQueueTest, "Net.ResizableCircularQueueTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

struct FResizableCircularQueueTestUtil
{
	typedef TResizableCircularQueue<uint32> QueueT;

	static bool VerifyQueueIntegrity(const QueueT& Queue, QueueT::ElementT ExpectedValueAtFront, QueueT::ElementT Increment)
	{
		bool bSuccess = true;
		SIZE_T Offset = Queue.Count() - 1;
	
		QueueT::ElementT ExpectedValue = ExpectedValueAtFront;

		// Peek elements in queue at given offset, peek from back to front
		for (SIZE_T It = 0, EndIt = Queue.Count(); It < EndIt; ++It)
		{
			bSuccess = bSuccess && (ExpectedValue == Queue.PeekAtOffset(It));
			ExpectedValue += Increment;
		}

		return bSuccess;
	}

	static void OverideHeadAndTail(QueueT& Queue, uint32 Head, uint32 Tail)
	{
		Queue.Head = Head;
		Queue.Tail = Tail;
	}
};

bool FResizableCircularQueueTest::RunTest(const FString& Parameters)
{
	typedef FResizableCircularQueueTestUtil::QueueT::ElementT ElementT;

	// Test empty
	{
		FResizableCircularQueueTestUtil::QueueT Q(0);

		TestEqual(TEXT("Test empty - Size"), Q.Count(), SIZE_T(0));
		TestTrue(TEXT("Test empty - IsEmpty"), Q.IsEmpty());
		TestEqual(TEXT("Test empty - Capacity"), Q.AllocatedCapacity(), SIZE_T(0));
	}

	// Test Push to Capacity
	{
		const ElementT ElementsToPush = 8;

		FResizableCircularQueueTestUtil::QueueT Q(ElementsToPush);

		for (ElementT It=0; It < ElementsToPush; ++It)
		{
			Q.Enqueue(It);
		}
		
		TestEqual(TEXT("Test Push to Capacity - Size"), Q.Count(), ElementsToPush);
		TestEqual(TEXT("Test Push to Capacity - Capacity"), Q.AllocatedCapacity(), ElementsToPush);
 		TestTrue(TEXT("Test Push to Capacity - Expected"), FResizableCircularQueueTestUtil::VerifyQueueIntegrity(Q, 0, 1));
	}

	// Test Push over Capacity
	{
		const ElementT InitialQueueCapacity = 32;

		FResizableCircularQueueTestUtil::QueueT Q(InitialQueueCapacity);

		for (ElementT It=0; It < InitialQueueCapacity; ++It)
		{
			Q.Enqueue(It);
		}
		Q.Pop();
		Q.Enqueue(InitialQueueCapacity);
		Q.Enqueue(InitialQueueCapacity + 1);
		
		TestEqual(TEXT("Test Push over Capacity - Size"), Q.Count(), InitialQueueCapacity+1);
		TestTrue(TEXT("Test Push over Capacity - Capacity"), Q.AllocatedCapacity() >= InitialQueueCapacity+1);
 		TestTrue(TEXT("Test Push over Capacity - Expected"), FResizableCircularQueueTestUtil::VerifyQueueIntegrity(Q, 1, 1));
	}

	// Test Push and Pop
	{
		const ElementT ElementsToPush = 256;
		const SIZE_T ElementPopMod = 16;
		const SIZE_T ExpectedSize = 256 - ElementPopMod;
		const SIZE_T ExpectedCapacity = 256;

		FResizableCircularQueueTestUtil::QueueT Q(4);

		uint32 ExpectedPoppedValue = 0;
		for (ElementT It = 0; It < ElementsToPush; ++It)
		{
			Q.Enqueue(It);
			TestEqual(TEXT("Test Push and pop - Push"), It, Q.PeekAtOffset(Q.Count() - 1));

			if (It % ElementPopMod == 0)
			{
				const uint32 PoppedValue = Q.PeekAtOffset(0);
				TestEqual(TEXT("Test Push and pop - Pop"), ExpectedPoppedValue, PoppedValue);
				++ExpectedPoppedValue;
				Q.Pop();
			}
		}

		TestEqual(TEXT("Test Push and pop - Size"), Q.Count(), ExpectedSize);
		TestEqual(TEXT("Test Push and pop - Capacity"), Q.AllocatedCapacity(), ExpectedCapacity);
 		TestTrue (TEXT("Test Push and pop - Expected"), FResizableCircularQueueTestUtil::VerifyQueueIntegrity(Q, ElementPopMod, 1));	
	}

	// Test Push and pop all
	{
		const ElementT ElementsToPush = 256;

		FResizableCircularQueueTestUtil::QueueT Q(ElementsToPush);

		TestTrue(TEXT ("Test Push and pop all - IsEmpty before"), Q.IsEmpty());
		TestEqual(TEXT("Test Push and pop all - Size before"), Q.Count(), SIZE_T(0));

		for (ElementT It=0; It < ElementsToPush; ++It)
		{
			Q.Enqueue(It);
		}
		
		TestEqual(TEXT("Test Push and pop all - Size"), Q.Count(), ElementsToPush);
		TestEqual(TEXT("Test Push and pop all - Capacity"), Q.AllocatedCapacity(), ElementsToPush);
 		TestTrue (TEXT("Test Push and pop all - Expected"), FResizableCircularQueueTestUtil::VerifyQueueIntegrity(Q, 0, 1));

		for (ElementT It=0; It < ElementsToPush; ++It)
		{
			Q.Pop();
		}
		
		TestTrue( TEXT("Test Push and pop all - IsEmpty after"), Q.IsEmpty());
		TestEqual(TEXT("Test Push and pop all - Size after"), Q.Count(), SIZE_T(0));
		TestEqual(TEXT("Test Push and pop all - Capacity after"), Q.AllocatedCapacity(), ElementsToPush);
	}

	// Test multi pop of element with non-trivial destructor
	{
		struct FStructWithDestructorForResizableCiruclarQueuePopTest
		{
			~FStructWithDestructorForResizableCiruclarQueuePopTest()
			{
				--Count;
				check(Count == 0);
			}

			int Count = 1;
		};

		using FMyTestQueue = TResizableCircularQueue<FStructWithDestructorForResizableCiruclarQueuePopTest>;

		// Pop zero
		{
			FMyTestQueue Q(0);
			Q.Enqueue(FStructWithDestructorForResizableCiruclarQueuePopTest());
			Q.Pop(0);
			TestEqual(TEXT("Pop zero elements leaves one element in the queue."), Q.Count(), SIZE_T(1));
		}

		// Pop all
		{
			FMyTestQueue Q(32);
			for (; Q.Count() < Q.AllocatedCapacity();)
			{
				Q.Enqueue(FStructWithDestructorForResizableCiruclarQueuePopTest());
			}

			// Want to setup the queue so the head and tail aren't at the "beginning" of the queue
			Q.Pop(1);
			TestEqual(TEXT("Pop one element leaves queue at capacity minus one."), Q.Count(), Q.AllocatedCapacity() - SIZE_T(1));

			Q.Enqueue(FStructWithDestructorForResizableCiruclarQueuePopTest());
			Q.Pop(Q.Count());
			TestTrue(TEXT("Pop of all elements leaves queue empty."), Q.IsEmpty());
		}
	}

	// Test index wrap
	{
		const ElementT ElementsToPush = 256;
		const SIZE_T ElementPopMod = 16;
		const SIZE_T ExpectedSize = 256 - ElementPopMod;
		const SIZE_T ExpectedCapacity = 256;

		FResizableCircularQueueTestUtil::QueueT Q(4);

		// Set head and tail at the end of the space to test index wraparound
		FResizableCircularQueueTestUtil::OverideHeadAndTail(Q, uint32(-2), uint32(-2));

		TestTrue(TEXT ("Test index wrap - IsEmpty before"), Q.IsEmpty());
		TestEqual(TEXT("Test index wrap - Size before"), Q.Count(), SIZE_T(0));

		for (ElementT It=0; It < ElementsToPush; ++It)
		{
			Q.Enqueue(It);
		}
		
		TestEqual(TEXT("Test index wrap - Size"), Q.Count(), ElementsToPush);
		TestEqual(TEXT("Test index wrap - Capacity"), Q.AllocatedCapacity(), ElementsToPush);
 		TestTrue (TEXT("Test index wrap - Expected"), FResizableCircularQueueTestUtil::VerifyQueueIntegrity(Q, 0, 1));

		for (ElementT It=0; It < ElementsToPush; ++It)
		{
			Q.Pop();
		}
		
		TestTrue( TEXT("Test index wrap - IsEmpty after"), Q.IsEmpty());
		TestEqual(TEXT("Test index wrap - Size after"), Q.Count(), SIZE_T(0));
		TestEqual(TEXT("Test index wrap - Capacity after"), Q.AllocatedCapacity(), ElementsToPush);
	}

	// Test Trim
	{
		const ElementT ElementsToPush = 9;
		const SIZE_T ElementsToPop = 5;
		const SIZE_T ExpectedCapacity = 16;
		const SIZE_T ExpectedCapacityAfterTrim = 4;

		FResizableCircularQueueTestUtil::QueueT Q(0);

		for (ElementT It=0; It < ElementsToPush; ++It)
		{
			Q.Enqueue(It);
		}
		
		TestEqual(TEXT("Test Trim - Size"), Q.Count(), ElementsToPush);
		TestEqual(TEXT("Test Trim - Capacity"), Q.AllocatedCapacity(), ExpectedCapacity);
 		TestTrue(TEXT("Test Trim - Expected"), FResizableCircularQueueTestUtil::VerifyQueueIntegrity(Q, 0, 1));

		for (SIZE_T It=0; It < ElementsToPop; ++It)
		{
			Q.Pop();
		}

		Q.Trim();

		TestEqual(TEXT("Test Trim - Size"), Q.Count(), ElementsToPush - ElementsToPop);
		TestEqual(TEXT("Test Trim - Capacity"), Q.AllocatedCapacity(), ExpectedCapacityAfterTrim);
 		TestTrue(TEXT("Test Trim - Expected"), FResizableCircularQueueTestUtil::VerifyQueueIntegrity(Q, ElementsToPop, 1));
	}

	// Test Trim empty
	{
		FResizableCircularQueueTestUtil::QueueT Q(0);

		Q.Trim();

		TestEqual(TEXT("Test trim empty - Size"), Q.Count(), SIZE_T(0));
		TestEqual(TEXT("Test trim empty - Capacity"), Q.AllocatedCapacity(), SIZE_T(0));
	}

	// Test non-trivial type
	{
		struct FNonTrivialStruct
		{
			FNonTrivialStruct() : Value(10) {}

			int Value;
		};

		using FQueue = TResizableCircularQueue<TArray<FNonTrivialStruct>>;

		{
			FQueue Q;
			FQueue::ElementT& Array = Q.Enqueue();
			TestEqual(TEXT("Test enqueue non-trivial element"), Array.Num(), 0);
		}

		{
			FQueue Q;
			FQueue::ElementT Array;
			Array.SetNum(2);
			Q.Enqueue(Array);
			
			const FQueue::ElementT& ArrayInQ = Q.Peek();
			TestEqual(TEXT("Test enqueue array with size 2"), ArrayInQ.Num(), 2);
			TestNotEqual(TEXT("Test enqueue array with size 2 has different allocation"), ArrayInQ.GetData(), static_cast<const FQueue::ElementT&>(Array).GetData());
		}
	}

	// Test trivial type with element construction
	{
		using FQueue = TResizableCircularQueue<int>;

		FQueue Q(8);
		int& Value = Q.EnqueueDefaulted_GetRef();
		TestEqual(TEXT("Test enqueue of initialized primitive type"), Value, 0);
	}

	check(!HasAnyErrors());

	return true;
}

#endif // #if WITH_DEV_AUTOMATION_TESTS