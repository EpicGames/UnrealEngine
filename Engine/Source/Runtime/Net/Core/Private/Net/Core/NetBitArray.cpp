// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/NetBitArray.h"

#include "Net/Core/NetBitArrayPrinter.h"

#include "Traits/IntType.h"

namespace UE::Net
{

//*************************************************************************************************
// FNetBitArray
//*************************************************************************************************
FString FNetBitArray::ToString() const
{
	return FNetBitArrayPrinter::PrintSetSummary(*this);
}

//*************************************************************************************************
// FNetBitArrayView
//*************************************************************************************************
FString FNetBitArrayView::ToString() const
{
	return FNetBitArrayPrinter::PrintSetSummary(*this);
}

//*************************************************************************************************
// FNetBitArrayHelper
//*************************************************************************************************

uint32 FNetBitArrayHelper::GetSetBitIndices(const FNetBitArrayHelper::StorageWordType* Storage, const uint32 BitCount, const uint32 StartIndex, const uint32 Count, uint32* const OutIndices, const uint32 OutIndicesCapacity)
{
	if ((!ensure(OutIndicesCapacity > 0)) | (StartIndex >= BitCount) | (Count == 0))
	{
		return 0U;
	}

	// We're ok with people passing ~0U for example so they don't have to check the capacity of this container.
	const uint32 EndIndex = static_cast<uint32>(FPlatformMath::Min(uint64(StartIndex) + Count - 1U, uint64(BitCount) - 1U));

	/**
	  * The algorithm iterates from the first bit in the start word and stores each index in OutIndices.
	  * If the index is outside the allowed range we don't increment the amount of stored indices though.
	  * This minimizes the amount of branches needed.
	  */
	uint32 StoredIndexCount = 0;
	for (uint32 WordIt = StartIndex/WordBitCount, Index = StartIndex & ~(WordBitCount - 1U); (Index <= EndIndex) & (StoredIndexCount < OutIndicesCapacity); Index += WordBitCount, ++WordIt)
	{
		StorageWordType CurrentWord = Storage[WordIt];
		while ((CurrentWord != 0) & (StoredIndexCount < OutIndicesCapacity))
		{
			const uint32 LeastSignificantBit = CurrentWord & StorageWordType(-TSignedIntType<sizeof(StorageWordType)>::Type(CurrentWord));
			CurrentWord ^= LeastSignificantBit;

			const uint32 OutIndex = Index + FPlatformMath::CountTrailingZeros(LeastSignificantBit);
			OutIndices[StoredIndexCount] = OutIndex;
			StoredIndexCount += (OutIndex >= StartIndex) & (OutIndex <= EndIndex);
		}
	}

	return StoredIndexCount;
}

uint32 FNetBitArrayHelper::CountSetBits(const StorageWordType* Storage, const uint32 BitCount, const uint32 StartIndex, const uint32 Count)
{
	if ((StartIndex >= BitCount) | (Count == 0))
	{
		return 0U;
	}

	const uint32 EndIndex = static_cast<uint32>(FPlatformMath::Min(uint64(StartIndex) + Count - 1U, uint64(BitCount) - 1U));
	const uint32 WordStartIt = StartIndex/WordBitCount;
	StorageWordType StartWordMask = (~StorageWordType(0) << (StartIndex & (WordBitCount - 1U)));
	const uint32 WordEndIt = EndIndex/WordBitCount;
	const StorageWordType EndWordMask = (~StorageWordType(0) >> (~EndIndex & (WordBitCount - 1U)));

	StorageWordType SetBitsCount = 0;

	for (uint32 WordIt = WordStartIt; WordIt <= WordEndIt; ++WordIt)
	{
		// Compute mask based on start and end word status.
		const bool bIsEndWord = (WordIt == WordEndIt);
		const StorageWordType WordMask = StartWordMask & (bIsEndWord ? EndWordMask : ~StorageWordType(0));
		// Only the first word is the start word so we can set all bits now.
		StartWordMask = ~StorageWordType(0);

		SetBitsCount += FPlatformMath::CountBits(Storage[WordIt] & WordMask);
	}

	return SetBitsCount;
}

} // end namespace UE::Net