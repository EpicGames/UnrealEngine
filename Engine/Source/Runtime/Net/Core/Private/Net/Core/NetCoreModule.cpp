// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/NetCoreModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Net/Core/Misc/NetCoreLog.h"

#include "HAL/IConsoleManager.h"
#include "Net/Core/DirtyNetObjectTracker/GlobalDirtyNetObjectTracker.h"
#include "Net/Core/NetHandle/NetHandleManager.h"
#include "Net/Serialization/FastArraySerializer.h"
#include "ProfilingDebugging/CsvProfiler.h"

bool GUseDetailedScopeCounters=true;
static FAutoConsoleVariableRef CVarNetEnableDetailedScopeCounters(
	TEXT("net.EnableDetailedScopeCounters"),
	GUseDetailedScopeCounters,
	TEXT("Enables detailed networking scope cycle counters. There are often lots of these which can negatively impact performance."),
	ECVF_Default);

DEFINE_LOG_CATEGORY(LogNetFastTArray);

DEFINE_LOG_CATEGORY(LogNetCore);

CSV_DEFINE_CATEGORY_MODULE(NETCORE_API, Networking, true);


class FNetCoreModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		UE::Net::FNetHandleManager::Init();
		UE::Net::FGlobalDirtyNetObjectTracker::Init();
	}

	virtual void ShutdownModule() override
	{
		UE::Net::FGlobalDirtyNetObjectTracker::Deinit();
		UE::Net::FNetHandleManager::Deinit();
	}

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
};


IMPLEMENT_MODULE(FNetCoreModule, NetCore);
