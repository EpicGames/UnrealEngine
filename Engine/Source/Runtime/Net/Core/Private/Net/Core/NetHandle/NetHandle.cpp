// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/NetHandle/NetHandle.h"
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"

namespace UE::Net
{

FString FNetHandle::ToString() const
{
	TStringBuilder<64> StringBuilder;
	StringBuilder << *this;
	return FString(StringBuilder.ToView());
}

}

FAnsiStringBuilderBase& operator<<(FAnsiStringBuilderBase& Builder, const UE::Net::FNetHandle& NetHandle)
{
	return Builder.Appendf("NetHandle (Id=%u)", NetHandle.GetId());
}

FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, const UE::Net::FNetHandle& NetHandle)
{
	return Builder.Appendf(UTF8TEXT("NetHandle (Id=%u)"), NetHandle.GetId());
}

FWideStringBuilderBase& operator<<(FWideStringBuilderBase& Builder, const UE::Net::FNetHandle& NetHandle)
{
	return Builder.Appendf(WIDETEXT("NetHandle (Id=%u)"), NetHandle.GetId());
}
