// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/NetHandle/NetHandleManager.h"
#include "Containers/Map.h"
#include "UObject/Object.h"

namespace UE::Net
{

class FNetHandleManager::FPimpl
{
private:
	friend FNetHandleManager;

	FNetHandle CreateNetHandle(const UObject* Object) const;

	TMap<const UObject*, FNetHandle> ObjectToNetHandle;
	TMap<FNetHandle, const UObject*> NetHandleToObject;
};

FNetHandleManager::FPimpl* FNetHandleManager::Instance = nullptr;

FNetHandle FNetHandleManager::GetNetHandle(const UObject* Object)
{
	if (!Instance)
	{
		return FNetHandle();
	}

	TMap<const UObject*, FNetHandle>::TKeyIterator KeyIt = Instance->ObjectToNetHandle.CreateKeyIterator(Object);
	if (KeyIt)
	{
		FNetHandle Handle = KeyIt.Value();
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (Handle.Value == FRemoteObjectId(Object))
#else
		UObject* ResolvedObject = Handle.Value.ResolveObjectPtrEvenIfUnreachable();
		if (ResolvedObject == Object)
#endif
		{
			return Handle;
		}
		else
		{
			// We've found a stale object. Remove all associations with it.
			KeyIt.RemoveCurrent();
			Instance->NetHandleToObject.Remove(Handle);
			return FNetHandle();
		}
	}

	return FNetHandle();
}

FNetHandle FNetHandleManager::GetOrCreateNetHandle(const UObject* Object)
{
	if (!Instance)
	{
		return FNetHandle();
	}

	TMap<const UObject*, FNetHandle>::TKeyIterator KeyIt = Instance->ObjectToNetHandle.CreateKeyIterator(Object);
	if (KeyIt)
	{
		FNetHandle Handle = KeyIt.Value();
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (Handle.Value == FRemoteObjectId(Object))
#else
		UObject* ResolvedObject = Handle.Value.ResolveObjectPtrEvenIfUnreachable();
		if (Object == ResolvedObject)
#endif
		{
			return Handle;
		}

		// We've found a stale object. Remove all associations with it.
		KeyIt.RemoveCurrent();
		Instance->NetHandleToObject.Remove(Handle);

		// Fall through and create new mappings between object and handle.
	}

	FNetHandle Handle = Instance->CreateNetHandle(Object);
	if (Handle.IsValid())
	{
		Instance->NetHandleToObject.Emplace(Handle, Object);
		Instance->ObjectToNetHandle.Emplace(Object, Handle);
		return Handle;
	}

	return FNetHandle();
}

FNetHandle FNetHandleManager::MakeNetHandleFromId(uint32 Id)
{
	// There aren't any valid NetHandles unless there's a manager.
	if (!Instance)
	{
		return FNetHandle();
	}

	if (FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(static_cast<int32>(Id)))
	{
		FNetHandle::FInternalValue InternalValue;
		InternalValue.Id = Id;
		InternalValue.Epoch = static_cast<uint32>(ObjectItem->GetSerialNumber());
		FNetHandle Handle(InternalValue);
		return Handle;
	}

	return FNetHandle();
}

void FNetHandleManager::DestroyNetHandle(FNetHandle Handle)
{
	if (!Instance)
	{
		return;
	}

	const UObject* Object = nullptr;
	if (Instance->NetHandleToObject.RemoveAndCopyValue(Handle, Object))
	{
		Instance->ObjectToNetHandle.Remove(Object);
	}
}

void FNetHandleManager::Init()
{
	checkf(Instance == nullptr, TEXT("%s"), TEXT("Only one FNetHandleManager instance may exist."));
	Instance = new FNetHandleManager::FPimpl();
}

void FNetHandleManager::Deinit()
{
	delete Instance;
	Instance = nullptr;
}

// FNetHandleManager::FPimpl implementation
FNetHandle FNetHandleManager::FPimpl::CreateNetHandle(const UObject* Object) const
{
	if (!IsValid(Object))
	{
		return FNetHandle();
	}

	FNetHandle Handle = FNetHandle(FObjectKey(Object));
	return Handle;
}

}
