// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/NetToken/NetTokenExportContext.h"
#include "UObject/CoreNet.h"

DEFINE_LOG_CATEGORY(LogNetToken);

namespace UE::Net
{

// Simple scope to make sure we set the correct ExportContext and restore the old one when we exit the scope
FNetTokenExportScope::FNetTokenExportScope(FNetBitWriter& InNetBitWriter, UE::Net::FNetTokenStore* InNetTokenStore, FNetTokenExportContext::FNetTokenExports& TargetExports, const char* InDebugName)
: ExportContext(InNetTokenStore, &TargetExports)
, NetBitWriter(InNetBitWriter)
, OldExportContext(InNetBitWriter.NetTokenExportContext.Get())
, DebugName(InDebugName)
{
	NetBitWriter.NetTokenExportContext.Set(&ExportContext);
}

FNetTokenExportScope::~FNetTokenExportScope()
{
	// Restore old export context
	NetBitWriter.NetTokenExportContext.Set(OldExportContext);

	if (!ExportContext.NetTokensPendingExport->IsEmpty() && (UE_GET_LOG_VERBOSITY(LogNetToken) >= ELogVerbosity::Verbose))
	{
		UE_LOGF(LogNetToken, Verbose, "FNetTokenExportScope %s added %d pending exports", DebugName, ExportContext.NetTokensPendingExport->Num());
		for (UE::Net::FNetToken NetToken : *ExportContext.NetTokensPendingExport)
		{
			UE_LOGF(LogNetToken, Verbose, "Pending export %ls", *NetToken.ToString());
		}
	}
}

FNetTokenExportContext::FNetTokenExportContext(FNetTokenStore* InNetTokenStore, FNetTokenExports* InNetTokensPendingExport)
: NetTokenStore(InNetTokenStore)
, NetTokensPendingExport(InNetTokensPendingExport)
{
}

FNetTokenExportContext* FNetTokenExportContext::GetNetTokenExportContext(FArchive& Ar)
{
	if (Ar.IsSaving() && ensureMsgf(Ar.IsNetArchive(), TEXT("Trying to export net tokens for archive that is not a NetBitWriter")))
	{
		// Assume that we are a NetBitWriter so that we have a way to pass NetTokenExportContext to NetSerialize() methods
		FNetBitWriter& NetBitWriter = static_cast<FNetBitWriter&>(Ar);
		return NetBitWriter.NetTokenExportContext.Get();
	}

	return nullptr;
}

void FNetTokenExportContext::AppendNetTokensPendingExport(TArrayView<const UE::Net::FNetToken> NetTokens)
{ 
	if (ensure(NetTokensPendingExport))
	{
		NetTokensPendingExport->Append(NetTokens);
	}
}

void FNetTokenExportContext::AddNetTokenPendingExport(UE::Net::FNetToken NetToken)
{
	if (ensure(NetTokensPendingExport) && NetToken.IsValid())
	{
		NetTokensPendingExport->Add(NetToken);
		UE_LOGF(LogNetToken, Verbose, "FNetTokenExportContext::AddNetTokenPendingExport %ls", *NetToken.ToString());
	}	
}

TArrayView<const UE::Net::FNetToken> FNetTokenExportContext::GetNetTokensPendingExport() const
{ 
	return NetTokensPendingExport ? MakeArrayView(*NetTokensPendingExport) : MakeArrayView<const UE::Net::FNetToken>(nullptr, 0U);
}


}
