// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/PropertyConditions/PropertyConditions.h"
#include "Net/Core/PropertyConditions/RepChangedPropertyTracker.h"
#include "Net/Core/PropertyConditions/PropertyConditionsDelegates.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Serialization/ArchiveCountMem.h"
#include "Stats/Stats.h"

DECLARE_CYCLE_STAT(TEXT("PropertyConditions PostGarbageCollect"), STAT_PropertyConditions_PostGarbageCollect, STATGROUP_Net);

namespace UE::Net::Private
{
FPropertyConditionDelegates::FOnPropertyCustomConditionChanged FPropertyConditionDelegates::OnPropertyCustomConditionChangedDelegate;
FPropertyConditionDelegates::FOnPropertyDynamicConditionChanged FPropertyConditionDelegates::OnPropertyDynamicConditionChangedDelegate;

FNetPropertyConditionManager::FNetPropertyConditionManager()
{
	PostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FNetPropertyConditionManager::PostGarbageCollect);
}

FNetPropertyConditionManager::~FNetPropertyConditionManager()
{
	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGarbageCollectHandle);
}

FNetPropertyConditionManager& FNetPropertyConditionManager::Get()
{
	static FNetPropertyConditionManager Singleton;
	return Singleton;
}

void FNetPropertyConditionManager::SetPropertyActive(const UObject* Object, const uint16 RepIndex, const bool bActive)
{
	const FObjectKey ObjectKey(Object);

	TSharedPtr<FRepChangedPropertyTracker> Tracker = FindPropertyTracker(ObjectKey);
	if (Tracker.IsValid())
	{
		Tracker->SetCustomIsActiveOverride(Object, RepIndex, bActive);
	}
}

void FNetPropertyConditionManager::SetPropertyActiveOverride(const UObject* Object, const uint16 RepIndex, const bool bActive)
{
	const FObjectKey ObjectKey(Object);

	// We cache the LastFoundTracker and Key to avoid the map lookup if we modify multiple properties at once.
	if (LastFoundTrackerKey != ObjectKey || LastFoundTracker == nullptr)
	{
		TSharedPtr<FRepChangedPropertyTracker> Tracker = bAllowCreateTrackerFromSetPropertyActiveOverride ? FindOrCreatePropertyTracker(ObjectKey) : FindPropertyTracker(ObjectKey);
		LastFoundTrackerKey = ObjectKey;
		LastFoundTracker = Tracker.Get();
	}

	if (LastFoundTracker)
	{
		LastFoundTracker->SetCustomIsActiveOverride(Object, RepIndex, bActive);
	}
}

void FNetPropertyConditionManager::SetPropertyDynamicCondition(const UObject* Object, const uint16 RepIndex, const ELifetimeCondition Condition)
{
	const FObjectKey ObjectKey(Object);

	TSharedPtr<FRepChangedPropertyTracker> Tracker = FindPropertyTracker(ObjectKey);
	if (Tracker.IsValid())
	{
		Tracker->SetDynamicCondition(Object, RepIndex, Condition);
	}
}

void FNetPropertyConditionManager::NotifyObjectDestroyed(const FObjectKey ObjectKey)
{
	if (ObjectKey == LastFoundTrackerKey)
	{
		LastFoundTrackerKey = FObjectKey();
		LastFoundTracker = nullptr;
	}

	PropertyTrackerMap.Remove(ObjectKey);
}

TSharedPtr<FRepChangedPropertyTracker> FNetPropertyConditionManager::FindOrCreatePropertyTracker(const FObjectKey ObjectKey)
{
	TSharedPtr<FRepChangedPropertyTracker> Tracker = FindPropertyTracker(ObjectKey);
	if (!Tracker.IsValid())
	{
		if (UObject* Obj = ObjectKey.ResolveObjectPtr())
		{
			UClass* ObjectClass = Obj->GetClass();
			check(ObjectClass);
			ObjectClass->SetUpRuntimeReplicationData();

			const int32 NumProperties = ObjectClass->ClassReps.Num();

			FCustomPropertyConditionState ActiveState(NumProperties);
			Obj->GetReplicatedCustomConditionState(ActiveState);

			Tracker = MakeShared<FRepChangedPropertyTracker>(MoveTemp(ActiveState));

			PropertyTrackerMap.Add(ObjectKey, Tracker);
		}
		else
		{
			ensureMsgf(false, TEXT("FindOrCreatePropertyTracker: Unable to resolve object key."));
		}
	}

	return Tracker;
}

TSharedPtr<FRepChangedPropertyTracker> FNetPropertyConditionManager::FindPropertyTracker(const FObjectKey ObjectKey) const
{
	return PropertyTrackerMap.FindRef(ObjectKey);
}

void FNetPropertyConditionManager::PostGarbageCollect()
{
	SCOPE_CYCLE_COUNTER(STAT_PropertyConditions_PostGarbageCollect);

	for (auto It = PropertyTrackerMap.CreateIterator(); It; ++It)
	{
		if (
#if UE_WITH_REMOTE_OBJECT_HANDLE
			FObjectPtr(It.Key().GetRemoteId()).GetResidence() == EResidence::Remote ||
#endif
			!It.Key().ResolveObjectPtr())
		{
			if (It.Key() == LastFoundTrackerKey)
			{
				LastFoundTrackerKey = FObjectKey();
				LastFoundTracker = nullptr;
			}
			It.RemoveCurrent();
		}
	}
}

void FNetPropertyConditionManager::LogMemory(FOutputDevice& Ar)
{
	FArchiveCountMem CountAr(nullptr);

	PropertyTrackerMap.CountBytes(CountAr);

	for (auto It = PropertyTrackerMap.CreateConstIterator(); It; ++It)
	{
		if (It->Value.IsValid())
		{
			It->Value->CountBytes(CountAr);
		}
	}

	const int32 CountBytes = sizeof(*this) + CountAr.GetNum();

	Ar.Logf(TEXT("  Property Condition Memory: %u"), CountBytes);
}

}; // UE::Net::Private