// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/PropertyConditions/RepChangedPropertyTracker.h"
#include "Net/Core/PropertyConditions/PropertyConditionsDelegates.h"
#include "Net/Core/PushModel/PushModel.h"

FRepChangedPropertyTracker::FRepChangedPropertyTracker(FCustomPropertyConditionState&& InActiveState)
	: ActiveState(MoveTemp(InActiveState))
{
}

void FRepChangedPropertyTracker::SetCustomIsActiveOverride(const UObject* OwningObject, const uint16 RepIndex, const bool bIsActive)
{
	const bool bOldActive = ActiveState.GetActiveState(RepIndex);
	ActiveState.SetActiveState(RepIndex, bIsActive);	// check for client replay recording moved to FReplicationFlags

#if WITH_PUSH_MODEL
	if (!bOldActive && bIsActive)
	{
		MARK_PROPERTY_DIRTY_UNSAFE(OwningObject, RepIndex);
	}
#endif

	if (bOldActive != bIsActive && UE::Net::Private::FPropertyConditionDelegates::GetOnPropertyCustomConditionChangedDelegate().IsBound())
	{
		UE::Net::Private::FPropertyConditionDelegates::GetOnPropertyCustomConditionChangedDelegate().Broadcast(OwningObject, RepIndex, bIsActive);
	}
}

void FRepChangedPropertyTracker::SetDynamicCondition(const UObject* OwningObject, const uint16 RepIndex, const ELifetimeCondition Condition)
{
	using namespace UE::Net::Private;

	const ELifetimeCondition OldCondition = ActiveState.GetDynamicCondition(RepIndex);
	if (Condition == OldCondition)
	{
		return;
	}

	ActiveState.SetDynamicCondition(RepIndex, Condition);

#if WITH_PUSH_MODEL
	MARK_PROPERTY_DIRTY_UNSAFE(OwningObject, RepIndex);
#endif

	FPropertyConditionDelegates::FOnPropertyDynamicConditionChanged& Delegate = FPropertyConditionDelegates::GetOnPropertyDynamicConditionChangedDelegate();
	if (Delegate.IsBound())
	{
		Delegate.Broadcast(OwningObject, RepIndex, Condition);
	}
}

void FRepChangedPropertyTracker::CountBytes(FArchive& Ar) const
{
	// Include our size here, because the caller won't know.
	Ar.CountBytes(sizeof(FRepChangedPropertyTracker), sizeof(FRepChangedPropertyTracker));
	ActiveState.CountBytes(Ar);
}
