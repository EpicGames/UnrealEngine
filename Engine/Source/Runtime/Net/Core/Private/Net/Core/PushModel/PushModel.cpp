// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/PushModel/PushModel.h"

#if WITH_PUSH_MODEL

#include "HAL/IConsoleManager.h"
#include "Algo/Sort.h"

#include "Types/PushModelUtils.h"
#include "Types/PushModelPerObjectState.h"
#include "Types/PushModelPerNetDriverState.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Stats/Stats.h"

DECLARE_CYCLE_STAT(TEXT("PushModel PostGarbageCollect"), STAT_PushModel_PostGarbageCollect, STATGROUP_Net);
DEFINE_LOG_CATEGORY_STATIC(LogPushModel, All, All);

struct FNetObjectManagerPushIdHelper
{
	static void ResetObjectNetPushId(UObject* Object)
	{
		check(Object);
		FObjectNetPushIdHelper::SetNetPushIdDynamic(Object, UEPushModelPrivate::FNetPushObjectId().GetValue());
	}
};

namespace UEPushModelPrivate
{
	//! Originally, multiple implementations of FPushModelObjectManagers were tested.
	//! The way this worked was by implementing the managers here, using Macros or
	//! Switches in the various API functions, and then changing any necessary helper
	//! Macros to make sure the correct arguments came through.
	//! Ideally, the interface will always stay the same, and any unused arguments or
	//! Returns are just ignored.
	
	//! TODO: We should add in a way for NetDrivers to opt out of PushModel.
	//! Things like the Beacon Net Driver, for example, don't need to care about it.
	//! Since most things are lazily created, this probably isn't a big deal, but
	//! having explicit behavior preventing it is probably worthwhile.

	/**
	 * Class that is used to manage Push Model Object states by using Custom IDs.
	 *
	 * This class relies on every Replicated UObject to have a Push Model Controlled ID
	 * associated with it. This ID is injected into the Base Most Replicated class
	 * during header generation.
	 *
	 * For example, UObject does not have the ID field added, but UActorComponent does.
	 * Since USceneComponent derives from UActorComponent, it can just use the injected
	 * ID from UActorComponent.
	 *
	 * Currently, this works fine for BP because you can't directly derive BPs from UObjects.
	 * They need to be either AActor or UActorComponent, and those will have the injected IDs.
	 * If that changes, we may need to move the custom IDs to base UObject and eat the memory.
	 *
	 * Also, instead of tracking Objects by pointer, we need another ID that can be used to track
	 * objects **outside** of networking contexts. We opt for FObjectKey. Although they are
	 * somewhat expensive to construct, they ensure uniqueness and ensure that the reference
	 * to a given object will be unique even across garbage collections if we're given stale
	 * pointers.
	 *
	 * **********************************************************************************
	 * ************* Adding Objects to and Remove Objects From the Manager **************
	 * **********************************************************************************
	 *
	 * Objects are lazily added to the manager (this will be true of any manager).
	 * Currently, this is tied directly to FRepChangelistState.
	 * Whenever a FRepChangelistState is created, we will call AddPushModelObject
	 * and whenever a FRepChangelistState is destroyed, we will call RemovePushModelObject.
	 *
	 * FRepChangelistState are only created once on a NetDriver for a given Object in game,
	 * and are then shared across all connections.
	 *
	 * This also means that multiple Net Drivers can "Add" or "Remove" push model objects.
	 *
	 * To cope with this, there will is one global FPushModelPerObjectState that is alive
	 * as long as **any** Net Drivers are replicating the object. There will also be one
	 * FPushModelPerNetDriverState alive for each individual Net Driver that is replicating
	 * an object.
	 *
	 * **********************************************************************************
	 * ****************************** Managing Dirty State ******************************
	 * **********************************************************************************
	 *
	 * When an Object is marked dirty from "Game Land", user's will pass a Object's
	 * Push Model ID as well as the Rep Index for the property (or properties). These
	 * things can mostly be determined at compile time using the MARK_PROPERTY_DIRTY_* macros,
	 * meaning that users only really need to pass the Property's Owning Class, the Property's
	 * name, and an Object Pointer.
	 *
	 * If the object is not currently replicated by any net drivers, its Push Model ID will be
	 * INDEX_NONE, and we will ignore it. Otherwise, we will set the necessary dirty bit
	 * on the FPushModelPerObjectState.
	 *
	 * That dirty state will remain set until we go to replicate the object.
	 *
	 * **********************************************************************************
	 * ************************* Replication using Dirty State **************************
	 * **********************************************************************************
	 *
	 * When a given Net Driver goes to Compare an Object's properties for replication,
	 * it will request the Push Model State (GetPerNetDriverState). At that point, we will
	 * push the Global Dirty State (on FPushModelPerObjectState) to the Per Net Driver
	 * Dirty States (on FPushModelPerNetDriverState). In this case, "Pushing State" is
	 * effectively a bitwise or of the Push Model Dirty States into the
	 * Per Net Driver Dirty State, and then clearing the Global Dirty State.
	 *
	 * There were 2 main reasons why this was done lazily:
	 *	1. The subset of Actors that we *will* replicate is always going to be much much
	 *		smaller than the total number of Actors that *may* replicate.
	 *
	 *	2. There isn't a good catch all spot to do this.
	 *		- Originally having the NetDriver's call this before replication occurred
	 *			was tested, but properties can be changed during AActor::PreReplication.
	 *			Further, because Reliable RPCs can force new Channel's open, and initial
	 *			replication, extra steps had to be taken to try and update states there
	 *			as well.
	 *
	 *		- Using AActor::CalPreReplication was also considered. However, there's
	 *			currently no good interface for getting all replicated subobjects of an
	 *			Actor (or other subobjects), so there's no guarantee we'd push states.
	 *			Additionally, even if there was an interface this wouldn't necessarily
	 *			respect Subobject Replication Keys.
	 *
	 * Doing this lazily is a simple solution to both of these problems, with the trade
	 * off that if multiple NetDrivers are replicating on the same frame, we may try to
	 * push the state multiple times. But, we can detect that no properties have changed
	 * since the last push, and not waste much time.
	 *
	 * When a property is *not* marked dirty but is Push Enabled, then we will skip comparing it.
	 * When a property is marked dirty, or is not Push Enabled, we will compare it.
	 * See the comments in PushModel.h for more info on that.
	 *
	 * **********************************************************************************
	 * ******************* Potential for "Automatic Subobject Keys" *********************
	 * **********************************************************************************
	 *
	 * Push Model could achieve the same effect as Subobject Keys without requiring users
	 * to do any extra work or manage additional state themselves.
	 *
	 * When an Object is about to be replicated (see FObjectReplicator::ReplicateProperties)
	 * we could check to see if all the Object's properties were Push Model based.
	 *
	 * If they were, we could efficiently check to see whether or not *any* properties had changed.
	 *
	 * If they hadn't, we could skip doing any additional work for standard properties.
	 * Some work may still need to happen for Custom Delta Properties (Fast Arrays).
	 *
	 * **********************************************************************************
	 * ********************** Potential for "Automatic Dormancy" ************************
	 * **********************************************************************************
	 *
	 * Push model may support the notion of Automatic Dormancy in the future. The premise
	 * being that instead of relying on designers or developers to to call FlushNetDormancy
	 * when changing object properties, instead Push Model could be used as a drop in
	 * substitute. If you're already adhering to the contract of Push Model, your system
	 * will already have all the necessary hooks to determine when properties change.
	 *
	 * This comes with a few nice benefits:
	 *	1. It would be impossible to forget to wake an Actor up when properties change.
	 *
	 *	2. Similarly, if you Components / Subobjects that contained shared logic,
	 *		there would be no extra special casing to mark the owning Actor awake.
	 *
	 *	3. **Every** type of object that fully relied on Push Model could automatically
	 *		make use of this system without the need for extra configuration.
	 *		This is especially useful considering all Blueprints derived directly from
	 *		AActor, UActorComponent, and USceneComponent would currently be in this
	 *		category. The more engine conversion that happens, the broader that becomes.
	 *
	 * Speculative plan for how this might work:
	 *
	 *	1. The fundamental machinery of Dormancy stays the same.
	 *		An Object is eligible for Dormancy if none of it's properties have changed
	 *		for some configurable timeout, and all connections have received its most
	 *		up to date information.
	 *
	 *	2. Alongside dirty property states, Push Model could also have a bitfield that
	 *		tracks whether or not an object was dirtied in a frame. Alternatively,
	 *		that state could be derived from FPushModelPerObjectState.
	 *
	 *	3. At some defined point in a frame before Replication of any NetDrivers
	 *		occurs, we push the Dirty Object State to the Net Driver.
	 *		This might work by just having each Net Driver grab the Dirty Object
	 *		bitfield and if a given object is dirty, mapping the Object ID back
	 *		to Actor / Object, and then calling the normal Flush Net Dormancy calls.
	 *
	 *	4. At some defined point in a frame *after* Replication of *every* NetDriver,
	 *		we reset the Object Dirty State.
	 *
	 * The trickiest part of this would likely be finding an efficient mapping back from a
	 * PushModel ID to a Networked Object. Push Model was designed to be mostly agnostic
	 * to Objects and because of that explicitly doesn't provide any API to go from a
	 * Push ID back to an Object.
	 */
	class FPushModelObjectManager_CustomId
	{
	private:

		using ThisClass = FPushModelObjectManager_CustomId;

	public:

		FPushModelObjectManager_CustomId()
		{
			PostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FPushModelObjectManager_CustomId::PostGarbageCollect);
		}

		~FPushModelObjectManager_CustomId()
		{
			FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGarbageCollectHandle);
		}

		void MarkPropertyDirty(const FNetLegacyPushObjectId ObjectId, const int32 RepIndex)
		{
			const int32 ObjectIndex = ObjectId;
			if (LIKELY(PerObjectStates.IsValidIndex(ObjectIndex)))
			{
				// The macros will take care of filtering out invalid objects, so we don't need to check here.
				PerObjectStates[ObjectIndex].MarkPropertyDirty(static_cast<uint16>(RepIndex));
			}
		}

		void MarkPropertyDirty(const FNetLegacyPushObjectId ObjectId, const int32 StartRepIndex, const int32 EndRepIndex)
		{
			const int32 ObjectIndex = ObjectId;
			if (LIKELY(PerObjectStates.IsValidIndex(ObjectIndex)))
			{
				FPushModelPerObjectState& ObjectState = PerObjectStates[ObjectIndex];
				for (int32 RepIndex = StartRepIndex; RepIndex <= EndRepIndex; ++RepIndex)
				{
					ObjectState.MarkPropertyDirty(static_cast<uint16>(RepIndex));
				}
			}
		}

		const FPushModelPerNetDriverHandle AddNetworkObject(const FObjectKey ObjectKey, const uint16 NumReplicatedProperties)
		{
			FNetLegacyPushObjectId& InternalPushId = ObjectKeyToInternalId.FindOrAdd(ObjectKey, INDEX_NONE);
			if (INDEX_NONE == InternalPushId)
			{
				InternalPushId = PerObjectStates.EmplaceAtLowestFreeIndex(NewObjectLookupPosition, ObjectKey, NumReplicatedProperties);
			}

			FPushModelPerObjectState& PerObjectState = PerObjectStates[InternalPushId];
			check(PerObjectState.GetNumberOfProperties() == NumReplicatedProperties);
			check(PerObjectState.GetObjectKey() == ObjectKey);

			const FNetPushPerNetDriverId NetDriverId = PerObjectState.AddPerNetDriverState();
			return FPushModelPerNetDriverHandle(NetDriverId, InternalPushId);
		}

		void RemoveNetworkObject(const FPushModelPerNetDriverHandle Handle)
		{
			const int32 ObjectIndex = Handle.ObjectId;
			if (LIKELY(PerObjectStates.IsValidIndex(ObjectIndex)))
			{
				FPushModelPerObjectState& PerObjectState = PerObjectStates[ObjectIndex];
				PerObjectState.RemovePerNetDriverState(Handle.NetDriverId);
			}
		}

		void PostGarbageCollect()
		{
			SCOPE_CYCLE_COUNTER(STAT_PushModel_PostGarbageCollect);

			// We can't compact PerObjectStates because we need ObjectIDs to be stable.
			// But we can shrink it.

			// Go ahead and remove any PerObjectStates that aren't being tracked by any NetDrivers.
			// We have to wait until GC for this, because the NetDriver's TickFlush() will periodically remove
			// Network Objects that are flagged for destruction.
			for (auto It = PerObjectStates.CreateIterator(); It; ++It)
			{
				if (!It->HasAnyNetDriverStates())
				{
					const FObjectKey& ObjectKey = It->GetObjectKey();

					if (UObject* DestroyedObj = ObjectKey.ResolveObjectPtrEvenIfUnreachable())
					{
						// If the UObject ptr still exists even if flagged for destruction, reset it's PushModelId to prevent it from trying to set dirty states while it is destroying itself.
						FNetObjectManagerPushIdHelper::ResetObjectNetPushId(DestroyedObj);
					}

					ObjectKeyToInternalId.Remove(ObjectKey);
					It.RemoveCurrent();
				}
				else
				{
					It->SetRecentlyCollectedGarbage();
				}
			}

			PerObjectStates.Shrink();
			ObjectKeyToInternalId.Compact();
			NewObjectLookupPosition = 0;
		}

		FPushModelPerNetDriverState* GetPerNetDriverState(const FPushModelPerNetDriverHandle Handle)
		{
			const int32 ObjectIndex = Handle.ObjectId;
			if (LIKELY(PerObjectStates.IsValidIndex(ObjectIndex)))
			{
				FPushModelPerObjectState& ObjectState = PerObjectStates[ObjectIndex];
				ObjectState.PushDirtyStateToNetDrivers();
				return &ObjectState.GetPerNetDriverState(Handle.NetDriverId);
			}

			return nullptr;
		}

		bool DoesHaveDirtyPropertiesOrRecentlyCollectedGarbage(const FPushModelPerNetDriverHandle Handle) const
		{
			const int32 ObjectIndex = Handle.ObjectId;
			if (LIKELY(PerObjectStates.IsValidIndex(ObjectIndex)))
			{
				const FPushModelPerObjectState& ObjectState = PerObjectStates[ObjectIndex];
				const FPushModelPerNetDriverState& NetDriverState = ObjectState.GetPerNetDriverState(Handle.NetDriverId);
				return ObjectState.HasDirtyProperties() || NetDriverState.HasDirtyProperties() || NetDriverState.DidRecentlyCollectGarbage();
			}

			return false;
		}

		bool ValidateObjectIdReassignment(FNetLegacyPushObjectId CurrentId, FNetLegacyPushObjectId NewId)
		{
			if (!PerObjectStates.IsValidIndex(CurrentId))
			{
				// This case is fine, because it means we already cleared out the old object slot.
				// Allow the reassignment.
				UE_LOGF(LogPushModel, Log, "ValidateObjectIdReassignment:  CurrentId is invalid.");
				return true;
			}
			if (!PerObjectStates.IsValidIndex(NewId))
			{
				// This should absolutely not be possible, because we should have just added the object
				// to the NewId slot. At this point, we're going to crash somewhere either way.
				UE_LOGF(LogPushModel, Fatal, "ValidateObjectIdReassignment: NewId is invalid.");
				return false;
			}

			const FPushModelPerObjectState& CurrentObjectState = PerObjectStates[CurrentId];
			const FPushModelPerObjectState& NewObjectState = PerObjectStates[NewId];

			if (CurrentObjectState.GetObjectKey() != NewObjectState.GetObjectKey())
			{
				// This is probably ok. Somehow, our object had a stale ID, but that slot is already taken up by some other object.
				// Allow the reassignment so we don't try to reference the object in our current slot.
				UE_LOGF(LogPushModel, Log, "ValidateObjectIdReassignment: IDs point to different objects. CurrentObject = %ls, NewObject = %ls",
					*GetPathNameSafe(CurrentObjectState.GetObjectKey().ResolveObjectPtr()), *GetPathNameSafe(NewObjectState.GetObjectKey().ResolveObjectPtr()));

				return true;
			}

			// Somehow, we still have a reference to the object in the old / current slot.
			// It's probably safer to just keep the old ID in case anything has it cached (not sure how that would be possible).
			// Bad things have already happened.

			// Want to log this even if ensures are disabled.
			UE_LOGF(LogPushModel, Warning, "ValidObjectIdReassignment: IDs point to the same object. Object=%ls",
				*GetPathNameSafe(CurrentObjectState.GetObjectKey().ResolveObjectPtr()));

			ensureMsgf(false, TEXT("ValidObjectIdReassignment: IDs point to the same object. Object=%s"),
				*GetPathNameSafe(CurrentObjectState.GetObjectKey().ResolveObjectPtr()));

			// Reset the new state because we're not going to use it.
			PerObjectStates.RemoveAt(NewId);
			NewObjectLookupPosition = NewId;

			return false;
		}

		uint32 CountBytes() const
		{
			FArchiveCountPushModelMem Ar;

			ObjectKeyToInternalId.CountBytes(Ar);
			PerObjectStates.CountBytes(Ar);

			for (TSparseArray<FPushModelPerObjectState>::TConstIterator It = PerObjectStates.CreateConstIterator(); It; ++It)
			{
				It->CountBytes(Ar);
			}

			return sizeof(*this) + Ar.GetMem();
		}

		bool HasHandlesAssigned() const
		{
			return !ObjectKeyToInternalId.IsEmpty();
		}

		void PrintAssignedHandles() const
		{
			UE_LOGF(LogPushModel, Display, "Starting print of %u objects currently assigned a PushModel handle", ObjectKeyToInternalId.Num());
			for (auto It=ObjectKeyToInternalId.CreateConstIterator(); It; ++It)
			{
				const FObjectKey& Key = It.Key();
				const FNetLegacyPushObjectId& Value = It.Value();

				UObject* ObjPtr = Key.ResolveObjectPtrEvenIfUnreachable();
				
				UE_LOGF(LogPushModel, Display, "Object: %ls (%ls) | PushObjectId: 0x%x", *GetNameSafe(ObjPtr), IsValid(ObjPtr)?TEXT("Valid"):TEXT("Invalid"), Value);
			}
		}

	private:
		int32 NewObjectLookupPosition = 0;
		TMap<FObjectKey, FNetLegacyPushObjectId> ObjectKeyToInternalId;
		TSparseArray<FPushModelPerObjectState> PerObjectStates;
		FDelegateHandle PostGarbageCollectHandle;

		class FArchiveCountPushModelMem : public FArchive
		{
		public:
			FArchiveCountPushModelMem() : Mem(0) 
			{ 
				ArIsCountingMemory = true; 
			}

			virtual void CountBytes(SIZE_T InNum, SIZE_T InMax) override { Mem += InMax; }

			SIZE_T GetMem() const { return Mem; }

		private:
			SIZE_T Mem;
		};
	};

	static FPushModelObjectManager_CustomId PushObjectManager;

	bool bIsPushModelEnabled = false;
	FAutoConsoleVariableRef CVarIsPushModelEnabled(
		TEXT("Net.IsPushModelEnabled"),
		bIsPushModelEnabled,
		TEXT("Whether or not Push Model is enabled. This networking mode allows game code to notify the networking system of changes, rather than scraping.")
	);
	
	bool bMakeBpPropertiesPushModel = true;
	FAutoConsoleVariableRef CVarMakeBpPropertiesPushModel(
		TEXT("Net.MakeBpPropertiesPushModel"),
		bMakeBpPropertiesPushModel,
		TEXT("Whether or not properties declared in Blueprints will be forced to used Push Model")
	);

	FAutoConsoleCommand PushModelPrintHandles(
		TEXT("Net.PushModelPrintHandles"),
		TEXT("Prints the list of replicated objects relevant to a specific connection"),
		FConsoleCommandDelegate::CreateLambda([]() { PushObjectManager.PrintAssignedHandles(); })
	);

	static FIrisMarkPropertyDirty IrisMarkPropertyDirtyDelegate;
	static FIrisMarkPropertiesDirty IrisMarkPropertiesDirtyDelegate;

	void SetIrisMarkPropertyDirtyDelegate(const FIrisMarkPropertyDirty& Delegate)
	{
		IrisMarkPropertyDirtyDelegate = Delegate;
	}

	void SetIrisMarkPropertiesDirtyDelegate(const FIrisMarkPropertiesDirty& Delegate)
	{
		IrisMarkPropertiesDirtyDelegate = Delegate;
	}

	void MarkPropertyDirty(const UObject* Object, const FNetPushObjectId ObjectId, const int32 RepIndex)
	{
		if (ObjectId.IsIrisId())
		{
			// Assume the delegate is bound. If it isn't we have an invalid ID, which is a bug that needs to be tracked down.
			IrisMarkPropertyDirtyDelegate.Execute(Object, ObjectId.GetIrisPushObjectId(), RepIndex);
		}
		else
		{
			PushObjectManager.MarkPropertyDirty(ObjectId.GetLegacyPushObjectId(), RepIndex);
		}
	}

	void MarkPropertyDirty(const UObject* Object, const FNetPushObjectId ObjectId, const int32 StartRepIndex, const int32 EndRepIndex)
	{
		if (ObjectId.IsIrisId())
		{
			IrisMarkPropertiesDirtyDelegate.Execute(Object, ObjectId.GetIrisPushObjectId(), StartRepIndex, EndRepIndex);
		}
		else
		{
			PushObjectManager.MarkPropertyDirty(ObjectId.GetLegacyPushObjectId(), StartRepIndex, EndRepIndex);
		}
	}

	/**
	 * Called by a given NetDriver to notify us that it's seen a given Object for the first time (or the first time
	 * since it was removed).
	 *
	 * This may be called multiple times for a given Object if there are multiple NetDrivers, but it's expected
	 * that each NetDriver only calls this once per object before RemoteNetworkObject is called.
	 *
	 * @param ObjectKey						An ObjectKey to uniquely identify the object.
	 * @param NumberOfReplicatedProperties	The number of replicated properties for this object.
	 *
	 * @return A Handle that can be used in other calls to uniquely identify this object per NetDriver.
	 */
	const FPushModelPerNetDriverHandle AddPushModelObject(const FObjectKey ObjectId, const uint16 NumberOfReplicatedProperties)
	{
		return PushObjectManager.AddNetworkObject(ObjectId, NumberOfReplicatedProperties);
	}

	/**
	 * Called by a given NetDriver to notify us that a given Object is no longer valid for Networking.
	 *
	 * This may be called multiple times for a given Object if there are multiple NetDrivers, but it's expected
	 * that each NetDriver only calls this once per object after AddNetworkObject is called, and never before
	 * AddNetworkObject is called.
	 *
	 * @param Handle	The Push Model Object handle (returned by AddPushModelObject).
	 */
	void RemovePushModelObject(const FPushModelPerNetDriverHandle Handle)
	{
		PushObjectManager.RemoveNetworkObject(Handle);
	}

	/**
	 * Gets the NetDriver specific state for a given Push Model Object.
	 * Note, calling this will flush dirty state to all NetDriver states for the Object.
	 *
	 * @param Handle	The Push Model Object handle (returned by AddPushModelObject).
	 */
	FPushModelPerNetDriverState* GetPerNetDriverState(const FPushModelPerNetDriverHandle Handle)
	{
		return PushObjectManager.GetPerNetDriverState(Handle);
	}

	/**
	 * @return True if the Object (or NetDriver state) have dirty properties, or have had GC 
	 */
	bool DoesHaveDirtyPropertiesOrRecentlyCollectedGarbage(const FPushModelPerNetDriverHandle Handle)
	{
		return PushObjectManager.DoesHaveDirtyPropertiesOrRecentlyCollectedGarbage(Handle);
	}

	bool ValidateObjectIdReassignment(FNetLegacyPushObjectId CurrentId, FNetLegacyPushObjectId NewId)
	{
		return PushObjectManager.ValidateObjectIdReassignment(CurrentId, NewId);
	}

	void LogMemory(FOutputDevice& Ar)
	{
		uint32 Count = PushObjectManager.CountBytes();

		Ar.Logf(TEXT("  Push Model Memory: %u"), Count);
	}

	bool bAllowHandleCreation = true;
	bool IsHandleCreationAllowed()
	{
		return bAllowHandleCreation;
	}

	void SetHandleCreationAllowed(bool bAllow)
	{
		UE_LOGF(LogPushModel, Log, "PushModel HandleCreation is now %ls", bAllow ? TEXT("enabled") : TEXT("disabled"));
		bAllowHandleCreation = bAllow;

		if (!bAllow && PushObjectManager.HasHandlesAssigned())
		{
			PushObjectManager.PrintAssignedHandles();
			ensureMsgf(!PushObjectManager.HasHandlesAssigned(), TEXT("PushModel Handle creation is disabled but there are already handles allocated. See logs for details on the assigned handles"));
		}
	}
} // end namespace UEPushModelPrivate

#endif // WITH_PUSH_MODEL