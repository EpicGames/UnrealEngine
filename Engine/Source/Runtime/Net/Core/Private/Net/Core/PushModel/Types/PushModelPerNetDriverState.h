// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/PushModel/PushModelMacros.h"

#if WITH_PUSH_MODEL

#include "CoreMinimal.h"
#include "Containers/BitArray.h"
#include "PushModelUtils.h"

namespace UEPushModelPrivate
{
	class FPushModelPerNetDriverState
	{
	public:

		FPushModelPerNetDriverState(const uint16 InNumberOfProperties)
			: PropertyDirtyStates(true, InNumberOfProperties)
			, bRecentlyCollectedGarbage(false)
			, bHasDirtyProperties(true)
		{
		}
		
		FPushModelPerNetDriverState(FPushModelPerNetDriverState&& Other)
			: PropertyDirtyStates(MoveTemp(Other.PropertyDirtyStates))
			, bRecentlyCollectedGarbage(Other.bRecentlyCollectedGarbage)
		{
		}
		
		FPushModelPerNetDriverState(const FPushModelPerNetDriverState& Other) = delete;
		FPushModelPerNetDriverState& operator=(const FPushModelPerNetDriverState& Other) = delete;

		void SetRecentlyCollectedGarbage()
		{
			bRecentlyCollectedGarbage = true;
		}

		void ResetDirtyStates()
		{
			ResetBitArray(PropertyDirtyStates);
			bRecentlyCollectedGarbage = false;
			bHasDirtyProperties = false;
		}

		void CountBytes(FArchive& Ar) const
		{
			PropertyDirtyStates.CountBytes(Ar);
		}

		bool IsPropertyDirty(const uint16 RepIndex) const
		{
			return PropertyDirtyStates[RepIndex];
		}

		TConstSetBitIterator<> GetDirtyProperties() const
		{
			return TConstSetBitIterator<>(PropertyDirtyStates);
		}

		bool DidRecentlyCollectGarbage() const
		{
			return bRecentlyCollectedGarbage;
		}

		void MarkPropertiesDirty(const TBitArray<>& OtherBitArray)
		{
			BitwiseOrBitArrays(OtherBitArray, PropertyDirtyStates);
			bHasDirtyProperties = true;
		}

		void MarkPropertyDirty(const uint16 RepIndex)
		{
			PropertyDirtyStates[RepIndex] = true;
			bHasDirtyProperties = true;
		}

		bool HasDirtyProperties() const
		{
			return bHasDirtyProperties;
		}

	private:

		/**
		 * Current state of our push model properties.
		 * Note, bits will be allocated for all replicated properties, not just push model properties.
		 */
		TBitArray<> PropertyDirtyStates;
		uint8 bRecentlyCollectedGarbage : 1;
		uint8 bHasDirtyProperties : 1;
	};
}

#endif