// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/PushModel/PushModelMacros.h"

#if WITH_PUSH_MODEL

#include "CoreMinimal.h"
#include "PushModelPerNetDriverState.h"
#include "PushModelUtils.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Containers/SparseArray.h"
#include "UObject/ObjectKey.h"

namespace UEPushModelPrivate
{
	/**
	 * This is a "state" for a given Object that is being tracked by a Push Model Object Manager.
	 * This state is shared across all NetDrivers, and so has a 1:1 mapping with actual UObjects.
	 */
	class FPushModelPerObjectState
	{
	public:

		/**
		 * Creates a new FPushModelPerObjectState.
		 *
		 * @param InObjectKey			An ObjectKey that can uniqely identify the object we're representing.
		 * @param InNumberOfProperties	The total number of replicated properties this object has.
		 */
		FPushModelPerObjectState(const FObjectKey InObjectKey, const uint16 InNumberOfProperties)
			: ObjectKey(InObjectKey)
			, DirtiedThisFrame(true, InNumberOfProperties)
			, bHasDirtyProperties(true)
		{
		}

		FPushModelPerObjectState(FPushModelPerObjectState&& Other)
			: ObjectKey(Other.ObjectKey)
			, DirtiedThisFrame(MoveTemp(Other.DirtiedThisFrame))
			, PerNetDriverStates(MoveTemp(Other.PerNetDriverStates))
			, bHasDirtyProperties(Other.bHasDirtyProperties)
		{
			Other.bHasDirtyProperties = false;
		}

		FPushModelPerObjectState(const FPushModelPerObjectState& Other) = delete;
		FPushModelPerObjectState& operator=(const FPushModelPerObjectState& Other) = delete;

		/**
		 * Currently, there are no notifications when a property gets nulled out from Garbage Collection.
		 * This should'nt happen frequently in networked scenarios as it can cause other bugs with things like
		 * Dormancy. However, it is easily reproducible on Clients recording Replays.
		 */
		void SetRecentlyCollectedGarbage()
		{
			for (FPushModelPerNetDriverState& NetDriverObject : PerNetDriverStates)
			{
				NetDriverObject.SetRecentlyCollectedGarbage();
			}
		}

		void MarkPropertyDirty(const uint16 RepIndex)
		{
			DirtiedThisFrame[RepIndex] = true;
			bHasDirtyProperties = true;
		}
		
		/**
		 * Pushes the current dirty state of the Push Model Object to each of the Net Driver States.
		 * and then reset the dirty state.
		 */
		void PushDirtyStateToNetDrivers()
		{
			if (bHasDirtyProperties)
			{
				for (FPushModelPerNetDriverState& NetDriverObject : PerNetDriverStates)
				{
					NetDriverObject.MarkPropertiesDirty(DirtiedThisFrame);
				}

				ResetBitArray(DirtiedThisFrame);
				bHasDirtyProperties = false;
			}
		}

		FPushModelPerNetDriverState& GetPerNetDriverState(FNetPushPerNetDriverId DriverId)
		{
			return PerNetDriverStates[DriverId];
		}

		const FPushModelPerNetDriverState& GetPerNetDriverState(FNetPushPerNetDriverId DriverId) const
		{
			return PerNetDriverStates[DriverId];
		}
		
		FNetPushPerNetDriverId AddPerNetDriverState()
		{
			FSparseArrayAllocationInfo AllocationInfo = PerNetDriverStates.AddUninitialized();
			new (AllocationInfo.Pointer) FPushModelPerNetDriverState(static_cast<uint16>(DirtiedThisFrame.Num()));
			
			return AllocationInfo.Index;
		}
		
		void RemovePerNetDriverState(const FNetPushPerNetDriverId DriverId)
		{
			PerNetDriverStates.RemoveAt(DriverId);
		}
		
		const bool HasAnyNetDriverStates()
		{
			return PerNetDriverStates.Num() != 0;
		}
		
		void CountBytes(FArchive& Ar) const
		{
			DirtiedThisFrame.CountBytes(Ar);
			PerNetDriverStates.CountBytes(Ar);
			for (const FPushModelPerNetDriverState& PerNetDriverState : PerNetDriverStates)
			{
				PerNetDriverState.CountBytes(Ar);
			}
		}
		
		const int32 GetNumberOfProperties() const
		{
			return DirtiedThisFrame.Num();
		}

		const FObjectKey& GetObjectKey() const
		{
			return ObjectKey;
		}

		bool HasDirtyProperties() const
		{
			return bHasDirtyProperties;
		}

	private:
	
		//! A unique ID for the object.
		const FObjectKey ObjectKey;

		//! Bitfield tracking which properties we've dirtied since the last time
		//! our state was pushed to NetDrivers.
		//! Note, bits will be allocated for all replicated properties, not just push model properties.
		TBitArray<> DirtiedThisFrame;

		//! Set of NetDriver states that have been requested and are currently tracking the object.
		TSparseArray<FPushModelPerNetDriverState> PerNetDriverStates;

		uint8 bHasDirtyProperties : 1;
	};
}

#endif