// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/PushModel/PushModelMacros.h"

#if WITH_PUSH_MODEL

#include "CoreMinimal.h"
#include "Containers/BitArray.h"

namespace UEPushModelPrivate
{
	static const int32 GetNumWordsInBitArray(const TBitArray<>& BitArray)
	{
		return (BitArray.Num() + 31) >> 5;
	}
	
	static void ResetBitArray(TBitArray<>& ToReset)
	{
		FMemory::Memzero(ToReset.GetData(), GetNumWordsInBitArray(ToReset) * 4);
	}
	
	static void SetBitArray(TBitArray<>& ToSet)
	{
		FMemory::Memset(ToSet.GetData(), 0xFF, GetNumWordsInBitArray(ToSet) * 4);
	}
	
	static void BitwiseOrBitArrays(const TBitArray<>& MaskBitArray, TBitArray<>& ResultBitArray)
	{
		check(MaskBitArray.Num() == ResultBitArray.Num());

		const uint32* InWords = MaskBitArray.GetData();
		uint32* OutWords = ResultBitArray.GetData();
		const int32 NumWords = GetNumWordsInBitArray(MaskBitArray);

		for (int32 i = 0; i < NumWords; ++i)
		{
			OutWords[i] |= InWords[i];
		}
	}

	static bool AreAnyBitsSet(const TBitArray<>& BitArray)
	{
		const int32 NumWords = GetNumWordsInBitArray(BitArray);
		const uint32* const Words = BitArray.GetData();

		for (int32 i = 0; i < NumWords; ++i)
		{
			if (Words[i] != 0)
			{
				return true;
			}
		}

		return false;
	}
}

#endif