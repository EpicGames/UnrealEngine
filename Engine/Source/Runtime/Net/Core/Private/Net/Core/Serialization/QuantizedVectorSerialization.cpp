// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/Serialization/QuantizedVectorSerialization.h"
#include "HAL/PlatformMath.h"
#include "Logging/LogMacros.h"
#include "Math/Vector.h"
#include "Traits/IntType.h"

namespace UE::Net::Private
{

/* Returns the number of bits needed for the int32 to be replicated properly. At least 1 for the sign bit. */
inline uint32 GetBitsNeeded(const int32 Value)
{
	const uint32 MassagedValue = uint32(Value ^ (Value >> 31U));
	return 33U - static_cast<uint32>(FPlatformMath::CountLeadingZeros(MassagedValue));
}

/* Returns the number of bits needed for the int32 to be replicated properly. At least 1 for the sign bit. */
inline uint32 GetBitsNeeded(const int64 Value)
{
	const uint64 MassagedValue = uint64(Value ^ (Value >> 63U));
	return 65U - static_cast<uint32>(FPlatformMath::CountLeadingZeros64(MassagedValue));
}

/* Round to integer */
inline int32 RoundFloatToInt(float F)
{
	return int32(F + FPlatformMath::Sign(F)*0.5f);
}

inline int64 RoundFloatToInt(double F)
{
	return int64(F + FPlatformMath::Sign(F)*0.5);
}

template<class T>
bool WriteQuantizedVector(const int32 Scale, const T& Value, FArchive& Ar)
{
	using ScalarType = decltype(T::X);
	constexpr SIZE_T ScalarTypeSize = sizeof(ScalarType);
	using IntType = typename TSignedIntType<ScalarTypeSize>::Type;

	static_assert(ScalarTypeSize == 4U || ScalarTypeSize == 8U, "Unknown floating point type.");

	// Beyond 2^MaxExponentForScaling scaling cannot improve the precision as the next floating point value is at least 1.0 more. 
	constexpr uint32 MaxExponentForScaling = ScalarTypeSize == 4 ? 23U : 52U;
	constexpr ScalarType MaxValueToScale = ScalarType(IntType(1) << MaxExponentForScaling);

	// Rounding of large values can introduce additional precision errors and the extra cost to serialize with full precision is small.
	constexpr uint32 MaxExponentAfterScaling = ScalarTypeSize == 4 ? 30U : 62U;
	constexpr ScalarType MaxScaledValue = ScalarType(IntType(1) << MaxExponentAfterScaling);

	// NaN values can be properly serialized using the full precision path, but they typically cause lots of errors
	// for the typical engine use case.
	if (Value.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("%s"), TEXT("WriteQuantizedVector: Value isn't finite. Clearing for safety."));
		WriteQuantizedVector(Scale, T{0,0,0}, Ar);
		return false;
	}

	const ScalarType Factor = IntCastChecked<int16>(Scale);
	T ScaledValue;
	ScaledValue.X = Value.X*Factor;
	ScaledValue.Y = Value.Y*Factor;
	ScaledValue.Z = Value.Z*Factor;

	// If the component values are within bounds then we optimize the bandwidth, otherwise we use full precision.
	if (ScaledValue.GetAbsMax() < MaxScaledValue)
	{
		const bool bUseScaledValue = Value.GetAbsMin() < MaxValueToScale;

		IntType X;
		IntType Y;
		IntType Z;
		if (bUseScaledValue)
		{
			X = RoundFloatToInt(ScaledValue.X);
			Y = RoundFloatToInt(ScaledValue.Y);
			Z = RoundFloatToInt(ScaledValue.Z);
		}
		else
		{
			X = RoundFloatToInt(Value.X);
			Y = RoundFloatToInt(Value.Y);
			Z = RoundFloatToInt(Value.Z);
		}

		const uint32 ComponentBitCount = FPlatformMath::Max(GetBitsNeeded(X), FPlatformMath::Max(GetBitsNeeded(Y), GetBitsNeeded(Z)));
		uint32 ComponentBitCountAndScaleInfo = (bUseScaledValue ? (1U << 6U) : 0U) | ComponentBitCount;
		Ar.SerializeInt(ComponentBitCountAndScaleInfo, 1U << 7U);

		Ar.SerializeBits(&X, ComponentBitCount);
		Ar.SerializeBits(&Y, ComponentBitCount);
		Ar.SerializeBits(&Z, ComponentBitCount);
	}
	else
	{
		// A component bit count of 0 indicates full precision.
		constexpr uint32 ComponentBitCount = 0;
		uint32 ComponentBitCountAndTypeInfo = (ScalarTypeSize == 8U ? (1U << 6U) : 0U) | ComponentBitCount;
		Ar.SerializeInt(ComponentBitCountAndTypeInfo, 1U << 7U);
		Ar.SerializeBits(const_cast<T*>(&Value), ScalarTypeSize*8U*3U);
	}
	
	return true;
}

template<class T>
bool ReadQuantizedVector(const int32 Scale, T& Value, FArchive& Ar)
{
	using ScalarType = decltype(T::X);
	constexpr SIZE_T ScalarTypeSize = sizeof(ScalarType);
	static_assert(ScalarTypeSize == 4 || ScalarTypeSize == 8, "Unknown floating point type.");
	using IntType = typename TSignedIntType<ScalarTypeSize>::Type;

	uint32 ComponentBitCountAndExtraInfo = 0;
	Ar.SerializeInt(ComponentBitCountAndExtraInfo, 1U << 7U);
	const uint32 ComponentBitCount = ComponentBitCountAndExtraInfo & 63U;
	const uint32 ExtraInfo = ComponentBitCountAndExtraInfo >> 6U;

	if (ComponentBitCount > 0U)
	{
		int64 X = 0;
		int64 Y = 0;
		int64 Z = 0;

		Ar.SerializeBits(&X, ComponentBitCount);
		Ar.SerializeBits(&Y, ComponentBitCount);
		Ar.SerializeBits(&Z, ComponentBitCount);

		// Sign-extend the values. The most significant bit read indicates the sign.
		const uint64 SignBit = (1ULL << (ComponentBitCount - 1U));
		X = (X ^ SignBit) - SignBit;
		Y = (Y ^ SignBit) - SignBit;
		Z = (Z ^ SignBit) - SignBit;

		T TempValue;
		TempValue.X = ScalarType(X);
		TempValue.Y = ScalarType(Y);
		TempValue.Z = ScalarType(Z);

		// Apply scaling if needed.
		if (ExtraInfo)
		{
			Value = TempValue/ScalarType(Scale);
		}
		else
		{
			Value = TempValue;
		}

		return true;
	}
	else
	{
		const uint32 ReceivedScalarTypeSize = ExtraInfo ? 8U : 4U;
		if (ReceivedScalarTypeSize == 8U)
		{
			FVector3d TempValue;
			Ar.SerializeBits(&TempValue, 8U*8U*3U);
			if (TempValue.ContainsNaN())
			{
				logOrEnsureNanError(TEXT("%s"), TEXT("ReadQuantizedVector: Value isn't finite. Clearing for safety."));
				Value = T{0,0,0};
				return false;
			}

			Value = T(TempValue);
			return true;
		}
		else
		{
			FVector3f TempValue;
			Ar.SerializeBits(&TempValue, 4U*8U*3U);
			if (TempValue.ContainsNaN())
			{
				logOrEnsureNanError(TEXT("%s"), TEXT("ReadQuantizedVector: Value isn't finite. Clearing for safety."));
				Value = T{0,0,0};
				return false;
			}

			Value = T(TempValue);
			return true;
		}
	}
}

template<class T>
T QuantizeVector(const int32 Scale, const T& Value)
{
	using ScalarType = decltype(T::X);
	constexpr SIZE_T ScalarTypeSize = sizeof(ScalarType);
	using IntType = typename TSignedIntType<ScalarTypeSize>::Type;

	static_assert(ScalarTypeSize == 4U || ScalarTypeSize == 8U, "Unknown floating point type.");

	// Beyond 2^MaxExponentForScaling scaling cannot improve the precision as the next floating point value is at least 1.0 more. 
	constexpr uint32 MaxExponentForScaling = ScalarTypeSize == 4 ? 23U : 52U;
	constexpr ScalarType MaxValueToScale = ScalarType(IntType(1) << MaxExponentForScaling);

	// Rounding of large values can introduce additional precision errors and the extra cost to serialize with full precision is small.
	constexpr uint32 MaxExponentAfterScaling = ScalarTypeSize == 4 ? 30U : 62U;
	constexpr ScalarType MaxScaledValue = ScalarType(IntType(1) << MaxExponentAfterScaling);

	// NaN values can be properly serialized using the full precision path, but they typically cause lots of errors
	// for the typical engine use case.
	if (Value.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("%s"), TEXT("QuantizeVector: Value isn't finite. Clearing for safety."));
		return T{ 0,0,0 };
	}

	const ScalarType Factor = IntCastChecked<int16>(Scale);
	T ScaledValue;
	ScaledValue.X = Value.X * Factor;
	ScaledValue.Y = Value.Y * Factor;
	ScaledValue.Z = Value.Z * Factor;

	// If the component values are within bounds then we optimize the bandwidth, otherwise we use full precision.
	if (ScaledValue.GetAbsMax() < MaxScaledValue)
	{
		const bool bUseScaledValue = Value.GetAbsMin() < MaxValueToScale;

		// 'Write' value
		IntType X;
		IntType Y;
		IntType Z;
		if (bUseScaledValue)
		{
			X = RoundFloatToInt(ScaledValue.X);
			Y = RoundFloatToInt(ScaledValue.Y);
			Z = RoundFloatToInt(ScaledValue.Z);
		}
		else
		{
			X = RoundFloatToInt(Value.X);
			Y = RoundFloatToInt(Value.Y);
			Z = RoundFloatToInt(Value.Z);
		}

		// 'Read' value
		T TempValue;
		TempValue.X = ScalarType(X);
		TempValue.Y = ScalarType(Y);
		TempValue.Z = ScalarType(Z);

		// Apply scaling if needed.
		if (bUseScaledValue)
		{
			return TempValue / ScalarType(Scale);
		}
		else
		{
			return TempValue;
		}
	}

	return Value;
}

}

namespace UE::Net
{

bool WriteQuantizedVector(int32 Scale, const FVector3d& Value, FArchive& Ar)
{
	return Private::WriteQuantizedVector(Scale, Value, Ar);
}

bool ReadQuantizedVector(int32 Scale, FVector3d& Value, FArchive& Ar)
{
	return Private::ReadQuantizedVector(Scale, Value, Ar);
}

FVector3d QuantizeVector(const int32 Scale, const FVector3d& Value)
{
	return Private::QuantizeVector(Scale, Value);
}

bool WriteQuantizedVector(int32 Scale, const FVector3f& Value, FArchive& Ar)
{
	return Private::WriteQuantizedVector(Scale, Value, Ar);
}

bool ReadQuantizedVector(int32 Scale, FVector3f& Value, FArchive& Ar)
{
	return Private::ReadQuantizedVector(Scale, Value, Ar);
}

FVector3f QuantizeVector(const int32 Scale, const FVector3f& Value)
{
	return Private::QuantizeVector(Scale, Value);
}

}
