// Copyright Epic Games, Inc. All Rights Reserved.
#include "Net/Core/Trace/Private/NetTraceInternal.h"
#include "AutoRTFM.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Net/Core/Trace/Reporters/NetTraceReporter.h"
#include "Hash/CityHash.h"
#include "HAL/IConsoleManager.h"
#include "Trace/Trace.h"
#include "UObject/NameTypes.h"
#include <atomic>

#define UE_NET_TRACE_VALIDATE 1

struct FNetTraceReporter;

#if UE_NET_TRACE_ENABLED

uint32 GNetTraceRuntimeVerbosity = 0U;

struct FNetTraceInternal
{
	typedef FNetTraceReporter Reporter;

	enum ENetTraceVersion
	{
		ENetTraceVersion_Initial = 1,
		ENetTraceVersion_BunchChannelIndex = 2,
		ENetTraceVersion_BunchChannelInfo = 3,
		ENetTraceVersion_FixedBunchSizeEncoding = 4,
		ENetTraceVersion_DebugNameIndexIs32Bits = 5,
	};

	struct FConnection
	{
		FString Address;
		FString OwningActor;
		uint8 State;
	};

	struct FNetObject
	{
		FName Name;
		const UE::Net::FNetDebugName* DebugName;
		uint64 TypeIdentifier; 
		uint64 ParentNetObjectId;
		uint32 OwnerId;
	};

	struct FGameInstance
	{
		TMap<uint32, FConnection> Connections;
		TMap<uint64, FNetObject> Objects;
		TSet<uint64> ObjectsTraced;
		TMap<uint64, FNetObject> SharedNewObjects; //Thread-safe, only used to track objects that were added during this frame
		FString Name;
		uint32 Id;//Same as the index of the owning GameInstances TMap
		bool bIsServer;
	};

	//Per-GameInstance variables that are stored per-thread
	struct FObjectTraceCounter
	{
		TSet<uint64> ObjectsTraced;
	};

	struct FThreadBuffer
	{
		// Trace Id allows us to discard the old Name Id maps when we start a new trace
		uint32 TraceId;

		// Map FName to NameId, used to know what FNames we have traced
		TMap<FName, UE::Net::FNetDebugNameId> DynamicFNameToNameIdMap;

		// Map hashed dynamic TCHAR* strings to NameId
		TMap<uint64, UE::Net::FNetDebugNameId> DynamicNameHashToNameIdMap;

		// Stores the Objects and ObjectsTraces structures on a Per-Game-Instance
		TMap<uint32, FObjectTraceCounter> GameInstanceToObjectTraceCounterMap;

		// Map FNetDebugName pointer to NameId - we can never move memory location of the Persistent FNetDebugNames
		TMap<const UE::Net::FNetDebugName*, UE::Net::FNetDebugNameId> DebugNameToNameIdMap;
	};

	// Get next NameId used to track what we already have traced
	static UE::Net::FNetDebugNameId GetNextNameId();

	static uint32 GetCurrentTraceId();

	static TUniquePtr<FNetTraceInternal::FThreadBuffer>& GetThreadBufferUniquePtr();
	static inline FThreadBuffer& GetThreadBuffer();

	static constexpr ENetTraceVersion NetTraceVersion = ENetTraceVersion::ENetTraceVersion_DebugNameIndexIs32Bits;
};

static std::atomic<uint32> CurrentTraceId(0);
static std::atomic<UE::Net::FNetDebugNameId> NextNameId(1);
static FDelegateHandle OnTraceConnectedHandle;
static FDelegateHandle OnTraceStoppedHandle;
static FTransactionallySafeCriticalSection NewObjectsCS;

// Container for GameInstances map which ensures each is set up with an Id, allowing
// trace internals to pass FGameInstance around and avoid double map lookups
class FGameInstanceContainer
{
public:
	typedef TMap<uint32, FNetTraceInternal::FGameInstance> MapType;

	FNetTraceInternal::FGameInstance& FindOrAdd(const uint32 GameInstanceId)
	{
		FNetTraceInternal::FGameInstance& GameInstance = GameInstances.FindOrAdd(GameInstanceId);
		GameInstance.Id = GameInstanceId;
		return GameInstance;
	}

	const MapType& GetGameInstances() const
	{
		return GameInstances;
	}

	void Remove(uint32 GameInstanceId)
	{
		GameInstances.Remove(GameInstanceId);
	}

private:
	MapType GameInstances;
};

static FGameInstanceContainer GameInstanceContainer;

FNetTrace::FOnNetTraceStarted FNetTrace::OnNetTraceStarted;

TUniquePtr<FNetTraceInternal::FThreadBuffer>& FNetTraceInternal::GetThreadBufferUniquePtr()
{
	UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(TUniquePtr<FNetTraceInternal::FThreadBuffer>, ThreadBuffer);
	return ThreadBuffer;
}

FNetTraceInternal::FThreadBuffer& FNetTraceInternal::GetThreadBuffer()
{
	TUniquePtr<FNetTraceInternal::FThreadBuffer>& ThreadBuffer = GetThreadBufferUniquePtr();
	FNetTraceInternal::FThreadBuffer* ThreadBufferPtr = ThreadBuffer.Get();
	uint32 TraceId = GetCurrentTraceId();
	if (!ThreadBufferPtr || ThreadBufferPtr->TraceId != TraceId)
	{
		ThreadBuffer = MakeUnique<FNetTraceInternal::FThreadBuffer>();
		ThreadBufferPtr = ThreadBuffer.Get();
		ThreadBufferPtr->TraceId = TraceId;
	}

	check(ThreadBufferPtr);

	return *ThreadBufferPtr;
}

UE_AUTORTFM_ALWAYS_OPEN UE::Net::FNetDebugNameId FNetTraceInternal::GetNextNameId()
{
	return NextNameId++;
}

UE_AUTORTFM_ALWAYS_OPEN uint32 FNetTraceInternal::GetCurrentTraceId()
{
	return CurrentTraceId;
}

void FNetTrace::SetupInstanceForNetTraceId(uint32 InNetTraceId)
{
	GameInstanceContainer.FindOrAdd(InNetTraceId);
}

void FNetTrace::OnTraceConnected()
{
	++CurrentTraceId;
	//We no longer reset NextNameId or PersistentNetDebugNames as that is incompatible with the Trace Tail Buffer which is always capturing

	FNetTraceInternal::Reporter::ReportInitEvent(FNetTraceInternal::NetTraceVersion);

	for (const TPair<uint32, FNetTraceInternal::FGameInstance>& InstanceIter : GameInstanceContainer.GetGameInstances())
	{
		const uint32 GameInstanceId = InstanceIter.Key;
		const FNetTraceInternal::FGameInstance& GameInstance = InstanceIter.Value;

		FNetTraceInternal::Reporter::ReportInstanceUpdated(GameInstanceId, GameInstance.bIsServer, *GameInstance.Name);

		for (const TPair<uint32, FNetTraceInternal::FConnection>& ConnectionIter : GameInstance.Connections)
		{
			const uint32 ConnectionId = ConnectionIter.Key;
			const FNetTraceInternal::FConnection& Connection = ConnectionIter.Value;

			FNetTraceInternal::Reporter::ReportConnectionCreated(GameInstanceId, ConnectionId);
			FNetTraceInternal::Reporter::ReportConnectionUpdated(GameInstanceId, ConnectionId, *Connection.Address, *Connection.OwningActor);
			FNetTraceInternal::Reporter::ReportConnectionStateUpdated(GameInstanceId, ConnectionId, Connection.State);
		};
	}

	OnNetTraceStarted.Broadcast();
}

void FNetTrace::OnTraceStopped(FTraceAuxiliary::EConnectionType /*TraceType*/, const FString& /*TraceDestination*/)
{
	TUniquePtr<FNetTraceInternal::FThreadBuffer>& ThreadBufferUniquePtr = FNetTraceInternal::GetThreadBufferUniquePtr();
	if (ThreadBufferUniquePtr)
	{
		ThreadBufferUniquePtr.Reset();
	}
}

void FNetTrace::SetTraceVerbosity(uint32 Verbosity)
{
	const uint32 NewVerbosity = FMath::Min(Verbosity, (uint32)UE_NET_TRACE_COMPILETIME_VERBOSITY);

	// Enable
	if (!GetTraceVerbosity() && NewVerbosity)
	{
		UE::Trace::ToggleChannel(TEXT("NetChannel"), true);
		UE::Trace::ToggleChannel(TEXT("FrameChannel"), true);

		if (UE::Trace::IsTracing())
		{
			OnTraceConnected();
		}

		OnTraceConnectedHandle = FTraceAuxiliary::OnConnection.AddLambda([]() 
		{
			// OnConnection will be called from a worker thread
			ExecuteOnGameThread(TEXT("FNetTrace::OnTraceConnected"), []()
			{
				FNetTrace::OnTraceConnected();
			});
		});
		OnTraceStoppedHandle = FTraceAuxiliary::OnTraceStopped.AddStatic(&FNetTrace::OnTraceStopped);
	}
	else if (GetTraceVerbosity() && !NewVerbosity)
	{
		UE::Trace::ToggleChannel(TEXT("NetChannel"), false);

		if (UE::Trace::IsTracing())
		{
			OnTraceStopped(FTraceAuxiliary::EConnectionType::None, TEXT(""));
		}
		FTraceAuxiliary::OnConnection.Remove(OnTraceConnectedHandle);
		FTraceAuxiliary::OnTraceStopped.Remove(OnTraceStoppedHandle);
	}

	GNetTraceRuntimeVerbosity = NewVerbosity;
}

void FNetTrace::TraceEndSession(uint32 GameInstanceId)
{
	GameInstanceContainer.Remove(GameInstanceId);

	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportInstanceDestroyed(GameInstanceId);
	}
}

void FNetTrace::TraceInstanceUpdated(uint32 GameInstanceId, bool bIsServer, const TCHAR* Name)
{
	FNetTraceInternal::FGameInstance& Instance = GameInstanceContainer.FindOrAdd(GameInstanceId);
	Instance.Name = Name;
	Instance.bIsServer = bIsServer;

	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportInstanceUpdated(GameInstanceId, bIsServer, Name);
	}
}

FNetTraceCollector* FNetTrace::CreateTraceCollector()
{
	if (!GNetTraceRuntimeVerbosity)
	{
		return nullptr;
	}

	// $TODO: Track and draw from pool/freelist
	FNetTraceCollector* Collector = new FNetTraceCollector();

	return Collector;
}

void FNetTrace::DestroyTraceCollector(FNetTraceCollector* Collector)
{
	if (Collector)
	{
		// $TODO: return to pool/freelist
		delete Collector;
	}
}

void FNetTrace::FoldTraceCollector(FNetTraceCollector* DstCollector, const FNetTraceCollector* SrcCollector, uint32 Offset, uint32 LevelOffset)
{
	if (DstCollector && SrcCollector && DstCollector != SrcCollector)
	{
		// Src cannot have any committed bunches or any pending events
		check(SrcCollector->BunchEventCount == 0 && SrcCollector->CurrentNestingLevel == 0);

		// When we fold non-bunch events we inject them at the current level
		const uint32 Level = DstCollector->CurrentNestingLevel + LevelOffset;

		// Make sure that the events fit
		if (SrcCollector->EventCount + DstCollector->EventCount > (uint32)DstCollector->Events.Num())
		{
			DstCollector->Events.SetNumUninitialized(SrcCollector->EventCount + DstCollector->EventCount, EAllowShrinking::No);
		}

		if (SrcCollector->EventCount + DstCollector->EventCount <= (uint32)DstCollector->Events.Num())
		{
			uint32 DstEventIndex = DstCollector->EventCount;
			FNetTracePacketContentEvent* DstEventData = DstCollector->Events.GetData();
			for (const FNetTracePacketContentEvent& SrcEvent : MakeArrayView(SrcCollector->Events.GetData(), SrcCollector->EventCount))
			{
				FNetTracePacketContentEvent& DstEvent = DstEventData[DstEventIndex];

				DstEvent = SrcEvent;
				DstEvent.StartPos += Offset;
				DstEvent.EndPos += Offset;
				DstEvent.NestingLevel += Level;
				++DstEventIndex;
			}

			DstCollector->EventCount += SrcCollector->EventCount;

			DstCollector->ReferencedObjectIds.Append(SrcCollector->ReferencedObjectIds);
		}
	}
}

void FNetTrace::PushStreamOffset(FNetTraceCollector* Collector, uint32 Offset)
{
	if (ensure(Collector->OffsetStackLevel < (FNetTraceCollector::MaxNestingLevel - 1U)))
	{
		const uint32 OffsetStackLevel = Collector->OffsetStackLevel;

		// Offset is additive, we always maintain a zero in the first slot
		Collector->OffsetStack[OffsetStackLevel + 1] = Collector->OffsetStack[OffsetStackLevel] + Offset;
		++Collector->OffsetStackLevel;
	}
}

void FNetTrace::PopStreamOffset(FNetTraceCollector* Collector)
{
	if (ensure(Collector->OffsetStackLevel > 0U))
	{
		--Collector->OffsetStackLevel;
	}
}

uint32 FNetTrace::BeginPacketContentEvent(FNetTraceCollector& Collector, ENetTracePacketContentEventType EventType, uint32 Pos)
{
#if UE_NET_TRACE_VALIDATE
	check(Collector.CurrentNestingLevel < (FNetTraceCollector::MaxNestingLevel - 1U));
#endif

	const uint32 EventCount = Collector.EventCount;

	const uint32 EventIndex = EventCount;
	if (EventIndex + 1U >= (uint32)Collector.Events.Num())
	{
		Collector.Events.SetNumUninitialized(EventIndex + 1U, EAllowShrinking::No);
	}

	FNetTracePacketContentEvent& Event = Collector.Events.GetData()[EventIndex];

	// Add offset
	Pos += Collector.OffsetStack[Collector.OffsetStackLevel];

	Event.StartPos = Pos;
	Event.EndPos = 0U;
	Event.EventType = (uint8)EventType;
	Event.NestingLevel = Collector.CurrentNestingLevel;

	Collector.NestingStack[Collector.CurrentNestingLevel] = Collector.EventCount;

	++Collector.CurrentNestingLevel;
	++Collector.EventCount;

	return EventIndex;
}

void FNetTrace::EndPacketContentEvent(FNetTraceCollector& Collector, uint32 EventIndex, uint32 Pos)
{
#if UE_NET_TRACE_VALIDATE
	check(EventIndex != FNetTrace::InvalidEventIndex);
	check(EventIndex < (uint32)Collector.Events.Num());
#endif

	FNetTracePacketContentEvent& Event = Collector.Events.GetData()[EventIndex];

	const uint32 EventNestingLevel = Event.NestingLevel;
	check(EventNestingLevel < Collector.CurrentNestingLevel);

	// Add offset
	Pos += Collector.OffsetStack[Collector.OffsetStackLevel];

	// When we retire events that did not write any data, do not report events with a higher nesting level
	Event.EndPos = Pos;
	if (Pos <= Event.StartPos)
	{
		// Roll back detected, remove events or mark them as not committed so we can report wasted writes?
		Collector.EventCount = Collector.NestingStack[EventNestingLevel];
	}
	--Collector.CurrentNestingLevel;
}

void FNetTrace::TracePacketContentEvent(FNetTraceCollector& Collector, UE::Net::FNetDebugNameId InNetTraceNameId, uint32 StartPos, uint32 EndPos, uint32 Verbosity)
{
	if (FNetTrace::GetTraceVerbosity() >= Verbosity)
	{
		const uint32 EventIndex = BeginPacketContentEvent(Collector, ENetTracePacketContentEventType::NameId, StartPos);

		FNetTracePacketContentEvent& Event = Collector.Events.GetData()[EventIndex];
		Event.DebugNameId = InNetTraceNameId;

		EndPacketContentEvent(Collector, EventIndex, EndPos);
	}
}

void FNetTrace::BeginBunch(FNetTraceCollector& Collector)
{
#if UE_NET_TRACE_VALIDATE
	check(Collector.PendingBunchEventIndex == ~0U);
	check(Collector.CurrentNestingLevel == 0U);
#endif

	Collector.PendingBunchEventIndex = Collector.EventCount;
}

void FNetTrace::DiscardBunch(FNetTraceCollector& Collector)
{
#if UE_NET_TRACE_VALIDATE
	check(Collector.PendingBunchEventIndex != ~0U);
	check(Collector.EventCount >= Collector.PendingBunchEventIndex);
	check(Collector.CurrentNestingLevel == 0U);
#endif

	// Just restore the event count 
	Collector.EventCount = Collector.PendingBunchEventIndex;
	Collector.CurrentNestingLevel = 0U;
	Collector.PendingBunchEventIndex = ~0U;
}

void FNetTrace::EndBunch(FNetTraceCollector& DstCollector, UE::Net::FNetDebugNameId BunchName, uint32 StartPos, uint32 HeaderBits, uint32 BunchBits, const FNetTraceBunchInfo& BunchInfo)
{
#if UE_NET_TRACE_VALIDATE
	check(DstCollector.PendingBunchEventIndex != ~0U);
	check(DstCollector.EventCount >= DstCollector.PendingBunchEventIndex);

	// Can only add bunch events at level 0
	check(DstCollector.CurrentNestingLevel == 0U);
#endif

	// Make sure that we have enough space for BunchEvent + BunchHeaderEvent
	const uint32 BunchEventIndex = DstCollector.EventCount;
	if (BunchEventIndex + 2U >= (uint32)DstCollector.Events.Num())
	{
		DstCollector.Events.SetNumUninitialized(BunchEventIndex + 2U, EAllowShrinking::No);
	}

	// Note that the bunch indices are different from storage indices	
	const uint32 BunchEventCount = DstCollector.EventCount - DstCollector.PendingBunchEventIndex;

	FNetTracePacketContentEvent* BunchEvent = &DstCollector.Events.GetData()[BunchEventIndex];

	// For bunch events we use the data a bit differently
	BunchEvent->DebugNameId = BunchName;
	BunchEvent->StartPos = StartPos;
	BunchEvent->EndPos = BunchBits;
	BunchEvent->EventType = (uint8)ENetTracePacketContentEventType::BunchEvent;
	BunchEvent->NestingLevel = 0U;

	// Mark the last bunch event
	DstCollector.LastBunchEventIndex = BunchEventIndex;

	//UE_LOGF(LogTemp, Display, "FNetTrace::EndBunch BunchBits: %u, EventCount: %u, BunchEventCount: %u", BunchBits, DstCollector.EventCount, BunchEventCount);

	++DstCollector.EventCount;

	// Store bunch header data as a separate event
	FNetTracePacketContentEvent* BunchHeaderEvent = &DstCollector.Events.GetData()[BunchEventIndex + 1];

	// ChannelIndex
	BunchHeaderEvent->BunchInfo = BunchInfo;

	// EventCount, is stored in start pos
	BunchHeaderEvent->StartPos = BunchEventCount;

	// Header bits if any
	BunchHeaderEvent->EndPos = HeaderBits;
	BunchHeaderEvent->EventType = (uint8)ENetTracePacketContentEventType::BunchHeaderEvent;
	BunchHeaderEvent->NestingLevel = 0U;

	DstCollector.PendingBunchEventIndex = ~0U;
	++DstCollector.BunchEventCount;
	++DstCollector.EventCount;
}

void FNetTrace::TraceBunch(FNetTraceCollector& DstCollector, const FNetTraceBunchInfo& BunchInfo, FName BunchName, uint32 StartPos, uint32 HeaderBits, uint32 BunchBits, const FNetTraceCollector* BunchCollector)
{
	if (&DstCollector != BunchCollector)
	{
		FNetTrace::BeginBunch(DstCollector);
		FNetTrace::FoldTraceCollector(&DstCollector, BunchCollector, 0U);
	}

	FNetTrace::EndBunch(DstCollector, TraceName(BunchName), StartPos, HeaderBits, BunchBits, BunchInfo);
}

void FNetTrace::TraceBunch(FNetTraceCollector& DstCollector, const FNetTraceBunchInfo& BunchInfo, const TCHAR* BunchName, uint32 StartPos, uint32 HeaderBits, uint32 BunchBits, const FNetTraceCollector* BunchCollector)
{
	if (&DstCollector != BunchCollector)
	{
		FNetTrace::BeginBunch(DstCollector);
		FNetTrace::FoldTraceCollector(&DstCollector, BunchCollector, 0U);
	}

	FNetTrace::EndBunch(DstCollector, TraceName(BunchName), StartPos, HeaderBits, BunchBits, BunchInfo);
}

void FNetTrace::PopSendBunch(FNetTraceCollector& Collector)
{
#if UE_NET_TRACE_VALIDATE
	check(Collector.CurrentNestingLevel == 0U);
	check(Collector.BunchEventCount > 0);
	check(Collector.EventCount > Collector.LastBunchEventIndex + 1U);
#endif

	const uint32 BunchEventIndex = Collector.LastBunchEventIndex;
	const uint32 BunchEventHeaderIndex = BunchEventIndex + 1U;

	FNetTracePacketContentEvent& BunchHeaderEvent = Collector.Events[BunchEventHeaderIndex];

#if UE_NET_TRACE_VALIDATE
	check(BunchHeaderEvent.EndPos != 0U);
	check(BunchHeaderEvent.EventType == (uint8)ENetTracePacketContentEventType::BunchHeaderEvent);
#endif

	//UE_LOGF(LogTemp, Display, "FNetTrace::PopSendBunch EventCount: %u",  Collector.EventCount);
	BunchHeaderEvent.EndPos = 0U;
}

static UE::Net::FNetDebugNameId& GetOrCreateDebugNameIdInLocalMap(const UE::Net::FNetDebugName* DebugNamePtr, FNetTraceInternal::FThreadBuffer& InThreadBuffer)
{
	UE::Net::FNetDebugNameId* NameIdPtr = InThreadBuffer.DebugNameToNameIdMap.Find(DebugNamePtr);
	if(!NameIdPtr)
	{
		NameIdPtr = &(InThreadBuffer.DebugNameToNameIdMap.Add(DebugNamePtr, FNetTrace::TraceName(DebugNamePtr->Name)));
	}

	check(NameIdPtr);
	return *NameIdPtr;
}

static void TraceExistingObject(FNetTraceInternal::FGameInstance& GameInstance, const TMap<uint64, FNetTraceInternal::FNetObject>& Objects, TSet<uint64>& ObjectsTraced, uint64 NetObjectId)
{
	if (!ObjectsTraced.Contains(NetObjectId))
	{
		//Try to look up object in the Objects list, otherwise fall back to the SharedNewObjects List which can be accessed from any thread
		const FNetTraceInternal::FNetObject* NetObject = Objects.Find(NetObjectId);
		FNetTraceInternal::FNetObject CachedNetObject;//Conditionally used for data in SharedNewObjects to avoid locking NewObjectsCS for the whole function
		if (!NetObject)
		{
			UE::TScopeLock CondScopeLock(NewObjectsCS);
			const FNetTraceInternal::FNetObject* TempNetObjectPtr = GameInstance.SharedNewObjects.Find(NetObjectId);
			if(TempNetObjectPtr)
			{
				//Copy data into CachedNetObject, in case another thread modifies SharedNewObjects and TempNetObjectPtr becomes invalid
				CachedNetObject = *TempNetObjectPtr;
				NetObject = &CachedNetObject;
			}
		}
		
		if (NetObject)
		{
			ObjectsTraced.Add(NetObjectId);

			if (NetObject->DebugName)
			{
				FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();
				const UE::Net::FNetDebugNameId DebugNameId = GetOrCreateDebugNameIdInLocalMap(NetObject->DebugName, LocalThreadBuffer);
					
				FNetTraceInternal::Reporter::ReportObjectExists(GameInstance.Id, NetObjectId, DebugNameId, NetObject->TypeIdentifier, NetObject->OwnerId);
			}
			else
			{
				FNetTraceInternal::Reporter::ReportObjectExists(GameInstance.Id, NetObjectId, FNetTrace::TraceName(NetObject->Name), NetObject->TypeIdentifier, NetObject->OwnerId);
			}

			if (NetObject->ParentNetObjectId != uint64(-1))
			{
				TraceExistingObject(GameInstance, Objects, ObjectsTraced, NetObject->ParentNetObjectId);
				FNetTraceInternal::Reporter::ReportSubObject(GameInstance.Id, NetObject->ParentNetObjectId, NetObjectId);
			}
		}
	}
}

void FNetTrace::TraceCollectedEvents(FNetTraceCollector& Collector, uint32 GameInstanceId, uint32 ConnectionId, ENetTracePacketType PacketType)
{
	FNetTracePacketInfo PacketInfo;
	PacketInfo.ConnectionId = (uint16)ConnectionId;
	PacketInfo.GameInstanceId = GameInstanceId;
	PacketInfo.PacketSequenceNumber = 0;
	PacketInfo.PacketType = PacketType;

	FNetTraceInternal::FGameInstance& GameInstance = GameInstanceContainer.FindOrAdd(GameInstanceId);
	FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();
	FNetTraceInternal::FObjectTraceCounter& ObjectTraceData = LocalThreadBuffer.GameInstanceToObjectTraceCounterMap.FindOrAdd(GameInstanceId);
	for (uint64 NetObjectId : Collector.ReferencedObjectIds)
	{
		TraceExistingObject(GameInstance, GameInstance.Objects, ObjectTraceData.ObjectsTraced, NetObjectId);
	}

	// Trace all collected events
	FNetTraceInternal::Reporter::ReportPacketContent(Collector.Events.GetData(), Collector.EventCount, PacketInfo);

	Collector.Reset();
}

void FNetTrace::TracePacketDropped(uint32 GameInstanceId, uint32 ConnectionId, uint32 PacketSequenceNumber, ENetTracePacketType PacketType)
{
	if (GNetTraceRuntimeVerbosity)
	{
		FNetTracePacketInfo PacketInfo;
		PacketInfo.ConnectionId = (uint16)ConnectionId;
		PacketInfo.GameInstanceId = GameInstanceId;
		PacketInfo.PacketSequenceNumber = PacketSequenceNumber;
		PacketInfo.PacketType = PacketType;

		FNetTraceInternal::Reporter::ReportPacketDropped(PacketInfo);
	}
}

void FNetTrace::TracePacket(uint32 GameInstanceId, uint32 ConnectionId, uint32 PacketSequenceNumber, uint32 PacketBits, ENetTracePacketType PacketType)
{
	if (GNetTraceRuntimeVerbosity)
	{
		FNetTracePacketInfo PacketInfo;
		PacketInfo.ConnectionId = (uint16)ConnectionId;
		PacketInfo.GameInstanceId = GameInstanceId;
		PacketInfo.PacketSequenceNumber = PacketSequenceNumber;
		PacketInfo.PacketType = PacketType;

		//UE_LOGF(LogTemp, Display, "FNetTrace::TracePacket GameInstance: %u, ConnectionId: %u Seq: %u NumBits: %u IsSend: %u",  GameInstanceId, ConnectionId, PacketSequenceNumber, PacketBits, PacketType == ENetTracePacketType::Outgoing ? 1U : 0U);

		FNetTraceInternal::Reporter::ReportPacket(PacketInfo, PacketBits);
	}
}

void FNetTrace::TraceObjectCreated(uint32 GameInstanceId, uint64 NetObjectId, const UE::Net::FNetDebugName* DebugName, uint64 TypeIdentifier, uint32 OwnerId)
{
	FNetTraceInternal::FGameInstance& GameInstance = GameInstanceContainer.FindOrAdd(GameInstanceId);

	{
		UE::TScopeLock NewObjectsScopeLock(NewObjectsCS);
		GameInstance.SharedNewObjects.Emplace(NetObjectId, { .Name = NAME_None, .DebugName = DebugName, .TypeIdentifier = TypeIdentifier, .ParentNetObjectId = uint64(-1), .OwnerId = OwnerId });
	}

	if (!GNetTraceRuntimeVerbosity)
	{
		return;
	}

	FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();
	FNetTraceInternal::FObjectTraceCounter& ObjectTraceData = LocalThreadBuffer.GameInstanceToObjectTraceCounterMap.FindOrAdd(GameInstanceId);

	ObjectTraceData.ObjectsTraced.Add(NetObjectId);

	if (DebugName->DebugNameId == 0U)
	{
		DebugName->DebugNameId = FNetTrace::TraceName(DebugName->Name);
	}
	FNetTraceInternal::Reporter::ReportObjectCreated(GameInstanceId, NetObjectId, DebugName->DebugNameId, TypeIdentifier, OwnerId);
}

void FNetTrace::TraceObjectCreated(uint32 GameInstanceId, uint64 NetObjectId, const FName ObjectName, uint64 TypeIdentifier, uint32 OwnerId)
{
	FNetTraceInternal::FGameInstance& GameInstance = GameInstanceContainer.FindOrAdd(GameInstanceId);
	{
		UE::TScopeLock NewObjectsScopeLock(NewObjectsCS);
		GameInstance.SharedNewObjects.Emplace(NetObjectId, { .Name = ObjectName, .DebugName = nullptr, .TypeIdentifier = TypeIdentifier, .ParentNetObjectId = uint64(-1), .OwnerId = OwnerId });
	}
	
	if (!GNetTraceRuntimeVerbosity)
	{
		return;
	}

	FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();
	FNetTraceInternal::FObjectTraceCounter& ObjectTraceData = LocalThreadBuffer.GameInstanceToObjectTraceCounterMap.FindOrAdd(GameInstanceId);

	ObjectTraceData.ObjectsTraced.Add(NetObjectId);

	FNetTraceInternal::Reporter::ReportObjectCreated(GameInstanceId, NetObjectId, FNetTrace::TraceName(ObjectName), TypeIdentifier, OwnerId);
}

void FNetTrace::TraceObjectDestroyed(uint32 GameInstanceId, uint64 NetObjectId)
{
	FNetTraceInternal::FGameInstance& GameInstance = GameInstanceContainer.FindOrAdd(GameInstanceId);
	FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();
	FNetTraceInternal::FObjectTraceCounter& ObjectTraceData = LocalThreadBuffer.GameInstanceToObjectTraceCounterMap.FindOrAdd(GameInstanceId);

	GameInstance.Objects.Remove(NetObjectId);

	if (GNetTraceRuntimeVerbosity)
	{
		ObjectTraceData.ObjectsTraced.Remove(NetObjectId);

		FNetTraceInternal::Reporter::ReportObjectDestroyed(GameInstanceId, NetObjectId);
	}
}

void FNetTrace::TraceConnectionCreated(uint32 GameInstanceId, uint32 ConnectionId)
{
	// TraceConnectionStateUpdated can be called before this, so use FindOrAdd
	GameInstanceContainer.FindOrAdd(GameInstanceId).Connections.FindOrAdd(ConnectionId);

	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportConnectionCreated(GameInstanceId, ConnectionId);
	}
}

void FNetTrace::TraceConnectionStateUpdated(uint32 GameInstanceId, uint32 ConnectionId, uint8 ConnectionStateValue)
{
	GameInstanceContainer.FindOrAdd(GameInstanceId).Connections.FindOrAdd(ConnectionId).State = ConnectionStateValue;

	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportConnectionStateUpdated(GameInstanceId, ConnectionId, ConnectionStateValue);
	}
}

void FNetTrace::TraceConnectionUpdated(uint32 GameInstanceId, uint32 ConnectionId, const TCHAR* AddressString, const TCHAR* OwningActor)
{
	FNetTraceInternal::FConnection& Connection = GameInstanceContainer.FindOrAdd(GameInstanceId).Connections.FindOrAdd(ConnectionId);
	Connection.Address = AddressString;
	Connection.OwningActor = OwningActor;

	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportConnectionUpdated(GameInstanceId, ConnectionId, AddressString, OwningActor);
	}
}

void FNetTrace::TraceConnectionClosed(uint32 GameInstanceId, uint32 ConnectionId)
{
	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportConnectionClosed(GameInstanceId, ConnectionId);
	}
}

void FNetTrace::TracePacketStatsCounter(uint32 GameInstanceId, uint32 ConnectionId, UE::Net::FNetDebugNameId CounterNameId, uint32 StatValue)
{
	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportPacketStatsCounter(GameInstanceId, ConnectionId, CounterNameId, StatValue);
	}
}

void FNetTrace::TracePacketStatsCounter(uint32 GameInstanceId, uint32 ConnectionId, const UE::Net::FNetDebugName* CounterName, uint32 StatValue)
{
	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportPacketStatsCounter(GameInstanceId, ConnectionId, CounterName->DebugNameId, StatValue);
	}
}

void FNetTrace::TraceFrameStatsCounter(uint32 GameInstanceId, UE::Net::FNetDebugNameId CounterNameId, uint32 StatValue)
{
	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportFrameStatsCounter(GameInstanceId, CounterNameId, StatValue);
	}
}

void FNetTrace::TraceFrameStatsCounter(uint32 GameInstanceId, const UE::Net::FNetDebugName* CounterName, uint32 StatValue)
{
	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportFrameStatsCounter(GameInstanceId, CounterName->DebugNameId, StatValue);
	}
}

UE::Net::FNetDebugNameId FNetTrace::TraceName(const TCHAR* Name)
{
	if ((GNetTraceRuntimeVerbosity == 0U) | (Name == nullptr))
	{
		return 0U;
	}

	// Get Thread buffer
	FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();

	// Hash the name using CityHash64
	const uint64 HashedName = CityHash64((const char*)Name, FCString::Strlen(Name) * sizeof(TCHAR));
	if (const UE::Net::FNetDebugNameId* FoundNameId = LocalThreadBuffer.DynamicNameHashToNameIdMap.Find(HashedName))
	{
		return *FoundNameId;
	}
	else
	{
		const UE::Net::FNetDebugNameId NameId = FNetTraceInternal::GetNextNameId();
		LocalThreadBuffer.DynamicNameHashToNameIdMap.Add(HashedName, NameId);

		FTCHARToUTF8 Converter(Name);
		FNetTraceInternal::Reporter::ReportAnsiName(NameId, Converter.Length() + 1, (const char*)Converter.Get());

		return NameId;
	}
}

UE::Net::FNetDebugNameId FNetTrace::TraceName(FName Name)
{
	using namespace UE::Net;

	if ((GNetTraceRuntimeVerbosity == 0U) || Name.IsNone())
	{
		return 0U;
	}

	// Get Thread buffer
	FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();

	if (const FNetDebugNameId* FoundNameId = LocalThreadBuffer.DynamicFNameToNameIdMap.Find(Name))
	{
		return *FoundNameId;
	}
	else
	{
		const FNetDebugNameId NameId = FNetTraceInternal::GetNextNameId();
		LocalThreadBuffer.DynamicFNameToNameIdMap.Add(Name, NameId);

		TCHAR Buffer[FName::StringBufferSize];
		uint32 NameLen = Name.ToString(Buffer);
		FTCHARToUTF8 Converter(Buffer);
		FNetTraceInternal::Reporter::ReportAnsiName(NameId, Converter.Length() + 1, (const char*)Converter.Get());

		return NameId;
	}
}

UE::Net::FNetDebugNameId FNetTrace::TraceName(const UE::Net::FNetDebugName* DebugName)
{
	if ((GNetTraceRuntimeVerbosity == 0U) | (DebugName == nullptr))
	{
		return 0U;
	}

	if (DebugName->DebugNameId == 0U)
	{
		FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();
		UE::Net::FNetDebugNameId NameId = GetOrCreateDebugNameIdInLocalMap(DebugName, LocalThreadBuffer);

		return NameId;
	}
	else
	{
		return DebugName->DebugNameId;
	}
}

void FNetTrace::TraceObjectPolled(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles, bool bIsWaste)
{
	// not rechecking GNetTraceRuntimeVerbosity here as it's already checked to be == VeryVerbose to have got this far
	FNetTraceInternal::FGameInstance& GameInstance = GameInstanceContainer.FindOrAdd(GameInstanceId);
	FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();
	FNetTraceInternal::FObjectTraceCounter& ObjectTraceData = LocalThreadBuffer.GameInstanceToObjectTraceCounterMap.FindOrAdd(GameInstanceId);
	
	TraceExistingObject(GameInstance, GameInstance.Objects, ObjectTraceData.ObjectsTraced, NetObjectId);
	
	FNetTraceInternal::Reporter::ReportObjectPolled(GameInstanceId, NetObjectId, Cycles, bIsWaste);
}

void FNetTrace::TraceObjectQuantized(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles)
{
	// not rechecking GNetTraceRuntimeVerbosity here as it's already checked to be == VeryVerbose to have got this far
	FNetTraceInternal::FGameInstance& GameInstance = GameInstanceContainer.FindOrAdd(GameInstanceId);
	FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();
	FNetTraceInternal::FObjectTraceCounter& ObjectTraceData = LocalThreadBuffer.GameInstanceToObjectTraceCounterMap.FindOrAdd(GameInstanceId);

	TraceExistingObject(GameInstance, GameInstance.Objects, ObjectTraceData.ObjectsTraced, NetObjectId);

	FNetTraceInternal::Reporter::ReportObjectQuantized(GameInstanceId, NetObjectId, Cycles);
}

void FNetTrace::TraceObjectWritten(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles)
{
	// not rechecking GNetTraceRuntimeVerbosity here as it's already checked to be == VeryVerbose to have got this far
	FNetTraceInternal::FGameInstance& GameInstance = GameInstanceContainer.FindOrAdd(GameInstanceId);
	FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();
	FNetTraceInternal::FObjectTraceCounter& ObjectTraceData = LocalThreadBuffer.GameInstanceToObjectTraceCounterMap.FindOrAdd(GameInstanceId);

	TraceExistingObject(GameInstance, GameInstance.Objects, ObjectTraceData.ObjectsTraced, NetObjectId);

	FNetTraceInternal::Reporter::ReportObjectWritten(GameInstanceId, NetObjectId, Cycles);
}

void FNetTrace::TraceSubObject(uint32 GameInstanceId, uint64 ParentNetObjectId, uint64 SubObjectNetObjectId)
{
	FNetTraceInternal::FGameInstance& GameInstance = GameInstanceContainer.FindOrAdd(GameInstanceId);
	FNetTraceInternal::FThreadBuffer& LocalThreadBuffer = FNetTraceInternal::GetThreadBuffer();
	if (FNetTraceInternal::FNetObject* SubObject = GameInstance.Objects.Find(SubObjectNetObjectId))
	{
		SubObject->ParentNetObjectId = ParentNetObjectId;
	}
	
	if (GNetTraceRuntimeVerbosity)
	{
		FNetTraceInternal::Reporter::ReportSubObject(GameInstanceId, ParentNetObjectId, SubObjectNetObjectId);
	}	
}

void FNetTrace::FinishConnectionTick(const uint32 GameInstanceId)
{
	FNetTraceInternal::FGameInstance& GameInstance = GameInstanceContainer.FindOrAdd(GameInstanceId);

	//Combine the SharedNewObjects into the Objects list, and wipe it, ready for the next frame
	for (const TPair<uint64, FNetTraceInternal::FNetObject>& KVP : GameInstance.SharedNewObjects)
	{
		if (!GameInstance.Objects.Find(KVP.Key))
		{
			GameInstance.Objects.Add(KVP.Key, { .Name = KVP.Value.Name, .DebugName = KVP.Value.DebugName, .TypeIdentifier = KVP.Value.TypeIdentifier, .ParentNetObjectId = KVP.Value.ParentNetObjectId, .OwnerId = KVP.Value.OwnerId });
		}
	}

	GameInstance.SharedNewObjects.Empty();
}

static FAutoConsoleCommand NeTraceSetVerbosityCmd = FAutoConsoleCommand(
	TEXT("NetTrace.SetTraceVerbosity"),
	TEXT("Start NetTrace with given verbositylevel."),
	FConsoleCommandWithArgsDelegate::CreateStatic(
		[](const TArray<FString>& Args)
		{
			if (Args.Num() == 0)
			{
				return;
			}

			FNetTrace::SetTraceVerbosity(FCString::Atoi(*Args[0]));
		}
	)
);

#endif
