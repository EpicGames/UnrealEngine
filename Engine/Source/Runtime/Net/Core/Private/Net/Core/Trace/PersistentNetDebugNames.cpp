// Copyright Epic Games, Inc. All Rights Reserved.
#include "Net/Core/Trace/NetDebugName.h"
#include "Hash/CityHash.h"
#include "Containers/Map.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Misc/CoreDelegates.h"
#include "Misc/MemStack.h"
#include "Misc/ScopeLock.h"


namespace UE::Net::Private
{
	static FNetDebugName const* ReliabilityNetDebugNames[2];
}

namespace UE::Net
{

class FPersistentNetDebugNames
{
public:
	explicit FPersistentNetDebugNames(uint32 InitialCapacity = 512);
	
	// Create persistent name
	// returns true if this was a new name and false if it already existed
	bool CreatePersistentNetDebugName(const TCHAR* Name, uint32 Length, const FNetDebugName*& OutDebugName);

	static FPersistentNetDebugNames& Get();
	static void ResetNameIds();
	static void TearDown();
private:

	FTransactionallySafeCriticalSection CriticalSection;
	TMap<uint64, const FNetDebugName*> PersistentNames;
	FMemStackBase Allocator;
	bool bIsInitialized;
};

FPersistentNetDebugNames::FPersistentNetDebugNames(uint32 InitialCapacity)
: Allocator()
, bIsInitialized(false)
{
	PersistentNames.Reserve(InitialCapacity);
}

FPersistentNetDebugNames& FPersistentNetDebugNames::Get()
{
	static FPersistentNetDebugNames Singleton;

	if (!Singleton.bIsInitialized)
	{
		FCoreDelegates::OnExit.AddStatic(&TearDown);
		Singleton.bIsInitialized = true;

#if UE_NET_TRACE_ENABLED
		Private::ReliabilityNetDebugNames[0] = UE::Net::CreatePersistentNetDebugName(TEXT("Unreliable"));
		Private::ReliabilityNetDebugNames[1] = UE::Net::CreatePersistentNetDebugName(TEXT("Reliable"));
#endif
	}

	return Singleton;
}

void FPersistentNetDebugNames::ResetNameIds()
{
	FPersistentNetDebugNames& Singleton = Get();
	{
		UE::TScopeLock Lock(Singleton.CriticalSection);

		for (const auto& Iter : Singleton.PersistentNames)
		{
			Iter.Value->DebugNameId = 0;
		}
	}
}

void FPersistentNetDebugNames::TearDown()
{
	FPersistentNetDebugNames& Singleton = Get();	
	Singleton.PersistentNames.Empty();
	Singleton.Allocator.Flush();
}

bool FPersistentNetDebugNames::CreatePersistentNetDebugName(const TCHAR* Name, uint32 Length, const FNetDebugName*& OutDebugName)
{
	const uint32 NameSize = Length * sizeof(TCHAR);

	// Hash name
	const uint64 HashedName = CityHash64((const char*)Name, NameSize);
	
	FNetDebugName* PersistentName = nullptr;

	UE::TScopeLock Lock(CriticalSection);
	const FNetDebugName** Entry = PersistentNames.Find(HashedName);
	if (Entry)
	{
		OutDebugName = *Entry;
		return false;
	}
	else
	{
		// Allocate FNetDebugName + extra space for TCHAR string including null terminator
		uint8* PersistentBuffer = (uint8*)Allocator.Alloc(sizeof(FNetDebugName) + NameSize + sizeof(TCHAR), alignof(FNetDebugName));

		PersistentName = (FNetDebugName*)PersistentBuffer;
		PersistentName->Name = (TCHAR*)(PersistentBuffer + sizeof(FNetDebugName));
		PersistentName->DebugNameId = 0U;
		// Store in map
		PersistentNames.Add(HashedName, PersistentName);
	}

	// copy the string to the allocated storage
	FCString::Strncpy((TCHAR*)PersistentName->Name, Name, Length + 1);

	OutDebugName = PersistentName;

	return true;
}

FNetDebugName const* GetReliabilityNetDebugName(bool bIsReliable)
{
	return Private::ReliabilityNetDebugNames[bIsReliable ? 1 : 0];
}

const FNetDebugName* CreatePersistentNetDebugName(const TCHAR* Name, uint32 NameLen)
{
	const FNetDebugName* DebugName;
	FPersistentNetDebugNames::Get().CreatePersistentNetDebugName(Name, NameLen, DebugName);

	return DebugName;
}

}
