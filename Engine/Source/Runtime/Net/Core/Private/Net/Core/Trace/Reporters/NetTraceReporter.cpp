// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Core/Trace/Reporters/NetTraceReporter.h"

#if UE_NET_TRACE_ENABLED

#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformTime.h"
#include "Net/Core/Trace/NetTrace.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Trace/Trace.inl"

uint32 FNetTraceReporter::NetTraceReporterVersion = 1;

NETCORE_API UE_TRACE_CHANNEL_EXTERN(NetChannel)
UE_TRACE_CHANNEL_DEFINE(NetChannel, "Network connections, packets (sent/received/dropped), packet content, replicated object lifecycle, and per-connection stats. Viewable in the Network Insights profiler.")

// We always output this event first to make sure we have a version number for backwards compatibility
UE_TRACE_EVENT_BEGIN(NetTrace, InitEvent)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, NetTraceVersion)
	UE_TRACE_EVENT_FIELD(uint32, NetTraceReporterVersion)
UE_TRACE_EVENT_END()

// Trace a name
UE_TRACE_EVENT_BEGIN(NetTrace, NameEvent, Important | NoSync)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetTrace, ObjectCreatedEvent, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, TypeId)
	UE_TRACE_EVENT_FIELD(uint64, ObjectId)
	UE_TRACE_EVENT_FIELD(uint32, OwnerId)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetTrace, ObjectExistsEvent, NoSync)	
	UE_TRACE_EVENT_FIELD(uint64, TypeId)
	UE_TRACE_EVENT_FIELD(uint64, ObjectId)
	UE_TRACE_EVENT_FIELD(uint32, OwnerId)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetTrace, ObjectDestroyedEvent)
	UE_TRACE_EVENT_FIELD(uint64, ObjectId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

// What else do we want to know? should we maybe call this a connectionEvent instead?
UE_TRACE_EVENT_BEGIN(NetTrace, ConnectionCreatedEvent)
	UE_TRACE_EVENT_FIELD(uint16, ConnectionId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetTrace, ConnectionStateUpdatedEvent)
	UE_TRACE_EVENT_FIELD(uint16, ConnectionId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
	UE_TRACE_EVENT_FIELD(uint8, ConnectionStateValue)
UE_TRACE_EVENT_END()

// Provides additional information about connection after it is created
UE_TRACE_EVENT_BEGIN(NetTrace, ConnectionUpdatedEvent)
	UE_TRACE_EVENT_FIELD(uint16, ConnectionId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Address)
UE_TRACE_EVENT_END()

// Add close reason?
UE_TRACE_EVENT_BEGIN(NetTrace, ConnectionClosedEvent)
	UE_TRACE_EVENT_FIELD(uint16, ConnectionId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

// StatsCounterEvent
UE_TRACE_EVENT_BEGIN(NetTrace, PacketStatsCounterEvent, NoSync)
	UE_TRACE_EVENT_FIELD(uint32, StatsValue)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
	UE_TRACE_EVENT_FIELD(uint16, ConnectionId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetTrace, FrameStatsCounterEvent, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, StatsValue)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

// Provides additional information about game instance
UE_TRACE_EVENT_BEGIN(NetTrace, InstanceUpdatedEvent)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
	UE_TRACE_EVENT_FIELD(bool, bIsServer)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()

// rename 
UE_TRACE_EVENT_BEGIN(NetTrace, InstanceDestroyedEvent)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

// Packet data is transmitted as attachment
UE_TRACE_EVENT_BEGIN(NetTrace, PacketContentEvent, NoSync)
	UE_TRACE_EVENT_FIELD(uint16, ConnectionId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
	UE_TRACE_EVENT_FIELD(uint8, PacketType)
	UE_TRACE_EVENT_FIELD(uint8[], Data)
UE_TRACE_EVENT_END()

//$TODO: Drop the timestamp when we can get them for free on the analysis side
UE_TRACE_EVENT_BEGIN(NetTrace, PacketEvent)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PacketBits)
	UE_TRACE_EVENT_FIELD(uint32, SequenceNumber)
	UE_TRACE_EVENT_FIELD(uint16, ConnectionId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
	UE_TRACE_EVENT_FIELD(uint8, PacketType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetTrace, PacketDroppedEvent)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, SequenceNumber)
	UE_TRACE_EVENT_FIELD(uint16, ConnectionId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
	UE_TRACE_EVENT_FIELD(uint8, PacketType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetTrace, ObjectPolledEvent, NoSync)
	UE_TRACE_EVENT_FIELD(uint64, ObjectId)
	UE_TRACE_EVENT_FIELD(uint64, Cycles)
	UE_TRACE_EVENT_FIELD(bool, bIsWaste)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetTrace, ObjectQuantizedEvent)
	UE_TRACE_EVENT_FIELD(uint64, ObjectId)
	UE_TRACE_EVENT_FIELD(uint64, Cycles)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetTrace, ObjectWrittenEvent)
	UE_TRACE_EVENT_FIELD(uint64, ObjectId)
	UE_TRACE_EVENT_FIELD(uint64, Cycles)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetTrace, SubObjectEvent)
	UE_TRACE_EVENT_FIELD(uint64, ParentObjectId)
	UE_TRACE_EVENT_FIELD(uint64, SubObjectId)
	UE_TRACE_EVENT_FIELD(uint8, GameInstanceId)
UE_TRACE_EVENT_END()

void FNetTraceReporter::ReportInitEvent(uint32 NetTraceVersion)
{
	UE_TRACE_LOG(NetTrace, InitEvent, NetChannel)
		<< InitEvent.Timestamp(FPlatformTime::Cycles64())
		<< InitEvent.NetTraceVersion(NetTraceVersion)
		<< InitEvent.NetTraceReporterVersion(NetTraceReporterVersion);
}

void FNetTraceReporter::ReportInstanceUpdated(uint32 GameInstanceId, bool bIsServer, const TCHAR* Name)
{
	UE_TRACE_LOG(NetTrace, InstanceUpdatedEvent, NetChannel)
		<< InstanceUpdatedEvent.GameInstanceId((uint8)GameInstanceId)
		<< InstanceUpdatedEvent.bIsServer(bIsServer)
		<< InstanceUpdatedEvent.Name(Name);
}

void FNetTraceReporter::ReportInstanceDestroyed(uint32 GameInstanceId)
{
	UE_TRACE_LOG(NetTrace, InstanceDestroyedEvent, NetChannel)
		<< InstanceDestroyedEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportAnsiName(UE::Net::FNetDebugNameId NameId, uint32 NameSize, const char* Name)
{
	UE_TRACE_LOG(NetTrace, NameEvent, NetChannel, NameSize * sizeof(char))
		<< NameEvent.NameId(NameId)
		<< NameEvent.Name(Name, NameSize);
}

void FNetTraceReporter::ReportPacketDropped(const FNetTracePacketInfo& PacketInfo)
{
	UE_TRACE_LOG(NetTrace, PacketDroppedEvent, NetChannel)
		<< PacketDroppedEvent.Timestamp(FPlatformTime::Cycles64())
		<< PacketDroppedEvent.SequenceNumber(PacketInfo.PacketSequenceNumber)
		<< PacketDroppedEvent.ConnectionId(PacketInfo.ConnectionId)
		<< PacketDroppedEvent.GameInstanceId((uint8)PacketInfo.GameInstanceId)
		<< PacketDroppedEvent.PacketType((uint8)PacketInfo.PacketType);
}

void FNetTraceReporter::ReportPacket(const FNetTracePacketInfo& PacketInfo, uint32 PacketBits)
{
	UE_TRACE_LOG(NetTrace, PacketEvent, NetChannel)
		<< PacketEvent.Timestamp(FPlatformTime::Cycles64())
		<< PacketEvent.PacketBits(PacketBits)
		<< PacketEvent.SequenceNumber(PacketInfo.PacketSequenceNumber)
		<< PacketEvent.ConnectionId(PacketInfo.ConnectionId)
		<< PacketEvent.GameInstanceId((uint8)PacketInfo.GameInstanceId)
		<< PacketEvent.PacketType((uint8)PacketInfo.PacketType);
}

void FNetTraceReporter::ReportPacketContent(FNetTracePacketContentEvent* Events, uint32 EventCount, const FNetTracePacketInfo& PacketInfo)
{
	// $IRIS: $TODO: Get Max attachmentsize when that is available from trace system
	const uint32 BufferSize = 3096u;
	const uint32 MaxEncodedEventSize = 20u;
	const uint32 FlushBufferThreshold = BufferSize - MaxEncodedEventSize;

	uint8 Buffer[BufferSize];
	uint8* BufferPtr = Buffer;
	
	uint64 LastOffset = 0u;

	auto FlushPacketContentBuffer = [](const FNetTracePacketInfo& InPacketInfo, const uint8* InBuffer, uint32 Count)
	{
		UE_TRACE_LOG(NetTrace, PacketContentEvent, NetChannel)
			<< PacketContentEvent.ConnectionId(InPacketInfo.ConnectionId)
			<< PacketContentEvent.GameInstanceId((uint8)InPacketInfo.GameInstanceId)
			<< PacketContentEvent.PacketType((uint8)InPacketInfo.PacketType)
			<< PacketContentEvent.Data(InBuffer, Count);
	};

	for (const FNetTracePacketContentEvent& CurrentEvent : MakeArrayView(Events, EventCount))
	{
		// Flush
		if ((BufferPtr - Buffer) > FlushBufferThreshold)
		{
			FlushPacketContentBuffer(PacketInfo, Buffer, (uint32)(BufferPtr - Buffer));
			BufferPtr = Buffer;
			LastOffset = 0;
		}

		// Encode event data to buffer

		// Type
		*(BufferPtr++) = CurrentEvent.EventType;

		switch (ENetTracePacketContentEventType(CurrentEvent.EventType))
		{
			case ENetTracePacketContentEventType::Object:
			{
				// NestingLevel
				*(BufferPtr++) = CurrentEvent.NestingLevel;

				FTraceUtils::Encode7bit(CurrentEvent.ObjectId, BufferPtr);

				// Encode event data, all offsets are delta compressed against previous begin marker
				const uint64 StartPos = CurrentEvent.StartPos;

				// Start
				FTraceUtils::Encode7bit(StartPos - LastOffset, BufferPtr);
				LastOffset = StartPos;

				// End
				FTraceUtils::Encode7bit(CurrentEvent.EndPos - StartPos, BufferPtr);
			}
			break;
			case ENetTracePacketContentEventType::NameId:
			{
				// NestingLevel
				*(BufferPtr++) = CurrentEvent.NestingLevel;

				FTraceUtils::Encode7bit(CurrentEvent.DebugNameId, BufferPtr);

				// Encode event data, all offsets are delta compressed against previous begin marker
				const uint64 StartPos = CurrentEvent.StartPos;

				// Start
				FTraceUtils::Encode7bit(StartPos - LastOffset, BufferPtr);
				LastOffset = StartPos;

				// End
				FTraceUtils::Encode7bit(CurrentEvent.EndPos - StartPos, BufferPtr);
			}
			break;
			case ENetTracePacketContentEventType::BunchEvent:
			{
				// DebugName
				const uint32 EventId = CurrentEvent.DebugNameId;
				FTraceUtils::Encode7bit(EventId, BufferPtr);

				// BunchSize
				const uint32 BunchSize = CurrentEvent.EndPos;
				FTraceUtils::Encode7bit(BunchSize, BufferPtr);

				// Must reset LastOffest when we begin a new bunch
				LastOffset = 0U;
			}
			break;
			case ENetTracePacketContentEventType::BunchHeaderEvent:
			{
				const uint32 BunchEventCount = CurrentEvent.StartPos;
				const uint32 HeaderSize = CurrentEvent.EndPos;

				// EventCount
				FTraceUtils::Encode7bit(BunchEventCount, BufferPtr);

				// HeaderSize if any
				FTraceUtils::Encode7bit(HeaderSize, BufferPtr);

				if (HeaderSize)
				{
					FTraceUtils::Encode7bit(CurrentEvent.BunchInfo.Value, BufferPtr);
				}
			}
			break;
		}
	}

	if (BufferPtr > Buffer)
	{
		FlushPacketContentBuffer(PacketInfo, Buffer, (uint32)(BufferPtr - Buffer));
		BufferPtr = Buffer;
	}
}

void FNetTraceReporter::ReportConnectionCreated(uint32 GameInstanceId, uint32 ConnectionId)
{
	UE_TRACE_LOG(NetTrace, ConnectionCreatedEvent, NetChannel)
		<< ConnectionCreatedEvent.ConnectionId((uint16)ConnectionId)
		<< ConnectionCreatedEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportConnectionStateUpdated(uint32 GameInstanceId, uint32 ConnectionId, uint8 ConnectionStateValue)
{
	UE_TRACE_LOG(NetTrace, ConnectionStateUpdatedEvent, NetChannel)
		<< ConnectionStateUpdatedEvent.ConnectionId((uint16)ConnectionId)
		<< ConnectionStateUpdatedEvent.GameInstanceId((uint8)GameInstanceId)
		<< ConnectionStateUpdatedEvent.ConnectionStateValue(ConnectionStateValue);
}

void FNetTraceReporter::ReportConnectionUpdated(uint32 GameInstanceId, uint32 ConnectionId, const TCHAR* AddressString, const TCHAR* OwningActor)
{
	UE_TRACE_LOG(NetTrace, ConnectionUpdatedEvent, NetChannel)
		<< ConnectionUpdatedEvent.GameInstanceId((uint8)GameInstanceId)
		<< ConnectionUpdatedEvent.ConnectionId((uint16)ConnectionId)
		<< ConnectionUpdatedEvent.Name(OwningActor)
		<< ConnectionUpdatedEvent.Address(AddressString);
}

void FNetTraceReporter::ReportConnectionClosed(uint32 GameInstanceId, uint32 ConnectionId)
{
	UE_TRACE_LOG(NetTrace, ConnectionClosedEvent, NetChannel)
		<< ConnectionClosedEvent.ConnectionId((uint16)ConnectionId)
		<< ConnectionClosedEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportPacketStatsCounter(uint32 GameInstanceId, uint32 ConnectionId, UE::Net::FNetDebugNameId CounterNameId, uint32 StatValue)
{
	UE_TRACE_LOG(NetTrace, PacketStatsCounterEvent, NetChannel)
		<< PacketStatsCounterEvent.StatsValue(StatValue)
		<< PacketStatsCounterEvent.NameId(CounterNameId)
		<< PacketStatsCounterEvent.ConnectionId((uint16)ConnectionId)
		<< PacketStatsCounterEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportFrameStatsCounter(uint32 GameInstanceId, UE::Net::FNetDebugNameId CounterNameId, uint32 StatValue)
{
	UE_TRACE_LOG(NetTrace, FrameStatsCounterEvent, NetChannel)
		<< FrameStatsCounterEvent.Timestamp(FPlatformTime::Cycles64())
		<< FrameStatsCounterEvent.StatsValue(StatValue)
		<< FrameStatsCounterEvent.NameId(CounterNameId)
		<< FrameStatsCounterEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportObjectCreated(uint32 GameInstanceId, uint64 NetObjectId, UE::Net::FNetDebugNameId NameId, uint64 TypeIdentifier, uint32 OwnerId)
{
	UE_TRACE_LOG(NetTrace, ObjectCreatedEvent, NetChannel)
		<< ObjectCreatedEvent.TypeId(TypeIdentifier)
		<< ObjectCreatedEvent.ObjectId(NetObjectId)
		<< ObjectCreatedEvent.OwnerId(OwnerId)
		<< ObjectCreatedEvent.NameId(NameId)
		<< ObjectCreatedEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportObjectExists(uint32 GameInstanceId, uint64 NetObjectId, UE::Net::FNetDebugNameId NameId, uint64 TypeIdentifier, uint32 OwnerId)
{
	UE_TRACE_LOG(NetTrace, ObjectExistsEvent, NetChannel)
		<< ObjectExistsEvent.TypeId(TypeIdentifier)
		<< ObjectExistsEvent.ObjectId(NetObjectId)
		<< ObjectExistsEvent.OwnerId(OwnerId)
		<< ObjectExistsEvent.NameId(NameId)
		<< ObjectExistsEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportObjectDestroyed(uint32 GameInstanceId, uint64 NetObjectId)
{
	UE_TRACE_LOG(NetTrace, ObjectDestroyedEvent, NetChannel)
		<< ObjectDestroyedEvent.ObjectId(NetObjectId)
		<< ObjectDestroyedEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportObjectPolled(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles, bool bIsWaste)
{
	UE_TRACE_LOG(NetTrace, ObjectPolledEvent, NetChannel)
		<< ObjectPolledEvent.ObjectId(NetObjectId)
		<< ObjectPolledEvent.Cycles(Cycles)
		<< ObjectPolledEvent.bIsWaste(bIsWaste)
		<< ObjectPolledEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportObjectQuantized(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles)
{
	UE_TRACE_LOG(NetTrace, ObjectQuantizedEvent, NetChannel)
		<< ObjectQuantizedEvent.ObjectId(NetObjectId)
		<< ObjectQuantizedEvent.Cycles(Cycles)
		<< ObjectQuantizedEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportObjectWritten(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles)
{
	UE_TRACE_LOG(NetTrace, ObjectWrittenEvent, NetChannel)
		<< ObjectWrittenEvent.ObjectId(NetObjectId)
		<< ObjectWrittenEvent.Cycles(Cycles)
		<< ObjectWrittenEvent.GameInstanceId((uint8)GameInstanceId);
}

void FNetTraceReporter::ReportSubObject(uint32 GameInstanceId, uint64 ParentObjectId, uint64 SubObjectId)
{
	UE_TRACE_LOG(NetTrace, SubObjectEvent, NetChannel)
		<< SubObjectEvent.ParentObjectId(ParentObjectId)
		<< SubObjectEvent.SubObjectId(SubObjectId)
		<< SubObjectEvent.GameInstanceId((uint8)GameInstanceId);
}

#endif
