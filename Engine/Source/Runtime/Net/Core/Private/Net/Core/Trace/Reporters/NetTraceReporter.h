// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/Trace/NetTrace.h"

#if UE_NET_TRACE_ENABLED

struct FNetTraceReporter
{
	static uint32 NetTraceReporterVersion;

	static void ReportInitEvent(uint32 NetTraceVersion);
	static void ReportPacketContent(FNetTracePacketContentEvent* Events, uint32 EventCount, const FNetTracePacketInfo& PacketInfo);
	static void ReportPacketDropped(const FNetTracePacketInfo& PacketInfo);
	static void ReportPacket(const FNetTracePacketInfo& PacketInfo, uint32 PacketBits);
	static void ReportAnsiName(UE::Net::FNetDebugNameId NameId, uint32 NameSize, const char* Name);	
	static void ReportObjectCreated(uint32 GameInstanceId, uint64 NetObjectId, UE::Net::FNetDebugNameId NameId, uint64 TypeIdentifier, uint32 OwnerId);
	static void ReportObjectExists(uint32 GameInstanceId, uint64 NetObjectId, UE::Net::FNetDebugNameId NameId, uint64 TypeIdentifier, uint32 OwnerId);
	static void ReportObjectDestroyed(uint32 GameInstanceId, uint64 NetObjectId);
	static void ReportConnectionCreated(uint32 GameInstanceId, uint32 ConnectionId);
	static void ReportConnectionStateUpdated(uint32 GameInstanceId, uint32 ConnectionId, uint8 ConnectionStateValue);
	static void ReportConnectionUpdated(uint32 GameInstanceId, uint32 ConnectionId, const TCHAR* AddressString, const TCHAR* OwningActor);
	static void ReportConnectionClosed(uint32 GameInstanceId, uint32 ConnectionId);
	static void ReportPacketStatsCounter(uint32 GameInstanceId, uint32 ConnectionId, UE::Net::FNetDebugNameId CounterNameId, uint32 StatValue);
	static void ReportFrameStatsCounter(uint32 GameInstanceId, UE::Net::FNetDebugNameId CounterNameId, uint32 StatValue);
	static void ReportInstanceUpdated(uint32 GameInstanceId, bool bIsServer, const TCHAR* Name);
	static void ReportInstanceDestroyed(uint32 GameInstanceId);
	static void ReportObjectPolled(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles, bool bIsWaste);
	static void ReportObjectQuantized(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles);
	static void ReportObjectWritten(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles);
	static void ReportSubObject(uint32 GameInstanceId, uint64 ParentObjectId, uint64 SubObjectId);
};

#endif
