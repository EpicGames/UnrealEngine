// Copyright Epic Games, Inc. All Rights Reserved.
#include "Net/Serialization/FastArraySerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FastArraySerializer)

DEFINE_STAT(STAT_NetSerializeFastArray);
DEFINE_STAT(STAT_NetSerializeFastArray_BuildMap);
DEFINE_STAT(STAT_NetSerializeFastArray_DeltaStruct);

int32 FFastArraySerializer::MaxNumberOfAllowedChangesPerUpdate = 2048;
FAutoConsoleVariableRef FFastArraySerializer::CVarMaxNumberOfAllowedChangesPerUpdate(
	TEXT("net.MaxNumberOfAllowedTArrayChangesPerUpdate"),
	FFastArraySerializer::MaxNumberOfAllowedChangesPerUpdate,
	TEXT("")
);

int32 FFastArraySerializer::MaxNumberOfAllowedDeletionsPerUpdate = 2048;
FAutoConsoleVariableRef FFastArraySerializer::CVarMaxNumberOfAllowedDeletionsPerUpdate(
	TEXT("net.MaxNumberOfAllowedTArrayDeletionsPerUpdate"),
	FFastArraySerializer::MaxNumberOfAllowedDeletionsPerUpdate,
	TEXT("")
);

FFastArraySerializer::FFastArraySerializer()
		: IDCounter(0)
		, ArrayReplicationKey(0)
#if WITH_PUSH_MODEL
		, OwningObject(nullptr)
		, RepIndex(INDEX_NONE)
#endif // WITH_PUSH_MODEL
		, CachedNumItems(INDEX_NONE)
		, CachedNumItemsToConsiderForWriting(INDEX_NONE)
		, DeltaFlags(EFastArraySerializerDeltaFlags::None)
	{
		SetDeltaSerializationEnabled(true);
	}

FFastArraySerializer::~FFastArraySerializer() = default;
