// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Net Analytics Aggregation
 *
 * Net analytics aggregation is used by the NetDriver (and primarily Oodle), to collect analytics during the lifetime of the NetDriver,
 * and to aggregate it so that it can be dispatched in one consolidated analytics event, at the end of the NetDriver's lifetime.
 *
 * This is useful for e.g. per-NetConnection analytics data, which we want to acquire but don't want to spam the analytics service with,
 * so we need an interface to aggregate at the NetDriver level (in principle, this should be reusable outside of the netcode too).
 *
 *
 * Usage:
 * To collect analytics data you will need to subclass FNetAnalyticsData and use the REGISTER_NET_ANALYTICS macro,
 * to create and register an instance of your data holder, with a NetDriver's NetAnalyticsAggregator,
 * using a unique name that only your code uses (preferably the analytics event name).
 *
 * This will also need to be registered in *Engine.ini, for each NetDriverName, e.g.:
 *	[GameNetDriver NetAnalyticsAggregatorConfig]
 *	+NetAnalyticsData=(DataName="Core.ServerNetConn", bEnabled=true)
 *
 * Then assign the result of REGISTER_NET_ANALYTICS, to store a reference to your data holder, in the location you collect analytics.
 *
 * The way your FNetAnalyticsData subclass collects and stores/aggregates data is up to you (and so is the thread-safety for that data),
 * just implement SendAnalytics to process and dispatch all of the aggregated data upon NetDriver shutdown. It's only called once.
 *
 *
 * Multithreaded lifetime:
 * All multithreaded analytics aggregation objects/data/events must be complete by NetDriver shut down.
 * Staying within the normal course of netcode send/receive paths should achieve this.
 */

// Includes
#include "AutoRTFM.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"

// Forward Declarations
class FNetAnalyticsAggregator;
class IAnalyticsProvider;
struct FAnalyticsEventAttribute;


// Defines

/** Whether or not to enable multithreaded support (future proofing for netcode multithreading) - comes with a performance cost */
#if !defined(NET_ANALYTICS_MULTITHREADING)
#define NET_ANALYTICS_MULTITHREADING 0
#endif


inline constexpr const ESPMode NetAnalyticsThreadSafety = ESPMode::ThreadSafe;


// Forward Declarations
struct FNetAnalyticsData;


// Typedefs
template<class T=FNetAnalyticsData> using TNetAnalyticsDataRef = TSharedRef<T, NetAnalyticsThreadSafety>;
template<class T=FNetAnalyticsData> using TNetAnalyticsDataPtr = TSharedPtr<T, NetAnalyticsThreadSafety>;
template<class T=FNetAnalyticsData> using TNetAnalyticsDataWeakPtr = TWeakPtr<T, NetAnalyticsThreadSafety>;


// Defines

/**
 * Registers a named FNetAnalyticsData instance with the net analytics aggregator.
 * Implemented this way, to support runtime name/type checks, without relying on statics.
 *
 * NOTE: May return nullptr! The analytics aggregator can selectively enable/disable analytics, based on the analytics data name
 *
 * @param Aggregator			The net analytics aggregator to register with
 * @param AnalyticsDataType		The net analytics data type to create and register
 * @param InDataName			The name associated with this analytics data - for looking it up
 * @return						Returns a shared pointer to the newly created net analytics data instance
 */
#define REGISTER_NET_ANALYTICS(Aggregator, AnalyticsDataType, InDataName) \
	StaticCastSharedPtr<AnalyticsDataType, FNetAnalyticsData, NetAnalyticsThreadSafety>( \
		(Aggregator->RegisterAnalyticsData_Internal(MakeShareable(new AnalyticsDataType(), FNetAnalyticsDataDeleter()), InDataName, \
			TEXT(#AnalyticsDataType))))


// Globals

/** Counter used to detect when a new FNetAnalyticsAggregator (i.e. new NetDriver) has been created - to trigger cleanup of TLS data */
extern NETCORE_API TAtomic<uint8> GNetAnalyticsCounter;


/**
 * For use with TThreadedNetAnalyticsData - this struct is subclassed, and used to define/hold the analytics variables,
 * that will be tracked/aggregated.
 *
 * This base type is required, due to how TThreadedNetAnalyticsData handles tracking of local analytics variables.
 */
struct FLocalNetAnalyticsStruct : public FVirtualDestructor
{
};


/**
 * Subclassed struct which holds net analytics data which is aggregated or to-be-aggregated, and dispatched upon SendAnalytics
 */
struct FNetAnalyticsData : public FVirtualDestructor, public TSharedFromThis<FNetAnalyticsData, NetAnalyticsThreadSafety>
{
	friend class FNetAnalyticsAggregator;
	friend struct FNetAnalyticsDataDeleter;

protected:
	/** The parent aggregator responsible for this data instance */
	FNetAnalyticsAggregator* Aggregator;


public:
	/**
	 * Default constructor
	 */
	NETCORE_API FNetAnalyticsData();
	NETCORE_API virtual ~FNetAnalyticsData();

	/**
	 * Dispatches the aggregated analytics data - no further data should be added
	 */
	virtual void SendAnalytics() = 0;


protected:
	/**
	 * Called by the Net Analytics Aggregator, when ready to send analytics - some implementations don't send immediately
	 */
	NETCORE_API virtual void InternalSendAnalytics();

	/**
	 * Called just before the last shared reference to this data is released - used to trigger analytics send in threadsafe version
	 */
	virtual void NotifyFinalRelease()
	{
	}
};

/**
 * Basic single-threaded-only (i.e. good for NetConnection level code) analytics data holder, which just wraps around a simple struct,
 * which defines the analytics variables and implements their aggregation.
 *
 * The passed in struct will have to implement a CommitAnalytics function, which implements aggregation of the struct data,
 * and the class which references this net analytics data will have to trigger this classes CommitAnalytics on the locally stored struct,
 * when finished collecting local analytics data.
 */
template<class TDataStruct> struct TBasicNetAnalyticsData : public FNetAnalyticsData, protected TDataStruct
{
	GENERATE_MEMBER_FUNCTION_CHECK(CommitAnalytics, void,, TDataStruct&);

	static_assert(THasMemberFunction_CommitAnalytics<TDataStruct>::Value, "TDataStruct must implement void CommitAnalytics(TDataStruct&)");

	/**
	 * Called by the class/code which is locally collecting analytics data, to commit the local data for aggregation when done.
	 *
	 * @param AnalyticsVars		The local analytics variables to be committed/aggregated
	 */
	void CommitAnalytics(TDataStruct& AnalyticsVars)
	{
		AnalyticsVars.CommitAnalytics(*this);
	}
};

#if NET_ANALYTICS_MULTITHREADING
/**
 * Special subclass of FNetAnalyticsData that is designed for thread-safety and infrequent access (e.g. at NetConnection Close only).
 * This is to future-proof for netcode multithreading, where PacketHandler level code is expected to run outside of the Game Thread.
 *
 * The last thread to release this analytics data, triggers SendAnalytics - thread safety is partially provided by shared pointer atomics.
 */
struct FThreadedNetAnalyticsData : public FNetAnalyticsData
{
	/**
	 * Default constructor
	 */
	NETCORE_API FThreadedNetAnalyticsData();


protected:
	NETCORE_API virtual void InternalSendAnalytics() override;

	NETCORE_API virtual void NotifyFinalRelease() override;


protected:
	/** Whether or not a thread has signaled that analytics are ready to be sent */
	TAtomic<bool> bReadyToSend;
};

/**
 * Special subclass of FThreadedNetAnalyticsData, which implements multithreaded synchronization, using Thread Local Storage (TLS),
 * providing a complete solution for aggregating net analytics data, by caching a local copy of TDataStruct in TLS (one per thread),
 * and automatically aggregating all instances upon NetDriver Shutdown.
 *
 * The passed in struct will have to implement a CommitAnalytics function, which implements aggregation of the struct data.
 *
 * Every time you need to update analytics, use GetLocalData to access the variables - don't permanently store the return value.
 *
 * There is a performance cost both to looking up thread_local's, and to mapping TDataStruct for TThreadedNetAnalyticsData.
 * This code must only be enabled in a multithreaded environment, due to the unnecessary performance cost in single-threaded code.
 */
template<class TDataStruct> struct TThreadedNetAnalyticsData : public FThreadedNetAnalyticsData, protected TDataStruct
{
	GENERATE_MEMBER_FUNCTION_CHECK(CommitAnalytics, void,, TDataStruct&);

	static_assert(THasMemberFunction_CommitAnalytics<TDataStruct>::Value, "TDataStruct must implement void CommitAnalytics(TDataStruct&)");
	static_assert(TIsDerivedFrom<TDataStruct, FLocalNetAnalyticsStruct>::IsDerived, "TDataStruct must be derived from FLocalNetAnalyticsStruct");


	/**
	 * Default constructor
	 */
	TThreadedNetAnalyticsData()
		: ThreadLocalData()
	{
	}

	/**
	 * Returns the current threads TDataStruct instance, for this net analytics data - creating/registering it, if necessary.
	 *
	 * @return	The current threads TDataStruct instance
	 */
	TDataStruct* GetLocalData()
	{
		/** Maps this threads net analytics variables, to the net analytics data handler which owns them - for fast lookup */
		typedef TMap<FThreadedNetAnalyticsData*, FLocalNetAnalyticsStruct*> FLocalNetAnalyticsStructMap;
		UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(FLocalNetAnalyticsStructMap, LocalNetAnalyticsMap);

		FLocalNetAnalyticsStruct** FoundVal = LocalNetAnalyticsMap.Find(this);

		return (FoundVal != nullptr ? static_cast<TDataStruct*>(*FoundVal) : AddLocalData(LocalNetAnalyticsMap));
	}


protected:
	/**
	 * Handles creation/registration of a new TDataStruct instance - as well as occasional cleanup of stale TLS data.
	 *
	 * @param LocalNetAnalyticsMap	Reference to the TLS map linking local net analytics vars, to a (potentially stale) net analytics data pointer
	 * @return						The newly created TDataStruct instance
	 */
	TDataStruct* AddLocalData(TMap<FThreadedNetAnalyticsData*, FLocalNetAnalyticsStruct*>& LocalNetAnalyticsMap)
	{
		TDataStruct* ReturnVal = new TDataStruct();

		/** Maps a weak net analytics data pointer, to its own raw pointer - to cleanup stale entries in LocalNetAnalyticsMap */
		typedef TMap<TNetAnalyticsDataWeakPtr<FThreadedNetAnalyticsData>, FThreadedNetAnalyticsData*> FThreadedNetAnalyticsDataMap;
		UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(FThreadedNetAnalyticsDataMap, StaleNetAnalyticsTracking);

		LocalNetAnalyticsMap.Add(this, ReturnVal);
		StaleNetAnalyticsTracking.Add(StaticCastSharedRef<FThreadedNetAnalyticsData, FNetAnalyticsData, NetAnalyticsThreadSafety>(AsShared()), this);
		ThreadLocalData.Enqueue(ReturnVal);


		/** Each thread tracks a counter signaling a new FNetAnalyticsAggregator/NetDriver instance, using this to trigger cleanup */
		UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(uint8, LastNetAnalyticsCounter);
		LastNetAnalyticsCounter = 0;

		uint8 CurNetAnalyticsCounter = GNetAnalyticsCounter;

		if (CurNetAnalyticsCounter != LastNetAnalyticsCounter)
		{
			for (auto It=StaleNetAnalyticsTracking.CreateIterator(); It; ++It)
			{
				if (!It.Key().IsValid())
				{
					LocalNetAnalyticsMap.Remove(It.Value());
					It.RemoveCurrent();
				}
			}

			LastNetAnalyticsCounter = CurNetAnalyticsCounter;
		}

		return ReturnVal;
	}

	virtual void NotifyFinalRelease() override
	{
		TDataStruct* CurAnalyticsData = nullptr;

		while (ThreadLocalData.Dequeue(CurAnalyticsData))
		{
			CurAnalyticsData->CommitAnalytics(*this);

			delete CurAnalyticsData;
		}

		FThreadedNetAnalyticsData::NotifyFinalRelease();
	}


protected:
	/** Every thread creates an instance of TDataStruct locally, and queues the pointer here for later processing/deletion */
	TQueue<TDataStruct*, EQueueMode::Mpsc> ThreadLocalData;
};
#endif

/**
 * Custom deleter for FNetAnalyticsData shared pointers
 */
struct FNetAnalyticsDataDeleter
{
	inline void operator()(FNetAnalyticsData* Data)
	{
		if (Data->Aggregator != nullptr)
		{
			Data->NotifyFinalRelease();
		}

		delete Data;
	}
};


/**
 * Central object (usually within NetDriver) which handles registration/retrieval/type-checking of net analytics data holders.
 */
class FNetAnalyticsAggregator
{
public:
	/**
	 * Base constructor
	 */
	NETCORE_API FNetAnalyticsAggregator(TSharedPtr<IAnalyticsProvider> InProvider, FName InNetDriverName);

	FNetAnalyticsAggregator() = delete;

	/**
	 * Initialize the net analytics aggregator
	 */
	NETCORE_API void Init();

	/**
	 * Initialize the net analytics aggregator config - must support hotfixing
	 */
	NETCORE_API void InitConfig();

	/**
	 * Adds game and engine specific attributes to the passed in array.
	 */
	NETCORE_API void AppendGameInstanceAttributes(TArray<FAnalyticsEventAttribute>& OutAttributes);

	/**
	 * Tells the analytics data holders to finish aggregating their analytics data, and to dispatch it.
	 * Only called once, at NetDriver shutdown.
	 */
	NETCORE_API void SendAnalytics();


	/**
	 * Use REGISTER_NET_ANALYTICS instead. Internal function, which registers a net analytics data holder, with an associated key name,
	 * and does type checking to ensure there have been no mixups with the data holder type.
	 *
	 * NOTE: May return nullptr! Analytics can be selectively enabled/disabled, based on the specified analytics data name.
	 *
	 * @param InData		The analytics data shared reference to be registered
	 * @param InDataName	The name given for referencing the analytics data
	 * @param InTypeName	Compile-time derived type name for the analytics data - for type checking
	 * @return				Returns the newly added analytics data shared reference
	 */
	NETCORE_API TNetAnalyticsDataPtr<> RegisterAnalyticsData_Internal(TNetAnalyticsDataRef<> InData, const FName& InDataName, FString InTypeName);


	/** Accessor for AnalyticsProvider */
	const TSharedPtr<IAnalyticsProvider>& GetAnalyticsProvider()
	{
		return AnalyticsProvider;
	}

	/** Accessor for NetDriverName */
	FName GetNetDriverName() const
	{
		return NetDriverName;
	}

public:

	using FAnalyticsAppender = TFunction<void(TArray<FAnalyticsEventAttribute>&)>;	
	NETCORE_API void SetAnalyticsAppender(FAnalyticsAppender AppenderFunction);

private:
	/** The analytics provider we are aggregating data for */
	TSharedPtr<IAnalyticsProvider> AnalyticsProvider;

	/** The name of the NetDriver which owns this analytics aggregator - for retrieving NetDriver-specific config values */
	FName NetDriverName;

	/** Maps net analytics data holders, to their specified name */
	TMap<FName, TNetAnalyticsDataRef<>> AnalyticsDataMap;

	/** Maps analytics data holder names, to their type name - to verify types and prevent miscasting */
	TMap<FName, FString> AnalyticsDataTypeMap;

	/** Maps analytics data holder names, to a config value specifying whether that data holder is enabled or not */
	TMap<FName, bool> AnalyticsDataConfigMap;

	/** Optional functor that can be used to append game specific attributes to your analytics */
	FAnalyticsAppender AnalyticsAppender;

	/** Whether or not analytics was already sent */
	bool bSentAnalytics;
};
