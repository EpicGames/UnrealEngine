// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/Optional.h"
#include "Templates/EnableIf.h"
#include "Templates/Models.h"

#include <limits>


namespace UE::Net
{

/**
 * Inherited class for a sample producer, which outputs samples to a consumer of the specified class.
 *
 * The consumer must implement: void AddSample(double Val)
 *
 * If the consumer wants to peek at each individual measurement that makes up the sample (for e.g. determining Min/Max measurements),
 * then the consumer must implement: void PeekMeasurement(double TimeVal, double Val)
 *
 * @param ConsumerType	The type which will be used for consuming samples.
 */
template<typename ConsumerType>
class TSampleProducer
{
	struct CSampleConsumer
	{
		template<typename T>
		auto Requires(T& A) -> decltype(
			A.AddSample(0.0)
		);
	};

	struct CPeekMeasurements
	{
		template<typename T>
		auto Requires(T& A) -> decltype(
			A.PeekMeasurement(0.0, 0.0)
		);
	};

	static_assert(TModels_V<CSampleConsumer, ConsumerType>, "ConsumerType must implement AddSample(double Val)");

public:
	/**
	 * Sets the consumer object to output samples to.
	 *
	 * @param InConsumer	The consumer object
	 */
	void SetConsumer(ConsumerType* InConsumer)
	{
		Consumer = InConsumer;
	}

protected:
	/**
	 * Called by subclasses to output a sample to the consumer.
	 *
	 * @param Value		The sample value being output
	 */
	void OutputSample(double Value)
	{
		if (Consumer != nullptr)
		{
			Consumer->AddSample(Value);
		}
	}

	/**
	 * Called by subclasses to let the consumer peek at individual measurements
	 *
	 * @param Value		The measurement value being collected for sampling
	 */

	template<typename InnerConsumerType=ConsumerType, typename = typename TEnableIf<TModels_V<CPeekMeasurements, InnerConsumerType>>::Type>
	inline void PeekMeasurement(double TimeVal, double Value)
	{
		if (Consumer != nullptr)
		{
			Consumer->PeekMeasurement(TimeVal, Value);
		}
	}

	template<typename InnerConsumerType=ConsumerType, typename = typename TEnableIf<!TModels_V<CPeekMeasurements, InnerConsumerType>>::Type, int32 UnusedParamForODR=0>
	inline void PeekMeasurement(double TimeVal, double Value)
	{
	}

private:
	/** The consumer object that receives samples */
	ConsumerType* Consumer;
};

/**
 * The type of moving value to calculate/track
 */
enum class EBinnedValueMode : uint8
{
	/** Tracks a moving average */
	MovingAvg,

	/** Tracks a moving sum */
	MovingSum
};

/**
 * Base class for moving values, implementing non-templatized data-structure/methods.
 * See TBinnedMovingValue.
 */
class FBinnedMovingValueBase
{
protected:
	struct FBin
	{
		/** Sum of all values in this bin */
		double Sum = 0.0;

		/** Number of values in this bin */
		int32 Count = 0;
	};

protected:
	FBinnedMovingValueBase() = delete;
	NETCORE_API explicit FBinnedMovingValueBase(const TArrayView<FBin> InBins, double InTimePerBin, EBinnedValueMode InMode);

	/**
	 * Internal implementation for AddMeasurement, which outputs a sample if the bins have filled and rolled back over to FirstBinIndex.
	 *
	 * @param TimeVal		The time value associated with the measurement, for time-based binning
	 * @param Value			The measurement value
	 * @param OutSample		Outputs a sample, if all bins have filled and rolled back over to FirstBinIndex
	 */
	NETCORE_API void AddMeasurement_Implementation(double TimeVal, double Value, TOptional<double>& OutSample);

public:
	/**
	 * Gets a sample of the calculated moving value, based on current bin data.
	 *
	 * @return	Outputs the calculated sample value
	 */
	NETCORE_API double GetSample() const;

	/**
	 * Resets the accumulated bin/sample data.
	 */
	NETCORE_API void Reset();

private:
	NETCORE_API void ResetNewAndSkippedBins(int32 BinIdx, TOptional<double>& OutSample);


private:
	/** A view of the allocated bins, for collecting moving values */
	const TArrayView<FBin> BinsView;

	/** The amount of time tracked by each individual bin */
	const double TimePerBin = 0.0;

	/** The total range of all bins */
	const double BinRange = 0.0;

	/** The mode for sample calculations - typically moving average, or moving sum */
	const EBinnedValueMode SampleMode = EBinnedValueMode::MovingAvg;

	/** The index of the first bin that a value was stored in, to detect when all bins are rolled over */
	int32 FirstBinIndex = INDEX_NONE;

	/** The index of the last bin to be written to */
	int32 LastWrittenBinIndex = INDEX_NONE;

	/** The total summed value of all bins */
	double TotalSum = 0.0;

	/** The total count of all bins */
	int32 TotalCount = 0;
};

/**
 * Tracks/stores a moving average or moving sum over a period of time, binning the accumulated values into different time periods,
 * for smoothing and balancing the memory/accuracy cost of dropping old values as they fall out of the measured time period.
 *
 * Use TBinnedMovingAvg/TBinnedMovingSum aliases, instead of using this directly.
 *
 * Measurements are input using AddMeasurement, and samples/values are output to the specified consumer, which is set using SetConsumer.
 *
 * Samples are automatically output when all of the data bins fill and are rolled over - this guarantees perfect accuracy over the full time period.
 * Samples can be manually output using Flush, but this can be less accurate depending on the use case - see Flush for more information.
 *
 * @param ConsumerType	The type which will be used for consuming samples.
 * @param NumBins		The number of bins to use.
 * @param InMode		Internal
 */
template<typename ConsumerType, int32 NumBins, EBinnedValueMode InMode>
class TBinnedMovingValue : public FBinnedMovingValueBase, public TSampleProducer<ConsumerType>
{
public:
	/**
	 * Main constructor
	 *
	 * @param TimePerBin	The amount of time that each bin represents.
	 */
	explicit TBinnedMovingValue(double TimePerBin)
		: FBinnedMovingValueBase(MakeArrayView(Bins, NumBins), TimePerBin, InMode)
	{
	}

	/**
	 * Inputs a measurement value and the approximate time it was measured.
	 *
	 * @param TimeVal		The time of the measurement.
	 * @param Value			The measurement value
	 */
	void AddMeasurement(double TimeVal, double Value)
	{
		TOptional<double> FinalSample;

		TSampleProducer<ConsumerType>::PeekMeasurement(TimeVal, Value);
		AddMeasurement_Implementation(TimeVal, Value, FinalSample);

		if (FinalSample.IsSet())
		{
			this->OutputSample(FinalSample.GetValue());
		}
	}

	/**
	 * Flushes a sample from the current bin values to the consumer.
	 *
	 * Since this is a moving value, none of the bins are reset when a sample is output (so the flushed sample can partially reflect old values),
	 * and the current bin will have been reset recently so it can be filled (so the flushed value may not reflect the full time period).
	 *
	 * The user must consider whether this is accurate/suitable for their use case.
	 */
	void Flush()
	{
		this->OutputSample(GetSample());
	}

private:
	/** Allocation for the bins which collect moving values */
	FBin Bins[NumBins] = {};
};

/**
 * Version of TBinnedMovingValue which takes permanently incrementing values (e.g. frame count),
 * and uses the delta since the previous value as the measurement (this means the very first value is not measured).
 *
 * Use TDeltaBinnedMovingAvg/TDeltaBinnedMovingSum aliases, instead of using this directly.
 *
 * @param ConsumerType	The type which will be used for consuming samples.
 * @param NumBins		The number of bins to use.
 * @param InMode		Internal
 */
template<typename ConsumerType, int32 NumBins, EBinnedValueMode InMode>
class TBinnedDeltaMovingValue : private TBinnedMovingValue<ConsumerType, NumBins, InMode>
{
	using Super = TBinnedMovingValue<ConsumerType, NumBins, InMode>;

public:
	// Using Super for constructors can break clang
	using TBinnedMovingValue<ConsumerType, NumBins, InMode>::TBinnedMovingValue;
	using Super::SetConsumer;
	using Super::GetSample;
	using Super::Flush;

	/**
	 * Inputs a value whose delta is calculated for measurement, and the approximate time it was measured.
	 *
	 * @param TimeVal		The time of the measurement.
	 * @param Value			The value to calculate the delta of, for measurement
	 */
	void AddMeasurement(double TimeVal, double Value)
	{
		if (LastValue != std::numeric_limits<double>::min())
		{
			Super::AddMeasurement(TimeVal, (Value - LastValue));
		}

		LastValue = Value;
	}

	/**
	 * Resets the accumulated bin/sample data (the very first value after this is not measured).
	 */
	void Reset()
	{
		Super::Reset();

		LastValue = std::numeric_limits<double>::min();
	}

private:
	/** The last input value, for tracking deltas */
	double LastValue = std::numeric_limits<double>::min();
};

/**
 * Alias for tracking a moving average. See TBinnedMovingValue.
 */
template<typename ConsumerType, int32 NumBins>
using TBinnedMovingAvg = TBinnedMovingValue<ConsumerType, NumBins, EBinnedValueMode::MovingAvg>;

/**
 * Alias for tracking a moving average using the delta of the input value. See TBinnedDeltaMovingValue/TBinnedMovingValue.
 */
template<typename ConsumerType, int32 NumBins>
using TDeltaBinnedMovingAvg = TBinnedDeltaMovingValue<ConsumerType, NumBins, EBinnedValueMode::MovingAvg>;

/**
 * Alias for tracking a moving sum. See TBinnedMovingValue.
 */
template<typename ConsumerType, int32 NumBins>
using TBinnedMovingSum = TBinnedMovingValue<ConsumerType, NumBins, EBinnedValueMode::MovingSum>;

/**
 * Alias for tracking a moving sum using the delta of the input value. See TBinnedDeltaMovingValue/TBinnedMovingValue.
 */
template<typename ConsumerType, int32 NumBins>
using TDeltaBinnedMovingSum = TBinnedDeltaMovingValue<ConsumerType, NumBins, EBinnedValueMode::MovingSum>;

}

/**
 * Quick named parameter emulation, e.g:
 *	TDeltaBinnedMovingSum<FSampleMinMaxAvg, TBinParms::NumBins(6)> PageFaultsPerMinTracking{TBinParms::TimePerBin(10.0)}
 */
namespace TBinParms
{
	static constexpr int32 NumBins(int32 InNumBins)
	{
		return InNumBins;
	}

	static constexpr double TimePerBin(double InBinSize)
	{
		return InBinSize;
	}
}


namespace UE::Net
{

/**
 * Determines how Min/Max values are calculated, for FSampleMinMaxAvg
 */
enum EMinMaxValueMode
{
	/** Min/Max values are calculated based on the final (possibly averaged/summed) sample */
	PerSample,

	/** Min/Max values are calculated based on every individual measurement accumulated in the sample (TSampleProducer only!) */
	PerMeasurement,

	/** Min/Max values are calculated based on the average of measurements accumulated over the specified time period (TSampleProducer only!) */
	PerTimeSeconds
};

/**
 * Base class for tracking the minimum, maximum and average of the passed in measurement values.
 *
 * This base class can handle reading.
 * The TSampleMinMaxAvg subclass is compatible as a consumer for TSampleProducer, and is used for writing.
 */
class FSampleMinMaxAvg
{
	template<EMinMaxValueMode, int32> friend class TSampleMinMaxAvg;

private:
	NETCORE_API void AddSample_Internal(double Value);

public:
	/**
	 * Resets the accumulated sample data
	 */
	NETCORE_API void Reset();

	/**
	 * Gets the minimum sample value encountered
	 *
	 * @return	The minimum sample value
	 */
	double GetMin() const
	{
		return ((MinValue == std::numeric_limits<double>::max()) ? 0.0 : MinValue);
	}

	/**
	 * Gets the maximum sample value encountered
	 *
	 * @return	The maximum sample value
	 */
	double GetMax() const
	{
		return ((MaxValue == std::numeric_limits<double>::min()) ? 0.0 : MaxValue);
	}

	/**
	 * Gets the current/latest sample value
	 *
	 * @return	The current sample value
	 */
	double GetCurrent() const
	{
		return CurrentValue;
	}

	/**
	 * Gets the average sample value
	 *
	 * @return	The average sample value
	 */
	double GetAvg() const
	{
		return TotalCount == 0 ? 0.0 : (TotalSum / static_cast<double>(TotalCount));
	}

	/**
	 * Gets the sum of all sample values tracked
	 *
	 * @return	The sum of samples
	 */
	double GetSum() const
	{
		return TotalSum;
	}

	/**
	 * Gets the number of samples tracked
	 *
	 * @return	The number of samples
	 */
	int32 GetSampleCount() const
	{
		return TotalCount;
	}

private:
	/** The minimum sample value encountered */
	double MinValue = std::numeric_limits<double>::max();

	/** The maximum sample value encountered */
	double MaxValue = std::numeric_limits<double>::min();

	/** The current sample value */
	double CurrentValue = 0.0;

	/** The total sum of all samples */
	double TotalSum = 0.0;

	/** The number of samples tracked */
	int32 TotalCount = 0;
};

/**
 * Writable subclass of FSampleMinMaxAvg. Can be cast to FSampleMinMaxAvg for read-only usage.
 *
 * TSampleMinMaxAvg<EMinMaxValueMode::PerSample>
 * TSampleMinMaxAvg<EMinMaxValueMode::PerMeasurement>
 * TSampleMinMaxAvg<EMinMaxValueMode::PerTimeSeconds, TSampleParms::TimeSeconds(1)>
 */
template<EMinMaxValueMode Mode, int32 MinMaxTimeSeconds=1>
class TSampleMinMaxAvg;

template<>
class TSampleMinMaxAvg<EMinMaxValueMode::PerSample> : public FSampleMinMaxAvg
{
public:
	/**
	 * Inputs a sample (usually from a TSampleProducer)
	 */
	void AddSample(double Value)
	{
		if (Value < MinValue)
		{
			MinValue = Value;
		}

		if (Value > MaxValue)
		{
			MaxValue = Value;
		}

		AddSample_Internal(Value);
	}
};

template<>
class TSampleMinMaxAvg<EMinMaxValueMode::PerMeasurement> : public FSampleMinMaxAvg
{
public:
	/**
	 * Inputs a sample (usually from a TSampleProducer)
	 */
	inline void AddSample(double Value)
	{
		AddSample_Internal(Value);
	}

	void PeekMeasurement(double TimeVal, double Value)
	{
		if (Value < MinValue)
		{
			MinValue = Value;
		}

		if (Value > MaxValue)
		{
			MaxValue = Value;
		}
	}
};

template<int32 MinMaxTimeSeconds>
class TSampleMinMaxAvg<EMinMaxValueMode::PerTimeSeconds, MinMaxTimeSeconds> : public FSampleMinMaxAvg
{
public:
	/**
	 * Inputs a sample (usually from a TSampleProducer)
	 */
	inline void AddSample(double Value)
	{
		AddSample_Internal(Value);
	}

	void PeekMeasurement(double TimeVal, double Value)
	{
		if (UNLIKELY(MinMaxPeriodStart == 0.0))
		{
			MinMaxPeriodStart = TimeVal;
		}
		else if ((TimeVal - MinMaxPeriodStart) >= static_cast<double>(MinMaxTimeSeconds))
		{
			const double NewMinMaxSample = MinMaxPeriodSum / MinMaxPeriodCount;

			if (NewMinMaxSample < MinValue)
			{
				MinValue = NewMinMaxSample;
			}

			if (NewMinMaxSample > MaxValue)
			{
				MaxValue = NewMinMaxSample;
			}

			MinMaxPeriodStart = TimeVal;
			MinMaxPeriodSum = 0.0;
			MinMaxPeriodCount = 0;
		}

		MinMaxPeriodSum += Value;
		MinMaxPeriodCount++;
	}

private:
	/** The time at which the last Min/Max averaging period started */
	double MinMaxPeriodStart = 0.0;

	/** The sum of measurements during the current Min/Max averaging period */
	double MinMaxPeriodSum = 0.0;

	/** The number of measurements during the current Min/Max averaging period */
	int32 MinMaxPeriodCount = 0;
};

/**
 * Class level 'using namespace UE::Net', for NetStatsUtils.h types. Inherit as protected.
 */
class FUsingNetStatsUtils
{
public:
	template<typename ConsumerType> using TSampleProducer = TSampleProducer<ConsumerType>;
	using EBinnedValueMode = EBinnedValueMode;
	template<typename ConsumerType, int32 NumBins, EBinnedValueMode InMode> using TBinnedMovingValue =
		TBinnedMovingValue<ConsumerType, NumBins, InMode>;
	template<typename ConsumerType, int32 NumBins, EBinnedValueMode InMode> using TBinnedDeltaMovingValue =
		TBinnedDeltaMovingValue<ConsumerType, NumBins, InMode>;
	template<typename ConsumerType, int32 NumBins> using TBinnedMovingAvg = TBinnedMovingAvg<ConsumerType, NumBins>;
	template<typename ConsumerType, int32 NumBins> using TDeltaBinnedMovingAvg = TDeltaBinnedMovingAvg<ConsumerType, NumBins>;
	template<typename ConsumerType, int32 NumBins> using TBinnedMovingSum = TBinnedMovingSum<ConsumerType, NumBins>;
	template<typename ConsumerType, int32 NumBins> using TDeltaBinnedMovingSum = TDeltaBinnedMovingSum<ConsumerType, NumBins>;
	using EMinMaxValueMode = EMinMaxValueMode;
	using FSampleMinMaxAvg = FSampleMinMaxAvg;
	template<EMinMaxValueMode Mode, int32 MinMaxTimeSeconds=1> using TSampleMinMaxAvg = TSampleMinMaxAvg<Mode, MinMaxTimeSeconds>;
};

}

/**
 * Quick named parameter emulation, e.g:
 *	TSampleMinMaxAvg<EMinMaxValueMode::PerTimeSeconds, TSampleParms::TimeSeconds(1)> GameThreadCPUPercentPerMinStats;
 */
namespace TSampleParms
{
	static constexpr int32 TimeSeconds(int32 InTimeSeconds)
	{
		return InTimeSeconds;
	}
}

