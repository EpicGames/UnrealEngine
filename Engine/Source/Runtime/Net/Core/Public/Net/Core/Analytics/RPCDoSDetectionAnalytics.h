// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


// Include
#include "Net/Core/Analytics/NetAnalytics.h"


// Forward declarations
struct FAnalyticsEventAttribute;
struct FRPCDoSAnalyticsData;


// Delegates

/**
 * Multicast delegate for modifying/adding-to RPC DoS analytics events.
 *
 * @param World			The World the RPC DoS Analytics and NetConnection is active in.
 * @param InOutAttrs	Specifies the analytics attributes array for performing modifications/additions
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FModifyRPCDoSAnalytics, UObject* /*World*/, TArray<FAnalyticsEventAttribute>& /*InOutAttrs*/);


// Globals

/** Globally accessible multicast delegate, for modifying/adding-to the RPC DoS analytics event */
NETCORE_API extern FModifyRPCDoSAnalytics GModifyRPCDoSAnalytics;

/** Globally accessible multicast delegate, for modifying/adding-to the RPC DoS escalation analytics event */
NETCORE_API extern FModifyRPCDoSAnalytics GModifyRPCDoSEscalationAnalytics;


// Structs

/**
 * Per-player tracking of the maximum RPC DoS escalation state
 */
struct FMaxRPCDoSEscalation
{
	/** The IP of the player */
	FString PlayerIP;

	/** The UID of the player */
	FString PlayerUID;

	/** The index of the maximum severity state that was reached */
	int32 MaxSeverityIndex = 0;

	/** The name of the maximum severity state that was reached */
	FString MaxSeverityCategory;

	/** The index of the maximum severity state that was confirmed for analytics */
	int32 MaxAnalyticsSeverityIndex = 0;

	/** The name of the maximum severity state that was confirmed for analytics */
	FString MaxAnalyticsSeverityCategory;


public:
	bool operator == (const FMaxRPCDoSEscalation& A) const
	{
		return PlayerIP == A.PlayerIP && PlayerUID == A.PlayerUID && MaxSeverityIndex == A.MaxSeverityIndex &&
				MaxSeverityCategory == A.MaxSeverityCategory && MaxAnalyticsSeverityIndex == A.MaxAnalyticsSeverityIndex &&
				MaxAnalyticsSeverityCategory == A.MaxAnalyticsSeverityCategory;
	}
};


/**
 * Per-RPC analytics for any RPC Tracking that is active within the RPC DoS Detection instance.
 */
struct FRPCAnalytics
{
	/** The name of the RPC */
	FName RPCName;

	/** The maximum number of calls to the RPC per second */
	int32 MaxCountPerSec = 0;


	/** The maximum amount of time spent executing the RPC per second (may be larger than a second, if an RPC is long-running) */
	double MaxTimePerSec = 0.0;

	/** The Game Thread CPU Usage, at the approximate time that MaxTimePerSec was set (used to detect CPU Saturation, not RPC cost) */
	uint8 MaxTimeGameThreadCPU = 0;


	/** The maximum amount of time spent executing a packet which contains only calls to this RPC */
	double MaxSinglePacketRPCTime = 0.0;

	/** The number of calls to this RPC within the packet, at the time that MaxSinglePacketRPCTime was set */
	int32 SinglePacketRPCCount = 0;

	/** The Game Thread CPU Usage, at the approximate time that MaxSinglePacketRPCTime was set (used to detect CPU Saturation, not RPC cost) */
	uint8 SinglePacketGameThreadCPU = 0;


	/** Counts the total number of times calls to this RPC were blocked */
	int32 BlockedCount = 0;


	/** Aggregation variables */

	/** The IP of the player */
	FString PlayerIP;

	/** The UID of the player */
	FString PlayerUID;


public:
	bool operator == (const FRPCAnalytics& A) const
	{
		return RPCName == A.RPCName && MaxCountPerSec == A.MaxCountPerSec && MaxTimePerSec == A.MaxTimePerSec &&
				MaxTimeGameThreadCPU == A.MaxTimeGameThreadCPU && MaxSinglePacketRPCTime == A.MaxSinglePacketRPCTime &&
				SinglePacketRPCCount == A.SinglePacketRPCCount && SinglePacketGameThreadCPU == A.SinglePacketGameThreadCPU &&
				BlockedCount == A.BlockedCount && PlayerIP == A.PlayerIP && PlayerUID == A.PlayerUID;
	}


	/**
	 * Whether or not the current RPC's analytics meet any of the minimum hardcoded thresholds, for inclusion in analytics.
	 *
	 * @return		Whether or not minimum analytics inclusion thresholds are met
	 */
	bool WithinMinAnalyticsThreshold() const
	{
		return	MaxCountPerSec > 10 ||
				MaxTimePerSec > 0.0001 ||
				MaxSinglePacketRPCTime > 0.0001 ||
				SinglePacketRPCCount > 10 ||
				BlockedCount > 0;
	}
};

/**
 * Container class for separating analytics variables and processing, from the main RPC DoS Detection code
 */
struct FRPCDoSAnalyticsVars
{
public:

	/** Default constructor */
	NETCORE_API FRPCDoSAnalyticsVars();

	NETCORE_API bool operator == (const FRPCDoSAnalyticsVars& A) const;

	void CommitAnalytics(FRPCDoSAnalyticsVars& AggregatedData);


public:
	/** The IP of the player */
	FString PlayerIP;

	/** The UID of the player */
	FString PlayerUID;

	/** The index of the maximum severity state that was reached */
	int32 MaxSeverityIndex = 0;

	/** The name of the maximum severity state that was reached */
	FString MaxSeverityCategory;

	/** The index of the maximum severity state confirmed for analytics that was reached */
	int32 MaxAnalyticsSeverityIndex = 0;

	/** The name of the maximum severity state confirmed for analytics that was reached */
	FString MaxAnalyticsSeverityCategory;


	/** Analytics for RPC's that are being tracked */
	TArray<TSharedPtr<FRPCAnalytics>> RPCTrackingAnalytics;

	/** The maximum number of RPC tracking analytics entries */
	const int32 MaxRPCAnalytics;


	/** Aggregation variables */

	/** (Not filled until final commit) The maximum RPC DoS severity state reached, for each player */
	TArray<FMaxRPCDoSEscalation> MaxPlayerSeverity;
};


/**
 * Class which is responsible for sending the actual analytics events (for implementation outside of NetCore, to avoid Analytics dependency)
 */
class FRPCDoSAnalyticsSender
{
public:
	virtual void SendAnalytics(FNetAnalyticsAggregator* Aggregator, FRPCDoSAnalyticsData* Data) {}

	virtual void FireEvent_ServerRPCDoSEscalation(FNetAnalyticsAggregator* Aggregator, FRPCDoSAnalyticsData* Data, int32 SeverityIndex,
													const FString& SeverityCategory, int32 WorstCountPerSec, double WorstTimePerSec,
													const FString& InPlayerIP, const FString& InPlayerUID, const TArray<FName>& InRPCGroup,
													double InRPCGroupTime=0.0) {}
};

/**
 * RPC DoS Detection implementation for basic aggregated net analytics data
 */
struct FRPCDoSAnalyticsData : public TBasicNetAnalyticsData<FRPCDoSAnalyticsVars>
{
	/** Implementation responsible for the analytics send event */
	TSharedPtr<FRPCDoSAnalyticsSender> AnalyticsSender;

	/** The worst RPC DoS severity state that has been active and passed confirmation thresholds - mirrors RPCDosDetection.WorstAnalyticsState */
	int8 WorstAnalyticsState = 0;


public:
	using FRPCDoSAnalyticsVars::operator ==;

	void SetAnalyticsSender(TSharedPtr<FRPCDoSAnalyticsSender> InSender);

	NETCORE_API FRPCDoSAnalyticsVars* GetVars();

	virtual void SendAnalytics() override;

	/**
	 * Triggers RPC DoS Escalation analytics. This code is separate to the RPC DoS Analytics aggregation,
	 * but is implemented here to keep the RPC DoS analytics in one place.
	 *
	 * @param SeverityIndex		The numeric value of the RPC DoS severity category, that the NetDriver escalated to
	 * @param SeverityCategory	The name of the RPC DoS severity category, that the NetDriver escalated to
	 * @param WorstCountPerSec	The worst per second RPC execution count that may have triggered this escalation
	 * @param WorstTimePerSec	The worst per second RPC execution time that may have triggered this escalation (for ranking)
	 * @param InPlayerIP		The IP of the player
	 * @param InPlayerUID		The UID of the player
	 * @apram InRPCGroup		Lists lightweight tracking RPC's contributing to escalation, which couldn't be individually timed
	 * @param InRPCGroupTime	The amount of time the whole RPC group spent executing
	 */
	void FireEvent_ServerRPCDoSEscalation(int32 SeverityIndex, const FString& SeverityCategory, int32 WorstCountPerSec,
											double WorstTimePerSec, const FString& InPlayerIP, const FString& InPlayerUID,
											const TArray<FName>& InRPCGroup, double InRPCGroupTime=0.0);
};

