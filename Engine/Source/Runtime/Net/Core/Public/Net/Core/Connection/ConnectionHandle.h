// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/TypeHash.h"

namespace UE::Net
{

/**
 * The handle is unique per netdriver. The child connection ID is not unique with respect to other handles with a different parent connection ID.
 * Child connection are not assigned proper connection IDs as we will never replicate to them. Only parent connections will be replicated to.
 * When there is need to deal with child connections in replication scenarios the ConnectionHandle can aid with that.
 */
class FConnectionHandle
{
public:
	/** Initializes an invalid handle. */
	inline FConnectionHandle() = default;
	explicit FConnectionHandle(uint32 InParentConnectionId);
	FConnectionHandle(uint32 InParentConnectionId, uint32 InChildConnectionId);

	bool operator==(const FConnectionHandle& Other) const;

	/** Returns true if this is a valid handle. */
	bool IsValid() const;
	/** Returns true if the handle is valid and represents a parent connection. */
	bool IsParentConnection() const;
	/** Returns true if the handle is valid and represents a child connection. */
	bool IsChildConnection() const;

	/** Returns the parent connection id for all types of valid handles. */
	uint32 GetParentConnectionId() const;
	/** Returns a non-zero ID for a valid child connection handle, zero for parent connections and invalid handles. */
	uint32 GetChildConnectionId() const;

private:
	friend uint32 GetTypeHash(const FConnectionHandle& Handle);

	uint32 ParentConnectionId = 0;
	// For a valid handle a ChildConnectionId of zero indicates it's the parent connection itself.
	uint32 ChildConnectionId = 0;
};

inline FConnectionHandle::FConnectionHandle(uint32 InParentConnectionId)
: ParentConnectionId(InParentConnectionId)
, ChildConnectionId(0)
{
}

inline FConnectionHandle::FConnectionHandle(uint32 InParentConnectionId, uint32 InChildConnectionId)
: ParentConnectionId(InParentConnectionId)
, ChildConnectionId(InChildConnectionId)
{
}

inline bool FConnectionHandle::operator==(const FConnectionHandle& Other) const
{
	return ParentConnectionId == Other.ParentConnectionId && ChildConnectionId == Other.ChildConnectionId;
}

inline bool FConnectionHandle::IsValid() const
{
	return ParentConnectionId > 0U;
}

inline bool FConnectionHandle::IsParentConnection() const
{
	return IsValid() && ChildConnectionId == 0U;
}

inline bool FConnectionHandle::IsChildConnection() const
{
	return IsValid() && ChildConnectionId != 0U;
}

inline uint32 FConnectionHandle::GetParentConnectionId() const
{
	return ParentConnectionId;
}

inline uint32 FConnectionHandle::GetChildConnectionId() const
{
	return IsValid() ? ChildConnectionId : 0U;
}

inline uint32 GetTypeHash(const FConnectionHandle& Handle)
{
	return ::GetTypeHash((uint64(Handle.ParentConnectionId) << 32U) | uint64(Handle.ChildConnectionId));
}

}
