// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Net/Core/Connection/StateStruct.h"
#include "Templates/EnableIf.h"
#include "Templates/Function.h"
#include "Templates/IsEnum.h"
#include "Templates/Models.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StructOnScope.h"
#include "UObject/UObjectGlobals.h"

#include "EscalationStates.generated.h"

class UObject;


/**
 * Escalation States
 *
 * Implements generic configurable 'escalation states' evolved from the RPC DoS and DDoS code,
 * for setting threshold limits on monitored variables/conditions, and escalating through complex states of gradually higher thresholds.
 *
 * Primarily used for monitoring and reacting to deteriorating network conditions, attempting mitigation and recovery from poor conditions.
 *
 *
 * Escalation State Definition
 *
 * See 'State Struct Definition' in StateStruct.h. Defined escalation states are automatically implemented, as a parameter to TEscalationManager.
 *
 * Additionally, you must implement the 'TEscalationStateStatics' interface in your state struct, for specifying hardcoded configuration values.
 *
 *
 * Escalation Counters
 *
 * Escalation Counters are used to monitor an incrementing count and/or accumulating time value, which are incremented/accumulated every frame,
 * and then automatically tracked internally in the Escalation Manager, to help efficiently calculate whether any thresholds/quota's have been hit.
 *
 * The number of counters is hardcoded at compile time, defined using an enum in the following format (which MUST contain 'Max' after last counter):
 *		enum class ENetFaultCounters : uint8
 *		{
 *			PacketCount,
 *			NetConnPacketFault,
 *			Max,
 *
 *			// NumPrealloc is optional, and will specify how many counters to inline-preallocate (beyond the already defined counters)
 *			NumPrealloc = 6
 *		};
 *
 * 'GetFrameCounter' in FEscalationManager, is used to reference the current frame counter, for incrementing.
 *
 * NOTE: If your escalation state calculates quotas over a period of seconds (e.g. 5/10 seconds), you must override 'GetHighestTimePeriod',
 *			and 'GetAllTimePeriods' - and specify all the time periods you need to monitor, so they are calculated/tracked behind the scenes.
 *
 *
 * Escalation Manager
 *
 * The Escalation Manager implements all of the state handling/tracking and internal counter accumulation for efficient quota/threshold checks.
 * Use TEscalationManager to implement a manager for your state/counters, like so:
 *		TEscalationManager<ENetFaultCounters, FNetFaultState> NetFaultEscalationManager;
 *
 * Call 'Init' on the escalation manager to initialize it (specifying a ConfigContext if needed - see 'Configuration and Defaults' in StateStruct.h),
 * and make sure a call to Tick is implemented (note that it takes the current time in seconds, not DeltaTime).
 *
 *
 * Quota checks
 *
 * Your escalation state must implement 'HasHitAnyQuota', using the input history/counters to determine if your custom state thresholds have been hit.
 * The various input counter values represent (partially) accumulated values for your counters, for testing thresholds over different time periods.
 *
 * Second counters do not have Frame counters pre-accumulated, and Period counters do not have Second counters pre-accumulated - this is done manually.
 *
 * See 'FNetFaultState' for a more detailed example of how quota checks are implemented.
 *
 *
 * Configuration
 *
 * See 'Configuration and Defaults' in StateStruct.h.
 *
 * Configuration works in a similar manner, except 'UEscalationManagerConfig' is subclassed instead,
 * the 'TEscalationStateStatics' interface specifies ConfigSection and the config class to use, and escalation manager Init specifies ConfigContext.
 *
 * UEscalationManagerConfig already loads the configuration state using EscalationSeverity/EscalationSeverityStates,
 * (override 'InitConfigDefaultsInternal' to set the default configuration state).
 */


// Defines

/** Time Quota debugging */
#define ESCALATION_QUOTA_DEBUG 0


// Forward declarations
class UEscalationManagerConfig;

namespace UE
{
	namespace Net
	{
		class FEscalationManager;
		struct FEscalationCounter;
	}
}


/**
 * The type of state change quota/thresholds to check
 */
enum class EQuotaType : uint8
{
	EscalateQuota,		// Check quota for escalating to a higher state
	DeescalateQuota		// Check quota for de-escalating to a lower state
};

/**
 * Parameters for FEscalationState.HasHitAnyQuota
 */
struct FHasHitAnyQuotaParms
{
	using FEscalationCounter = UE::Net::FEscalationCounter;

	/** The list of custom registered counters, by category, for any custom processing implementation */
	const TArrayView<TArrayView<int32>>& RegisteredCounters;

	/** The per-period historical counters being compared against (excludes SecondCounters and FrameCounters) */
	const TArrayView<FEscalationCounter>(&PerPeriodHistory)[16];

	/** The current seconds counters for factoring-in to comparisons (excludes FrameCounters) */
	const TArrayView<FEscalationCounter>& SecondCounters;

	/** The current frames counters for factoring-in to comparisons */
	const TArrayView<FEscalationCounter>& FrameCounters;

	/** The type of state change quota/thresholds to check */
	EQuotaType QuotaType;
};


/**
 * Base struct which defines an escalation state, which is subclassed to implement custom state variables and (de-)escalation quota's.
 * NOTE: Subclasses must also implement the TEscalationStateStatics interface.
 */
USTRUCT()
struct FEscalationState : public FStateStruct
{
	friend UE::Net::FEscalationManager;
	friend UEscalationManagerConfig;

	GENERATED_BODY()


protected:
	enum class EValidateTime : uint8
	{
		Optional,
		MustBeSet
	};


public:
	/** Whether or not to log when escalating to this state */
	UPROPERTY(config)
	bool bLogEscalate				= false;

	/** This escalation state is considered to be dormant/inactive - and the escalation manager may no longer need ticking in this state */
	UPROPERTY(config)
	bool bDormant					= false;

	/** The amount of time, in seconds, before the current severity state cools off and de-escalates */
	UPROPERTY(config)
	int16 CooloffTime				= -1;

	/** The amount of time, in seconds, spent in the current severity state before it automatically escalates to the next state */
	UPROPERTY(config)
	int16 AutoEscalateTime			= -1;


protected:
	/** Cached runtime config values (UPROPERTY's are copied to FEscalationManager.State during ApplyState, everything else is not) */

	/** Cached value for the highest time period in this config state */
	UPROPERTY()
	int8 HighestTimePeriod = 0;

	/** Cached value for all different time periods in this state */
	UPROPERTY()
	TArray<int8> AllTimePeriods;


public:
	/**
	 * Whether or not this escalation state is a 'dormant' state, which allows the escalation manager to disable ticking
	 */
	NETCORE_API bool IsDormant() const;

	/**
	 * Gets the highest counter time period specified by escalation state settings (used for limiting 'CountersPerPeriodHistory' size)
	 *
	 * @return		The highest counter time period in the state config settings
	 */
	NETCORE_API int8 GetHighestTimePeriod() const;

	/**
	 * Gets all counter time periods specified by the escalation state settings
	 *
	 * @return		All of the time periods in the state config settings
	 */
	NETCORE_API const TArray<int8>& GetAllTimePeriods() const;


protected:
	NETCORE_API virtual void ValidateConfigInternal() override;

	/**
	 * Validate that the specified escalation time period is within thresholds, and clamp it if necessary
	 *
	 * @param Value			The time period value to validate
	 * @param PropertyName	The name of the property, for logging
	 * @param Requirement	Whether the time period value must be set, or is optional
	 */
	NETCORE_API void ValidateTimePeriod(int8& Value, const TCHAR* PropertyName, EValidateTime Requirement=EValidateTime::Optional);


private:
	/**
	 * Whether or not the specified per-period counters have hit escalation quota's specified by this state.
	 *
	 * NOTE: This quota check should not be used in high performance code - it should be decomposed into non-virtual calls in a subclass,
	 *			which are then used directly in code which needs them - to avoid virtual calls.
	 *
	 * @param Parms		Parameters needed for performing quota checks
	 * @return			Whether or not any escalation quota has been hit
	 */
	virtual bool HasHitAnyQuota(FHasHitAnyQuotaParms Parms) const
	{
		return false;
	}
};

/**
 * Static interface which FEscalateState subclasses must implement (called during construction, so virtuals won't do)
 */
template<class T>
struct TEscalationStateStatics
{
	/**
	 * Gets the base configuration section name escalation states will use (not the full/complete section name)
	 */
	static inline const TCHAR* GetConfigSection()
	{
		return T::GetConfigSection();
	}

	/**
	 * Gets the UEscalationManagerConfig subclass escalation states are associated with
	 */
	static inline UClass* GetBaseConfigClass()
	{
		return T::GetBaseConfigClass();
	}
};


namespace UE
{
namespace Net
{
// Forward declarations
enum class ESeverityUpdate : uint8;


// Typedefs

/**
 * Callback notifying when the escalation state has been updated
 *
 * @param OldState		The old state being escalated from (same as new state - but different object - on first initialization)
 * @param NewState		The new/active escalation state
 * @param UpdateType	Whether the update was an escalation (including auto-escalation) or de-escalation
 */
using FNotifySeverityUpdate = TUniqueFunction<void(const FEscalationState& OldState, const FEscalationState& NewState, ESeverityUpdate UpdateType)>;


/**
 * Struct containing escalation counters covering any time period (e.g. frame/second/arbitrary-period)
 */
struct FEscalationCounter
{
	/** Counter for the number of times an event occurred that could lead to an escalation */
	int32 Counter			= 0;

	/** Accumulated time spent executing code that could lead to an escalation */
	double AccumTime		= 0.0;

#if ESCALATION_QUOTA_DEBUG
	/** Debug version of the value above, using more accurate timing, for testing accuracy of approximate time */
	double DebugAccumTime	= 0.0;
#endif


public:
	/**
	 * Reset the values of this escalation counter
	 */
	void ResetCounters()
	{
		Counter = 0;
		AccumTime = 0.0;

#if ESCALATION_QUOTA_DEBUG
		DebugAccumTime = 0.0;
#endif
	}

	/**
	 * Accumulate another escalation counter into this one
	 *
	 * @param InCounter		The escalation counter to accumulate with
	 */
	void AccumulateCounter(const FEscalationCounter& InCounter)
	{
		Counter += InCounter.Counter;
		AccumTime += InCounter.AccumTime;

#if ESCALATION_QUOTA_DEBUG
		DebugAccumTime += InCounter.DebugAccumTime;
#endif
	}
};

/**
 * Escalation severity update types
 */
enum class ESeverityUpdate : uint8
{
	Escalate,		// Escalating to a higher escalation state
	AutoEscalate,	// Automatic/non-quota escalation to a higher escalation state, e.g. when too much time was spent in a lower state
	Deescalate		// Deescalating to a lower escalation state
};

/**
 * The reason for a severity update
 */
enum class EEscalateReason : uint8
{
	QuotaLimit,		// Escalation quota's were hit (followed by additional ReasonContext)
	AutoEscalate,	// Automatic escalation from a lower state, e.g. after spending too much time in a lower state
	Deescalate		// Deescalation to a lower state
};

/**
 * The result of an UpdateSeverity call
 */
enum class EEscalateResult : uint8
{
	Escalated,		// Escalated to a higher/more-severe state
	Deescalated,	// De-escalated to a lower/less-severe state
	NoChange		// No state change occurred
};


/**
 * Manages initialization/application of escalation states, and tracking various counters/timers used to calculate state change quotas/thresholds.
 */
class FEscalationManager
{
	template<typename, typename, typename> friend class TEscalationManager;
	friend UEscalationManagerConfig;

private:
	/**
	 * Simple type wrapper for FStructOnScope, without any allocation and without owning the memory
	 */
	template<typename T, typename = decltype(TBaseStructure<T>::Get())>
	class TStructOnScopeLite final : public FStructOnScope
	{
	public:
		TStructOnScopeLite(const UStruct* InScriptStruct, uint8* Data)
			: FStructOnScope(InScriptStruct, Data)
		{
			check(InScriptStruct->IsChildOf(TBaseStructure<T>::Get()));
		}

		T* Get() const
		{
			return reinterpret_cast<T*>(SampleStructMemory);
		}

		T* operator->() const
		{
			return Get();
		}

		explicit operator bool() const
		{
			return IsValid();
		}

		template<typename U>
		U* Cast()
		{
			if (GetStruct()->IsChildOf(TBaseStructure<U>::Get()))
			{
				return reinterpret_cast<U*>(SampleStructMemory);
			}
			return nullptr;
		}

		template<typename U>
		const U* Cast() const
		{
			return const_cast<TStructOnScopeLite*>(this)->Cast<U>();
		}
	};


	struct FEscalationManagerParms
	{
		int32 NumCounters																		= 0;
		const TArrayView<TArrayView<int32>> RegisteredCountersCache								= {};
		const UStruct* StateStruct																= nullptr;
		uint8* StateMemory																		= nullptr;
	};

	struct FEscalationManagerInitParms
	{
		const TArrayView<FEscalationCounter> FrameCounters										= {};
		const TArrayView<FEscalationCounter> SecondCounters										= {};
		const TArrayView<FEscalationCounter> CountersPerPeriodHistoryAlloc						= {};
	};


private:
	NETCORE_API FEscalationManager(FEscalationManagerParms Parms);

	NETCORE_API void InitParms(FEscalationManagerInitParms Parms);

	FEscalationManager() = delete;

	/**
	 * Loads configuration settings using the specified parameters, and initializes severity states
	 *
	 * @param ConfigParms	Specifies the parameters for the config section/object
	 */
	NETCORE_API void InitConfig(FStateConfigParms ConfigParms);

	/**
	 * Updates the current escalation manager severity state
	 *
	 * @param Update			Whether or not we are escalating or de-escalating the severity state
	 * @param Reason			The reason for the escalation change
	 * @param ReasonContext		Additional customizable context for the escalation change
	 * @return					The result of the severity update. Escalation/Deescalation, or NoChange
	 */
	NETCORE_API EEscalateResult UpdateSeverity(ESeverityUpdate Update, EEscalateReason Reason, FString ReasonContext=TEXT(""));

	/**
	 * Recalculates cached counter history information, after an interval or state change which affects time period tracking.
	 *
	 * @param InTimePeriods				The time period values which need caching (e.g. a period of 2 seconds, and a period of 8 seconds).
	 * @param OutPerPeriodHistory		Outputs the recalculated counter cache, for the specified time periods (e.g. elements 1 and 7 only).
	 * @param StartPerSecHistoryIdx		The index in CounterPerSecHistory to start calculating from
	 */
	NETCORE_API void RecalculatePeriodHistory(const TArray<int8>& InTimePeriods, TArrayView<FEscalationCounter>(&OutPerPeriodHistory)[16],
									int32 StartPerSecHistoryIdx=INDEX_NONE);

	NETCORE_API int32 GetHighestHistoryRequirement() const;

	/**
	 * Resets all counters - typically when entering dormancy
	 */
	NETCORE_API void ResetAllCounters();

	NETCORE_API int32 AddNewCounter_Internal(int32 Count, const TArrayView<FEscalationCounter>& CountersPerPeriodAlloc);

public:
	/**
	 * Accessor for the current frames counter (this is the primary place where counters are incremented).
	 *
	 * @param CounterTndex	The index of the counter
	 * @return				The current frames counter, of the specified index
	 */
	FEscalationCounter& GetFrameCounter(int32 CounterIndex)
	{
		check(CounterIndex < NumCounters);

		return FrameCounters[CounterIndex];
	}

	/**
	 * Checks if any escalation quota's have been hit, and if so, updates/escalates the severity state
	 */
	NETCORE_API void CheckQuotas();

	/**
	 * Ticks the escalation manager
	 * NOTE: This takes the current time, in seconds, rather than DeltaTime.
	 *
	 * @param TimeSeconds	The (can be approximate) current time
	 */
	NETCORE_API void TickRealtime(double TimeSeconds);

	/**
	 * Whether or not calls to Tick are presently required
	 *
	 * @return	Whether Tick is required
	 */
	bool DoesRequireTick() const
	{
		return !IsDormant();
	}


	// Accessors

	/**
	 * Whether or not the current state is a dormant state, which does not require ticking
	 */
	NETCORE_API bool IsDormant() const;

	/**
	 * Returns a reference to the BaseConfig object
	 */
	NETCORE_API const UEscalationManagerConfig* GetBaseConfig() const;

	/**
	 * Sets a new ManagerContext value
	 */
	NETCORE_API void SetManagerContext(FString InManagerContext);

	/**
	 * Sets a new NotifySeverityUpdate value
	 */
	NETCORE_API void SetNotifySeverityUpdate(FNotifySeverityUpdate&& InNotifySeverityUpdate);


private:
	/** The state configuration object for this manager */
	const UEscalationManagerConfig* BaseConfig													= nullptr;

	/** Extra context to use when logging severity state updates (e.g. Player IP) */
	FString ManagerContext;

	/** The callback to use to notify of state severity escalation/de-escalation */
	FNotifySeverityUpdate NotifySeverityUpdate;

	/** The number of different counter types this escalation manager keeps track of */
	int32 NumCounters																			= 0;

	/** Whether or not this escalation manager is enabled */
	bool bEnabled																				= false;


	/** The managers escalation state */
	TStructOnScopeLite<FEscalationState> State;

	/** The currently active escalation severity state settings */
	int8 ActiveState																			= 0;

	/** The last time the previous severity states escalation conditions were met (to prevent bouncing up/down between states) */
	double LastMetEscalationConditions															= 0.0;


	/** Per-frame counters */
	TArrayView<FEscalationCounter> FrameCounters;

	/** Per-second counters (approximate - may cover more than one second, in the case of expensive/long-running counted events) */
	TArrayView<FEscalationCounter> SecondCounters;

	/** Timestamp for the last time per-second quota counting began */
	double LastPerSecQuotaBegin																	= 0.0;

	/** Stores enough per second quota history, to allow all EscalationSeverity states to recalculate if their CooloffTime is reached */
	TArray<TArrayView<FEscalationCounter>> CountersPerSecHistory;

	/** Allocation for the above array, to reduce indirection */
	TArray<FEscalationCounter> CountersPerSecHistoryAlloc;

	/** The last written index of CountersPerSecHistory */
	int32 LastCountersPerSecHistoryIdx															= 0;

	/** Counter history which is precalculated/cached from CountersPerSecHistory, for tracking per-period time (up to a maximum of 16s) */
	TArrayView<FEscalationCounter> CountersPerPeriodHistory[16]									= {};


	/** Lists of registered counters, per category */
	const TArrayView<TArrayView<int32>> RegisteredCounters;
};

/** Default value for TEscalationManager's CounterCategoriesEnum */
enum EEmptyCategories : uint32
{
	Max
};


/**
 * Manages initialization/application of escalation states for the specified escalation state type,
 * and allocation/tracking of the specified counters/timers used to calculate state change quotas/thresholds.
 *
 * The different counter types are specified in sequential order in the 'CountersEnum' enum, followed by a final/unused Max value.
 */
template<typename CountersEnum, typename EscalationStateType, typename CounterCategoriesEnum=EEmptyCategories>
class TEscalationManager final : public FEscalationManager
{
	static_assert(TIsEnum<CountersEnum>::Value, "CountersEnum must be an enum");
	static_assert(TIsEnum<CounterCategoriesEnum>::Value, "CounterCategoriesEnum must be an enum");
	static_assert(TIsDerivedFrom<EscalationStateType, FEscalationState>::Value, "EscalationStateType must be a subclass of FEscalationState");


	struct CHasEnumPreallocNum
	{
		template<typename T>
		auto Requires() -> decltype(
			T::NumPrealloc
		);
	};

	// 'typename = typename TEnableIf' does not work here, unusually
	template<typename T, typename TEnableIf<TModels_V<CHasEnumPreallocNum, T>>::Type* = nullptr>
	inline static constexpr int32 GetCounterNumPrealloc()
	{
		return FMath::Max((int32)T::NumPrealloc, (int32)T::Max);
	}

	template<typename T, typename TEnableIf<!TModels_V<CHasEnumPreallocNum, T>>::Type* = nullptr>
	inline static constexpr int32 GetCounterNumPrealloc()
	{
		return (int32)T::Max;
	}

	/** The number of static counters in the specified enum (Max must come after the last static counter) */
	static constexpr int32 TNum = (int32)CountersEnum::Max;

	/** The number of counters to preallocate inline, for newly added counters */
	static constexpr int32 TNumPrealloc = GetCounterNumPrealloc<CountersEnum>();

	/** The number of categories in the counter categories enum */
	static constexpr int32 TNumCategories = (int32)CounterCategoriesEnum::Max;


public:
	/**
	 * Base constructor
	 */
	TEscalationManager()
		: FEscalationManager(
			FEscalationManagerParms
			{
				TNum, MakeArrayView(RegisteredCountersCacheAlloc, TNumCategories),
				EscalationStateType::StaticStruct(), (uint8*)&StateAlloc //-V1050
			})
	{
		static_assert(UE_ARRAY_COUNT(CountersPerPeriodHistory) == 16,
						"CountersPerPeriodAlloc initialization must match CountersPerPeriodHistory length");

		FrameCountersAlloc.SetNum(TNumPrealloc);
		SecondCountersAlloc.SetNum(TNumPrealloc);
		CountersPerPeriodAlloc.SetNum(TNumPrealloc * UE_ARRAY_COUNT(CountersPerPeriodHistory));

		InitParms(
			FEscalationManagerInitParms
			{
				MakeArrayView(FrameCountersAlloc.GetData(), TNum), MakeArrayView(SecondCountersAlloc.GetData(), TNum),
				MakeArrayView(CountersPerPeriodAlloc.GetData(), TNum * UE_ARRAY_COUNT(CountersPerPeriodHistory))
			});
	}

	/**
	 * Initializes the escalation manager
	 *
	 * @param ConfigContext		Additional context to use for state configuration (e.g. NetDriver name, to separate config for each NetDriver)
	 */
	void Init(FString ConfigContext)
	{
		FEscalationManager::InitConfig({TEscalationStateStatics<EscalationStateType>::GetConfigSection(), ConfigContext,
											TEscalationStateStatics<EscalationStateType>::GetBaseConfigClass(),
											EscalationStateType::StaticStruct()
										});
	}

	/**
	 * Dynamically adds a new counter not specified in CountersEnum, and returns its index.
	 * If NumPrealloc (or Max if not specified) is exceeded, counter tracking will switch from inline allocation to the heap.
	 *
	 * @param Count		The number of counters to add
	 * @return			The index of the first added counter
	 */
	int32 AddNewCounter(int32 Count=1)
	{
		FrameCountersAlloc.AddDefaulted(Count);
		SecondCountersAlloc.AddDefaulted(Count);
		CountersPerPeriodAlloc.AddDefaulted(Count * UE_ARRAY_COUNT(CountersPerPeriodHistory));

		FrameCounters = MakeArrayView(FrameCountersAlloc);
		SecondCounters = MakeArrayView(SecondCountersAlloc);

		return AddNewCounter_Internal(Count, MakeArrayView(CountersPerPeriodAlloc));
	}

	/**
	 * Registers a counter for automatic processing under the specified counter category, as set by CounterCategoriesEnum
	 * (each category can implement a different type of custom processing, in HasHitAnyQuota)
	 *
	 * @param CategoryIndex		The processing category the counter should be registered under
	 * @param CounterIndex		The index of the counter to register
	 */
	void RegisterCounterCategory(int32 CategoryIndex, int32 CounterIndex)
	{
		check(CategoryIndex < UE_ARRAY_COUNT(RegisteredCountersAlloc));
		check(CounterIndex >= 0);

		TArray<int32, TInlineAllocator<TNumPrealloc>>& CurRegisteredCounters = RegisteredCountersAlloc[CategoryIndex];
	
		CurRegisteredCounters.AddUnique(CounterIndex);
		RegisteredCounters[CategoryIndex] = MakeArrayView(CurRegisteredCounters);
	}

private:
	/** Inline allocation of state and other variables, for cache locality */

	EscalationStateType StateAlloc;

	TArray<FEscalationCounter, TInlineAllocator<TNumPrealloc>> FrameCountersAlloc;
	TArray<FEscalationCounter, TInlineAllocator<TNumPrealloc>> SecondCountersAlloc;
	TArray<FEscalationCounter, TInlineAllocator<TNumPrealloc * UE_ARRAY_COUNT(CountersPerPeriodHistory)>> CountersPerPeriodAlloc;
	TArray<int32, TInlineAllocator<TNumPrealloc>> RegisteredCountersAlloc[FMath::Max(TNumCategories, 1)];
	TArrayView<int32> RegisteredCountersCacheAlloc[FMath::Max(TNumCategories, 1)];
};

}
}


/**
 * Base class for defining escalation state configuration.
 *
 * Subclass and override 'InitConfigDefaultsInternal' to initialize EscalationSeverity and bEnabled (and other custom config variables).
 */
UCLASS(config=Engine, PerObjectConfig, MinimalAPI)
class UEscalationManagerConfig : public UStatePerObjectConfig
{
	GENERATED_BODY()

private:
	NETCORE_API virtual void LoadStateConfig() override;


public:
	/** Names of the different states for escalating severity, depending on conditions for each state */
	UPROPERTY(config)
	TArray<FString> EscalationSeverity;

	/** The different states of escalating severity, depending on the escalation thresholds that have been met */
	TArray<TStructOnScope<FEscalationState>> EscalationSeverityState;
};


