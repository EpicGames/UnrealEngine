// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Net/Core/Connection/NetEnums.h"
#include "Net/Core/Connection/NetResult.h"
#include "Templates/PimplPtr.h"
#include "UObject/ObjectMacros.h"

#include "NetCloseResult.generated.h"


/**
 * Network Close results (can occur before NetDriver/NetConnection creation, and can be success or error)
 *
 * Licensees should use ENetCloseResult::Extended, and specify an ErrorContext value to FNetCloseResult, to specify custom results
 *
 * NOTE: If you modify this, run the enum-checking smoke test to verify you didn't forget to update anything else:
 *			System.Core.Networking.FNetCloseResult.EnumTest
 */
UENUM()
enum class ENetCloseResult : uint32
{
	/** ENetworkFailure values */

	/** A relevant net driver has already been created for this service */
	NetDriverAlreadyExists,

	/** The net driver creation failed */
	NetDriverCreateFailure,

	/** The net driver failed its Listen() call */
	NetDriverListenFailure,

	/** A connection to the net driver has been lost */
	ConnectionLost,

	/** A connection to the net driver has timed out */
	ConnectionTimeout,

	/** The net driver received an NMT_Failure message */
	FailureReceived,

	/** The client needs to upgrade their game */
	OutdatedClient,

	/** The server needs to upgrade their game */
	OutdatedServer,

	/** There was an error during connection to the game */
	PendingConnectionFailure,

	/** NetGuid mismatch */
	NetGuidMismatch,

	/** Network checksum mismatch */
	NetChecksumMismatch,


	/** ESecurityEvent values */

	/** The packet didn't follow protocol */
	SecurityMalformedPacket,

	/** The packet contained invalid data */
	SecurityInvalidData,

	/** The connection had issues (possibly malicious) and was closed */
	SecurityClosed,


	/** Base result types */

	/** Default state (do not move the position of this in enum) */
	Unknown,

	/** Close was successful, happening under normal conditions */
	Success,

	/** Extended/custom ErrorContext state. To be interpreted by code */
	Extended,


	/** NetDriver Error result types */

	/** Host closed the connection */
	HostClosedConnection,

	/** Disconnected remotely */
	Disconnect,

	/** Client needs to upgrade */
	Upgrade,

	/** PreLogin failed */
	PreLoginFailure,

	/** Join failed */
	JoinFailure,

	/** JoinSplit failed */
	JoinSplitFailure,

	/** Address Rssolution Failed */
	AddressResolutionFailed,


	/** NetConnection Error result types */

	/** RPC DoS Detection kicked the player */
	RPCDoS,

	/** NetConnection Cleanup was triggered */
	Cleanup,

	/** UpdateLevelVisibility missing level package */
	MissingLevelPackage,

	/** PacketHandler Error processing incoming packet */
	PacketHandlerIncomingError,

	/** Packet had zeros in the last byte */
	ZeroLastByte,

	/** Zero size packet */
	ZeroSize,

	/** Failed to read PacketHeader */
	ReadHeaderFail,

	/** Failed to read extra PacketHeader information */
	ReadHeaderExtraFail,

	/** Sequence mismatch while processing acks */
	AckSequenceMismatch,

	/** Bunch channel index exceeds maximum channel limit */
	BunchBadChannelIndex,

	/** Bunch channel name serialization failed */
	BunchChannelNameFail,

	/** Bunch specified wrong channel type for an existing channel */
	BunchWrongChannelType,

	/** Bunch header serialization overflowed */
	BunchHeaderOverflow,

	/** Bunch data serialization overflowed */
	BunchDataOverflow,

	/** Server received bHasPackageMapExports packet */
	BunchServerPackageMapExports,

	/** Received control channel bunch before control channel was created */
	BunchPrematureControlChannel,

	/** Received non-control-channel bunch when not yet ready */
	BunchPrematureChannel,

	/** Received control channel close before open */
	BunchPrematureControlClose,

	/** Unknown channel type specified */
	UnknownChannelType,

	/** Attempted to send data before handshake is complete */
	PrematureSend,

	/** Server received corrupt data from the client - or there was otherwise an error during channel processing */
	CorruptData,

	/** Socket send failure */
	SocketSendFailure,

	/** Failed to find splitscreen player matching ChildConnection NetPlayerIndex */
	BadChildConnectionIndex,

	/** Log limiting per-second checks insta-kicked the player */
	LogLimitInstant,

	/** Log limiting repeated threshold hits kicked the player */
	LogLimitSustained,

	/** Encryption failure */
	EncryptionFailure,

	/** EncryptionToken is missing */
	EncryptionTokenMissing,


	/** Channel Error result types */

	/** ReceiveNetGUIDBunch serialization failed */
	ReceivedNetGUIDBunchFail,

	/** ReceiveNetExtBunch serialization failed */
	ReceivedNetExtBunchFail,

	/** Too many reliable bunches queued */
	MaxReliableExceeded,

	/** ReceivedNextBunch serialization failed */
	ReceivedNextBunchFail,

	/** Queued ReceivedNextBunch serialization failed */
	ReceivedNextBunchQueueFail,

	/** Reliable partial initial bunch attempting to destroy incomplete reliable partial bunch */
	PartialInitialReliableDestroy,

	/** Reliable partial merge bunch attempting to destroy incomplete reliable partial bunch */
	PartialMergeReliableDestroy,

	/** Partial initial bunch which is not byte aligned */
	PartialInitialNonByteAligned,

	/** Non-final partial bunch which is not byte aligned */
	PartialNonByteAligned,

	/** Final partial bunch has package map exports */
	PartialFinalPackageMapExports,

	/** Partial bunch exceeded maximum partial bunch size */
	PartialTooLarge,

	/** Received open bunch when channel already open */
	AlreadyOpen,

	/** Received reliable bunch before channel was fully open */
	ReliableBeforeOpen,

	/** Reliable buffer overflowed when attempting to send */
	ReliableBufferOverflow,

	/** Reliable buffer overflowed when attempting to send RPC */
	RPCReliableBufferOverflow,


	/** Control Channel result types */

	/** Control channel closing */
	ControlChannelClose,

	/** Endianness check failed */
	ControlChannelEndianCheck,

	/** The ActorChannel for the connections PlayerController failed */
	ControlChannelPlayerChannelFail,

	/** Received unknown control channel message */
	ControlChannelMessageUnknown,

	/** Failed to read control channel message */
	ControlChannelMessageFail,

	/** Failed to read control channel message payload */
	ControlChannelMessagePayloadFail,

	/** Control channel send bunch overflowed */
	ControlChannelBunchOverflowed,

	/** Queued control channel send bunch overflowed */
	ControlChannelQueueBunchOverflowed,


	/** Actor Channel Error result types */

	/** Client tried to set bHasMustBeMappedGUIDs */
	ClientHasMustBeMappedGUIDs,

	/** Client tried to send a destruction info */
	ClientSentDestructionInfo,

	/** Received MustBeMappedGUID that isn't registered */
	UnregisteredMustBeMappedGUID,

	/** FObjectReplicator ReceivedBunch failed */
	ObjectReplicatorReceivedBunchFail,

	/** Content block serialization failed */
	ContentBlockFail,

	/** Content block header bOutHasRepLayout serialization failed */
	ContentBlockHeaderRepLayoutFail,

	/** Content block header bIsActor serialization failed */
	ContentBlockHeaderIsActorFail,

	/** Content block header object serialization failed */
	ContentBlockHeaderObjFail,

	/** Reached end of bunch prematurely, after reading content block header */
	ContentBlockHeaderPrematureEnd,

	/** Content block header sub-object was an Actor */
	ContentBlockHeaderSubObjectActor,

	/** Content block header sub-object not in parent actor */
	ContentBlockHeaderBadParent,

	/** The client tried to create a sub-object */
	ContentBlockHeaderInvalidCreate,

	/** Content block header bStablyNamed serialization failed */
	ContentBlockHeaderStablyNamedFail,

	/** Content block header unable to ready sub-object class */
	ContentBlockHeaderNoSubObjectClass,

	/** Content block header sub-object was UObject base class */
	ContentBlockHeaderUObjectSubObject,

	/** Content block header sub-object was an AActor subclass */
	ContentBlockHeaderAActorSubObject,

	/** Content block header serialization failed */
	ContentBlockHeaderFail,

	/** Content block payload NumPayloadBits serialization failed */
	ContentBlockPayloadBitsFail,

	/** Field header RepIndex serialization failed */
	FieldHeaderRepIndex,

	/** Field header RepIndex exceeds maximum index limit */
	FieldHeaderBadRepIndex,

	/** Field header payload NumPayloadBits serialization failed */
	FieldHeaderPayloadBitsFail,

	/** Field payload serialization failed */
	FieldPayloadFail,

	/** Replication Channel Count was exceeded and channel creation failed */
	ReplicationChannelCountMaxedOut,

	/** Beacon Error result types */

	/** Control flow error */
	BeaconControlFlowError,

	/** Unable to parse expected packet structure */
	BeaconUnableToParsePacket,

	/** Failed to verify user authentication */
	BeaconAuthenticationFailure,

	/** Login Failure, invalid ID */
	BeaconLoginInvalidIdError,

	/** Login Failure, unable to process authentication */
	BeaconLoginInvalidAuthHandlerError,

	/** Unable to authenticate for beacon, wrong PlayerId */
	BeaconAuthError,

	/** Join failure, existing ClientWorldPackageName */
	BeaconSpawnClientWorldPackageNameError,

	/** Join failure, existing beacon actor */
	BeaconSpawnExistingActorError,

	/** Join failure, couldn't spawn client beacon actor */
	BeaconSpawnFailureError,

	/** Join failure, no actor at NetGUIDAck */
	BeaconSpawnNetGUIDAckNoActor,

	/** Join failure, no host object at NetGUIDAck */
	BeaconSpawnNetGUIDAckNoHost,

	/** Join failure, unexpected control message */
	BeaconSpawnUnexpectedError,

	
	/** Iris Error result types */

	/** Protocol mismatch in Iris preventing a critical object instantiation */
	IrisProtocolMismatch,

	/** When a specific handle caused a reading error */
	IrisNetRefHandleError,

	/** Fault Handler Error result types */

	/** Net Fault Recovery failed to recover from a fault, and is triggering a disconnect */
	FaultDisconnect,


	/** PacketHandler Error result types */

	/** Marks a PacketHandler/HandlerComponent error, that is not recoverable */
	NotRecoverable,
};

DECLARE_NETRESULT_ENUM(ENetCloseResult);


NETCORE_API const TCHAR* LexToString(ENetCloseResult Enum);

/** Converts from ENetworkFailure to ENetCloseResult */
NETCORE_API ENetCloseResult FromNetworkFailure(ENetworkFailure::Type Val);

/** Converts from ENetCloseResult to ENetworkFailure */
NETCORE_API ENetworkFailure::Type ToNetworkFailure(ENetCloseResult Val);


/** Converts from ESecurityEvent to ENetCloseResult */
NETCORE_API ENetCloseResult FromSecurityEvent(ESecurityEvent::Type Val);



namespace UE
{
namespace Net
{

/**
 * FNetCloseResult
 *
 * TNetResult alias using ENetCloseResult, for use with NetConnection Close errors and recoverable network faults.
 */
using FNetCloseResult = TNetResult<ENetCloseResult>;

}
}

