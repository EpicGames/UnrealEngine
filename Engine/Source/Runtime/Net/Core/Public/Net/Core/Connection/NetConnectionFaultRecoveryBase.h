// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/PlatformCrt.h"
#include "Math/UnrealMathSSE.h"
#include "Net/Core/Connection/EscalationStates.h"
#include "Net/Core/Connection/NetCloseResult.h"
#include "Net/Core/Connection/NetResultManager.h"
#include "Net/Core/Connection/StateStruct.h"
#include "Templates/PimplPtr.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/ObjectMacros.h"

#include "NetConnectionFaultRecoveryBase.generated.h"

class UClass;

// Forward declarations
namespace UE
{
	namespace Net
	{
		class FNetConnectionFaultRecovery;
		struct FNetResult;
	}
}


/**
 * Generic escalation state definition used to implement attempted recovery from faults/errors in the NetConnection level netcode.
 * Fault handlers may have their own separate escalation tracking.
 */
USTRUCT()
struct FNetFaultState : public FEscalationState
{
	GENERATED_BODY()

	friend UE::Net::FNetConnectionFaultRecovery;

public:
	/** Whether or not the current escalation state should immediately Close the connection */
	UPROPERTY(config)
	bool bCloseConnection						= false;


	/** Escalation limits - for escalating to a more strict fault state */

	/** The number of faults per period before the next stage of escalation is triggered */
	UPROPERTY(config)
	int16 EscalateQuotaFaultsPerPeriod			= -1;

	/** Percentage of faults out of total number of recent packets, before the next stage of escalation is triggered */
	UPROPERTY(config)
	int8 EscalateQuotaFaultPercentPerPeriod		= -1;

	/** The number of faults per period before de-escalating into this state (adds hysteresis/lag to state changes) */
	UPROPERTY(config)
	int16 DescalateQuotaFaultsPerPeriod			= -1;

	/** Percentage of faults out of total number of recent packets, before de-escalating into this state (adds hysteresis/lag to state changes) */
	UPROPERTY(config)
	int8 DescalateQuotaFaultPercentPerPeriod	= -1;

	/** The time period to use for determining escalation/de-escalation quotas (Max: 16) */
	UPROPERTY(config)
	int8 EscalateQuotaTimePeriod				= -1;


public:
	static NETCORE_API const TCHAR* GetConfigSection();
	static NETCORE_API UClass* GetBaseConfigClass();


protected:
	NETCORE_API virtual EInitStateDefaultsResult InitConfigDefaultsInternal() override;
	NETCORE_API virtual void ApplyImpliedValuesInternal() override;
	NETCORE_API virtual void ValidateConfigInternal() override;

private:
	NETCORE_API virtual bool HasHitAnyQuota(FHasHitAnyQuotaParms Parms) const override;
};


namespace UE
{
namespace Net
{

/**
 * Defines the static counters used for FNetFaultState escalation manager implementation.
 * Custom counters (such as from PacketHandler's, Oodle/AES) can be dynamically added.
 */
enum class ENetFaultCounters : uint8
{
	PacketCount,		// Total number of packets received per-period
	NetConnPacketFault,	// Faults encountered by low level NetConnection packet processing (signifying packet corruption - excludes PacketHandler)

	Max,

	NumPrealloc = 6
};


/**
 * Categories that Net Fault Counters can optionally be registered into, so that they are automatically tracked for escalation management.
 * Primarily used by custom counters (e.g. Oodle/AES), to plug into the main NetConnection Fault Recovery escalation quota tracking.
 */
enum class ENetFaultCounterCategory : uint8
{
	/**
	 * Only worst value counter is used for quota checks, as network corruption should only affect outermost packet layer
	 * (Oodle/AES level primarily - but can occur at NetConnection level, if there are no compression/encryption handlers)
	 */
	NetworkCorruption,

	Max
};

/**
 * Inline integer conversion of ENetFaultCounters
 *
 * @param CounterVal	The value to convert to an integer
 * @return				The integer value of the specified ENetFaultCounters value
 */
inline uint8 ToInt(ENetFaultCounters CounterVal)
{
	return static_cast<uint8>(CounterVal);
}

/**
 * Inline integer conversion of ENetFaultCounterCategory
 *
 * @param CounterVal	The value to convert to an integer
 * @return				The integer value of the specified ENetFaultCounterCategory value
 */
inline uint8 ToInt(ENetFaultCounterCategory CategoryVal)
{
	return static_cast<uint8>(CategoryVal);
}

// Hacky base class to avoid 8 bytes of padding after the vtable
class FNetConnectionFaultRecoveryBaseFixLayout
{
public:
	virtual ~FNetConnectionFaultRecoveryBaseFixLayout() = default;
};

/**
 * Implements the base/public interface for FNetConnectionFaultRecovery - defined here, to eliminate the need for Engine dependencies.
 */
class FNetConnectionFaultRecoveryBase : public FNetConnectionFaultRecoveryBaseFixLayout
{
public:
	using FNetFaultEscalationHandler = TEscalationManager<ENetFaultCounters, FNetFaultState, ENetFaultCounterCategory>;

	/**
	 * (For Counter Categories) Use this to notify fault recovery of a successfully handled fault it should be aware of, within HandleNetResult;
	 * this is required for escalation tracking, if that fault type is registered with a counter category.
	 *
	 * @param InResult	The successfully handled result, passed on to fault recovery
	 * @return			Returns whether or not fault recovery handled the fault (this value should be returned by the calling HandleNetResult)
	 */
	NETCORE_API EHandleNetResult NotifyHandledFault(FNetResult&& InResult);


	/** Accessors */

	/**
	 * Accessor for FaultManager
	 */
	FNetResultManager& GetFaultManager()
	{
		return FaultManager;
	}

	/**
	 * Convenience Forwarding function for FNetResultManager.HandleNetResult, taking FNetCloseResult/ENetCloseResult
	 */
	NETCORE_API EHandleNetResult HandleNetResult(FNetCloseResult&& InResult);

	/**
	 * Passthrough for TEscalationManager.AddNewCounter (with caching, NetFaultEscalationManager has not been initialized yet)
	 */
	NETCORE_API int32 AddNewCounter(int32 Count=1);

	/**
	 * Passthrough for FEscalationManager.GetFrameCounter
	 */
	NETCORE_API FEscalationCounter& GetFrameCounter(int32 CounterIndex);

	/**
	 * Passthrough for TEscalationManager.RegisterCounterCategory (with caching, NetFaultEscalationManager has not been initialized yet)
	 */
	NETCORE_API void RegisterCounterCategory(ENetFaultCounterCategory Category, int32 CounterIndex);


private:
	virtual void InitEscalationManager()
	{
	}


protected:
	/** Whether or not fault recovery has disconnected the NetConnection */
	bool bDisconnected = false;

	/** Tracks the index of the last added counter, before NetFaultEscalationManager is created */
	int32 LastCounterIndex = static_cast<int32>(ENetFaultCounters::Max);

	/** The escalation manager instance used for tracking/implementing recovery for the various fault types */
	TUniquePtr<FNetFaultEscalationHandler> NetFaultEscalationManager;

	struct FPendingCategoryRegister
	{
		int32 CategoryIndex		= 0;
		int32 CounterIndex		= 0;

		friend bool operator == (const FPendingCategoryRegister& A, const FPendingCategoryRegister& B)
		{
			return A.CategoryIndex == B.CategoryIndex && A.CounterIndex == B.CounterIndex;
		}
	};

	/** Tracks counter categories and indexes that are pending registration, before NetFaultEscalationManager is created */
	TArray<FPendingCategoryRegister> PendingCategories;

	/** Net Faults which are currently being tracked (reset when escalation manager reaches base/normal state) */
	TPimplPtr<FNetResult, EPimplPtrMode::DeepCopy> TrackedFaults;

	/** Enum/fast-lookup version of TrackedFaults, using GetTypeHash */
	TArray<uint32, TInlineAllocator<FMath::Max(static_cast<int32>(ENetFaultCounters::Max), 8)>> TrackedFaultEnumHashes;


	/** Fault Manager for attempting to recover from connection faults, before triggering Close */
	FNetResultManager FaultManager;
};

}
}

