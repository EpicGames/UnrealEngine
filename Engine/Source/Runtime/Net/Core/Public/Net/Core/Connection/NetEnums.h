// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Misc/EnumClassFlags.h"

#include "NetEnums.generated.h"

namespace UE::Net
{
	enum class EJoinFlags
	{
		None,
		NoPawn = 1,
	};
	ENUM_CLASS_FLAGS(EJoinFlags)
}

/** Types of network failures broadcast from the engine */
UENUM(BlueprintType)
namespace ENetworkFailure
{
	enum Type : int
	{
		/** A relevant net driver has already been created for this service */
		NetDriverAlreadyExists,
		/** The net driver creation failed */
		NetDriverCreateFailure,
		/** The net driver failed its Listen() call */
		NetDriverListenFailure,
		/** A connection to the net driver has been lost */
		ConnectionLost,
		/** A connection to the net driver has timed out */
		ConnectionTimeout,
		/** The net driver received an NMT_Failure message */
		FailureReceived,
		/** The client needs to upgrade their game */
		OutdatedClient,
		/** The server needs to upgrade their game */
		OutdatedServer,
		/** There was an error during connection to the game */
		PendingConnectionFailure,
		/** NetGuid mismatch */
		NetGuidMismatch,
		/** Network checksum mismatch */
		NetChecksumMismatch
	};
}

namespace ENetworkFailure
{
	inline const TCHAR* ToString(ENetworkFailure::Type FailureType)
	{
		switch (FailureType)
		{
		case NetDriverAlreadyExists:
			return TEXT("NetDriverAlreadyExists");
		case NetDriverCreateFailure:
			return TEXT("NetDriverCreateFailure");
		case NetDriverListenFailure:
			return TEXT("NetDriverListenFailure");
		case ConnectionLost:
			return TEXT("ConnectionLost");
		case ConnectionTimeout:
			return TEXT("ConnectionTimeout");
		case FailureReceived:
			return TEXT("FailureReceived");
		case OutdatedClient:
			return TEXT("OutdatedClient");
		case OutdatedServer:
			return TEXT("OutdatedServer");
		case PendingConnectionFailure:
			return TEXT("PendingConnectionFailure");
		case NetGuidMismatch:
			return TEXT("NetGuidMismatch");
		case NetChecksumMismatch:
			return TEXT("NetChecksumMismatch");
		}
		return TEXT("Unknown ENetworkFailure error occurred.");
	}
}

UENUM(BlueprintType)
enum class EReplicationSystem : uint8
{
	Default,
	Generic,
	Iris,
};

// (DEPRECATED) Security event types used for UE_SECURITY_LOG
namespace ESecurityEvent
{ 
	enum Type
	{
		Malformed_Packet = 0, // The packet didn't follow protocol
		Invalid_Data = 1,     // The packet contained invalid data
		Closed = 2            // The connection had issues (potentially malicious) and was closed
	};
	
	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(const ESecurityEvent::Type EnumVal)
	{
		switch (EnumVal)
		{
			case Malformed_Packet:
			{
				return TEXT("Malformed_Packet");
			}
			case Invalid_Data:
			{
				return TEXT("Invalid_Data");
			}
			case Closed:
			{
				return TEXT("Closed");
			}
		}
		return TEXT("");
	}
}

namespace UE::Net
{

/** Traits of a UNetConnection that are of importance to propagate to replication systems not depending on Engine. */
enum class ENetConnectionTraits : uint32
{
	/** No traits. Must be zero. */
	None = 0U,
	/** The connection is for replay recording */
	ReplayRecording = 1U,
	/** The connection is for replay playback */
	ReplayPlayback = 2U,
};
ENUM_CLASS_FLAGS(ENetConnectionTraits);

enum class EProxyType : uint8
{
	None = 0,
	/** Replicate objects from different game servers into the world. */
	Backend,
	/** Replicate objects to clients that were replicated into the world by another replication system. */
	Frontend,
};

/**
 * Error types used when reporting an error on a specific Iris handle
 */
enum class ENetRefHandleError : uint32
{
	None = 0x00,
	BitstreamCorrupted, // An error while reading detected an irrecoverable bitstream corruption. Client was forced to disconnect
	ReplicationDisabled, // An error while reading was recoverable but prevents this NetObject from replicating.
	BlockedByMustBeMapped, // An object is blocked due to MustBeMapped references not resolving fast enough
	BlockedByCreationParents, // An object is blocked waiting on creation parents that do not exist yet on the client.
};

inline const TCHAR* LexToString(ENetRefHandleError NetRefHandleError)
{
	switch (NetRefHandleError)
	{
		case ENetRefHandleError::None:
		{
			return TEXT("None");
		} break;
		case ENetRefHandleError::BitstreamCorrupted:
		{
			return TEXT("BitstreamCorrupted");
		} break;
		case ENetRefHandleError::ReplicationDisabled:
		{
			return TEXT("ReplicationDisabled");
		} break;
		case ENetRefHandleError::BlockedByMustBeMapped:
		{
			return TEXT("BlockedByMustBeMapped");
		} break;
		case ENetRefHandleError::BlockedByCreationParents:
		{
			return TEXT("BlockedByCreationParents");
		} break;
		default:
		{
			return TEXT("missing");
		}
	}
}

} // end namespace UE::Net
