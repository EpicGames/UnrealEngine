// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Templates/EnableIf.h"
#include "Templates/IsEnum.h"
#include "Templates/PimplPtr.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"


namespace UE
{
namespace Net
{

// Forward declarations
struct FNetResult;
template<typename> struct TNetResult;


// Expose FNetResult friend functions for Argument Dependent Lookup
template<typename T, typename = typename TEnableIf<TIsEnum<T>::Value>::Type>
TNetResult<T>* Cast(FNetResult* InResult);

template<typename T, typename = typename TEnableIf<TIsEnum<T>::Value>::Type>
const TNetResult<T>* Cast(const FNetResult* InResult);


/**
 * Conversion types for TNetResult::ToString
 */
enum class ENetResultString : uint8
{
	/** Only include the base TNetResult in the string */
	WithoutChain,

	/** Include the whole chain of TNetResult's in the string */
	WithChain,

	/** Only include the raw result enum in the string */
	ResultEnumOnly
};

namespace Private
{
	NETCORE_API void NetResultToString(FString& OutResultStr, const TCHAR* LexResult, const FString& ErrorContext,
												ENetResultString ConversionType);
}


/**
 * Base/non-templatized implementation for TNetResult
 */
struct FNetResult
{
	template<typename> friend struct TNetResult;

private:
	inline FNetResult(uint64 InResult, const UEnum* InResultEnumObj)
		: Result(InResult)
		, ResultEnumObj(InResultEnumObj)
		, RawResultEnumObj(InResultEnumObj)
	{
	}

	inline FNetResult(uint64 InResult, const UEnum* InResultEnumObj, const FString& InErrorContext)
		: Result(InResult)
		, ResultEnumObj(InResultEnumObj)
		, RawResultEnumObj(InResultEnumObj)
		, ErrorContext(InErrorContext)
	{
	}

	inline FNetResult(uint64 InResult, const UEnum* InResultEnumObj, FString&& InErrorContext)
		: Result(InResult)
		, ResultEnumObj(InResultEnumObj)
		, RawResultEnumObj(InResultEnumObj)
		, ErrorContext(MoveTemp(InErrorContext))
	{
	}

public:
	FNetResult(const FNetResult&) = default;
	FNetResult& operator=(const FNetResult&) = default;
	FNetResult(FNetResult&&) = default;
	FNetResult& operator=(FNetResult&&) = default;
	~FNetResult() = default;


	/** Operators */

	bool operator == (const FNetResult& A) const
	{
		return Result == A.Result && ErrorContext == A.ErrorContext && RawResultEnumObj == A.RawResultEnumObj &&
			NextResult.IsValid() == A.NextResult.IsValid() && (!NextResult.IsValid() || *NextResult == *A.NextResult);
	}

	bool operator != (const FNetResult& A) const
	{
		return !(FNetResult::operator == (A));
	}


	/**
	 * Casts an FNetResult to a TNetResult<T>, with enum type checking
	 *
	 * @param InResult	The FNetResult to cast
	 * @return			Returns the resulting TNetResult<T>
	 */
	template<typename T, typename>
	friend TNetResult<T>* Cast(FNetResult* InResult)
	{
		TNetResult<T>* ReturnVal = nullptr;

		if (InResult != nullptr && InResult->RawResultEnumObj == StaticEnum<T>())
		{
			ReturnVal = static_cast<TNetResult<T>*>(InResult);
		}

		return ReturnVal;
	}

	/**
	 * Casts an FNetResult to a TNetResult<T>, with enum type checking
	 *
	 * @param InResult	The FNetResult to cast
	 * @return			Returns the resulting TNetResult<T>
	 */
	template<typename T, typename>
	friend const TNetResult<T>* Cast(const FNetResult* InResult)
	{
		const TNetResult<T>* ReturnVal = nullptr;

		if (InResult != nullptr && InResult->RawResultEnumObj == StaticEnum<T>())
		{
			ReturnVal = static_cast<const TNetResult<T>*>(InResult);
		}

		return ReturnVal;
	}


	/**
	 * Use UEnum reflection to convert result to human readable string (ExportText style). Use ToString instead, when casting is possible.
	 *
	 * @param ConversionType	The type of string conversion to perform (e.g. whether or not to include the whole chain of results)
	 * @return					Returns the result converted to a string
	 */
	NETCORE_API FString DynamicToString(ENetResultString ConversionType=ENetResultString::WithoutChain) const;

	friend uint32 GetTypeHash(FNetResult const& Hash)
	{
		return HashCombine(::GetTypeHash((uint64)Hash.Result), ::GetTypeHash(Hash.RawResultEnumObj));
	}


	/**
	 * Chains a new result to an existing result, putting the new result at the end of the chain.
	 *
	 * @param InResult		The result to chain to this one. Should use MoveTemp.
	 */
	void AddChainResult(FNetResult&& InResult)
	{
		FNetResult* ChainEnd = this;

		// Not intended to be fast, just in a useful order while using less members
		while (ChainEnd->NextResult.IsValid())
		{
			ChainEnd = ChainEnd->NextResult.Get();
		}

		ChainEnd->NextResult = MakePimpl<FNetResult, EPimplPtrMode::DeepCopy>(MoveTemp(InResult));
	}

	/**
	 * Chains a new result to an existing result, putting the new result at the end of the chain.
	 *
	 * @param InResult		The result to chain to this one. Should use MoveTemp.
	 */
	template<typename T, typename = typename TEnableIf<TIsEnum<T>::Value>::Type>
	void AddChainResult(TNetResult<T>&& InResult)
	{
		FNetResult* ChainEnd = this;

		// Not intended to be fast, just in a useful order while using less members
		while (ChainEnd->NextResult.IsValid())
		{
			ChainEnd = ChainEnd->NextResult.Get();
		}

		ChainEnd->NextResult = MakePimpl<FNetResult, EPimplPtrMode::DeepCopy>(MoveTemp(InResult));
	}

	/**
	 * Determines whether the specified enum result is contained within this result chain
	 * NOTE: Does not check the error code
	 *
	 * @param A		The result to search for
	 * @return		Whether or not the result exists in the chain
	 */
	template<typename ResultEnum, typename = typename TEnableIf<TIsEnum<ResultEnum>::Value>::Type>
	bool HasChainResult(ResultEnum A) const
	{
		for (FConstIterator It(*this); It; ++It)
		{
			const TNetResult<ResultEnum>* CurResult = Cast<ResultEnum>(&*It);

			if (CurResult != nullptr && static_cast<ResultEnum>(CurResult->Result) == A)
			{
				return true;
			}
		}

		return false;
	}


public:
	/** Chain iterator */
	struct FConstIterator final
	{
		FConstIterator(const FNetResult& InResult)
			: CurResult(&InResult)
		{
		}

		explicit operator bool() const
		{
			return CurResult != nullptr;
		}

		const FNetResult& operator *() const
		{
			check(CurResult != nullptr);

			return *CurResult;
		}

		const FNetResult* operator ->() const
		{
			return CurResult;
		}

		void operator ++()
		{
			if (CurResult != nullptr)
			{
				CurResult = CurResult->NextResult.Get();
			}
		}

	private:
		const FNetResult* CurResult;
	};


private:
	/** Base result value */
	uint64 Result	= 0;

	/** Weak pointer to the UEnum for the result values type */
	TWeakObjectPtr<const UEnum> ResultEnumObj;

	/** Raw pointer to the result UEnum type, for comparison only */
	const void* RawResultEnumObj = nullptr;

	/** Error context, intended to be interpreted by code */
	FString ErrorContext;

	/** If multiple sequential results (typically errors) are returned, accumulate them as a linked list */
	TPimplPtr<FNetResult, EPimplPtrMode::DeepCopy> NextResult;
};

/**
 * Chains a new result to the result pointer, putting it at the end of the chain - or assigns it to the result pointer, if not set.
 *
 * @param ResultPtr		The result pointer to chain or assign the new result to.
 * @param Result		The result to chain to the result pointer. Should use MoveTemp.
 */
inline void AddToChainResultPtr(TPimplPtr<FNetResult, EPimplPtrMode::DeepCopy>& ResultPtr, FNetResult&& Result)
{
	if (ResultPtr.IsValid())
	{
		ResultPtr->AddChainResult(MoveTemp(Result));
	}
	else
	{
		ResultPtr = MakePimpl<FNetResult, EPimplPtrMode::DeepCopy>(MoveTemp(Result));
	}
}

/**
 * Chains a new result to the end of the result pointer (or takes its place if not set), then consumes/moves/returns the pointer result
 *
 * @param ResultPtr		The result pointer to consume and chain the new result to.
 * @param Result		The result to chain to the consumed result pointer. Should use MoveTemp.
 * @return				Returns the final result, which is either the combined result pointer and new result, or just the new result
 */
inline FNetResult AddToAndConsumeChainResultPtr(TPimplPtr<FNetResult, EPimplPtrMode::DeepCopy>& ResultPtr, FNetResult&& Result)
{
	const bool bValidResultPtr = ResultPtr.IsValid();
	FNetResult ReturnVal(bValidResultPtr ? MoveTemp(*ResultPtr) : MoveTemp(Result));

	if (bValidResultPtr)
	{
		ReturnVal.AddChainResult(MoveTemp(Result));
	}

	return ReturnVal;
}


/**
 * Generic result struct for Network Results (e.g. Close and failures) - partially adapted from FOnlineError/FResult.
 * ResultEnum must have LexToString implemented, and DECLARE_NETRESULT_ENUM(ResultEnum) must be declared for each enum type.
 *
 * NOTE: Support for using results to modify control flow, depends on how they are implemented in the netcode.
 */
template<typename ResultEnum>
struct TNetResult final : public FNetResult
{
	static_assert(TIsEnum<ResultEnum>::Value, "ResultEnum must be an enum");
	static_assert(ResultEnum::Success != ResultEnum::Unknown, "ResultEnum must contain a 'Success' and 'Unknown' (i.e. undetermined) value");

	using UnderlyingResultType = __underlying_type(ResultEnum);

public:
	/**
	 * Default constructor
	 *
	 * @param InResult			The type of result
	 */
	TNetResult(ResultEnum InResult=ResultEnum::Unknown)
		: FNetResult(static_cast<UnderlyingResultType>(InResult), StaticEnum<ResultEnum>())
	{
		if (InResult != ResultEnum::Unknown && InResult != ResultEnum::Success)
		{
			ErrorContext = DefaultErrorContext(InResult);
		}
	}

	/**
	 * Main constructor, specifying full result (usually error)
	 *
	 * @param InResult			The type of result
	 * @param InErrorContext	Additional context associated with error results (to be interpreted by code)
	 */
	TNetResult(ResultEnum InResult, const FString& InErrorContext)
		: FNetResult(static_cast<UnderlyingResultType>(InResult), StaticEnum<ResultEnum>())
	{
		if (InResult != ResultEnum::Unknown && InResult != ResultEnum::Success)
		{
			ErrorContext = (InErrorContext.IsEmpty() ? DefaultErrorContext(InResult) : InErrorContext);
		}
	}


	/**
	 * Convert result to human readable string (ExportText style)
	 *
	 * @param ConversionType	The type of string conversion to perform (e.g. whether or not to include the whole chain of results)
	 * @return					Returns the result converted to a string
	 */
	FString ToString(ENetResultString ConversionType=ENetResultString::WithoutChain) const
	{
		using namespace UE::Net::Private;

		FString ReturnVal;
		const bool bIncludeChain = ConversionType == ENetResultString::WithChain;

		for (FConstIterator It(*this); It; ++It)
		{
			const TCHAR* ResultLex = ToCStr(LexToString(static_cast<ResultEnum>(It->Result)));

			if (ConversionType == ENetResultString::ResultEnumOnly)
			{
				ReturnVal = ResultLex;
			}
			else
			{
				NetResultToString(ReturnVal, ResultLex, It->ErrorContext, ConversionType);
			}

			if (!bIncludeChain)
			{
				break;
			}
		}

		return ReturnVal;
	}


	/** Accessors */

	bool WasSuccessful() const
	{
		return static_cast<ResultEnum>(Result) == ResultEnum::Success;
	}

	ResultEnum GetResult() const
	{
		return static_cast<ResultEnum>(Result);
	}

	const FString& GetErrorContext() const
	{
		return ErrorContext;
	}


	/**
	 * Determines whether the specified result is contained within this result chain
	 * NOTE: Also checks against the error code
	 *
	 * @param A		The result to search for
	 * @return		Whether or not the result exists in the chain
	 */
	bool HasChainResult(const TNetResult& A) const
	{
		for (FConstIterator It(*this); It; ++It)
		{
			if (*It == A)
			{
				return true;
			}
		}

		return false;
	}


	/** Operators */

	bool operator == (ResultEnum A) const
	{
		return static_cast<ResultEnum>(Result) == A;
	}

	bool operator != (ResultEnum A) const
	{
		return !(TNetResult::operator == (A));
	}

private:
	/** Default ErrorContext based on ResultEnum */
	static const TCHAR* DefaultErrorContext(ResultEnum InResult)
	{
		return (InResult == ResultEnum::Success ? TEXT("") : ToCStr(LexToString(InResult)));
	}
};

/**
 * Exposes TNetResult friend functions to Argument-Dependent-Lookup, so these functions are found when 'EnumType' is passed as a parameter.
 * This is normally automatic when TNetResult is passed as a parameter, but it's not automatic when 'EnumType' is implicitly casted, e.g:
 *	AddToChainResultPtr(Bunch.ExtendedError, ENetCloseResult::MaxReliableExceeded);
 *
 * @param EnumType		The TNetResult enum type being declared
 */
#define DECLARE_NETRESULT_ENUM(EnumType) \
	namespace UE \
	{ \
	namespace Net \
	{ \
		inline void AddToChainResultPtr(TPimplPtr<FNetResult, EPimplPtrMode::DeepCopy>& ResultPtr, TNetResult<EnumType>&& Result) { AddToChainResultPtr(ResultPtr, static_cast<FNetResult&&>(Result)); } \
		inline FNetResult AddToAndConsumeChainResultPtr(TPimplPtr<FNetResult, EPimplPtrMode::DeepCopy>& ResultPtr, TNetResult<EnumType>&& Result) { return AddToAndConsumeChainResultPtr(ResultPtr, static_cast<FNetResult&&>(Result)); } \
	} \
	}

}
}
