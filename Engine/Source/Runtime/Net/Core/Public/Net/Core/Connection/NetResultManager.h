// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/Platform.h"
#include "Net/Core/Connection/NetResult.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"


namespace UE
{
namespace Net
{

// Forward declarations
class FNetResultHandler;


/**
 * The result of a HandleNetResult call
 */
enum class EHandleNetResult : uint8
{
	Closed,			// The connection was closed (certain code paths should return early)
	Handled,		// The result was handled, but (if a fault) has not yet resulted in a connection Close
	NotHandled		// The result was not handled (no result handler or 'unhandled result' callback handled it)
};

/**
 * The position to add a new FNetResultHandler, in the list of result handlers
 */
enum class EAddResultHandlerPos : uint8
{
	First,			// Places the new handler at the start of the list (higher precedence)
	Last			// Places the new handler at the end of the list (lower precedence)
};


/**
 * Net Result Manager
 *
 * Handles arbitrary net results, which may attempt recovery from errors instead of e.g. immediately closing the NetConnection
 */
class FNetResultManager final
{
public:
	/**
	 * Callback for handling results which no FNetResultHandler took ownership of
	 *
	 * @param InResult		Specifies the result
	 */
	using FUnhandledResultFunc = TUniqueFunction<EHandleNetResult(FNetResult&& InResult)>;


public:
	FNetResultManager() = default;

	FNetResultManager(FNetResultManager&) = delete;
	FNetResultManager& operator=(const FNetResultManager&) = delete;
	FNetResultManager(FNetResultManager&&) = delete;
	FNetResultManager& operator=(FNetResultManager&&) = delete;

	/**
	 * Adds a new result handler to the result manager (owned by the result manager)
	 *
	 * @param InResultHandler	The new result handler to be added
	 * @param Position			The position/precedence in the result handler list, to place the new handler
	 */
	NETCORE_API void AddResultHandler(TUniquePtr<FNetResultHandler>&& InResultHandler, EAddResultHandlerPos Position=EAddResultHandlerPos::Last);

	/**
	 * Adds a new result handler pointer to the result manager (not owned by the result manager)
	 *
	 * @param InResultHandler	The new result handler to be added
	 * @param Position			The position/precedence in the result handler list, to place the new handler
	 */
	NETCORE_API void AddResultHandlerPtr(FNetResultHandler* InResultHandler, EAddResultHandlerPos Position=EAddResultHandlerPos::Last);

	/**
	 * Takes a net result and passes it around to the result handlers and callbacks until it is handled, or returns 'EHandleNetResult::NotHandled'.
	 *
	 * NOTE: InResult is moved when result is Closed/Handled, and not moved when result is NotHandled.
	 *
	 * @param InResult		Specifies the result
	 * @return				Whether or not the result has handled or resulted in a close, or went unhandled
	 */
	NETCORE_API EHandleNetResult HandleNetResult(FNetResult&& InResult);

	/**
	 * Sets a callback for handling net results which no result handlers have dealt with
	 *
	 * @param InCallback	The callback to use for unhandled results
	 */
	NETCORE_API void SetUnhandledResultCallback(FUnhandledResultFunc InCallback);

private:
	/** The list of result handlers, for attempting to handle results with */
	TArray<FNetResultHandler*, TInlineAllocator<8>> ResultHandlers;

	/** Result handlers passed to and owned by the result manager */
	TArray<TUniquePtr<FNetResultHandler>> OwnedResultHandlers;

	/** The callback for handling results which no result handler has dealt with */
	FUnhandledResultFunc UnhandledResultCallback;
};


/**
 * Net Result Handler
 *
 * Result handlers implement result tracking and recovery attempts for select fault types, with the ability to expand custom result handling,
 * within NetDriver subclasses or other arbitrary parts of the netcode.
 */
class FNetResultHandler
{
	friend FNetResultManager;

public:
	FNetResultHandler() = default;
	virtual ~FNetResultHandler() = default;

	FNetResultHandler(const FNetResultHandler&) = delete;
	FNetResultHandler& operator=(const FNetResultHandler&) = delete;
	FNetResultHandler(FNetResultHandler&&) = delete;
	FNetResultHandler& operator=(FNetResultHandler&&) = delete;


	/**
	 * Initializes the result handler
	 */
	virtual void Init()
	{
	}

	/**
	 * Takes a net result and either handles it or returns 'EHandleNetResult::NotHandled'.
	 *
	 * NOTE: InResult is moved when result is Closed/Handled, and not moved when result is NotHandled.
	 *
	 * @param InResult		Specifies the result
	 * @return				Whether or not the result has handled or resulted in close, or went unhandled
	 */
	virtual EHandleNetResult HandleNetResult(FNetResult&& InCloseResult)
	{
		return EHandleNetResult::NotHandled;
	}

protected:
	/** The Result Manager which the result handler is assigned to */
	FNetResultManager* ResultManager = nullptr;
};

}
}
