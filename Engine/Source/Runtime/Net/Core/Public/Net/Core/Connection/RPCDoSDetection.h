// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/CoreNet.h"
#include "Net/Core/Analytics/RPCDoSDetectionAnalytics.h"

#include "RPCDoSDetection.generated.h"


// Defines

/** Count/Time Quota debugging */
#define RPC_QUOTA_DEBUG 0

/** Whether or not CSV stats should be enabled for the RPC DoS checks (development only - for stress testing the RPC DoS code) */
#define RPC_DOS_DEV_STATS 0

/** Tick/Packet scope correctness debugging */
#define RPC_DOS_SCOPE_DEBUG 1


// Typedefs

/**
 * Callback usually passed in by the NetConnection, for getting the address of the owning connection
 *
 * @return	The address of the owning NetConnection
 */
using FGetRPCDoSAddress = TUniqueFunction<FString()>;

/**
 * Callback usually passed in by the NetConnection, for getting the unique id of the owning player/connection
 *
 * @return	The unique id of the owning player/connection
 */
using FGetRPCDoSPlayerUID = TUniqueFunction<FString()>;

/**
 * Callback usually passed in by the NetConection, for kicking the player after exceeding RPC DoS kick thresholds
 */
using FRPCDoSKickPlayer = TUniqueFunction<void()>;


// Globals/CVars

#if RPC_DOS_SCOPE_DEBUG
namespace UE::Net
{
	/** Whether or not debugging/ensures for RPC DoS Tick/Packet scopes should be enabled */
	extern NETCORE_API int32 GRPCDoSScopeDebugging;
}
#endif


// Structs

/**
 * Struct containing RPC counters covering any time period (e.g. frame/second/arbitrary-period)
 */
struct FRPCDoSCounters
{
	/** Counter for the number of RPCs received */
	int32 RPCCounter			= 0;

	/** Accumulated time spent executing RPCs */
	double AccumRPCTime			= 0.0;

#if RPC_QUOTA_DEBUG
	/** Debug version of the value above, using more accurate timing, for testing accuracy of approximate time */
	double DebugAccumRPCTime	= 0.0;
#endif


public:
	/**
	 * Reset the values of this RPC counter
	 */
	void ResetRPCCounters()
	{
		RPCCounter = 0;
		AccumRPCTime = 0.0;

#if RPC_QUOTA_DEBUG
		DebugAccumRPCTime = 0.0;
#endif
	}

	/**
	 * Accumulate another RPC counter into this one
	 *
	 * @param InCounter		The RPC counter to accumulate with
	 */
	void AccumulateCounter(const FRPCDoSCounters& InCounter)
	{
		RPCCounter += InCounter.RPCCounter;
		AccumRPCTime += InCounter.AccumRPCTime;

#if RPC_QUOTA_DEBUG
		DebugAccumRPCTime += InCounter.DebugAccumRPCTime;
#endif
	}
};

/**
 * The status of blocking for an individual RPC
 */
enum class ERPCBlockState : uint8
{
	Unchecked,		// The status for this RPC has not been checked
	OnAllowList,	// The RPC is allow listed and can't be blocked
	NotBlocked,		// The RPC is not blocked
	Blocked			// The RPC is blocked
};

/**
 * Live tracking information for individual RPC's, for timing of RPC's, analytics, and handling blocking
 */
struct FRPCTrackingInfo
{
	/** Contains up to 'HistoryCount' seconds of counter/timing data for the RPC. [0] = last second of data, [15] = last 16 seconds of data.  */
	FRPCDoSCounters PerPeriodHistory[16]		= {};

	/** The number of valid 'PerPeriodHistory' entries */
	uint8 HistoryCount							= 0;

	/** The last time this RPC was called/tracked (based on 'SecondsIncrementer'). Indirect index to the active PerPeriodHistory entry. */
	uint8 LastTrackedSecondIncrement			= 0;

	/** Whether or not the RPC is currently being blocked, after having hit a blocking quota. */
	ERPCBlockState BlockState					= ERPCBlockState::Unchecked;

	/** Caches a pointer to analytics for this RPC, if it's within analytics thresholds */
	TSharedPtr<FRPCAnalytics> RPCTrackingAnalyticsEntry;


public:
	/**
	 * Gets the active 'PerPeriodHistory' index, based on LastTrackedSecondIncrement.
	 *
	 * @return	The currently active 'PerPeriodHistory' index.
	 */
	uint8 GetCurrentHistoryIdx() const
	{
		static_assert(UE_ARRAY_COUNT(PerPeriodHistory) == 16, "PerPeriodHistory must be 16 in size, for GetCurrentHistoryIdx to work.");

		return LastTrackedSecondIncrement & 0xF;
	}
};

/**
 * Stores a reference to active/valid FRPCTrackingInfo entries, for occasional cleanup.
 */
struct FActiveRPCTrackingInfo
{
	/** The 'RPCTracking' map key for this entry (UFunction pointer, which may not point to a valid UFunction, hence void*) */
	const void* Key							= nullptr;

	/** Direct pointer to the RPC tracking info, for checking if it's expired */
	const FRPCTrackingInfo* TrackingInfo	= nullptr;
};


/**
 * Stores the RPC DoS detection state (either settings from the config file, or the active DDoS detection state)
 */
USTRUCT()
struct FRPCDoSState
{
	GENERATED_USTRUCT_BODY()

	/** Whether or not to log when escalating to this state */
	UPROPERTY(config)
	bool bLogEscalate				= false;

	/** Whether or not to send analytics when escalating to this state */
	UPROPERTY(config)
	bool bSendEscalateAnalytics		= false;

	/** Whether or not to kick the player when they escalate to this state */
	UPROPERTY(config)
	bool bKickPlayer				= false;

	/** Whether or not to keep a temporary record of recent RPC's, for potential logging/analytics */
	UPROPERTY(config)
	bool bTrackRecentRPCs			= false;


	/** Escalation limits - for escalating to a more strict FRPCDoSState */

	/** The number of RPC's per frame before the next stage of DoS detection is triggered */
	UPROPERTY(config)
	int16 EscalateQuotaRPCsPerFrame		= -1;

	/** The amount of time spent executing RPC's per frame, before the next stage of DoS detection is triggered */
	UPROPERTY(config)
	int16 EscalateTimeQuotaMSPerFrame	= -1;

	/** The number of RPC's per EscalateQuotaPeriod before the next stage of DoS detection is triggered */
	UPROPERTY(config)
	int16 EscalateQuotaRPCsPerPeriod	= -1;

	/** The amount of time spent executing RPC's per EscalateQuotaPeriod, before the next stage of DoS detection is triggered */
	UPROPERTY(config)
	int16 EscalateTimeQuotaMSPerPeriod	= -1;

	/** The time period to use for determining RPC count and time escalation quotas (Max: 16) */
	UPROPERTY(config)
	int8 EscalateQuotaTimePeriod		= -1;


	/** Escalation monitoring - for setting thresholds for triggering analytics (does not affect escalation quota's and RPC limiting) */

	/** The number of times this stage must be escalated to, before it is 'confirmed' as having been escalated to (for analytics) */
	UPROPERTY(config)
	int8 EscalationCountTolerance		= 1;

	/** The maximum time spent executing RPC's per frame, before this escalation stage is automatically 'confirmed' for analytics */
	UPROPERTY(config)
	int16 EscalationTimeToleranceMS		= -1;


	/** RPC Execution limits - for preventing execution of specific RPC's, after they have reached a count/time limit */

	/** The limit for the number of times a single RPC can be repeated, over the time period specified by RPCRepeatTimeLimitPeriod */
	UPROPERTY(config)
	int16 RPCRepeatLimitPerPeriod		= -1;

	/** The limit for the number of milliseconds a single RPC can spend executing, over the time period specified by RPCRepeatTimeLimitPeriod */
	UPROPERTY(config)
	int16 RPCRepeatLimitMSPerPeriod		= -1;

	/** The time period to use for measuring excessive execution time for a single RPC (Max: 16) */
	UPROPERTY(config)
	int8 RPCRepeatLimitTimePeriod		= -1;


	/** The amount of time, in seconds, before the current DoS severity category cools off and de-escalates */
	UPROPERTY(config)
	int16 CooloffTime					= -1;

	/** The amount of time, in seconds, spent in the current DoS severity category before it automatically escalates to the next category */
	UPROPERTY(config)
	int16 AutoEscalateTime				= -1;


	/** Cached/converted values */

	/** EscalateTimeQuotaMSPerFrame converted to seconds */
	double EscalateTimeQuotaSecsPerFrame	= 0.0;

	/** EscalateTimeQuotaMSPerPeriod converted to seconds */
	double EscalateTimeQuotaSecsPerPeriod	= 0.0;

	/** RPCRepeatLimitMSPerPeriod converted to seconds */
	double RPCRepeatLimitSecsPerPeriod		= 0.0;

	/** EscalationTimeToleranceMS converted to seconds */
	double EscalationTimeToleranceSeconds	= 0.0;


public:
	virtual ~FRPCDoSState() = default;

	/**
	 * Some configuration values are implicitly enabled by other configuration values - this function applies them
	 */
	NETCORE_API virtual void ApplyImpliedValues();

	/**
	 * Whether or not the specified per-period counters have hit count escalation quota's specified by this state.
	 *
	 * @param PerPeriodHistory		The per-period historical counters being compared against
	 * @param InFrameCounter		The current frames counter for factoring-in to comparisons
	 * @return						Whether or not the count escalation quota has been hit
	 */
	NETCORE_API bool HasHitQuota_Count(const FRPCDoSCounters(&PerPeriodHistory)[16], FRPCDoSCounters& InFrameCounter) const;

	/**
	 * Whether or not the specified per-period counters have hit time escalation quota's specified by this state.
	 *
	 * @param PerPeriodHistory		The per-period historical counters being compared against
	 * @param InFrameCounter		The current frames counter for factoring-in to comparisons
	 * @return						Whether or not the time escalation quota has been hit
	 */
	bool HasHitQuota_Time(const FRPCDoSCounters(&PerPeriodHistory)[16], FRPCDoSCounters& InFrameCounter) const;
};

/**
 * Configuration for RPC DoS Detection states
 */
USTRUCT()
struct FRPCDoSStateConfig : public FRPCDoSState
{
	GENERATED_USTRUCT_BODY()

	/** The name of the RPC DoS severity level this config section represents */
	FString SeverityCategory;


private:
	/** Cached/converted values */

	/** Cached value for the highest time period in this config state */
	int8 HighestTimePeriod = 0;

	/** Cached value for all different time periods in this state */
	TArray<int8> AllTimePeriods;


public:
	/** Runtime values */

	/** Whether or not reaching this escalation stage has been 'confirmed' for analytics */
	bool bEscalationConfirmed = false;

	/** The number of times this stage has been escalated to */
	int16 EscalationCount = 0;


public:
	NETCORE_API virtual void ApplyImpliedValues() override;

	/**
	 * Uses reflection to load all struct config variables from the specified ini section.
	 * NOTE: Reflection does not work with multiple inheritance.
	 *
	 * @param SectionName	The ini section name containing the struct configuration
	 * @param InFilename	The ini filename to read from
	 * @return				Whether or not the struct config variables were read successfully
	 */
	bool LoadStructConfig(const TCHAR* SectionName, const TCHAR* InFilename=nullptr);

	/**
	 * Validates loaded struct config variables
	 */
	void ValidateConfig();

	/**
	 * Gets the highest counter time period specified by RPC DoS state settings (used for limiting 'CounterPerPeriodHistory' size)
	 *
	 * @return		The highest counter time period in the state config settings
	 */
	int8 GetHighestTimePeriod() const;

	/**
	 * Gets all counter time periods specified by the RPC DoS state settings
	 *
	 * @return		All of the time periods in the state config settings
	 */
	const TArray<int8>& GetAllTimePeriods() const;

	/**
	 * Applies the loaded config state, to an object implementing RPC DoS State's.
	 * NOTE: Does not use reflection, as this does not work with multiple inheritance.
	 *
	 * @param Target	The object implementing FRPCDoSState, to apply the config settings to.
	 */
	void ApplyState(FRPCDoSState& Target);
};



/**
 * Result of NotifyReceivedRPC, for specifying whether the RPC should execute or should be blocked
 */
enum class ERPCNotifyResult : uint8
{
	ExecuteRPC,		// Allow the RPC to execute
	BlockRPC		// Block the RPC from executing
};

/**
 * RPC DoS escalation severity update types
 */
enum class ERPCDoSSeverityUpdate : uint8
{
	Escalate,		// Escalating to a higher RPC DoS escalation state
	AutoEscalate,	// Automatic/non-RPC escalation to a higher RPC DoS escalation state, e.g. when too much time was spent in a lower state
	Deescalate		// Deescalating to a lower RPC DoS escalation state
};

/**
 * The reason for an RPC DoS severity update
 */
enum class ERPCDoSEscalateReason : uint8
{
	CountLimit,		// RPC Count quota's were hit
	TimeLimit,		// RPC Time quota's were hit
	AutoEscalate,	// Automatic escalation from a lower state, e.g. after spending too much time in a lower state
	Deescalate		// Deescalation to a lower state
};

/**
 * The type of a call to PostSequentialRPC
 */
enum class EPostSequentialRPCType : uint8
{
	MidPacket,		// End of RPC sequence while still processing packet
	PostPacket		// End of RPC sequence after a packet has finished processing
};



/**
 * RPC DoS detection
 *
 * Implements DoS detection for NetConnection RPC's, using an escalating series of detection states depending on the severity of the DoS,
 * to prevent false positives and minimize any performance impact.
 *
 * RPC's are profiled based on count and execution time, with careful implementation of tracking to balance lookup speed, memory usage, timing,
 * and analytics tracking.
 */
class FRPCDoSDetection : protected FRPCDoSState
{
	friend struct FRPCDoSStateConfig;

	/**
	 * Lightweight RPC tracking state
	 */
	struct FLightweightRPCTracking
	{
		/** The size of the tracking static arrays - must be 256 in case Count wraps (unlikely, but saves some checks) */
		static constexpr int32 TrackingNum = 256;

		struct FLightweightRPCEntry
		{
			/** The function being tracked */
			const UFunction* Function;

			/** Name of the function being tracked */
			FName Name;
		};


		/** Static array of tracked RPC's*/
		FLightweightRPCEntry RPC[TrackingNum] = {};

		/** The number of tracked RPC's in the static array */
		uint8 Count = 0;
	};

public:
	/**
	 * Initialize RPC DoS Detection, inputting callbacks for retrieving external information necessary for analytics.
	 *
	 * @param NetDriverName			The name of the NetDriver which RPC DoS Detection belongs to (e.g. GameNetDriver)
	 * @param AnalyticsAggregator	Reference to the NetDriver analytics aggregator
	 * @param AnalyticsSender		Reference to the Engine implementation of RPC DoS Analytics sending
	 * @param InAddressFunc			Callback for getting the current NetConnection address.
	 * @param InPlayerUIDFunc		Callback for getting the current NetConnection player UID.
	 * @param InKickPlayerFunc		Callback for kicking the player after a bad enough RPC DoS.
	 */
	NETCORE_API void Init(FName NetDriverName, TSharedPtr<FNetAnalyticsAggregator>& AnalyticsAggregator, TSharedPtr<FRPCDoSAnalyticsSender> AnalyticsSender,
				FGetRPCDoSAddress&& InAddressFunc, FGetRPCDoSPlayerUID&& InPlayerUIDFunc, FRPCDoSKickPlayer&& InKickPlayerFunc);

	/**
	 * Initializes configuration settings for RPC DoS Detection - with support for hot reloading
	 *
	 * @param NetDriverName		The name of the NetDriver which RPC DoS Detection belongs to (e.g. GameNetDriver)
	 */
	void InitConfig(FName NetDriverName);


	/**
	 * Called prior to the packet-loop TickDispatch
	 *
	 * @param TimeSeconds	Cached/approximate timestamp, to save grabbing a new timestamp
	 */
	NETCORE_API void PreTickDispatch(double TimeSeconds);

	/**
	 * Called prior to the current NetConnection receiving a packet.
	 *
	 * @param TimeSeconds	Cached/approximate timestamp, to save grabbing a new timestamp
	 */
	void PreReceivedPacket(double TimeSeconds)
	{
		using namespace UE::Net;

#if RPC_DOS_DEV_STATS
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RPCDoS_Checks);
#endif

#if RPC_DOS_SCOPE_DEBUG
		ensure(GRPCDoSScopeDebugging == 0 || TickScopePrivate.IsActive());
		ensure(GRPCDoSScopeDebugging == 0 || !SequentialRPCScopePrivate.IsActive());
#endif

		PacketScopePrivate.SetActive(true);

		FPacketScope& PacketScope = GetPacketScope();

		PacketScope.ReceivedPacketStartTime = TimeSeconds;

		// Redundant call to Reset for sequential RPC scope - remove when scope debugging verifies the sequential RPC scope is never active here
#if 1 // !RPC_DOS_SCOPE_DEBUG
		SequentialRPCScopePrivate.Reset();
#endif
	}

	/**
	 * If RPC monitoring is enabled, provides a notification for every RPC that is received, with the ability to block it.
	 *
	 * @param Object			The UObject the RPC will be called on
	 * @param Function			The UFunction representing the RPC
	 * @param FunctionName		The cached name of the function
	 * @return					Whether or not to block the RPC
	 */
	ERPCNotifyResult NotifyReceivedRPC(UObject* Object, const UFunction* Function, FName FunctionName)
	{
		ERPCNotifyResult Result = ERPCNotifyResult::ExecuteRPC;

#if RPC_DOS_DEV_STATS
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RPCDoS_Checks);
#endif

		if (bRPCDoSDetection && !bHitchSuspendDetection)
		{
			if (!SequentialRPCScopePrivate.IsActive())
			{
				PreSequentialRPC();
			}

			if (bRPCTrackingEnabled)
			{
				FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();

				Result = SequentialRPCScope.CheckRPCTracking(*this, Function, FunctionName);
			}

#if RPC_QUOTA_DEBUG
			FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();

			SequentialRPCScope.DebugReceivedRPCStartTime = FPlatformTime::Seconds();
#endif
		}


		return Result;
	}

	/**
	 * If RPC monitoring is disabled, does extremely lightweight tracking of RPC calls, for monitoring tracking triggers,
	 * and to allow a chance for deducing full RPC tracking information if monitoring is enabled while receiving
	 * (so that e.g. if an RPC DoS attack enables monitoring, information on the RPC responsible can be recovered).
	 *
	 * @param Function			The UFunction representing the RPC
	 * @param FunctionName		The cached name of the function
	 */
	void LightweightReceivedRPC(const UFunction* Function, FName FunctionName)
	{
#if RPC_DOS_DEV_STATS
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RPCDoS_Checks);
#endif

		if (bRPCDoSDetection && !bHitchSuspendDetection)
		{
			if (!SequentialRPCScopePrivate.IsActive())
			{
				PreSequentialRPC();
			}

			if (UNLIKELY(ForcedRPCTracking == FunctionName && FMath::FRand() <= ForcedRPCTrackingChance))
			{
				EnableForcedRPCTracking(Function, FunctionName, FPlatformTime::Seconds());
			}
			else
			{
				FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();
				FLightweightRPCTracking& LightweightRPCTracking = SequentialRPCScope.LightweightRPCTracking;
				FLightweightRPCTracking::FLightweightRPCEntry& CurEntry = LightweightRPCTracking.RPC[LightweightRPCTracking.Count++];

				CurEntry.Function = Function;
				CurEntry.Name = FunctionName;
			}
		}
	}

	/**
	 * Called after the current NetConnection receives an RPC.
	 */
	void PostReceivedRPC()
	{
#if RPC_DOS_DEV_STATS
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RPCDoS_Checks);
#endif

#if RPC_QUOTA_DEBUG
		{
			FPacketScope& PacketScope = GetPacketScope();

			PacketScope.DebugReceivedRPCEndTime = FPlatformTime::Seconds();
			FrameCounter.DebugAccumRPCTime += (PacketScope.DebugReceivedRPCEndTime - PacketScope.DebugReceivedRPCStartTime);
		}
#endif

		if (bRPCDoSDetection && !bHitchSuspendDetection)
		{
			FTickScope& TickScope = GetTickScope();
			FPacketScope& PacketScope = GetPacketScope();
			FSequentialRPCScope& SequentialRPCScope = GetSequentialRPCScope();

			TickScope.FrameCounter.RPCCounter++;

			RPCIntervalCounter++;
			SequentialRPCScope.ReceivedPacketRPCCount++;
			PacketScope.bPacketContainsRPC = true;

			TickScope.CondCheckCountQuota(*this);
		}
	}

	/**
	 * Called after the same RPC was received once or multiple times sequentially, for optimal timing
	 *
	 * @param SequenceType		The type of sequential RPC call (a sequence stopping mid-packet, or at the end of the packet)
	 * @param TimeSeconds		The current time, in seconds (may be approximate)
	 * @param RPCCounter		Cached pointer to the currently active counter, for this RPC
	 * @param RPCTrackingInfo	Cached pointer to the tracking info, for this RPC
	 */
	void PostSequentialRPC(EPostSequentialRPCType SequenceType, double TimeSeconds, FRPCDoSCounters* RPCCounter,
							FRPCTrackingInfo* RPCTrackingInfo);

	/**
	 * Called after the current NetConnection receives a packet.
	 *
	 * @param TimeSeconds	Cached/approximate timestamp, to save grabbing a new timestamp
	 */
	void PostReceivedPacket(double TimeSeconds)
	{
		using namespace UE::Net;

#if RPC_DOS_DEV_STATS
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RPCDoS_Checks);
#endif

		if (GetPacketScope().bPacketContainsRPC)
		{
			PostReceivedRPCPacket(TimeSeconds);
		}

		ReceivedPacketEndTime = TimeSeconds;

		PacketScopePrivate.SetActive(false);

#if RPC_DOS_SCOPE_DEBUG
		ensure(GRPCDoSScopeDebugging == 0 || !SequentialRPCScopePrivate.IsActive());
		ensure(GRPCDoSScopeDebugging == 0 || TickScopePrivate.IsActive());
#endif
	}

	/**
	 * Called after the current NetConnection receives a packet, when RPC DoS Detection is active and the packet contains an RPC.
	 * NOTE: Timing is approximate - may include multiple-RPC and non-RPC packet processing time - but low cost
	 *
	 * @param TimeSeconds	Cached/approximate timestamp, to save grabbing a new timestamp
	 */
	NETCORE_API void PostReceivedRPCPacket(double TimeSeconds);

	/**
	 * Called after the packet-loop TickDispatch
	 */
	NETCORE_API void PostTickDispatch();

	/**
	 * Triggered upon NetConnection Close (for committing analytics)
	 */
	NETCORE_API void NotifyClose();


	/**
	 * Tracks a blocked RPC for aggregated logging. Below the log threshold, logs individually.
	 * Above the threshold, accumulates counts for per-tick aggregated logging.
	 *
	 * @param RPCName	The name of the RPC that was blocked
	 */
	NETCORE_API void TrackBlockedRPC(FName RPCName);


	/**
	 * Whether or not monitoring of received RPC's (i.e. calling of NotifyReceivedRPC) should be performed
	 *
	 * @return		Whether or not to monitor received RPC's.
	 */
	bool ShouldMonitorReceivedRPC() const
	{
#if RPC_QUOTA_DEBUG
		return true;
#endif

		return bRPCTrackingEnabled;
	}

	/**
	 * Whether or not RPC DoS Detection is enabled
	 */
	bool IsRPCDoSDetectionEnabled() const
	{
		return bRPCDoSDetection;
	}

	/**
	 * Overrides the current AddressFunc
	 *
	 * @param InAddressFunc		The new AddressFunc
	 */
	void SetAddressFunc(FGetRPCDoSAddress&& InAddressFunc);

	/**
	 * Overrides the current PlayerUIDFunc
	 *
	 * @param InPlayerUIDFunc	The new PlayerUIDFunc
	 */
	void SetPlayerUIDFunc(FGetRPCDoSPlayerUID&& InPlayerUIDFunc);

	/**
	 * Overrides the current KickPlayerFunc
	 *
	 * @param InKickPlayerFunc	The new KickPlayerFunc
	 */
	void SetKickPlayerFunc(FRPCDoSKickPlayer&& InKickPlayerFunc);


private:
	/**
	 * Called when we begin receiving the same RPC once or multiple times sequentially.
	 */
	void PreSequentialRPC()
	{
		using namespace UE::Net;

#if RPC_DOS_SCOPE_DEBUG
		ensure(GRPCDoSScopeDebugging == 0 || TickScopePrivate.IsActive());
		ensure(GRPCDoSScopeDebugging == 0 || PacketScopePrivate.IsActive());
#endif

		SequentialRPCScopePrivate.SetActive(true);
	}

	/**
	 * Updates the current RPC DoS detection severity state
	 *
	 * @param Update	Whether or not we are escalating or de-escalating the severity state
	 * @param Reason	The reason for the escalation change
	 */
	NETCORE_API void UpdateSeverity_Private(ERPCDoSSeverityUpdate Update, ERPCDoSEscalateReason Reason);

	/**
	 * Initializes a newly active escalation state
	 *
	 * @param TimeSeconds	Cached/approximate timestamp, to save grabbing a new timestamp
	 */
	void InitState(double TimeSeconds);

	/**
	 * Heavily rate limited RPC count quota checks
	 */
	void CondCheckCountQuota()
	{
		// 64 RPC interval
		if ((RPCIntervalCounter & 0x3F) == 0)
		{
			FTickScope& TickScope = GetTickScope();

			if (HasHitQuota_Count(CounterPerPeriodHistory, TickScope.FrameCounter))
			{
				TickScope.UpdateSeverity(*this, ERPCDoSSeverityUpdate::Escalate, ERPCDoSEscalateReason::CountLimit);
			}
		}
	}

	/**
	 * Heavily rate limited RPC time quota checks
	 *
	 * @param TimeSeconds	Cached/approximate timestamp, to save grabbing a new timestamp
	 */
	void CondCheckTimeQuota(double TimeSeconds)
	{
		if (TimeSeconds > NextTimeQuotaCheck)
		{
			FTickScope& TickScope = GetTickScope();

			NextTimeQuotaCheck = TimeSeconds + TimeQuotaCheckInterval;

			if (HasHitQuota_Time(CounterPerPeriodHistory, TickScope.FrameCounter))
			{
				TickScope.UpdateSeverity(*this, ERPCDoSSeverityUpdate::Escalate, ERPCDoSEscalateReason::TimeLimit);
			}
		}
	}

	/**
	 * When RPC monitoring and tracking are enabled, tracks every RPC received.
	 * WARNING: Potentially expensive! RPC DoS Detection should be configured, so that tracking is rarely enabled.
	 *
	 * @param Function			The UFunction representing the RPC
	 * @param FunctionName		The cached name of the function
	 * @return					Whether or not to block the RPC
	 */
	NETCORE_API ERPCNotifyResult CheckRPCTracking(const UFunction* Function, FName FunctionName);

	/**
	 * Recalculates cached RPC counter history information, after an interval or state change which affects time period tracking.
	 *
	 * @param InTimePeriods				The time period values which need caching (e.g. a period of 2 seconds, and a period of 8 seconds).
	 * @param OutPerPeriodHistory		Outputs the recalculated RPC counter cache, for the specified time periods (e.g. elements 1 and 7 only).
	 * @param StartPerSecHistoryIdx		The index in CounterPerSecHistory to start calculating from
	 */
	void RecalculatePeriodHistory(const TArray<int8>& InTimePeriods, FRPCDoSCounters(&OutPerPeriodHistory)[16],
									int32 StartPerSecHistoryIdx=INDEX_NONE);


	/**
	 * Enables tracking of individual RPC's.
	 *
	 * @param TimeSeconds	Cached/approximate timestamp, to save grabbing a new timestamp
	 */
	void EnableRPCTracking(double TimeSeconds);

	/**
	 * Enables forced tracking of individual RPC's
	 *
	 * @param Function		The function which triggered forced RPC tracking
	 * @param FunctionName	The name of the function which triggered forced RPC tracking
	 * @param TimeSeconds	Cached/approximate timestamp, to save grabbing a new timestamp
	 */
	NETCORE_API void EnableForcedRPCTracking(const UFunction* Function, FName FunctionName, double TimeSeconds);

	/**
	 * Disables tracking of individual RPC's.
	 *
	 * @param TimeSeconds	Cached/approximate timestamp, to save grabbing a new timestamp
	 */
	void DisableRPCTracking(double TimeSeconds);

	/**
	 * Adds or retrieves the specified RPC from tracking
	 *
	 * @param InFunc			The RPC UFunction for tracking
	 * @param bOutNewTracking	Whether or not the RPC was newly added to tracking
	 * @return					Returns the RPC tracking info
	 */
	FRPCTrackingInfo& FindOrAddRPCTracking(const UFunction* InFunc, bool& bOutNewTracking)
	{
		TSharedPtr<FRPCTrackingInfo>& SharedResult = RPCTracking.FindOrAdd(InFunc);

		if (!SharedResult.IsValid())
		{
			SharedResult = MakeShared<FRPCTrackingInfo>();

			ActiveRPCTracking.Add({InFunc, SharedResult.Get()});

			bOutNewTracking = true;
		}

		return *SharedResult.Get();
	}

	/**
	 * Used to periodically clear stale RPC tracking.
	 *
	 * @param TimeSeconds	Cached/approximate timestamp, to save grabbing a new timestamp
	 */
	void ClearStaleRPCTracking(double TimeSeconds);


	/**
	 * Caches the player address, based on AddressFunc
	 */
	void CachePlayerAddress();

	/**
	 * Caches the player UID, based on PlayerUIDFunc
	 */
	void CachePlayerUID();

	/**
	 * Returns the net address for the owning NetConnection.
	 *
	 * @return	The net address for the owning connection.
	 */
	const FString& GetPlayerAddress()
	{
		return CachedAddress;
	}

	/**
	 * Returns the unique id for the owning player.
	 *
	 * @return	The unique id for the owning player.
	 */
	const FString& GetPlayerUID()
	{
		return CachedPlayerUID;
	}


private:
	/** Base class for setting code scope activity */
	template<class T>
	class TScopeBase
	{
	public:
		/** Sets whether or not the current code scope is active */
		inline void SetActive(bool bInVal)
		{
			using namespace UE::Net;

#if RPC_DOS_SCOPE_DEBUG
			ensure(GRPCDoSScopeDebugging == 0 || bScopeActive != bInVal);
#endif

			bScopeActive = bInVal;

			Reset();
		}

		/** Whether or not the current code scope is active */
		inline bool IsActive() const
		{
			return bScopeActive;
		}

		/** Resets the properties covered by this scope, at both the start/end of the scope */
		inline void Reset()
		{
			static_cast<T*>(this)->Reset();
		}

	private:
		/** Whether or not the current code scope is active */
		bool bScopeActive = false;
	};

	/** Variables and functions that should only be accessible during TickDispatch */
	class FTickScope : public TScopeBase<FTickScope>
	{
	public:
		/** Wrapper for UpdateSeverity which forces FTickScope acquisition */
		inline void UpdateSeverity(FRPCDoSDetection& This, ERPCDoSSeverityUpdate Update, ERPCDoSEscalateReason Reason)
		{
			This.UpdateSeverity_Private(Update, Reason);
		}

		/** Wrapper for CondCheckCountQuota which forces FTickScope acquisition */
		inline void CondCheckCountQuota(FRPCDoSDetection& This)
		{
			This.CondCheckCountQuota();
		}

		/** Wrapper for CondCheckTimeQuota which forces FTickScope acquisition */
		inline void CondCheckTimeQuota(FRPCDoSDetection& This, double TimeSeconds)
		{
			This.CondCheckTimeQuota(TimeSeconds);
		}

		void Reset()
		{
			FrameCounter.ResetRPCCounters();
		}

	public:
		/** Per-frame RPC counting (multiple disjointed packets may be processed for same connection, during a frame) */
		FRPCDoSCounters FrameCounter;
	};

	/** Scoped variables/functions accessible during TickDispatch */
	FTickScope TickScopePrivate;


	/** Gets a reference to the TickDispatch scoped variable/function accessor */
	inline FTickScope& GetTickScope()
	{
		using namespace UE::Net;

#if RPC_DOS_SCOPE_DEBUG
		ensure(GRPCDoSScopeDebugging == 0 || TickScopePrivate.IsActive());
#endif

		return TickScopePrivate;
	}


	/** Variables and functions that should only be accessible while receiving an individual packet */
	class FPacketScope : public TScopeBase<FPacketScope>
	{
	public:
		/** Whether or not the current packet being received, contains an RPC */
		bool bPacketContainsRPC											= false;

		/** Cached free/external timestamp, for when the current received packet began processing */
		double ReceivedPacketStartTime									= 0.0;


	public:
		void Reset()
		{
			bPacketContainsRPC = false;
			ReceivedPacketStartTime = 0.0;
		}
	};

	/** Scoped variables/functions accessible while receiving an individual packet */
	FPacketScope PacketScopePrivate;


	/** Gets a reference to the packet receive scoped variable/function accessor */
	inline FPacketScope& GetPacketScope()
	{
		using namespace UE::Net;

#if RPC_DOS_SCOPE_DEBUG
		ensure(GRPCDoSScopeDebugging == 0 || PacketScopePrivate.IsActive());
#endif

		return PacketScopePrivate;
	}

	/** Variables and functions that should only be accessible while receiving the same RPC individually/sequentially */
	class FSequentialRPCScope : public TScopeBase<FSequentialRPCScope>
	{
	public:
		/** Wrapper for CondCheckCountQuota which forces FSequentialRPCScope acquisition */
		inline void CondCheckCountQuota(FRPCDoSDetection& This)
		{
			This.CondCheckCountQuota();
		}

		/** Wrapper for CondCheckTimeQuota which forces FSequentialRPCScope acquisition */
		inline void CondCheckTimeQuota(FRPCDoSDetection& This, double TimeSeconds)
		{
			This.CondCheckTimeQuota(TimeSeconds);
		}

		/** Wrapper for CheckRPCTracking which forces FSequentialRPCScope acquisition */
		inline ERPCNotifyResult CheckRPCTracking(FRPCDoSDetection& This, const UFunction* Function, FName FunctionName)
		{
			return This.CheckRPCTracking(Function, FunctionName);
		}

		/** Wrapper for FindOrAddRPCTracking which forces FSequentialRPCScope acquisition */
		inline FRPCTrackingInfo& FindOrAddRPCTracking(FRPCDoSDetection& This, const UFunction* InFunc, bool& bOutNewTracking)
		{
			return This.FindOrAddRPCTracking(InFunc, bOutNewTracking);
		}

		void Reset()
		{
			bReceivedPacketRPCUnique = true;
			ReceivedPacketRPCCount = 0;
			LightweightRPCTracking.Count = 0;
			PostReceivedRPCName = NAME_None;
			PostReceivedRPCTracking = nullptr;
			PostReceivedRPCCounter = nullptr;
			PostReceivedRPCBlockCount = 0;
			LastReceivedRPCTimeCache = 0.0;

#if RPC_QUOTA_DEBUG
			DebugReceivedRPCStartTime = 0.0;
			DebugReceivedRPCEndTime = 0.0;
#endif
		}

	public:
		/** Whether or not the packet only contains references to the same unique RPC (may be multiple RPC calls to same RPC) */
		bool bReceivedPacketRPCUnique									= true;

		/** Counts the number of RPC's in the current packet, for unique-RPC analytics */
		int32 ReceivedPacketRPCCount									= 0;

		/** State for lightweight tracking of RPC's */
		FLightweightRPCTracking LightweightRPCTracking;


		/** Temporarily caches the currently active RPC name, while processing a packet */
		FName PostReceivedRPCName;

		/** Temporarily caches the currently active RPC tracking, while processing a packet */
		FRPCTrackingInfo* PostReceivedRPCTracking						= nullptr;

		/** Temporarily caches the currently active RPC counter, while processing a packet */
		FRPCDoSCounters* PostReceivedRPCCounter							= nullptr;

		/** Temporarily caches the number of blocked RPC calls for the current RPC, while processing a packet */
		int32 PostReceivedRPCBlockCount									= 0;

		/** Temporarily caches timestamps for sequentially executed RPC's, to minimize new timestamp generation. */
		double LastReceivedRPCTimeCache									= 0.0;

#if RPC_QUOTA_DEBUG
		/** Direct/accurate timestamp of RPC receive start time, for debugging against approximate RPC timestamps */
		double DebugReceivedRPCStartTime								= 0.0;

		/** Direct/accurate timestamp of RPC receive end time, for debugging against approximate RPC timestamps */
		double DebugReceivedRPCEndTime									= 0.0;
#endif
	};

	/** Scoped variables/functions accessible while receiving the same RPC individually/sequentially */
	FSequentialRPCScope SequentialRPCScopePrivate;

	/** Gets a reference to the sequential RPC scoped variable/function accessor */
	inline FSequentialRPCScope& GetSequentialRPCScope()
	{
		using namespace UE::Net;

#if RPC_DOS_SCOPE_DEBUG
		ensure(GRPCDoSScopeDebugging == 0 || SequentialRPCScopePrivate.IsActive());
#endif

		return SequentialRPCScopePrivate;
	}


private:
	/** Whether or not RPC DoS detection is presently enabled */
	bool bRPCDoSDetection											= false;

	/** Whether or not analytics for RPC DoS detection is enabled */
	bool bRPCDoSAnalytics											= false;

	/** The amount of time since the previous frame, for detecting hitches, to prevent false positives from built-up packets */
	int32 HitchTimeQuotaMS											= 0;

	/** The amount of time to suspend RPC DoS Detection, once a hitch is encountered, to prevent false positives from built-up packets */
	int32 HitchSuspendDetectionTimeMS								= 0;

	/** List of RPC's which should never be blocked */
	TArray<FName> RPCBlockAllowList;

	/** If the related CVar is set, the name of the RPC which should forcibly enable tracking. */
	FName ForcedRPCTracking;

	/** Specifies the chance (between 0.0 and 1.0) that tracking will be enabled, when encountering the ForcedRPCTracking RPC */
	double ForcedRPCTrackingChance									= 1.0;

	/** Specifies the length of time forced RPC tracking should be enabled (disabled the next tick, if 0.0) */
	double ForcedRPCTrackingLength									= 0.0;

	/** Specifies the time at which forced RPC tracking should end, if tracking isn't enabled by the active state */
	double ForcedRPCTrackingEndTime									= 0.0;


	/** Callback used for getting the address of the NetConnection the RPC DoS Detection is associated with */
	FGetRPCDoSAddress AddressFunc;

	/** Player IP cached from AddressFunc */
	FString CachedAddress;

	/** Callback used for getting the unique id of the NetConnection the RPC DoS Detection is associated with */
	FGetRPCDoSPlayerUID PlayerUIDFunc;

	/** Player UID cached from PlayerUIDFunc */
	FString CachedPlayerUID;

	/** Callback used for kicking the player after a bad enough RPC DoS */
	FRPCDoSKickPlayer KickPlayerFunc;


	/** The different RPC DoS detection states, of escalating severity, depending on the amount of RPC spam */
	TArray<FRPCDoSStateConfig> DetectionSeverity;

	/** The currently active RPC DoS severity state settings */
	int8 ActiveState												= 0;

	/** The worst RPC DoS severity state that has been active */
	int8 WorstActiveState											= 0;

	/** The worst RPC DoS severity state that has been active and passed confirmation thresholds - used for limiting analytics events */
	int8 WorstAnalyticsState										= 0;

	/** The last time the previous severity states escalation conditions were met (to prevent bouncing up/down between states) */
	double LastMetEscalationConditions								= 0.0;

	/** Whether or not RPC DoS Detection has been suspended, due to a hitch impairing the accuracy of measurements */
	bool bHitchSuspendDetection										= false;

	/** Timestamp for when RPC DoS Detection was suspended, due to a hitch */
	double HitchSuspendDetectionStartTime							= 0.0;


	/** Per-second RPC counting (approximate - may cover more than one second, in the case of expensive/long-running RPCs) */
	FRPCDoSCounters SecondCounter;

	/** Timestamp for the last time per-second quota counting began */
	double LastPerSecQuotaBegin										= 0.0;

	/** Stores enough per second quota history, to allow all DetectionSeverity states to recalculate if their CooloffTime is reached */
	TArray<FRPCDoSCounters> CounterPerSecHistory;

	/** The last written index of CounterPerSecHistory */
	int32 LastCounterPerSecHistoryIdx								= 0;

	/** Counter history which is precalculated/cached from CounterPerSecHistory, for tracking per-period time (up to a maximum of 16s) */
	FRPCDoSCounters CounterPerPeriodHistory[16]						= {};

	/** Wrapping count of seconds passed, for RPC tracking */
	uint8 SecondsIncrementer										= 0;


	/** Cached free/external timestamp, for when the last received packet finished processing */
	double ReceivedPacketEndTime									= 0.0;

	/** Cached PreTickDispatch timestamp, to reuse for minimizing timestamp retrieval */
	double LastPreTickDispatchTime									= 0.0;

	/** Unbounded RPC counter, for performing RPC checks at every 'x' intervals */
	uint32 RPCIntervalCounter										= 0;

	/** Timestamp for when time quota's should next be checked */
	double NextTimeQuotaCheck										= 0.0;

	/** Interval for time quota checks, in seconds */
	static constexpr double TimeQuotaCheckInterval					= 0.005;


	/** Whether or not RPC Tracking has been enabled by RPC DoS escalation or other checks */
	bool bRPCTrackingEnabled										= false;

	/** The runtime maximum number of seconds of RPC tracking that is used */
	int8 MaxRPCTrackingPeriod										= 0;

	/** Maps UFunction pointers to live function tracking structs (NOTE: Treats the UFunction's as potentially invalid/GC'd, hence void*) */
	TMap<const void*, TSharedPtr<FRPCTrackingInfo>> RPCTracking;

	/** Stores a reference to active/valid RPCTracking entries, for faster cleanup - as the above map stores values sparsely. */
	TArray<FActiveRPCTrackingInfo> ActiveRPCTracking;

	/** Default size for ActiveRPCTracking, to minimize realloc */
	static constexpr int32 DefaultActiveRPCTrackingSize				= 128;

	/** The last time RPCTracking was cleaned out */
	double LastRPCTrackingClean										= 0.0;


	/** The locally cached/updated analytics variables, for the RPC DoS Detection - aggregated upon connection Close */
	FRPCDoSAnalyticsVars AnalyticsVars;

	/** The net analytics data holder for the RPC DoS analytics, which is where analytics variables are aggregated upon Close */
	TNetAnalyticsDataPtr<FRPCDoSAnalyticsData> RPCDoSAnalyticsData;

	/** The minimum RPC time per second threshold, for doing analytics updates/checks */
	double AnalyticsMinTimePerSecThreshold							= 0.0;

	/** The last time the 'AnalyticsMinTimePerSecThreshold' was recalculated */
	double LastAnalyticsThresholdRecalc								= 0.0;


	/**
	 * Flushes aggregated blocked RPC log data. Called once per tick from PostTickDispatch.
	 * If the per-tick log threshold was reached, prints one aggregated log line and resets counts (not map keys).
	 */
	void FlushBlockedRPCLogs();

	/**
	 * Fully resets blocked RPC tracking state, including wiping the map. Called on cooldown/de-escalation.
	 */
	void ResetBlockedRPCTracking();


	/** Accumulated blocked RPC counts, keyed by RPC name. Counts reset per-tick on flush; map wiped on de-escalation. */
	TMap<FName, int32> BlockedRPCCounts;

	/** Cached RPC name for fast-path blocked RPC tracking (avoids map lookup when the same RPC is blocked repeatedly) */
	FName CachedBlockedRPCName;

	/** Cached pointer to the count in BlockedRPCCounts for CachedBlockedRPCName */
	int32* CachedBlockedRPCCount									= nullptr;

	/** Number of individual blocked RPC log messages emitted this tick */
	int32 BlockedRPCLogCounter										= 0;
};

