// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/EnableIf.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StructOnScope.h"
#include "UObject/UObjectGlobals.h"

#include "StateStruct.generated.h"

class UClass;
class UStruct;


/**
 * State Structs
 *
 * State structs implement an extensible way to use UPROPERTY's to set up 'state' variables in a USTRUCT,
 * define many different named 'states' and set their default values (with the easy ability to override their values from the .ini file),
 * and apply named states to a 'live'/active instance of the state struct (using UPROPERTY reflection, so that it's mostly automatic).
 *
 * This has many of the benefits of UCLASS reflection and configuration, without the overhead - and implements more intuitive .ini configuration.
 *
 *
 * State Struct Definition
 *
 * To define a custom state struct, create a new USTRUCT extending from FStateStruct, and implement state variables as UPROPERTY() variables.
 * To make variables configurable from the .ini, use UPROPERTY(config).
 *
 * Override 'InitConfigDefaultsInternal' to set default values for each state (StateName specifies the current state).
 *
 *
 * State Struct Implementation
 *
 * To implement loading for a custom state struct, extend from 'UStatePerObjectConfig' and define a UPROPERTY for the state names, e.g:
 *		UPROPERTY(config)
 *		TArray<FString> EscalationSeverity;
 *
 * Then create a variable to hold the state defaults ('FEscalationState' here is a subclass of 'FStateStruct'):
 *		TArray<TStructOnScope<FEscalationState>> EscalationSeverityState;
 *
 * Override 'LoadStateConfig' and use 'RegisterStateConfig' to link the state names to the state defaults (which will automatically be populated):
 *		RegisterStateConfig(EscalationSeverity, EscalationSeverityState);
 *
 * Override 'InitConfigDefaultsInternal' to set default values for the state names, and to set bEnabled to true (if desired by default).
 *
 *
 * State Struct Usage
 *
 * To use state structs, call and store the result of 'UStatePerObjectConfig::Get', specifying the configuration settings and classes to use, e.g:
 *		UEscalationManagerConfig* BaseConfig = CastChecked<UEscalationManagerConfig>(UStatePerObjectConfig::Get(
 *			{TEXT("ConfigSection"), TEXT("ConfigContext"), UEscalationManagerConfig::StaticClass(), FEscalationState::StaticStruct()}));
 *
 * The resulting UStatePerObjectConfig subclass object will contain all of the default states as set by 'RegisterStateConfig',
 * and 'ApplyState' can be used to apply default states, to an active state instance.
 *
 * To define an active state instance in an arbitrary class, specify:
 *		TStructOnScopeLite<FEscalationState> State;
 *
 * To apply a default state to an active state instance, call:
 *		UStatePerObjectConfig::ApplyState(BaseConfig->EscalationSeverityState[StateIdx], State.Get());
 *
 * NOTE: It's safe to store the config UObject without worrying about garbage collection, as there will only be one instance and it's added to root.
 *
 *
 * Configuration and Defaults
 *
 * Default/hardcoded values are implemented for UStatePerObjectConfig and FStateStruct by overriding 'InitConfigDefaultsInternal'.
 * In both cases, .ini settings can be specified, which override the hardcoded defaults.
 *
 * State structs have non-standard ini configuration, with the following components for .ini section names:
 *	- ConfigSection:	The base configuration section name, shared with all related state struct config sections (e.g. NetFault, RPCDoSDetection)
 *	- ConfigContext:	Additional context for modifying which UStatePerObjectConfig section is loaded (e.g. GameNetDriver/BeaconNetDriver)
 *	- StateName:		For the configuration of individual states, this specifies the state name
 *
 * The main UStatePerObjectConfig .ini section follows this format:
 *		[ConfigContext ConfigSection]
 *
 *		Or if there is no ConfigContext specified:
 *		[ConfigSection]
 *
 * The configuration for individual FStateStruct states follows this format:
 *		[ConfigSection.StateName]
 */


/**
 * The result of a call to InitConfigDefaults
 */
enum class EInitStateDefaultsResult : uint8
{
	Initialized,		// Default values have been set for the struct
	NotInitialized		// Default values have not been set for the struct
};


/**
 * Parameters for creating/getting and initializing a new UStatePerObjectConfig instance
 */
struct FStateConfigParms
{
	/** The base configuration section name to use (not the full/complete section name) */
	const TCHAR* ConfigSection	= nullptr;

	/** Additional context for the configuration section (e.g. the NetDriver name, if configuration differs between NetDriver's) */
	FString ConfigContext;

	/** The UStatePerObjectConfig subclass which will be created */
	UClass* ConfigClass			= nullptr;

	/** The FStateStruct subclass struct object (StaticStruct), which will be used for initializing new states */
	UStruct* StateStruct		= nullptr;
};


/**
 * Base struct used for states, which is subclassed to define/implement custom states.
 */
USTRUCT()
struct FStateStruct
{
	friend class UStatePerObjectConfig;

	GENERATED_BODY()

private:
	/** Cached/runtime config values */

	/** The name of the state this config section represents (valid/usable during InitConfigDefaults) */
	UPROPERTY()
	FString StateName;


public:
	virtual ~FStateStruct() = default;

	/**
	 * Returns the name of this state
	 *
	 * @return	The name of this state
	 */
	NETCORE_API FString GetStateName() const;


private:
	/**
	 * Initializes the default settings for this state. Overridden by config settings.
	 *
	 * @return	Whether or not default settings were set.
	 */
	EInitStateDefaultsResult InitConfigDefaults()
	{
		return InitConfigDefaultsInternal();
	}

	/**
	 * Some configuration values are implicitly enabled by other configuration values - this function applies them
	 */
	void ApplyImpliedValues()
	{
		ApplyImpliedValuesInternal();
	}

	/**
	 * Validates loaded struct config variables
	 */
	void ValidateConfig()
	{
		ValidateConfigInternal();
	}


protected:
	/** To be implemented by subclasses */

	virtual EInitStateDefaultsResult InitConfigDefaultsInternal()
	{
		return EInitStateDefaultsResult::NotInitialized;
	}

	virtual void ApplyImpliedValuesInternal()
	{
	}

	virtual void ValidateConfigInternal()
	{
	}
};


/**
 * Base class for loading and initializing state configuration
 */
UCLASS(config=Engine, PerObjectConfig, MinimalAPI)
class UStatePerObjectConfig : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Singleton for getting a named instance of this config section/object
	 *
	 * @param ConfigParms	Specifies the parameters for the config section/object to get/create, and how it should be created
	 * @return				Returns a cached or newly created instance of the config section/object
	 */
	static NETCORE_API const UStatePerObjectConfig* Get(FStateConfigParms ConfigParms);

	/**
	 * Applies the specified config state, to an object implementing FStateStruct's.
	 * NOTE: Reflection does not work with multiple inheritance.
	 *
	 * @param ConfigState	The configuration state containing the settings to be applied
	 * @param TargetState	The object implementing FStateStruct, to apply the config settings to.
	 */
	static NETCORE_API void ApplyState(const FStructOnScope& ConfigState, FStateStruct* TargetState);


#if !UE_BUILD_SHIPPING
	/**
	 * Debug function which prints the entire state configuration to the log
	 */
	NETCORE_API virtual void DebugDump() const;
#endif


protected:
	/**
	 * Registers an array of state names to an array of state instances (of a type derived from FStateStruct).
	 * Should be called during LoadStateConfig in subclasses.
	 *
	 * @param StateNames	The (usually configurable) array of state names defining the state instances to be created
	 * @param OutStates		The array of state instances to be initialized based on StateNames
	 */
	template<typename U, typename = typename TEnableIf<TIsDerivedFrom<U, FStateStruct>::IsDerived, void>::Type>
	void RegisterStateConfig(const TArray<FString>& StateNames, TArray<TStructOnScope<U>>& OutStates)
	{
		RegisterStateConfig(StateNames, reinterpret_cast<TArray<TStructOnScope<FStateStruct>>&>(OutStates));
	}


private:
	/**
	 * Implement in subclasses to call RegisterStateConfig, in order to load states
	 */
	virtual void LoadStateConfig()
	{
	}

	/**
	 * Initializes the default settings for this class. Overridden by config settings.
	 */
	void InitConfigDefaults()
	{
		InitConfigDefaultsInternal();
	}

	/**
	 * Implement in subclasses to initialize the default settings for this class. Overridden by config settings.
	 */
	virtual void InitConfigDefaultsInternal()
	{
	}


	NETCORE_API void RegisterStateConfig(const TArray<FString>& StateNames, TArray<TStructOnScope<FStateStruct>>& OutStates);

	NETCORE_API virtual void OverridePerObjectConfigSection(FString& SectionName) override;


	/**
	 * Internal singleton for getting the custom archetype/default-object instance for a config section/object (to allow custom default values)
	 *
	 * @param ConfigParms		Specifies the parameters for the config section/object to get/create, and how it should be created
	 * @param FullSection		The full section name used by the config section/object
	 * @param FormattedName		The base formatted object name to use, for unique/singleton object lookup/caching
	 * @return					Returns a cached or newly created instance of the config section/object archetype
	 */
	static NETCORE_API UStatePerObjectConfig* GetArchetype(FStateConfigParms ConfigParms, FString FullSection, FString FormattedName);

	/**
	 * Uses reflection to load all struct config variables from the specified ini section.
	 * NOTE: Reflection does not work with multiple inheritance.
	 *
	 * @param OutStruct		The struct to load the settings into
	 * @param SectionName	The ini section name containing the struct configuration
	 * @param InFilename	The ini filename to read from
	 * @return				Whether or not the struct config variables were read successfully
	 */
	static NETCORE_API bool LoadStructConfig(FStructOnScope& OutStruct, const TCHAR* SectionName, const TCHAR* InFilename=nullptr);


private:
#if !UE_BUILD_SHIPPING
	struct FStateConfigRegister
	{
		const TArray<FString>&					StateNames;
		TArray<TStructOnScope<FStateStruct>>&	States;
	};
#endif

private:
	/** Cached parameters that were used to create this instance */
	FStateConfigParms ConfigParms;

#if !UE_BUILD_SHIPPING
	/** Stores a list of state name arrays registered to state instance arrays, to allow debug dumping of values */
	TArray<FStateConfigRegister> RegisteredStateConfigs;
#endif

	/** Overrides the config section name, using the CDO */
	UPROPERTY()
	FString PerObjectConfigSection;

public:
	/** Whether or not this state configuration instance is enabled (states will not load, if not) */
	UPROPERTY(config)
	bool bEnabled;
};

