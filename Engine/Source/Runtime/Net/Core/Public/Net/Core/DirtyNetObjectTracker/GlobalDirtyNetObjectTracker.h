// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/NetBitArray.h"
#include "Net/Core/NetHandle/NetHandle.h"
#include "Containers/ContainersFwd.h"

class FNetCoreModule;

namespace UE::Net
{

void MarkNetObjectStateDirty(FNetHandle NetHandle, int32 StartRepIndex, int32 EndRepIndex);

class FGlobalDirtyNetObjectTracker
{
public:
	struct FPollHandle
	{
	public:
		FPollHandle() = default;
		FPollHandle(FPollHandle&&);
		~FPollHandle();

		FPollHandle& operator=(FPollHandle&&);

		FPollHandle(const FPollHandle&) = delete;
		FPollHandle& operator=(const FPollHandle&) = delete;

		bool IsValid() const;

		/** Destroy the handle, making it invalid in the process. */
		void Destroy();

	private:
		friend FGlobalDirtyNetObjectTracker;

		enum : uint32
		{
			InvalidIndex = ~0U,
		};

		explicit FPollHandle(uint32 InIndex);

		uint32 Index = InvalidIndex;
	};

	using FDirtyPropertyStorage = TArray<FNetBitArrayBase::StorageWordType, TInlineAllocator<4>>;
	using FDirtyHandleAndPropertyMap = TMap<FNetHandle, FDirtyPropertyStorage>;

public:
	/** Returns true if we are tracking dirtiness per property. */
	NETCORE_API static bool IsUsingPerPropertyDirtyTracking();

	/** Marks an object as dirty. Assumes Init() has been called. */
	NETCORE_API static void MarkNetObjectStateDirty(FNetHandle, const int32, const int32);

	/** Marks an object as dirty. Assumes Init() has been called. */
	inline static void MarkNetObjectStateDirty(FNetHandle Handle) 
	{
		MarkNetObjectStateDirty(Handle, INDEX_NONE, INDEX_NONE);
	}

	/** Delegate called on pollers that haven't gathered dirty state before it resets */
	DECLARE_DELEGATE(FPreResetDelegate);

	/** Create a poller which is assumed to call PollDirtyNetObjects(). Assumes Init() has been called. */
	NETCORE_API static FPollHandle CreatePoller(FPreResetDelegate InPreResetDelegate);

	/**
	 * Returns the objects with dirty state. The reference can only be assumed to be valid until the next call to ResetDirtyNetObjects().
	 * A matching call to ResetDirtyNetObjects() is required but should not be called until all pollers have had the chance to call GetDirtyNetObjects().
	 * Assumes Init() has been called.
	 */
	NETCORE_API static const TSet<FNetHandle>& GetDirtyNetObjects(const FPollHandle& Handle);

	/**
	 * Returns the objects with dirty state including per property dirtiness. The reference can only be assumed to be valid until the next call to ResetDirtyNetObjects().
	 * A matching call to ResetDirtyNetObjects() is required but should not be called until all pollers have had the chance to call GetDirtyNetObjects().
	 * Assumes Init() has been called.
	 */
	NETCORE_API static const FDirtyHandleAndPropertyMap& GetDirtyNetObjectsAndProperties(const FPollHandle& Handle);

	/**
	 * Detect any updates to the dirty list until all pollers have gathered and reset the list.
	 */
	NETCORE_API static void LockDirtyListUntilReset(const FPollHandle& Handle);

	/**
	 * Clears the set of dirty net objects. GetDirtyNetObjects() must be called first. Some synchronization is needed between pollers such that all pollers have the chance to call GetDirtyNetObjects() first.
	 * It may be a good idea to have the polling done in a NetDriver TickFlush() or similar and resetting done in PostTickFlush().
	 * Assumes Init() has been called.
	 */
	NETCORE_API static void ResetDirtyNetObjects(const FPollHandle&);

	/**
	 * Reset the list of dirty net objects but only if there is a single poller registered in the system.
	 * Returns true if the reset was executed.
	 */
	NETCORE_API static bool ResetDirtyNetObjectsIfSinglePoller(const FPollHandle&);

protected:
	friend FNetCoreModule;

	/** If push model is compiled in Init() creates a new instance that is used by functions that need it. Checks that no instance exists. */
	static void Init();

	/** Destroys the instance if it exists. */
	static void Deinit();

private:
	/** Called by NetPollHandle destructor. */
	NETCORE_API static void DestroyPoller(uint32 HandleIndex);

	class FPimpl;
	static FPimpl* Instance;
};

inline void MarkNetObjectStateDirty(FNetHandle NetHandle)
{
	FGlobalDirtyNetObjectTracker::MarkNetObjectStateDirty(NetHandle, INDEX_NONE, INDEX_NONE);
}

inline void MarkNetObjectStateDirty(FNetHandle NetHandle, int32 StartRepIndex, int32 EndRepIndex)
{
	FGlobalDirtyNetObjectTracker::MarkNetObjectStateDirty(NetHandle, StartRepIndex, EndRepIndex);
}

inline FGlobalDirtyNetObjectTracker::FPollHandle::FPollHandle(uint32 InIndex)
: Index(InIndex) 
{
}

inline FGlobalDirtyNetObjectTracker::FPollHandle::FPollHandle(FPollHandle&& Other)
{
	Index = Other.Index;
	Other.Index = InvalidIndex;
}

inline FGlobalDirtyNetObjectTracker::FPollHandle& FGlobalDirtyNetObjectTracker::FPollHandle::operator=(FPollHandle&& Other)
{
	if (IsValid())
	{
		FGlobalDirtyNetObjectTracker::DestroyPoller(Index);
	}

	Index = Other.Index;
	Other.Index = InvalidIndex;

	return *this;
}

inline FGlobalDirtyNetObjectTracker::FPollHandle::~FPollHandle()
{
	Destroy();
}

inline bool FGlobalDirtyNetObjectTracker::FPollHandle::IsValid() const
{
	return Index != InvalidIndex;
}

inline void FGlobalDirtyNetObjectTracker::FPollHandle::Destroy()
{
	if (IsValid())
	{
		FGlobalDirtyNetObjectTracker::DestroyPoller(Index);
		Index = InvalidIndex;
	}
}

}
