// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformTime.h"


// Delegates

/**
 * Delegate for allowing analytics to receive notification of detected DDoS attacks
 */
DECLARE_DELEGATE_OneParam(FDDoSSeverityEscalation, FString /*SeverityCategory*/);



/**
 * Struct containing the per-second packet counters
 */
struct FDDoSPacketCounters
{
	/** Counter for non-NetConnection packets received, since the last per second quota period began */
	int32 NonConnPacketCounter;

	/** Counter for NetConnection packets received, since the last per second quota period began */
	int32 NetConnPacketCounter;

	/** Counter for recently-disconnected NetConnection packets received, since the last per second quota period beegan */
	int32 DisconnPacketCounter;

	/** Counter for bad non-NetConnection packets received, since the last per second quota period began */
	int32 BadPacketCounter;

	/** Counter for non-DDoS packet errors received */
	int32 ErrorPacketCounter;

	/** Counter for the number of packets (of any type) dropped, since the last per second quota period began */
	int32 DroppedPacketCounter;

	/** The worst per-frame packet receive time in milliseconds, over the ~1 second this packet counter history represents */
	int32 WorstFrameReceiveTimeMS;


	FDDoSPacketCounters()
		: NonConnPacketCounter(0)
		, NetConnPacketCounter(0)
		, DisconnPacketCounter(0)
		, BadPacketCounter(0)
		, ErrorPacketCounter(0)
		, DroppedPacketCounter(0)
		, WorstFrameReceiveTimeMS(0)
	{
	}
};


/**
 * Stores the DDoS detection state (either settings from the config file, or the active DDoS detection state)
 */
struct FDDoSState
{
	/** Whether or not to send analytics when escalating to this state */
	bool bSendEscalateAnalytics;

	/** The number of packets/sec before the next stage of DDoS detection is triggered */
	int32 EscalateQuotaPacketsPerSec;

	/** The number of recently disconnected NetConnection packets/sec, before the next stage of DDoS detection is triggered. */
	int32 EscalateQuotaDisconnPacketsPerSec;

	/** The number of bad (failed to process correctly) packets/sec, before the next stage of DDoS detection is triggered */
	int32 EscalateQuotaBadPacketsPerSec;

	/** The amount of time spent processing packets, before the next stage of DDoS detection is triggered */
	int16 EscalateTimeQuotaMSPerFrame;

	/** The limit for the number of non-NetConnection packets to process, each frame */
	int32 PacketLimitPerFrame;

	/** The limit for time spent processing non-NetConnection packets, each frame (counts all packets time, non-NetConn and NetConn) */
	int32 PacketTimeLimitMSPerFrame;

	/** The limit for time spent processing NetConnection packets, each frame (counts all packets time, non-NetConn and NetConn) */
	int32 NetConnPacketTimeLimitMSPerFrame;

	/** The amount of time, in seconds, before the current DDoS severity category cools off and de-escalates */
	int32 CooloffTime;


	FDDoSState()
		: bSendEscalateAnalytics(true)
		, EscalateQuotaPacketsPerSec(-1)
		, EscalateQuotaDisconnPacketsPerSec(-1)
		, EscalateQuotaBadPacketsPerSec(-1)
		, EscalateTimeQuotaMSPerFrame(-1)
		, PacketLimitPerFrame(-1)
		, PacketTimeLimitMSPerFrame(-1)
		, NetConnPacketTimeLimitMSPerFrame(-1)
		, CooloffTime(-1)
	{
	}

	/**
	 * Whether or not the specified counters and time passed has hit any of the quota's
	 *
	 * @param InCounter		The counters to check against
	 * @param TimePassedMS	The amount of time passed in milliseconds, since the beginning of this state
	 * @return				Whether or not the quota was hit
	 */
	bool HasHitQuota(FDDoSPacketCounters& InCounter, int32 TimePassedMS) const
	{
		const bool bAtQuota = EscalateQuotaPacketsPerSec > 0 && InCounter.NonConnPacketCounter >= EscalateQuotaPacketsPerSec;
		const bool bAtDisconnQuota = EscalateQuotaDisconnPacketsPerSec > 0 && InCounter.DisconnPacketCounter >= EscalateQuotaDisconnPacketsPerSec;
		const bool bAtBadQuota = EscalateQuotaBadPacketsPerSec > 0 && InCounter.BadPacketCounter >= EscalateQuotaBadPacketsPerSec;
		const bool bAtTimeQuota = EscalateTimeQuotaMSPerFrame > 0 && TimePassedMS > EscalateTimeQuotaMSPerFrame;

		return bAtQuota || bAtDisconnQuota || bAtBadQuota || bAtTimeQuota;
	}
};

/**
 * DDoS detection state, with functions for applying the state to active DDoS detection
 */
struct FDDoSStateConfig : public FDDoSState
{
	/** The name of the DDoS severity level this config section represents */
	FString SeverityCategory;


	FDDoSStateConfig()
		: SeverityCategory(TEXT(""))
	{
	}

	void ApplyState(FDDoSState& Target)
	{
		Target.bSendEscalateAnalytics				= bSendEscalateAnalytics;
		Target.EscalateQuotaPacketsPerSec			= EscalateQuotaPacketsPerSec;
		Target.EscalateQuotaDisconnPacketsPerSec	= EscalateQuotaDisconnPacketsPerSec;
		Target.EscalateQuotaBadPacketsPerSec		= EscalateQuotaBadPacketsPerSec;
		Target.EscalateTimeQuotaMSPerFrame			= EscalateTimeQuotaMSPerFrame;
		Target.PacketLimitPerFrame					= PacketLimitPerFrame;
		Target.PacketTimeLimitMSPerFrame			= PacketTimeLimitMSPerFrame;
		Target.NetConnPacketTimeLimitMSPerFrame		= NetConnPacketTimeLimitMSPerFrame;
		Target.CooloffTime							= CooloffTime;
	}

	/**
	 * Applies only the per-frame adjusted state (based on expected vs actual framerate), to active DDoS protection.
	 * ApplyState should be called, first.
	 */
	void ApplyAdjustedState(FDDoSState& Target, float FrameAdjustment)
	{
		// Exclude escalation triggers from this
		//Target.EscalateTimeQuotaMSPerFrame		= (EscalateTimeQuotaMSPerFrame == -1 ? -1 : EscalateTimeQuotaMSPerFrame * FrameAdjustment);
		Target.PacketLimitPerFrame				= (PacketLimitPerFrame == -1 ? -1 : (int32)((float)PacketLimitPerFrame * FrameAdjustment));
		Target.PacketTimeLimitMSPerFrame		= (PacketTimeLimitMSPerFrame == -1 ? -1 : (int32)((float)PacketTimeLimitMSPerFrame * FrameAdjustment));
		Target.NetConnPacketTimeLimitMSPerFrame	= (NetConnPacketTimeLimitMSPerFrame == -1 ? -1 : (int32)((float)NetConnPacketTimeLimitMSPerFrame * FrameAdjustment));
	}
};


/**
 * The main DDoS detection tracking class, for counting packets and applying restrictions.
 * Implemented separate to the NetDriver, to allow wider use e.g. potentially at socket level, if useful.
 */
class FDDoSDetection : protected FDDoSPacketCounters, protected FDDoSState
{
public:
	/**
	 * Default constructor
	 */
	NETCORE_API FDDoSDetection();


	/**
	 * Initializes the DDoS detection settings
	 *
	 * @param MaxTickRate	The maximum tick rate of the server
	 */
	NETCORE_API void Init(int32 MaxTickRate);

	/**
	 * Initializes the settings from the .ini file - must support reloading of settings on-the-fly
	 */
	NETCORE_API void InitConfig();

	/**
	* Initialize the expected tick rate.
	*/
	NETCORE_API void SetMaxTickRate(int32 MaxTickRate);

	/**
	 * Updates the current DDoS detection severity state
	 *
	 * @param bEscalate		Whether or not we are escalating or de-escalating the severity state
	 */
	NETCORE_API void UpdateSeverity(bool bEscalate);


	/**
	 * Triggered before packet receive begins, during the current frame
	 */
	NETCORE_API void PreFrameReceive(float DeltaTime);

	/**
	 * Triggered after packet receive ends, during the current frame
	 */
	NETCORE_API void PostFrameReceive();


	/**
	 * Rate limited call to CheckNonConnQuotasAndLimits
	 */
	void CondCheckNonConnQuotasAndLimits()
	{
		// Limit checks to once every 128 packets
		if (((NonConnPacketCounter + DisconnPacketCounter) & 0x7F) == 0)
		{
			bHitFrameNonConnLimit = CheckNonConnQuotasAndLimits();
		}
	}

	/**
	 * Rate limited call to CheckNetConnLimits
	 */
	void CondCheckNetConnLimits()
	{
		// Limit checks to once every 128 packets
		if ((NetConnPacketCounter & 0x7F) == 0)
		{
			bHitFrameNetConnLimit = CheckNetConnLimits();
		}
	}


	/**
	 * Accessor for bDDoSLogRestrictions - doubles as a per-frame logspam counter, automatically disabling logs after a quota
	 */
	bool CheckLogRestrictions()
	{
		return bDDoSLogRestrictions || (bDDoSDetection && ++LogHitCounter > DDoSLogSpamLimit);
	}


	// Brief accessors

	bool IsDDoSDetectionEnabled() const		{ return bDDoSDetection; }
	bool IsDDoSAnalyticsEnabled() const		{ return bDDoSAnalytics; }
	bool ShouldBlockNonConnPackets() const	{ return bHitFrameNonConnLimit; }
	bool ShouldBlockNetConnPackets() const	{ return bHitFrameNetConnLimit; }

	void IncNonConnPacketCounter()			{ ++NonConnPacketCounter; }
	int32 GetNonConnPacketCounter() const	{ return NonConnPacketCounter; }
	void IncNetConnPacketCounter()			{ ++NetConnPacketCounter; }
	int32 GetNetConnPacketCounter() const	{ return NetConnPacketCounter; }
	void IncDisconnPacketCounter()			{ ++ DisconnPacketCounter; }
	int32 GetDisconnPacketCounter() const	{ return DisconnPacketCounter; }
	void IncBadPacketCounter()				{ ++BadPacketCounter; }
	int32 GetBadPacketCounter() const		{ return BadPacketCounter; }
	void IncErrorPacketCounter()			{ ++ErrorPacketCounter; }
	int32 GetErrorPacketCounter() const		{ return ErrorPacketCounter; }
	void IncDroppedPacketCounter(int32 Count=1)			{ DroppedPacketCounter += Count; }
	int32 GetDroppedPacketCounter() const	{ return DroppedPacketCounter; }

protected:
	/**
	 * Performs periodic checks on trigger quota's and packet limits, for non-NetConnection packets
	 *
	 * @return	Whether or not non-NetConnection packet limits have been reached
	 */
	NETCORE_API bool CheckNonConnQuotasAndLimits();

	/**
	 * Performs periodic checks on NetConnection packet limits
	 *
	 * @return	Whether or not NetColnnection packet limits have been reached
	 */
	bool CheckNetConnLimits()
	{
		return NetConnPacketTimeLimitMSPerFrame > 0 &&
				(int32)((FPlatformTime::Seconds() - StartFrameRecvTimestamp) * 1000.0) > NetConnPacketTimeLimitMSPerFrame;
	}



protected:
	/** Whether or not DDoS detection is presently enabled */
	bool bDDoSDetection;

	/** Whether or not analytics for DDoS detection is enabled */
	bool bDDoSAnalytics;

	/** Whether or not the current frame has reached non-NetConnection packet limits, and should block non-NetConnection packets */
	bool bHitFrameNonConnLimit;

	/** Whether or not the current frame has reached NetConnection packet limits, and should block ALL further packets */
	bool bHitFrameNetConnLimit;


	/** The different DDoS detection states, of escalating severity, depending on the strength of the DDoS */
	TArray<FDDoSStateConfig> DetectionSeverity;

	/** The currently active DDoS severity state settings */
	int8 ActiveState;

	/** The worst DDoS severity state that has been active - used for limiting analytics events */
	int8 WorstActiveState;

	/** The last time the previous severity states escalation conditions were met (to prevent bouncing up/down between states) */
	double LastMetEscalationConditions;

	/** Limit checking previous states escalation conditions to once per frame */
	bool bMetEscalationConditionsThisFrame;


	/** Whether or not restriction of log messages from non-NetConnection packets is enabled */
	bool bDDoSLogRestrictions;

	/** The maximum number of non-NetConnection triggered log messages per frame, before further logs are dropped this frame */
	int32 DDoSLogSpamLimit;

	/** Counter for log restriction hits, in the current frame */
	int32 LogHitCounter;


	/** The amount of time since the previous frame, for detecting frame hitches, to prevent DDoS detection false positives */
	int32 HitchTimeQuotaMS;

	/** The number of frames spent hitching, before disabling false positive detection, and treating packet buildup as potential DDoS */
	int8 HitchFrameTolerance;

	/** The number of consecutive frames spent hitching */
	int32 HitchFrameCount;


	/** Timestamp for the last time per-second quota counting began */
	double LastPerSecQuotaBegin;

	/** Stores enough per second quota history, to allow all DetectionSeverity states to recalculate if their CooloffTime is reached */
	TArray<FDDoSPacketCounters> CounterPerSecHistory;

	/** The last written index of CounterPerSecHistory */
	int32 LastCounterPerSecHistoryIdx;


	/** The timestamp for the start of the current frames receive */
	double StartFrameRecvTimestamp;

	/** Timestamp for the end of the last frames receive loop */
	double EndFrameRecvTimestamp;

	/** Counts the packets from the start of the current frame */
	int32 StartFramePacketCount;

	/** The expected time between frames (1.0 / MaxTickRate) - used for adjusting limits/quota's based on DeltaTime */
	double ExpectedFrameTime;

	/** The current frames adjustment/deviation, from ExpectedFrameTime */
	float FrameAdjustment;


public:
	/** Analytics delegate for notifying of severity state escalations */
	FDDoSSeverityEscalation NotifySeverityEscalation;
};
