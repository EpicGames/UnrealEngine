// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/NetworkGuid.h"
#include "Serialization/BitReader.h"
#include "Templates/Tuple.h"
#include "UObject/CoreNet.h"
#include "UObject/WeakObjectPtr.h"

class FArchive;
class FGuidReferences;
class FProperty;
class UPackageMap;

using FGuidReferencesMap = TMap<int32, FGuidReferences>;

namespace UE::Net::Private
{
	extern NETCORE_API bool bRemapStableSubobjects;
}

/**
 * References to Objects (including Actors, Components, etc.) are replicated as NetGUIDs, since
 * the literal memory pointers will be different across game instances. In these cases, actual
 * replicated data for the Object will be handled elsewhere (either on its own Actor channel,
 * or on its Owning Actor's channel, as a replicated subobject).
 *
 * As Objects are replicated and received, these GUID References will become "mapped", and the
 * networking system will update the corresponding properties with pointers to the appropriate objects.
 *
 * As Objects are destroyed (due to game play, actor channels closing, relevancy, etc.), GUID References
 * will be "unmapped", and their corresponding properties will nulled out.
 *
 * The process of Mapping and Unmapping can happen numerous times for the same object (e.g., if an Actor
 * goes in and out of relevancy repeatedly).
 *
 * This class helps manage those references for specific replicated properties.
 * A FGuidReferences instance will be created for each Replicated Property that is a reference to an object.
 *
 * Guid References may also be nested in properties (like dynamic arrays), so we'll recursively track
 * those as well.
 */
class FGuidReferences
{
public:
	NETCORE_API FGuidReferences(
		FBitReader&					InReader,
		FBitReaderMark&				InMark,
		const TSet<FNetworkGUID>&	InUnmappedGUIDs,
		const TSet<FNetworkGUID>&	InMappedDynamicGUIDs,
		const int32					InParentIndex,
		const int32					InCmdIndex,
		UPackageMap*				InPackageMap
	);

	FGuidReferences(
		FGuidReferencesMap*	InArray,
		const int32			InParentIndex,
		const int32			InCmdIndex,
		UPackageMap*		InPackageMap
	):
		ParentIndex(InParentIndex),
		CmdIndex(InCmdIndex),
		NumBufferBits(0),
		Array(InArray),
		PackageMap(InPackageMap)
	{}


	NETCORE_API ~FGuidReferences();

	FGuidReferences& operator=(const FGuidReferences& Other) = delete;
	NETCORE_API FGuidReferences(const FGuidReferences& Other);

	NETCORE_API FGuidReferences& operator=(FGuidReferences&&);
	NETCORE_API FGuidReferences(FGuidReferences&& Other);

	void CountBytes(FArchive& Ar) const
	{
		UnmappedGUIDs.CountBytes(Ar);
		MappedDynamicGUIDs.CountBytes(Ar);
		Buffer.CountBytes(Ar);

		if (Array)
		{
			Array->CountBytes(Ar);
			for (const auto& GuidReferencePair : *Array)
			{
				GuidReferencePair.Value.CountBytes(Ar);
			}
		}
	}

	/** Get the set of GUIDs for objects that haven't been loaded / created yet. */
	const TSet<FNetworkGUID>& GetUnmappedGUIDs() const { return UnmappedGUIDs; }

	/** Starts tracking InGUID if not already and calls UPackageMap::AddUnmappedNetGUIDReference if bRemapStableSubobjects is enabled. */
	NETCORE_API void AddUnmappedGUID(FNetworkGUID InGUID);

	/** Stops tracking a GUID and calls UPackageMap::RemoveUnmappedNetGUIDReference if bRemapStableSubobjects is enabled. */
	NETCORE_API void RemoveUnmappedGUID(FNetworkGUID InGUID);

	/** Called by FRepLayout to update GUID tracking when updating unmapped objects. */
	NETCORE_API bool UpdateUnmappedGUIDs(UPackageMap* InPackageMap, UObject* OriginalObject, const FProperty* Property, int32 AbsOffset);

	/** The Property Command index of the top level property that references the GUID. */
	int32 ParentIndex;

	/** The Property Command index of the actual property that references the GUID. */
	int32 CmdIndex;

	int32 NumBufferBits;

private:
	/** GUIDs for objects that haven't been loaded / created yet. */
	TSet<FNetworkGUID> UnmappedGUIDs;

public:
	/** GUIDs for dynamically spawned objects that have already been created. */
	TSet<FNetworkGUID> MappedDynamicGUIDs;

	/** A copy of the last network data read related to this GUID Reference. */
	TArray<uint8> Buffer;

	/**
	 * If this FGuidReferences instance is owned by an Array Property that contains nested GUID References,
	 * then this will be a valid FGuidReferencesMap containing the nested FGuidReferences.
	 */
	FGuidReferencesMap* Array;

private:
	/** Stored PackageMap reference to update with tracked UnmappedGUIDs. */
	TWeakObjectPtr<UPackageMap> PackageMap;

	/** Calls UPackageMap::AddUnmappedNetGUIDReference for all GUIDs in UnmappedGUIDs, if bRemapStableSubobjects si enabled. */
	void TrackAllUnmappedGUIDs() const;
};
