// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "HAL/PlatformCrt.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectKey.h"

class FArchive;
class UObject;

namespace UE::Net
{
	/**
	* Special netcondition groups that allow a subobject to be replicated to unique conditions.
	* For these groups the network engine will determine automatically if the connection matches the conditions needed.
	* So the playercontroller should never be added to any of these groups.
	*/

	/** Replicate a subobject to it's owning connection. */
	extern NETCORE_API const FName NetGroupOwner;

	/** Replicate a subobject to the replay recording connection */
	extern NETCORE_API const FName NetGroupReplay;

	/** Returns true if the netcondition is one of the special explicit groups. */
	NETCORE_API bool IsSpecialNetConditionGroup(FName NetConditionGroup);
}


namespace UE::Net
{

/**
* Manage the membership of subobject's replicated with a COND_NetGroup condition.
* For a COND_NetGroup subobject to be replicated to a connection, they both need to be part of the same netcondition group.
* A netcondition group is simply an FName. 
* This manager holds onto the groups the subobject is a part of, while the connection's player controller is
* where we hold information on the user's groups.
* 
* Groups are inclusive and non-binding so subobject's can be part of multiple groups at the same time.
* As long as a player is a member of one of the subobject's group, that subobject will be replicated to that connection.
*/
class FNetConditionGroupManager
{
public:

	/** Add a subobject into a netcondition group */
	NETCORE_API void RegisterSubObjectInGroup(UObject* SubObject, FName NetConditionGroup);

	/** Add a subobject to many netcondition groups */
	NETCORE_API void RegisterSubObjectInMultipleGroups(UObject* SubObject, TArrayView<FName> NetConditionGroups);

	/** Remove a subobject from a netcondition group */
	NETCORE_API void UnregisterSubObjectFromGroup(UObject* SubObject, FName NetConditionGroup);

	/** Remove a subobject from many netcondition groups */
	NETCORE_API void UnregisterSubObjectFromMultipleGroups(UObject* SubObject, TArrayView<FName> NetConditionGroups);

	/** Remove the subobject from all groups it was included into */
	NETCORE_API void UnregisterSubObjectFromAllGroups(UObject* SubObject);

	/** Return the groups the subobject is a part of. Will be empty if it wasn't added to any. */
	TArrayView<const FName> GetSubObjectNetConditionGroups(FObjectKey SubObjectKey) const
	{
		const FSubObjectGroupInfo* GroupInfo = SubObjectGroupMap.Find(SubObjectKey);
		return GroupInfo ? TArrayView<const FName>(GroupInfo->NetGroups) : TArrayView<const FName>();
	}

	NETCORE_API void CountBytes(FArchive& Ar) const;

private:

#ifndef DEFAULT_NETGROUP_INLINE_SIZE
	// Sets the inlined array size of each subobject entry.  Configure it to the average groups the subobjects are registered to make them quicker to access.
	#define DEFAULT_NETGROUP_INLINE_SIZE 4
#endif

	struct FSubObjectGroupInfo
	{
		TArray<FName, TInlineAllocator<DEFAULT_NETGROUP_INLINE_SIZE>> NetGroups;
	};

	/** Map of the subobjects and the groups they are a part of */
	TMap<FObjectKey, FSubObjectGroupInfo> SubObjectGroupMap;
};


} // end namespace UE::Net
