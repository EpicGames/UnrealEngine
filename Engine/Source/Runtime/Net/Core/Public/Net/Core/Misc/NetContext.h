// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMisc.h"

// Forward declarations
namespace UE::Net::Private
{
	class FNetRPC;
}
class FObjectReplicator;

namespace UE::Net
{

/** Stateless class that provides misc network context information */
class FNetContext
{
public:
	
	/** 
	* Tells true if we are inside code executed from a remote RPC.
	* Will be false if the RPC is executed locally.
	*/
	static bool IsInsideNetRPC() { return bIsInRPCStack; }

private:

	friend class FScopedNetContextRPC;

	FNetContext() = delete;
	~FNetContext() = delete;

	static NETCORE_API bool bIsInRPCStack;
};


/** Used by friendly class to set the right network context */
class FScopedNetContextRPC
{
private:

	// Only friend class are allowed to push the net context
	friend UE::Net::Private::FNetRPC;
	friend FObjectReplicator;
	
	NETCORE_API FScopedNetContextRPC();
	NETCORE_API ~FScopedNetContextRPC();
};

} // end namespace UE::Net
