// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/ObjectKey.h"

class UActorComponent;
class UObject;

#ifndef UE_NET_SUBOBJECTLIST_WEAKPTR
	#define UE_NET_SUBOBJECTLIST_WEAKPTR 1
#endif

namespace UE::Net
{
	class FSubObjectRegistryGetter;
}

namespace UE::Net 
{

/**
* Stores the SubObjects to replicate and the network condition dictating to which connection they can replicate to.
*/
struct FSubObjectRegistry
{
public:

	enum class EResult : uint8
	{
		/** SubObject is new in the registry */
		NewEntry,
		/** SubObject already was in the registry */
		AlreadyRegistered,
		/** SubObject was already registered but the NetCondition passed is different from the entry */
		NetConditionConflict,
	};

	/** Adds a subobject and returns if it's a new entry or existing entry */
	NETCORE_API FSubObjectRegistry::EResult AddSubObjectUnique(UObject* InSubObject, ELifetimeCondition InNetCondition);

	/** Remove the subobject from the replicated list. Returns true if the subobject had been registered. */
	NETCORE_API bool RemoveSubObject(UObject* InSubObject);

	bool IsEmpty() const { return Registry.Num() == 0; }
	int32 Num() const	 { return Registry.Num(); }

	/**
	* Remove all the indexes specified by the passed array
	* @param IndexesToClean Array of indexes sorted from smallest to biggest. The sorted order is assumed to have been done by the caller
	*/
	NETCORE_API void CleanRegistryIndexes(const TArrayView<int32>& IndexesToClean);

	/** Find the NetCondition of a SubObject. Returns COND_MAX if not registered */
	NETCORE_API ELifetimeCondition GetNetCondition(UObject* SubObject) const;

	struct FEntry
	{
	private:
		/** 
		* Subobjects can be stored via raw pointers or weakobject pointers.  
		* Use raw pointers in a environment where you want the fastest replication code and can guarantee that every replicated subobject will be properly unregistered before getting deleted. Server's are one such place.
		* Otherwise use weak pointers in a environment where the replicated subobjects might get garbage'd before being unregistered.  PIE or clients using the list for demo recording are exemples of that.
		*/
#if UE_NET_SUBOBJECTLIST_WEAKPTR
		FWeakObjectPtr SubObject;
#else
		UObject* SubObject = nullptr;
#endif

	public:

		/** Store the object key info for fast access later */
		FObjectKey Key;

		/** The network condition that chooses which connection this subobject can be replicated to. Default is none which means all connections receive it. */
		ELifetimeCondition NetCondition = COND_None;

#if UE_NET_REPACTOR_NAME_DEBUG
		/** Cached name of the subobject class, for minidump debugging */
		FName SubObjectClassName;

		/** Cached name of the subobject, for minidump debugging */
		FName SubObjectName;
#endif

		FEntry(UObject* InSubObject, ELifetimeCondition InNetCondition = COND_None)
			: SubObject(InSubObject)
			, Key(InSubObject)
			, NetCondition(InNetCondition)
#if UE_NET_REPACTOR_NAME_DEBUG
			, SubObjectClassName(InSubObject != nullptr ? ((UObjectBase*)((UObjectBase*)InSubObject)->GetClass())->GetFName() : NAME_None)
			, SubObjectName(InSubObject != nullptr ? ((UObjectBase*)InSubObject)->GetFName() : NAME_None)
#endif
		{
		}

		inline bool operator==(const FEntry& rhs) const	{ return Key == rhs.Key; }
		inline bool operator==(const UObject* rhs) const	{ return SubObject == rhs; }

		inline UObject* GetSubObject() const
		{
#if UE_NET_SUBOBJECTLIST_WEAKPTR
			return SubObject.Get();
#else
			return SubObject;
#endif
		}
	};

	/** Returns the list of registered subobjects */
	const TArray<FSubObjectRegistry::FEntry>& GetRegistryList() const { return Registry; }

	/** Returns true if the subobject is contained in this list */
	NETCORE_API bool IsSubObjectInRegistry(const UObject* SubObject) const;

private:

	TArray<FEntry> Registry;
};

/** Keep track of replicated components and their subobject list */
struct FReplicatedComponentInfo
{
	/** Component that will be replicated */
	UActorComponent* Component = nullptr;

	/** Store the object key info for fast access later */
	FObjectKey Key;

	/** NetCondition of the component */
	ELifetimeCondition NetCondition = COND_None;

	/** Collection of subobjects replicated with this component */
	FSubObjectRegistry SubObjects;

#if UE_NET_REPACTOR_NAME_DEBUG
	/** Cached name of the component class, for minidump debugging */
	FName ComponentClassName;

	/** Cached name of the component, for minidump debugging */
	FName ComponentName;
#endif

	FReplicatedComponentInfo(UActorComponent* InComponent, ELifetimeCondition InNetCondition = COND_None)
		: Component(InComponent)
		, Key((UObject*)InComponent)
		, NetCondition(InNetCondition)
#if UE_NET_REPACTOR_NAME_DEBUG
		, ComponentClassName(InComponent != nullptr ? ((UObjectBase*)((UObjectBase*)InComponent)->GetClass())->GetFName() : NAME_None)
		, ComponentName(InComponent != nullptr ? ((UObjectBase*)InComponent)->GetFName() : NAME_None)
#endif
	{
	};

	bool operator==(const FReplicatedComponentInfo& rhs) const { return Key == rhs.Key; }
	bool operator==(const UActorComponent* rhs) const { return Component == rhs; }
};


} // namespace UE::Net


