// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/**
 * Packet Auditing
 *
 * Description:
 *	This implements a means of verifying the correct serialization of packets between a client and server,
 *	and is primarily for use during development of the netcode (particularly with the PacketHandler).
 *
 *	If used liberally, it should quickly zoom in on the precise location of packet serialization problems,
 *	greatly reducing time spent debugging, and - critically - avoiding having to manually reverse engineer packet dumps.
 *
 *
 * Basic usage:
 *	In code where you are writing data for sending, you can mark a stage of packet writing for auditing, like so:
 *		FPacketAudit::AddStage(TEXT("UniqueName"), OutPacket);
 *
 *	In code where you are reading incoming packet data, you can implement auditing for that stage as so:
 *		FPacketAudit::CheckStage(TEXT("UniqueName"), InPacket);
 *
 *	Replace 'UniqueName', with a descriptive and unique name, for marking the stage of serialization being checked.
 *	OutPacket must refer to an FBitWriter, and InPacket must refer to an FBitReader.
 *
 *	Run the client/server with '-PacketAudit' added to the commandline.
 *
 *
 * Notes/Limitations:
 *	- It's safe to leave AddStage/CheckStage in the netcode, without wrapping them in #if blocks, as they're compiled-out in shipping
 *	- You need to run the server/client as administrator
 *	- Only a single client and server can perform packet auditing at any given time.
 *	- Does not function well with ReliabilityHandlerComponent or packetloss.
 *	- Auditing assumes simple bit reader/writer serialization, and may choke when used with more complex packet parsing
 *	- Every call to AddStage, must have a corresponding call to CheckStage - or packet data will leak and eventually cause an assert
 *
 *
 * Technicals:
 *	The auditing code uses Inter-Process-Communication, by means of shared memory and a mutex for guarding access,
 *	to serialize two TMap's containing packet auditing data, into binary format, and passes them between the client/server processes.
 */

// Includes
#include "Containers/UnrealString.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformProcess.h"
#include "Serialization/BitReader.h"
#include "Serialization/BitWriter.h"

// Forward declarations
class FPacketAudit;


// Globals

#if !UE_BUILD_SHIPPING
/** The handler for managing packet auditing */
extern NETCORE_API FPacketAudit* GPacketAuditor;
#endif


/**
 * FPacketAudit
 *
 * Uses inter-process-communication, to audit every stage of packet processing, for verifying that packets are processing properly,
 * and that the client/server are in sync at every stage of reading/writing.
 *
 * Since this relies on IPC, it is only suitable for testing on a single machine.
 *
 * IMPORTANT: Only supports interaction between a single server, and a single client.
 */
class FPacketAudit
{
#if !UE_BUILD_SHIPPING
protected:
	/**
	 * Provides scoped exclusive access to GSendPacketAudits and GReceivePacketAudits within the .cpp file
	 */
	struct FScopedAuditAccess
	{
		FScopedAuditAccess(FPacketAudit& InAudtor);

		~FScopedAuditAccess();

	private:
		FPacketAudit& Auditor;
	};
#endif

private:
	/**
	 * Default constructor
	 */
	FPacketAudit()
	{
	}

	/**
	 * Default constructor
	 */
	NETCORE_API FPacketAudit(FPlatformProcess::FSemaphore* InGameMutex);

	/**
	 * Default destructor
	 */
	NETCORE_API ~FPacketAudit();

public:
	/**
	 * Initialize GPacketAudtor
	 */
	static NETCORE_API void Init();

	/**
	 * Destroy GPacketAuditor
	 */
	static NETCORE_API void Destruct();

	/**
	 * On the send side, marks a named stage during packet writing, which should be audited on the receive side.
	 *
	 * @param StageName		The unique name to provide this stage of packet writing
	 * @param OutPacket		The packet the data is being written to - used for gathering audit information
	 * @param bByteAligned	Whether or not to treat the packet as if it is byte aligned (used in rare circumstances)
	 */
	static inline void AddStage(FString StageName, FBitWriter& OutPacket, bool bByteAligned=false)
	{
#if !UE_BUILD_SHIPPING
		if (GPacketAuditor != nullptr)
		{
			GPacketAuditor->AddStage_Internal(StageName, OutPacket, bByteAligned);
		}
#endif
	}

	/**
	 * On the receive side, checks to see that a named stage was marked for auditing on the send side, and audits the packet data.
	 *
	 * @param StageName		The unique name to this stage of packet processing
	 * @param InPacket		The packet the data is being read from - used for gathering audit information
	 * @param bByteAligned	Whether or not to treat the packet as if it is byte aligned (used in rare circumstances)
	 */
	static inline void CheckStage(FString StageName, FBitReader& InPacket, bool bByteAligned=false)
	{
#if !UE_BUILD_SHIPPING
		if (GPacketAuditor != nullptr)
		{
			GPacketAuditor->CheckStage_Internal(StageName, InPacket, bByteAligned);
		}
#endif
	}

	/**
	 * Low level netcode hook, notifying of outgoing packets
	 *
	 * @param OutPacket		The current outgoing packet
	 */
	static inline void NotifyLowLevelSend(FBitWriter& OutPacket)
	{
#if !UE_BUILD_SHIPPING
		if (GPacketAuditor != nullptr)
		{
			GPacketAuditor->NotifyLowLevelSend_Internal(OutPacket);
		}
#endif
	}

	/**
	 * Low level netcode hook, notifying of incoming packets
	 *
	 * @param InPacket		The current incoming packet
	 */
	static inline void NotifyLowLevelReceive(FBitReader& InPacket)
	{
#if !UE_BUILD_SHIPPING
		if (GPacketAuditor != nullptr)
		{
			GPacketAuditor->NotifyLowLevelReceive_Internal(InPacket);
		}
#endif
	}

	// @todo #JohnB: Deprecate, once the pipeline uses FBitReader/Writers all the way
#if 1
	static inline void NotifyLowLevelSend(uint8* Data, uint32 DataLen, uint32 DataLenBits)
	{
#if !UE_BUILD_SHIPPING
		if (GPacketAuditor != nullptr)
		{
			FBitWriter OutPacket(DataLenBits);

			OutPacket.SerializeBits(Data, DataLenBits);

			GPacketAuditor->NotifyLowLevelSend_Internal(OutPacket);
		}
#endif
	}

	static inline void NotifyLowLevelReceive(uint8* Data, uint32 DataLen)
	{
#if !UE_BUILD_SHIPPING
		if (GPacketAuditor != nullptr)
		{
			FBitReader InPacket(Data, DataLen * 8);

			NotifyLowLevelReceive(InPacket);
		}
#endif
	}
#endif

protected:
	NETCORE_API void AddStage_Internal(FString StageName, FBitWriter& OutPacket, bool bByteAligned);

	NETCORE_API void CheckStage_Internal(FString StageName, FBitReader& InPacket, bool bByteAligned);

	NETCORE_API void NotifyLowLevelSend_Internal(FBitWriter& OutPacket);

	NETCORE_API void NotifyLowLevelReceive_Internal(FBitReader& InPacket);


	/**
	 * Gets the CRC of a packet, zeroing any unwritten bits in the last byte beforehand.
	 *
	 * @param Data			The packet data
	 * @param DataLenBits	The length of the data, in bits
	 * @return				Returns the CRC of the packet
	 */
	static NETCORE_API uint32 PacketCRC(uint8* Data, uint32 DataLenBits);

#if !UE_BUILD_SHIPPING
	/**
	 * Dumps the collected packet audit data to the log
	 */
	NETCORE_API void DumpAuditData(FScopedAuditAccess& AuditLock, uint32 InPacketCRC=0);
#endif


protected:
	/** Mutex used for enforcing a single instance of client/server packet auditing */
	FPlatformProcess::FSemaphore*			GameMutex;

	/** Mutex used for exclusive access to shared memory */
	FPlatformProcess::FSemaphore*			SharedMutex;

	/** Shared memory region for packet sends */
	FPlatformMemory::FSharedMemoryRegion*	SendSharedMemory;

	/** Shared memory region for packet receives */
	FPlatformMemory::FSharedMemoryRegion*	ReceiveSharedMemory;
};

