// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/Array.h"
#include "Templates/IsPODType.h"
#include "Templates/MemoryOps.h"
#include <type_traits>

/**
 * Simple ResizableCircularQueue.
 * Relies on unsigned arithmetics and ever increasing head and tail indices to avoid having to store 
 * an extra element or maintain explicit empty state.
 *
 * InitialCapacity must be a power of two.
 */
template<typename T, typename AllocatorT = FDefaultAllocator>
class TResizableCircularQueue
{
public:
	typedef T ElementT;

	/** Construct Empty Queue with the given initial Capacity, Capacity must be a power of two since we rely on unsigned arithmetics for wraparound */
	explicit TResizableCircularQueue(SIZE_T InitialCapacity);
	inline TResizableCircularQueue() : TResizableCircularQueue(0U) {}
	~TResizableCircularQueue();

	/** Returns true if the queue is empty */
	bool IsEmpty() const { return Head == Tail; }

	/** Gets the number of elements in the queue. */
	SIZE_T Count() const { return Head - Tail; }

	/** Current allocated Capacity */
	SIZE_T AllocatedCapacity() const { return Storage.Num(); }

	/** Push single element to the back of the Queue */
	void Enqueue(const ElementT& SrcData);

	/**
	 * Push single default constructed element to the back of the Queue, returning a reference to it.
	 *
	 * @see Enqueue_GetRef
	 */
	ElementT& EnqueueDefaulted_GetRef();

	/**
	 * Push single element to the back of the Queue, returning a reference to it. POD types will not be initialized.
	 *
	 * @see EnqueueDefaulted_GetRef
	 */
	ElementT& Enqueue_GetRef();

	/**
	 * Push single element to the back of the Queue, returning a reference to it. POD types will not be initialized.
	 *
	 * @see Enqueue_GetRef, EnqueueDefaulted_GetRef
	 */
	ElementT& Enqueue();

	/** Pop elements from the front of the queue */
	void Pop();

	/** Pop Count elements from the front of the queue */
	void Pop(SIZE_T Count);

	/**  Unchecked version, Pop a single element from the front of the queue */
	void PopNoCheck();

	/** Unchecked version, Pop Count elements from the front of the queue */
	void PopNoCheck(SIZE_T Count);

	/** Peek with the given offset from the front of the queue */
	const ElementT& PeekAtOffset(SIZE_T Offset = 0) const { check(Offset < Count()); return PeekAtOffsetNoCheck(Offset); }

	/** Poke at the element with the given offset from the front of the queue */
	ElementT& PokeAtOffset(SIZE_T Offset = 0) { check(Offset < Count()); return PokeAtOffsetNoCheck(Offset); }

	/** Peek at front element */
	const ElementT& Peek() const { return PeekAtOffset(0); }

	/** Peek at front element */
	ElementT& Poke() { return PokeAtOffset(0); }

	/**  Unchecked version, Peek with the given offset from the front of the queue */
	const ElementT& PeekAtOffsetNoCheck(SIZE_T Offset = 0) const { return Storage.GetData()[(Tail + Offset) & IndexMask]; }

	/**  Unchecked version, Peek with the given offset from the front of the queue */
	ElementT& PokeAtOffsetNoCheck(SIZE_T Offset = 0) { return Storage.GetData()[(Tail + Offset) & IndexMask]; }

	/** Peek at front element with no check */
	const ElementT& PeekNoCheck() const { return PeekAtOffsetNoCheck(0); }

	/**  Trim memory usage to the next power of two for the current size */
	void Trim();

	/** Empty queue without releasing memory */
	void Reset();

	/** Empty queue and release memory */
	void Empty();
	
private:
	enum : uint32
	{
		bConstructElements = (TIsPODType<T>::Value ? 0U : 1U),
		bDestructElements = (std::is_trivially_destructible_v<T> ? 0U : 1U),
	};

#if WITH_DEV_AUTOMATION_TESTS
	friend struct FResizableCircularQueueTestUtil;
#endif

	// Resize buffer maintaining validity of stored data.
	void SetCapacity(SIZE_T NewCapacity);

	typedef uint32 IndexT;
	typedef TArray<ElementT, AllocatorT> StorageT;

	IndexT Head;
	IndexT Tail;

	IndexT IndexMask;
	StorageT Storage;
};


template<typename T, typename AllocatorT>
TResizableCircularQueue<T, AllocatorT>::TResizableCircularQueue(SIZE_T InitialCapacity)
: Head(0u)
, Tail(0u)
, IndexMask(0u)
{
	// Capacity should be power of two, it will be rounded up but lets warn the user anyway in debug builds.
	checkSlow((InitialCapacity & (InitialCapacity - 1)) == 0);
	if (InitialCapacity > 0)
	{
		SetCapacity(InitialCapacity);
	}
}

template<typename T, typename AllocatorT>
TResizableCircularQueue<T, AllocatorT>::~TResizableCircularQueue()
{
	PopNoCheck(Count());
	Storage.SetNumUnsafeInternal(0);
}

template<typename T, typename AllocatorT>
void TResizableCircularQueue<T, AllocatorT>::Enqueue(const ElementT& SrcData)
{ 
	const SIZE_T RequiredCapacity = Count() + 1;
	if (RequiredCapacity > AllocatedCapacity())
	{
		// Capacity must be power of two
		SetCapacity(RequiredCapacity);
	}

	const IndexT MaskedIndex = Head++ & IndexMask;
	T* DstData = Storage.GetData() + MaskedIndex;
	new (DstData) T(SrcData);
}

template<typename T, typename AllocatorT>
typename TResizableCircularQueue<T, AllocatorT>::ElementT& TResizableCircularQueue<T, AllocatorT>::EnqueueDefaulted_GetRef()
{
	const SIZE_T RequiredCapacity = Count() + 1;
	if (RequiredCapacity > AllocatedCapacity())
	{
		// Capacity must be power of two
		SetCapacity(RequiredCapacity);
	}

	const IndexT MaskedIndex = Head++ & IndexMask;
	T* DstData = Storage.GetData() + MaskedIndex;
	new (DstData) T();

	return *DstData;
}

template<typename T, typename AllocatorT>
typename TResizableCircularQueue<T, AllocatorT>::ElementT& TResizableCircularQueue<T, AllocatorT>::Enqueue_GetRef()
{
	const SIZE_T RequiredCapacity = Count() + 1;
	if (RequiredCapacity > AllocatedCapacity())
	{
		// Capacity must be power of two
		SetCapacity(RequiredCapacity);
	}

	const IndexT MaskedIndex = Head++ & IndexMask;
	T* DstData = Storage.GetData() + MaskedIndex;
	if (bConstructElements)
	{
		new (DstData) T();
	}

	return *DstData;
}

template<typename T, typename AllocatorT>
typename TResizableCircularQueue<T, AllocatorT>::ElementT& TResizableCircularQueue<T, AllocatorT>::Enqueue()
{
	return Enqueue_GetRef();
}

template<typename T, typename AllocatorT>
void
TResizableCircularQueue<T, AllocatorT>::Pop(SIZE_T PopCount)
{
	if (ensure(Count() >= PopCount))
	{
		PopNoCheck(PopCount);
	}
}

template<typename T, typename AllocatorT>
void TResizableCircularQueue<T, AllocatorT>::Pop()
{
	if (ensure(Count() > 0))
	{
		PopNoCheck();
	}
}

template<typename T, typename AllocatorT>
void TResizableCircularQueue<T, AllocatorT>::PopNoCheck()
{ 
	if (bDestructElements)
	{
		const IndexT MaskedIndex = Tail & IndexMask;
		T* Data = Storage.GetData() + MaskedIndex;
		DestructItem(Data);
	}

	++Tail;
}

template<typename T, typename AllocatorT>
void TResizableCircularQueue<T, AllocatorT>::PopNoCheck(SIZE_T Count)
{
	if (bDestructElements)
	{
		const IndexT MaskedTailStart = Tail & IndexMask;
		const IndexT MaskedTailEnd = (Tail + Count) & IndexMask;

		if ((Count > 0) & (MaskedTailStart >= MaskedTailEnd))
		{
			T* Data = Storage.GetData();
			const SIZE_T FirstDestructCount = (Storage.Num() - MaskedTailStart);
			DestructItems(Data + MaskedTailStart, FirstDestructCount);
			DestructItems(Data, MaskedTailEnd);
		}
		else
		{
			T* Data = Storage.GetData() + MaskedTailStart;
			DestructItems(Data, Count);
		}
	}

	check(SIZE_T(Tail) + Count <= TNumericLimits<IndexT>::Max());
	Tail += (IndexT)Count;
}

template<typename T, typename AllocatorT>
void TResizableCircularQueue<T, AllocatorT>::SetCapacity(SIZE_T RequiredCapacity)
{
	using SizeType = typename StorageT::SizeType; 

 	SIZE_T NewCapacity = FMath::RoundUpToPowerOfTwo64(RequiredCapacity);

	if ((NewCapacity == Storage.Num()) || (NewCapacity < Count()))
	{
		return;
	}

	if (Storage.Num() > 0)
	{
		StorageT NewStorage;
		NewStorage.Empty(static_cast<SizeType>(NewCapacity));
		
		// copy data to new storage
		const IndexT MaskedTail = Tail & IndexMask;
		const IndexT MaskedHead = Head & IndexMask;

		const ElementT* SrcData = Storage.GetData();
		const SIZE_T SrcCapacity = Storage.Num();
		const SIZE_T SrcSize = Count();
		ElementT* DstData = NewStorage.GetData();

		// MaskedTail will be equal to MaskedHead both when the queue is full and when it is empty.
		if ((SrcSize > 0) & (MaskedTail >= MaskedHead))
		{
			const SIZE_T CopyCount = (SrcCapacity - MaskedTail);
			NewStorage.Append(SrcData + MaskedTail, static_cast<SizeType>(CopyCount));
			NewStorage.Append(SrcData, MaskedHead);
		}
		else
		{
			NewStorage.Append(SrcData + MaskedTail, static_cast<SizeType>(SrcSize));
		}

		NewStorage.AddUninitialized(static_cast<SizeType>(NewCapacity - SrcSize));

		this->Storage = MoveTemp(NewStorage);
		IndexMask = static_cast<IndexT>(NewCapacity - 1);
		Tail = 0u;
		Head = static_cast<IndexT>(SrcSize);
	}
	else
	{
		IndexMask = static_cast<IndexT>(NewCapacity - 1);
		Storage.AddUninitialized(static_cast<SizeType>(NewCapacity));
	}
}

template<typename T, typename AllocatorT>
void TResizableCircularQueue<T, AllocatorT>::Trim()
{
	if (IsEmpty())
	{
		Empty();
	}
	else
	{
		SetCapacity(Count());
	}
}

template<typename T, typename AllocatorT>
void TResizableCircularQueue<T, AllocatorT>::Reset()
{
	PopNoCheck(Count());
	Head = 0;
	Tail = 0;
}

template<typename T, typename AllocatorT>
void TResizableCircularQueue<T, AllocatorT>::Empty()
{
	PopNoCheck(Count());
	Head = 0;
	Tail = 0;
	IndexMask = IndexT(-1);

	// We've already destructed items. We just want to free the memory.
	Storage.SetNumUnsafeInternal(0);
	Storage.Empty();
}
