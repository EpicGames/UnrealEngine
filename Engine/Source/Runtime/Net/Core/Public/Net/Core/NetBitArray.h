// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformMemory.h"
#include "Misc/AssertionMacros.h"
#include <type_traits>

namespace UE::Net
{
	class FNetBitArray;
	class FNetBitArrayView;

	namespace Private
	{
		class FNetBitArrayRangedForConstIterator;
	}
}


/* NetBitArray validation support. */
#ifndef UE_NETBITARRAY_VALIDATE
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#	define UE_NETBITARRAY_VALIDATE 0
#else
#	define UE_NETBITARRAY_VALIDATE 1
#endif
#endif

#if UE_NETBITARRAY_VALIDATE
#	define UE_NETBITARRAY_CHECK(x) check(x)
#else
#	define UE_NETBITARRAY_CHECK(...)
#endif

#define UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(lhs,rhs) UE_NETBITARRAY_CHECK((lhs).GetData() != (rhs).GetData() && (lhs).GetNumBits() == (rhs).GetNumBits())

namespace UE::Net
{

class FNetBitArrayBase
{
public:
	enum ENoResetNoValidateType { NoResetNoValidate };
	enum EResetOnInitType { ResetOnInit };

	typedef uint32 StorageWordType;
	static constexpr uint32 WordBitCount = sizeof(StorageWordType)*8U;

	/** Returned from various methods to indicate failure. */
	static constexpr uint32 InvalidIndex = ~0U;

	// Some WordOps that can be passed to Combine() and ForAllSetBits()
	static constexpr StorageWordType AndOp(StorageWordType A, StorageWordType B) { return A & B; }
	static constexpr StorageWordType AndNotOp(StorageWordType A, StorageWordType B) { return A & ~B; }
	static constexpr StorageWordType OrOp(StorageWordType A, StorageWordType B) { return A | B; }
	static constexpr StorageWordType XorOp(StorageWordType A, StorageWordType B) { return A ^ B; }

	// Round up to a value that uses all available bits in a single NetBitArray Word
	static uint32 RoundUpToMaxWordBitCount(uint32 Value)
	{
		return Value > 0  ? ((Value + WordBitCount - 1) & ~(WordBitCount - 1)) : WordBitCount;
	}
};

/**
 * Simple bit array with internal storage.
 * Has very little error checking as it is used in performance critical code.
 */
class FNetBitArray : public FNetBitArrayBase
{
public:
	/**  Creates an empty NetBitArray. */
	FNetBitArray();

	/**  Creates an array with size BitCount bits. */
	explicit FNetBitArray(uint32 BitCount);

	/** Construct NetBitArray with the given BitCount but doesn't zero the allocated memory. Useful when the entire bit array will be overwritten before being read.*/
	explicit FNetBitArray(uint32 BitCountIn, const ENoResetNoValidateType);

	/** Return true if equal including BitCount and padding bits */
	bool operator==(const FNetBitArray& Other) const;

	/**
	 * Sets a new size and resets the array's contents.
	 * @param BitCount - The number of bits in the array after the operation.
	 */
	void Init(uint32 BitCount);

	/**
	 * Initialize a bit array with another array.
	 */
	void InitAndCopy(const FNetBitArray& Source);
	void InitAndCopy(const FNetBitArrayView& Source);
	/**
	 * Release all memory and set capacity to 0 bits
	 */
	void Empty();

	/**
	 * Sets the number of bits to BitCount. If the array grows the new bits will be cleared.
	 * Existing bits will remain intact.
	 *
	 * @param BitCount - The number of bits in the array after the operation.
	 */
	void SetNumBits(uint32 BitCount);

	/**
	 * Grows the size with the specified number of bits. New bits will be cleared.
	 *
	 * @param BitCount - The number of bits to add to the array.
	 * @see SetNumBits
	 */
	void AddBits(uint32 BitCount);

	/** Returns the number of bits */
	uint32 GetNumBits() const;

	/** Returns the number of words in our storage */
	uint32 GetNumWords() const;

	/**
	 * Returns a pointer to the internal storage but validates that the array buffer is the expected size.
	 * Use this before directly iterating over one or multiple buffers and ensure they all support the max iteration index.
	 */
	inline const StorageWordType* GetDataChecked(uint32 MaxWordIterationCount) const UE_LIFETIMEBOUND
	{
		checkf((int32)MaxWordIterationCount <= Storage.Num(), TEXT("MaxWordIterationCount (%u) is higher than this NetBitArray word count (%d). Out of Bounds memory access will occur."), MaxWordIterationCount, Storage.Num());
		return Storage.GetData();
	}

	/**
	 * Returns a pointer to the internal storage but validates that the array buffer is the expected size.
	 * Use this before directly iterating over one or multiple buffers and ensure they all support the max iteration index.
	 */
	inline StorageWordType* GetDataChecked(uint32 MaxWordIterationCount) UE_LIFETIMEBOUND
	{
		checkf((int32)MaxWordIterationCount <= Storage.Num(), TEXT("MaxWordIterationCount (%u) is higher than this NetBitArray word count (%d). Out of Bounds memory access will occur."), MaxWordIterationCount, Storage.Num());
		return Storage.GetData();
	}

	/** Returns a pointer to the internal storage. It's safer to use GetWord() or GetDataChecked() instead. */
	inline const StorageWordType* GetData() const UE_LIFETIMEBOUND { return Storage.GetData(); }

	/** Returns a pointer to the internal storage. It's safer to use GetWord() or GetDataChecked() instead. */
	inline StorageWordType* GetData() UE_LIFETIMEBOUND { return Storage.GetData(); }

	/** Read an entire word. Useful when you want to read 'WordBitCount(32)' bits at once */
	inline StorageWordType GetWord(uint32 WordIndex) const UE_LIFETIMEBOUND 
	{ 
		checkf(WordIndex < (uint32)Storage.Num(), TEXT("NetBitArray index out of bounds: %u / max %u"), WordIndex, Storage.Num());
		return Storage.GetData()[WordIndex]; 
	}

	/** Access an entire word. Useful when you want to write 'WordBitCount(32)' bits at once */
	inline StorageWordType& GetWord(uint32 WordIndex) UE_LIFETIMEBOUND 
	{  
		checkf(WordIndex < (uint32)Storage.Num(), TEXT("NetBitArray index out of bounds: %u / max %u"), WordIndex, Storage.Num());
		return Storage.GetData()[WordIndex]; 
	}

	/** Clear all bits in the array. */
	UE_DEPRECATED(5.5, "Use ClearAllBits instead.")
	void Reset() { ClearAllBits(); }

	/** Clear all bits in the array. */
	void ClearAllBits();

	/** Sets all bits in the array, but clears padding bits. */
	void SetAllBits();

	/** Return if a specified bit is set or not */
	bool IsBitSet(uint32 Index) const { return GetBit(Index); }

	/** Returns true if any bit is set in the bitset. Note: Padding bits in storage are expected to be zero. */
	bool IsAnyBitSet() const;

	/** Returns true if any bit is set in the specified range. The range will be clamped if needed.  Note: Padding bits in storage are expected to be zero. */
	bool IsAnyBitSet(uint32 StartIndex, uint32 Count) const;

	/** Returns true if no bit is set in the bitset. Note: Padding bits in storage are expected to be zero. */
	bool IsNoBitSet() const;

	/** Set the bit with the specified index */
	void SetBit(uint32 Index);

	/** Set the bit with the specified index to bValue */
	void SetBitValue(uint32 Index, bool bValue);

	/** Set the bit with the specified index to bValue */
	void SetBits(uint32 StartIndex, uint32 Count);

	/** Clear the bits in the specified range */
	void ClearBits(uint32 StartIndex, uint32 Count);

	/** Clear the bit with the specified index */
	void ClearBit(uint32 Index);

	/** Do an Or operation on the specified index */
	inline void OrBitValue(uint32 Index, bool bValue);

	/** Do an And operation on the specified index */
	inline void AndBitValue(uint32 Index, bool bValue);

	/** Get the bit with the specified index */
	bool GetBit(uint32 Index) const;

	/** Find first zero bit. Returns InvalidIndex if no zero bit was found */
	uint32 FindFirstZero() const;

	/** Find first set bit. Returns InvalidIndex if no zero bit was found */
	uint32 FindFirstOne() const;

	/** Find first zero bit starting from StartIndex. Returns InvalidIndex if no zero bit was found. */
	uint32 FindFirstZero(uint32 StartIndex) const;

	/** Find first set bit starting from StartIndex. Returns InvalidIndex if no set bit was found. */
	uint32 FindFirstOne(uint32 StartIndex) const;

	/** Find last zero bit. Returns InvalidIndex if no zero bit was found. */
	uint32 FindLastZero() const;

	/** Find last set bit. Returns InvalidIndex if no set bit was found. */
	uint32 FindLastOne() const;

	/** Only prints the amount of set bits in the array. @see FNetBitArrayPrinter for more print options */
	NETCORE_API FString ToString() const;

	/**
	 * Retrieves set bits in the provided range and returns how many indices were written to OutIndices.
	 * If OutIndices is filled to its capacity the search for set bits will end. OutIndices may be modified
	 * beyond the returned count but within the capacity.
	 * 
	 * @param StartIndex Which bit index to start searching for set bits.
	 * @param Count How many bits to check. The count will be clamped to number of bits in the array so you can pass ~0U to check all bits.
	 * @param OutIndices Where to store the indices of the set bits. Indices with lower numbers will be stored first.
	 * @param OutIndicesCapacity How many indices can be stored in OutIndices.
	 * @returns Number of indices OutIndices was populated with.
	 */
	uint32 GetSetBitIndices(uint32 StartIndex, uint32 Count, uint32* OutIndices, uint32 OutIndicesCapacity) const;

	/** Counts the number of set bits in this array in the provided range. */
	uint32 CountSetBits(uint32 StartIndex = 0, uint32 Count = ~0U) const;

	/** Copy bits from other bit array. The other array must have the same bit count. */
	void Copy(const FNetBitArray& Other);
	void Copy(const FNetBitArrayView& Other);

	/**
	 * Overwrite this bit array with the result of a word operation from two other arrays
	 * 
	 * @param First The first bit array to pass in the word operation
	 * @param WordOp The operation to perform on the two bit arrays
	 * @param Second The second bit array to pass in the word operation
	 * @see FNetBitArrayHelper::AndOp, FNetBitArrayHelper::AndNotOp, FNetBitArrayHelper::OrOp, FNetBitArrayHelper::XorOp
	 */
	template<typename WordOpFunctor> void Set(const FNetBitArray& First, WordOpFunctor&& WordOp, const FNetBitArray& Second);

	/**
	 * Combine this array with another array using a word operation functor. Bit counts need to match.
	 *
	 * @param Other The other array to combine with.
	 * @param WordOpFunctor The operation to perform for each word to combine. Signature is StorageWordType(StorageWordType ThisWord, StorageWordType OtherWord).
	 * @see FNetBitArrayHelper::AndOp, FNetBitArrayHelper::AndNotOp, FNetBitArrayHelper::OrOp, FNetBitArrayHelper::XorOp
	 */
	template<typename WordOpFunctor> void Combine(const FNetBitArray& Other, WordOpFunctor&& WordOp);

	/**
	 * Combine this array with the result of a bitwise operation of two other arrays.
	 * eg: ThisArray = ThisArray Op (ArrayA Op2 ArrayB)
	 *
	 * @param Op Operation to execute on the word of this array and the result of the operation on the two other arrays
	 * @param ArrayA The first bit array to execute the other operation on
	 * @param Op2 The operation to execute on the word of the read-only array passed in parameter
	 * @param ArrayB The second bit array to execute the other operation on
	 * @see FNetBitArrayHelper::AndOp, FNetBitArrayHelper::AndNotOp, FNetBitArrayHelper::OrOp, FNetBitArrayHelper::XorOp
	 */
	template<typename WordOpFunctor1, typename WordOpFunctor2> void CombineMultiple(WordOpFunctor1&& Op, const FNetBitArray& ArrayA, WordOpFunctor2&& Op2, const FNetBitArray& ArrayB);

	/** Iterate over all set bits and invoke functor with signature void(uint32 BitIndex). */
	template<typename T>
	void ForAllSetBits(T&& Functor) const;

	/** Iterate over all set bits and invoke functor with signature void(uint32 BitIndex). EndBit is clamped to the the number of bits in the array minus one. */
	template<typename T>
	void ForAllSetBitsInRange(const uint32 StartBit, const uint32 EndBit, T&& Functor) const;

	/** Iterate over all the zero bits and invoke functor with signature void(uint32 BitIndex). */
	template<typename T>
	void ForAllUnsetBits(T&& Functor) const;

	/** Iterate over all words and execute Functor, with signature void(uint32 BitIndex), for all set bits after executing WordOpFunctor */
	template<typename T, typename V>
	static void ForAllSetBits(const FNetBitArray& A, const FNetBitArray& B, T&& WordOpFunctor, V&& Functor);

	/** Call the functor for every bit still set after the following operation: "A Op1 (B Op2 C)" */
	template <typename WordOpFunctor1, typename WordOpFunctor2, typename UserFunctor>
	static void ForAllSetBitsMultiple(const FNetBitArray& A, WordOpFunctor1&& Op1, const FNetBitArray& B, WordOpFunctor2&& Op2, const FNetBitArray& C, UserFunctor&& Functor);

	/** Compare two BitArrays bit by bit and invoke FunctorA(BitIndex) for bits set only in array A and invoke FunctorB(BitIndex) for bits set only in array B. */
	template<typename T, typename V>
	static void ForAllExclusiveBits(const FNetBitArray& A, const FNetBitArray& B, T&& FunctorA, V&& FunctorB);

	/** Compare two BitArrays bit by bit. Where they differ and Predicate(BitIndex) returns true FunctorA(BitIndex) is invoked for bits set only in array A and FunctorB(BitIndex) is invoked for bits set only in array B. */
	template<typename T, typename V, typename Predicate>
	static void ForAllExclusiveBitsByPredicate(const FNetBitArray& A, const FNetBitArray& B, T&& FunctorA, V&& FunctorB, Predicate&& Pred);

	/**
	 * DO NOT USE DIRECTLY
	 * Range-based for loop support iterating over set bits and returning the index to the set bit.
	 */
	Private::FNetBitArrayRangedForConstIterator begin() const;
	Private::FNetBitArrayRangedForConstIterator end() const;

private:
	StorageWordType GetLastWordMask() const { return (~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1))); }
	void ClearPaddingBits();

	/** Sets a new size but doesn't initialize the contents with zero'd memory. */
	void Reserve(uint32 BitCount);

	TArray<StorageWordType> Storage;
	uint32 BitCount;
};

/**
 * Simple bit array view.
 * Assumes external storage and has very little error checking as it is used in performance critical code.
 * Note: Trailing padding bits in storage will be zeroed unless the view is explicitly constructed to allow it.
 *       Certain operations assume the padding bits are cleared.
 */
class FNetBitArrayView : public FNetBitArrayBase
{
public:

	/** Constructor will produce a valid but empty bitarray. */
	inline FNetBitArrayView();

	/** Construct NetBitArray from external storage. Storage must be large enough to fit given the BitCount. No clearing of storage and no validation that padding bits are cleared. */
	inline FNetBitArrayView(StorageWordType* StorageIn, uint32 BitCountIn, const ENoResetNoValidateType);
	/** Construct NetBitArray from external storage. Storage must be large enough to fit given the BitCount. Clears the entire storage. */
	inline FNetBitArrayView(StorageWordType* StorageIn, uint32 BitCountIn, const EResetOnInitType);
	/** Construct NetBitArray from external storage. Storage must be large enough to fit given the BitCount. Validates that padding bits are cleared. */
	inline FNetBitArrayView(StorageWordType* StorageIn, uint32 BitCountIn);

	/** Return true if equal including BitCount and padding bits */
	bool operator==(const FNetBitArrayView& Other) const;

	/** Return if a specific bit is set or not */
	inline bool IsBitSet(uint32 Index) const { return GetBit(Index); }

	/** Returns true if any bit is set in the bitset Note: Padding bits in storage are expected to be zero. */
	inline bool IsAnyBitSet() const;

	/** Returns true if any bit is set in the specified range. The range will be clamped if needed. Note: Padding bits in storage are expected to be zero.*/
	inline bool IsAnyBitSet(uint32 StartIndex, uint32 Count) const;

	/** Returns true if no bit is set in the bitset. Note: Padding bits in storage are expected to be zero. */
	inline bool IsNoBitSet() const;

	/** Reset the storage of the BitArray including any padding bits */
	UE_DEPRECATED(5.5, "Use ClearAllBits instead.")
	inline void Reset() { ClearAllBits(); }

	/** Clear all bits to zero*/
	inline void ClearAllBits();

	/** All padding bits will be set to zero */
	inline void ClearPaddingBits();

	/** Set all bits, padding bits will be zeroed*/
	inline void SetAllBits();

	/** Set the bit with the specified index */
	inline void SetBit(uint32 Index);

	/** Set the bit with the specified index to bValue */
	inline void SetBitValue(uint32 Index, bool bValue);

	/** Set the bits in the specified range to true */
	inline void SetBits(uint32 StartIndex, uint32 Count);

	/** Clear the bits in the specified range */
	inline void ClearBits(uint32 StartIndex, uint32 Count);

	/** Clear the bit with the specified index */
	inline void ClearBit(uint32 Index) const;

	/** Get the bit with the specified index */
	inline bool GetBit(uint32 Index) const;

	/** Do an Or operation on the specified index */
	inline void OrBitValue(uint32 Index, bool bValue);

	/** Do an And operation on the specified index */
	inline void AndBitValue(uint32 Index, bool bValue);

	/** Find first zero bit. Returns InvalidIndex if no zero bit was found */
	inline uint32 FindFirstZero() const;

	/** Find first set bit. Returns InvalidIndex if no zero bit was found */
	inline uint32 FindFirstOne() const;

	/** Find first zero bit starting from StartIndex. Returns InvalidIndex if no zero bit was found. */
	inline uint32 FindFirstZero(uint32 StartIndex) const;

	/** Find first set bit starting from StartIndex. Returns InvalidIndex if no set bit was found. */
	inline uint32 FindFirstOne(uint32 StartIndex) const;

	/** Find last zero bit. Returns InvalidIndex if no zero bit was found. */
	uint32 FindLastZero() const;

	/** Find last set bit. Returns InvalidIndex if no set bit was found. */
	uint32 FindLastOne() const;

	/** Returns the number of bits */
	inline uint32 GetNumBits() const;

	/** Returns the number of words in our storage */
	inline uint32 GetNumWords() const;

	/**
	 * Returns a pointer to the internal storage but validates that the array buffer is the expected size.
	 * Use this before directly iterating over one or multiple buffers and ensure they all support the max iteration index.
	 */
	inline const StorageWordType* GetDataChecked(uint32 MaxWordIterationCount) const UE_LIFETIMEBOUND
	{
		checkf(MaxWordIterationCount <= WordCount, TEXT("MaxWordIterationCount (%u) is higher than this NetBitArrayView word count (%u). Out of Bounds memory access will occur."), MaxWordIterationCount, WordCount);
		return Storage;
	}

	/**
	 * Returns a pointer to the internal storage but validates that the array buffer is the expected size.
	 * Use this before directly iterating over one or multiple buffers and ensure they all support the max iteration index.
	 */
	inline StorageWordType* GetDataChecked(uint32 MaxWordIterationCount) UE_LIFETIMEBOUND
	{
		checkf(MaxWordIterationCount <= WordCount, TEXT("MaxWordIterationCount (%u) is higher than this NetBitArrayView word count (%u). Out of Bounds memory access will occur."), MaxWordIterationCount, WordCount);
		return Storage;
	}

	/** Returns a pointer to the internal storage. It's safer to use GetWord() or GetDataChecked() instead. */
	inline const StorageWordType* GetData() const UE_LIFETIMEBOUND { return Storage; }

	/** Returns a pointer to the internal storage. It's safer to use GetWord() or GetDataChecked() instead. */
	inline StorageWordType* GetData() UE_LIFETIMEBOUND { return Storage; }

	/** Read an entire word. Useful when you want to read 'WordBitCount(32)' bits at once */
	inline StorageWordType GetWord(uint32 WordIndex) const UE_LIFETIMEBOUND 
	{ 
		checkf(WordIndex < WordCount, TEXT("NetBitArrayView index out of bounds: %u / max %u"), WordIndex, WordCount);
		return Storage[WordIndex]; 
	}

	/** Access an entire word. Useful when you want to write 'WordBitCount(32)' bits at once */
	inline StorageWordType& GetWord(uint32 WordIndex) UE_LIFETIMEBOUND 
	{
		checkf(WordIndex < WordCount, TEXT("NetBitArrayView index out of bounds: %u / max %u"), WordIndex, WordCount);
		return Storage[WordIndex]; 
	}

	/** Only prints the amount of set bits in the array. @see FNetBitArrayPrinter for more print options */
	NETCORE_API FString ToString() const;

	/**
	 * Retrieves set bits in the provided range and returns how many indices were written to OutIndices.
	 * If OutIndices is filled to its capacity the search for set bits will end. OutIndices may be modified
	 * beyond the returned count but within the capacity.
	 * 
	 * @param StartIndex Which bit index to start searching for set bits.
	 * @param Count How many bits to check. The count will be clamped to number of bits in the array so you can pass ~0U to check all bits.
	 * @param OutIndices Where to store the indices of the set bits. Indices with lower numbers will be stored first.
	 * @param OutIndicesCapacity How many indices can be stored in OutIndices.
	 * @returns Number of indices OutIndices was populated with.
	 */
	uint32 GetSetBitIndices(uint32 StartIndex, uint32 Count, uint32* OutIndices, uint32 OutIndicesCapacity) const;

	/** Counts the number of set bits in this array in the provided range. */
	uint32 CountSetBits(uint32 StartIndex = 0, uint32 Count = ~0U) const;

	/** Copy bits from other bit array. The other array must have the same bit count. */
	inline void Copy(const FNetBitArrayView& Other);
	inline void Copy(const FNetBitArray& Other);

	/**
	 * Overwrite this bit array with the result of a word operation from two other arrays
	 *
	 * @param First The first bit array to pass in the word operation
	 * @param WordOp The operation to perform on the two bit arrays
	 * @param Second The second bit array to pass in the word operation
	 * @see FNetBitArrayHelper::AndOp, FNetBitArrayHelper::AndNotOp, FNetBitArrayHelper::OrOp, FNetBitArrayHelper::XorOp
	 */
	template<typename WordOpFunctor> void Set(const FNetBitArrayView& First, WordOpFunctor&& WordOp, const FNetBitArrayView& Second);

	/**
	 * Combine this array with another array using a word operation functor. Bit counts need to match.
	 *
	 * @param Other The other array to combine with.
	 * @param WordOpFunctor The operation to perform for each word to combine. Signature is StorageWordType(StorageWordType ThisWord, StorageWordType OtherWord).
	 * @see FNetBitArrayHelper::AndOp, FNetBitArrayHelper::AndNotOp, FNetBitArrayHelper::OrOp, FNetBitArrayHelper::XorOp
	 */
	template<typename WordOpFunctor> void Combine(const FNetBitArrayView& Other, WordOpFunctor&& WordOp);
	
	/**
	 * Combine this array with the result of a bitwise operation of two other arrays.
	 * eg: ThisArray = ThisArray Op (ArrayA Op2 ArrayB)
	 *
	 * @param Op Operation to execute on the word of this array and the result of the operation on the two other arrays
	 * @param ArrayA The first bit array to execute the other operation on
	 * @param Op2 The operation to execute on the word of the read-only array passed in parameter
	 * @param ArrayB The second bit array to execute the other operation on
	 * @see FNetBitArrayHelper::AndOp, FNetBitArrayHelper::AndNotOp, FNetBitArrayHelper::OrOp, FNetBitArrayHelper::XorOp
	 */
	template<typename WordOpFunctor1, typename WordOpFunctor2> void CombineMultiple(WordOpFunctor1&& Op, const FNetBitArrayView& ArrayA, WordOpFunctor2&& Op2, const FNetBitArrayView& ArrayB);

	/** Iterate over all set bits and invoke functor with signature void(uint32 BitIndex). */
	template <typename T>
	void ForAllSetBits(T&& Functor) const;

	/** Iterate over all set bits from StartBit to EndBit (inclusive) and invoke functor with signature void(uint32 BitIndex). EndBit is clamped to the the number of bits in the array minus one. */
	template <typename T>
	void ForAllSetBitsInRange(const uint32 StartBit, const uint32 EndBit, T&& Functor) const;

	/** Iterate over all the zero bits and invoke functor with signature void(uint32 BitIndex). */
	template<typename T>
	void ForAllUnsetBits(T&& Functor) const;

	/** Iterate over all words and execute Functor, with signature void(uint32 BitIndex), for all set bits after executing WordOpFunctor */
	template <typename T, typename V>
	static void ForAllSetBits(const FNetBitArrayView& A, const FNetBitArrayView& B, T&& WordOpFunctor, V&& Functor);

	/** Call the functor for every bit still set after the following operation: "A Op1 (B Op2 C)" */
	template <typename WordOpFunctor1, typename WordOpFunctor2, typename UserFunctor>
	static void ForAllSetBitsMultiple(const FNetBitArrayView& A, WordOpFunctor1&& Op1, const FNetBitArrayView& B, WordOpFunctor2&& Op2, const FNetBitArrayView& C, UserFunctor&& Functor);

	/** Compare two BitArrays bit by bit and invoke FunctorA(BitIndex) for bits set only in array A and invoke FunctorB(BitIndex) for bits set only in array B. */
	template <typename T, typename V>
	static void ForAllExclusiveBits(const FNetBitArrayView& A, const FNetBitArrayView& B, T&& FunctorA, V&& FunctorB);

	/** Compare two BitArrays bit by bit. Where they differ and Predicate(BitIndex) returns true FunctorA(BitIndex) is invoked for bits set only in array A and FunctorB(BitIndex) is invoked for bits set only in array B. */
	template<typename T, typename V, typename Predicate>
	static void ForAllExclusiveBitsByPredicate(const FNetBitArrayView& A, const FNetBitArrayView& B, T&& FunctorA, V&& FunctorB, Predicate&& Pred);

	static constexpr inline uint32 CalculateRequiredWordCount(uint32 BitCount);

	/**
	 * DO NOT USE DIRECTLY
	 * Range-based for loop support iterating over set bits and returning the index to the set bit.
	 */
	Private::FNetBitArrayRangedForConstIterator begin() const;
	Private::FNetBitArrayRangedForConstIterator end() const;

private:
	inline StorageWordType GetLastWordMask() const { return (~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1))); }

	StorageWordType* Storage;
	uint32 WordCount;
	uint32 BitCount;
};

//*************************************************************************************************
// FNetBitArrayHelper 
// Implementation helper
//*************************************************************************************************

class FNetBitArrayHelper final
{
private:
	friend FNetBitArray;
	friend FNetBitArrayView;

	typedef FNetBitArray::StorageWordType StorageWordType;
	static constexpr uint32 WordBitCount = sizeof(StorageWordType) * 8u;

	static StorageWordType GetLastWordMask(uint32 BitCount)
	{
		return (~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1)));
	}

	static bool IsAnyBitSet(const StorageWordType* Storage, const uint32 WordCount)
	{
		for (uint32 WordIt = 0; WordIt < WordCount; ++WordIt)
		{
			if (Storage[WordIt])
			{
				return true;
			}
		}

		return false;
	}

	static bool IsAnyBitSet(const StorageWordType* Storage, const uint32 BitCount, const uint32 StartIndex, const uint32 Count)
	{
		// Range validation
		if ((Count == 0) | (StartIndex >= BitCount))
		{
			return false;
		}

		const uint32 EndIndex = static_cast<uint32>(FPlatformMath::Min(uint64(StartIndex) + Count - 1U, uint64(BitCount) - 1U));
		const uint32 WordStartIt = StartIndex / WordBitCount;
		StorageWordType StartWordMask = (~StorageWordType(0) << (StartIndex & (WordBitCount - 1)));
		const uint32 WordEndIt = EndIndex / WordBitCount;
		const StorageWordType EndWordMask = (~StorageWordType(0) >> (~EndIndex & (WordBitCount - 1)));

		for (uint32 WordIt = WordStartIt; WordIt <= WordEndIt; ++WordIt)
		{
			// Compute mask based on start and end word status.
			const bool bIsEndWord = (WordIt == WordEndIt);
			const StorageWordType WordMask = StartWordMask & (bIsEndWord ? EndWordMask : ~StorageWordType(0));
			// Only the first word is the start word so we can set all bits now.
			StartWordMask = ~StorageWordType(0);

			if (Storage[WordIt] & WordMask)
			{
				return true;
			}
		}

		return false;
	}

	static void SetBits(StorageWordType* Storage, const uint32 BitCount, const uint32 StartIndex, const uint32 Count)
	{
		UE_NETBITARRAY_CHECK((Count > 0) & (StartIndex < BitCount));

		const uint32 EndIndex = static_cast<uint32>(FPlatformMath::Min(uint64(StartIndex) + Count - 1U, uint64(BitCount) - 1U));
		const uint32 WordStartIt = StartIndex / WordBitCount;
		StorageWordType StartWordMask = (~StorageWordType(0) << (StartIndex & (WordBitCount - 1)));
		const uint32 WordEndIt = EndIndex / WordBitCount;
		const StorageWordType EndWordMask = (~StorageWordType(0) >> (~EndIndex & (WordBitCount - 1)));

		for (uint32 WordIt = WordStartIt; WordIt <= WordEndIt; ++WordIt)
		{
			// Compute mask based on start and end word status.
			const bool bIsEndWord = (WordIt == WordEndIt);
			const StorageWordType WordMask = StartWordMask & (bIsEndWord ? EndWordMask : ~StorageWordType(0));
			// Only the first word is the start word so we can set all bits now.
			StartWordMask = ~StorageWordType(0);

			Storage[WordIt] |= ~StorageWordType(0) & WordMask;
		}
	}

	static void ClearBits(StorageWordType* Storage, const uint32 BitCount, const uint32 StartIndex, const uint32 Count)
	{
		UE_NETBITARRAY_CHECK((Count > 0) & (StartIndex < BitCount));

		const uint32 EndIndex = static_cast<uint32>(FPlatformMath::Min(uint64(StartIndex) + Count - 1U, uint64(BitCount) - 1U));
		const uint32 WordStartIt = StartIndex / WordBitCount;
		StorageWordType StartWordMask = (~StorageWordType(0) << (StartIndex & (WordBitCount - 1)));
		const uint32 WordEndIt = EndIndex / WordBitCount;
		const StorageWordType EndWordMask = (~StorageWordType(0) >> (~EndIndex & (WordBitCount - 1)));

		for (uint32 WordIt = WordStartIt; WordIt <= WordEndIt; ++WordIt)
		{
			// Compute mask based on start and end word status.
			const bool bIsEndWord = (WordIt == WordEndIt);
			const StorageWordType WordMask = StartWordMask & (bIsEndWord ? EndWordMask : ~StorageWordType(0));
			// Only the first word is the start word so we can set all bits now.
			StartWordMask = ~StorageWordType(0);

			Storage[WordIt] = Storage[WordIt] & ~WordMask;
		}
	}

	static void SetBit(StorageWordType* Storage, const uint32 BitCount, const uint32 Index)
	{
		UE_NETBITARRAY_CHECK(Index < BitCount);

		const SIZE_T WordIndex = Index/WordBitCount;
		const StorageWordType WordMask = (StorageWordType(1) << (Index & (WordBitCount - 1)));

		Storage[WordIndex] |= WordMask;
	}

	static void SetBitValue(StorageWordType* Storage, const uint32 BitCount, const uint32 Index, const bool bValue)
	{
		UE_NETBITARRAY_CHECK(Index < BitCount);

		const SIZE_T WordIndex = Index/WordBitCount;
		const StorageWordType WordMask = (StorageWordType(1) << (Index & (WordBitCount - 1)));
		const StorageWordType ValueMask = bValue ? WordMask : StorageWordType(0);

		Storage[WordIndex] = (Storage[WordIndex] & ~WordMask) | ValueMask;
	}

	static void ClearBit(StorageWordType* Storage, const uint32 BitCount, const uint32 Index)
	{
		UE_NETBITARRAY_CHECK(Index < BitCount);

		const SIZE_T WordIndex = Index/WordBitCount;
		const StorageWordType WordMask = (StorageWordType(1) << (Index & (WordBitCount - 1)));

		Storage[WordIndex] &= ~WordMask;
	}

	static bool GetBit(const StorageWordType* Storage, const uint32 BitCount, const uint32 Index)
	{
		UE_NETBITARRAY_CHECK(Index < BitCount);

		const SIZE_T WordIndex = Index/WordBitCount;
		const StorageWordType WordMask = (StorageWordType(1) << (Index & (WordBitCount - 1)));

		return (Storage[WordIndex] & WordMask) != 0u;
	}

	static void OrBitValue(StorageWordType* Storage, const uint32 BitCount, const uint32 Index, const bool bValue)
	{
		UE_NETBITARRAY_CHECK(Index < BitCount);

		const SIZE_T WordIndex = Index / WordBitCount;
		const StorageWordType WordMask = (StorageWordType(1) << (Index & (WordBitCount - 1)));
		const StorageWordType ValueMask = bValue ? WordMask : StorageWordType(0);

		Storage[WordIndex] = Storage[WordIndex] | ValueMask;
	}

	static void AndBitValue(StorageWordType* Storage, const uint32 BitCount, const uint32 Index, const bool bValue)
	{
		UE_NETBITARRAY_CHECK(Index < BitCount);

		const SIZE_T WordIndex = Index / WordBitCount;
		const StorageWordType WordMask = (StorageWordType(1) << (Index & (WordBitCount - 1)));
		const StorageWordType ValueMask = bValue ? ~StorageWordType(0) : ~WordMask;

		Storage[WordIndex] = Storage[WordIndex] & ValueMask;
	}

	static uint32 FindFirstZero(const StorageWordType* Storage, const uint32 WordCount, const uint32 BitCount)
	{
		for (uint32 WordIt = 0, CurrentBitIndex = 0; WordIt < WordCount; ++WordIt, CurrentBitIndex += WordBitCount)
		{
			const StorageWordType Word = Storage[WordIt];
			if (Word != StorageWordType(~StorageWordType(0)))
			{
				const uint32 Index = CurrentBitIndex + FPlatformMath::CountTrailingZeros(~Word);
				// Need to make sure the index is not out of bounds
				return (Index < BitCount ? Index : FNetBitArrayBase::InvalidIndex);
			}
		}

		return FNetBitArrayBase::InvalidIndex;
	}

	static uint32 FindFirstOne(const StorageWordType* Storage, const uint32 WordCount, const uint32 BitCount)
	{
		for (uint32 WordIt = 0, CurrentBitIndex = 0; WordIt < WordCount; ++WordIt, CurrentBitIndex += WordBitCount)
		{
			const StorageWordType Word = Storage[WordIt];
			if (Word != 0)
			{
				const uint32 Index = CurrentBitIndex + FPlatformMath::CountTrailingZeros(Word);
				// Need to make sure the index is not out of bounds
				return (Index < BitCount ? Index : FNetBitArrayBase::InvalidIndex);
			}
		}

		return FNetBitArrayBase::InvalidIndex;
	}

	static uint32 FindFirstZero(const StorageWordType* Storage, const uint32 WordCount, const uint32 BitCount, const uint32 StartIndex)
	{
		// WordMask is used to hide zero bits in the first word we check to prevent returning an index before StartIndex.
		uint32 CurrentBitIndex = StartIndex & ~(WordBitCount - 1U);
		uint32 WordMask = ~(~0U << (StartIndex & (WordBitCount - 1U)));
		for (uint32 WordIt = StartIndex/WordBitCount; WordIt < WordCount; ++WordIt, CurrentBitIndex += WordBitCount)
		{
			const StorageWordType Word = Storage[WordIt] | WordMask;
			WordMask = 0U;
			if (Word != StorageWordType(~StorageWordType(0)))
			{
				const uint32 Index = CurrentBitIndex + FPlatformMath::CountTrailingZeros(~Word);
				// Need to make sure the index is not out of bounds
				return (Index < BitCount ? Index : FNetBitArrayBase::InvalidIndex);
			}
		}

		return FNetBitArrayBase::InvalidIndex;
	}

	static uint32 FindFirstOne(const StorageWordType* Storage, const uint32 WordCount, const uint32 BitCount, const uint32 StartIndex)
	{
		uint32 CurrentBitIndex = StartIndex & ~(WordBitCount - 1U);
		// WordMask is needed only in the first word we check to prevent returning an index before StartIndex.
		uint32 WordMask = ~0U << (StartIndex & (WordBitCount - 1U));
		for (uint32 WordIt = StartIndex/WordBitCount; WordIt < WordCount; ++WordIt, CurrentBitIndex += WordBitCount)
		{
			const StorageWordType Word = Storage[WordIt] & WordMask;
			// From here on we accept all words as they are.
			WordMask = ~0U;
			if (Word != 0)
			{
				const uint32 Index = CurrentBitIndex + FPlatformMath::CountTrailingZeros(Word);
				// Need to make sure the index is not out of bounds
				return (Index < BitCount ? Index : FNetBitArrayBase::InvalidIndex);
			}
		}

		return FNetBitArrayBase::InvalidIndex;
	}

	static uint32 FindLastZero(const StorageWordType* Storage, const uint32 WordCount, const uint32 BitCount)
	{
		StorageWordType WordMask = GetLastWordMask(BitCount);

		for (uint32 WordIt = WordCount; WordIt > 0; )
		{
			--WordIt;
			const StorageWordType Word = Storage[WordIt];
			if (Word != WordMask)
			{
				const uint32 BitOffset = WordIt*WordBitCount;
				const uint32 Index = BitOffset + WordBitCount - 1U - FPlatformMath::CountLeadingZeros(~Word & WordMask);
				// Need to make sure the index is not out of bounds
				return (Index < BitCount ? Index : FNetBitArrayBase::InvalidIndex);
			}

			WordMask = StorageWordType(~StorageWordType(0));
		}

		return FNetBitArrayBase::InvalidIndex;
	}

	static uint32 FindLastOne(const StorageWordType* Storage, const uint32 WordCount, const uint32 BitCount)
	{
		for (uint32 WordIt = WordCount; WordIt > 0; )
		{
			--WordIt;
			const StorageWordType Word = Storage[WordIt];
			if (Word != 0U)
			{
				const uint32 BitOffset = WordIt*WordBitCount;
				const uint32 Index = BitOffset + WordBitCount - 1U - FPlatformMath::CountLeadingZeros(Word);
				return Index;
			}
		}

		return FNetBitArrayBase::InvalidIndex;
	}

	NETCORE_API static uint32 GetSetBitIndices(const StorageWordType* Storage, const uint32 BitCount, const uint32 StartIndex, const uint32 Count, uint32* const OutIndices, const uint32 OutIndicesCapacity);
	
	NETCORE_API static uint32 CountSetBits(const StorageWordType* Storage, const uint32 BitCount, const uint32 StartIndex, const uint32 Count);

	template<typename WordOpFunctor> static void Set(StorageWordType* Storage, const StorageWordType* FirstSource, const StorageWordType* SecondSource, const uint32 WordCount, WordOpFunctor&& WordOp)
	{
		for (uint32 WordIt = 0; WordIt < WordCount; ++WordIt)
		{
			Storage[WordIt] = WordOp(FirstSource[WordIt], SecondSource[WordIt]);
		}
	}

	template<typename Functor> static void Combine(StorageWordType* Storage, const StorageWordType* OtherStorage, const uint32 WordCount, Functor&& WordOp)
	{
		for (uint32 WordIt = 0; WordIt < WordCount; ++WordIt)
		{
			Storage[WordIt] = WordOp(Storage[WordIt], OtherStorage[WordIt]);
		}
	}

	template<typename WordOpFunctor1, typename WordOpFunctor2> static void CombineMultiple(StorageWordType* WriteStorage, const StorageWordType* const ReadStorageA, const StorageWordType* const ReadStorageB, const uint32 WordCount, WordOpFunctor1&& WordOpWrite, WordOpFunctor2&& WordOpRead)
	{
		for (uint32 WordIt = 0; WordIt < WordCount; ++WordIt)
		{
			WriteStorage[WordIt] = WordOpWrite(WriteStorage[WordIt], WordOpRead(ReadStorageA[WordIt], ReadStorageB[WordIt]));
		}
	}

	static bool IsEqual(const StorageWordType* Storage, const StorageWordType* OtherStorage, const uint32 WordCount)
	{
		StorageWordType Result = 0U;
		for (uint32 WordIt = 0; WordIt < WordCount; ++WordIt)
		{
			Result |= Storage[WordIt] ^ OtherStorage[WordIt];
		}
		return Result == 0U;
	}

	template<typename T> static void ForAllSetBits(const StorageWordType* Storage, const uint32 WordCount, const uint32 BitCount, T&& Functor)
	{
		const StorageWordType LastWordMask = ~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1));
		const uint32 LastWordIt = WordCount - 1;

		for (uint32 WordIt = 0, CurrentBitIndex = 0; WordIt < WordCount; ++WordIt, CurrentBitIndex += WordBitCount)
		{
			StorageWordType CurrentWord = Storage[WordIt] & ((WordIt == LastWordIt) ? LastWordMask : ~0U);

			// Invoke functor for all set bits in the CurrentWord
			while (CurrentWord)
			{
				const uint32 BitIndex = CurrentBitIndex + FPlatformMath::CountTrailingZeros(CurrentWord);
				Functor(BitIndex);
				CurrentWord &= CurrentWord - 1;
			}
		}
	}

	//Calls Functor for every set bit from StartBit to EndBit (inclusive)
	template<typename T> static void ForAllSetBitsInRange(const StorageWordType* Storage, const uint32 TotalWordCount, const uint32 TotalBitCount, const uint32 StartBit, const uint32 InEndBit, T&& Functor)
	{
		const uint32 EndBit = FPlatformMath::Min(InEndBit, TotalBitCount - 1U);
		if ((TotalBitCount == 0U) || (StartBit >= TotalBitCount) || (EndBit < StartBit))
		{
			return;
		}

		const uint32 StartWord = StartBit / WordBitCount;
		const uint32 EndWord = EndBit / WordBitCount;
		const uint32 StartWordBitIndex = StartWord * WordBitCount;//Beginning bit of the StartWord (may be lower than StartBit)
		const uint32 EndWordBitIndex = EndWord * WordBitCount;//Beginning bit of the EndWord (may be lower than EndBit)
		const StorageWordType StartWordMask = ~StorageWordType(0) << (StartBit - StartWordBitIndex);
		const uint32 EndWordBitCount = (EndBit - EndWordBitIndex) + 1;
		const StorageWordType EndWordMask = ~StorageWordType(0) >> (WordBitCount - EndWordBitCount);

		for (uint32 WordIt = StartWord, CurrentBitIndex = StartWordBitIndex; WordIt <= EndWord; ++WordIt, CurrentBitIndex += WordBitCount)
		{
			StorageWordType WordMask = (~StorageWordType(0) & ((WordIt == StartWord) ? StartWordMask : ~0U)) & ((WordIt == EndWord) ? EndWordMask : ~0U);
			StorageWordType CurrentWord = Storage[WordIt] & WordMask;

			// Invoke functor for all set bits in the CurrentWord
			while (CurrentWord)
			{
				const uint32 BitIndex = CurrentBitIndex + FPlatformMath::CountTrailingZeros(CurrentWord);
				Functor(BitIndex);
				CurrentWord &= CurrentWord - 1;
			}
		}
	}

	template<typename T> static void ForAllUnsetBits(const StorageWordType* Storage, const uint32 WordCount, const uint32 BitCount, T&& Functor)
	{
		const StorageWordType LastWordMask = ~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1));
		const uint32 LastWordIt = WordCount - 1;

		for (uint32 WordIt = 0, CurrentBitIndex = 0; WordIt < WordCount; ++WordIt, CurrentBitIndex += WordBitCount)
		{
			const StorageWordType WordMask = ((WordIt == LastWordIt) ? LastWordMask : ~0U);

			// Flip bits to test the zero values
			StorageWordType CurrentWord = ~Storage[WordIt] & WordMask;

			// Invoke functor for all set bits in the CurrentWord
			while (CurrentWord)
			{
				const uint32 BitIndex = CurrentBitIndex + FPlatformMath::CountTrailingZeros(CurrentWord);
				Functor(BitIndex);
				CurrentWord &= CurrentWord - 1;
			}
		}
	}

	template<typename T, typename V> static void ForAllSetBits(const StorageWordType* StorageA, const StorageWordType* StorageB, const uint32 WordCount, const uint32 BitCount, T&& WordOpFunctor, V&& Functor)
	{
		const StorageWordType LastWordMask = ~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1));
		const uint32 LastWordIt = WordCount - 1;

		for (uint32 WordIt = 0, CurrentBitIndex = 0; WordIt < WordCount; ++WordIt, CurrentBitIndex += WordBitCount)
		{
			const StorageWordType CurrentWordA = StorageA[WordIt];
			const StorageWordType CurrentWordB = StorageB[WordIt];

			StorageWordType CurrentWord = WordOpFunctor(CurrentWordA, CurrentWordB) & ((WordIt == LastWordIt) ? LastWordMask : ~0U);

			// Invoke functor for all set bits in the CurrentWord
			while (CurrentWord)
			{
				const uint32 BitIndex = CurrentBitIndex + FPlatformMath::CountTrailingZeros(CurrentWord);
				Functor(BitIndex);
				CurrentWord &= CurrentWord - 1;
			}
		}
	}

	template <typename WordOpFunctor1, typename WordOpFunctor2, typename UserFunctor> static void ForAllSetBitsMultiple(const StorageWordType* StorageA, const StorageWordType* StorageB, const StorageWordType* StorageC, const uint32 WordCount, const uint32 BitCount, WordOpFunctor1&& WordOp1, WordOpFunctor2&& WordOp2, UserFunctor&& Functor)
	{
		const StorageWordType LastWordMask = ~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1));
		const uint32 LastWordIt = WordCount - 1;

		for (uint32 WordIt = 0, CurrentBitIndex = 0; WordIt < WordCount; ++WordIt, CurrentBitIndex += WordBitCount)
		{
			const StorageWordType CurrentWordA = StorageA[WordIt];
			const StorageWordType CurrentWordB = StorageB[WordIt];
			const StorageWordType CurrentWordC = StorageC[WordIt];

			StorageWordType CurrentWord = WordOp1(CurrentWordA, WordOp2(CurrentWordB, CurrentWordC) ) & ((WordIt == LastWordIt) ? LastWordMask : ~0U);

			// Invoke functor for all set bits in the CurrentWord
			while (CurrentWord)
			{
				const uint32 BitIndex = CurrentBitIndex + FPlatformMath::CountTrailingZeros(CurrentWord);
				Functor(BitIndex);
				CurrentWord &= CurrentWord - 1;
			}
		}
	}

	template<typename T, typename V, typename Predicate> static void ForAllExclusiveBitsByPredicate(const StorageWordType* StorageA, const StorageWordType* StorageB, const uint32 WordCount, const uint32 BitCount, T&& FunctorA, V&& FunctorB, Predicate&& Pred)
	{
		const StorageWordType LastWordMask = ~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1));
		const uint32 LastWordIt = WordCount - 1;

		for (uint32 WordIt = 0, CurrentBitIndex = 0; WordIt < WordCount; ++WordIt, CurrentBitIndex += WordBitCount)
		{
			StorageWordType CurrentWordA = StorageA[WordIt];
			const StorageWordType CurrentWordB = StorageB[WordIt];
			StorageWordType CurrentWordXOR = (CurrentWordA ^ CurrentWordB) & ((WordIt == LastWordIt) ? LastWordMask : ~0U);

			// if CurrentWord contains any set bits invoke the correct functor
			uint32 LocalBitIndex = CurrentBitIndex;
			while (CurrentWordXOR)
			{
				if (CurrentWordXOR & 0x1)
				{
					if (Pred(LocalBitIndex))
					{
						if (CurrentWordA & 0x1)
						{
							FunctorA(LocalBitIndex);
						}
						else
						{
							FunctorB(LocalBitIndex);
						}
					}
				}

				CurrentWordXOR >>= 1;
				CurrentWordA >>= 1;

				++LocalBitIndex;
			}
		}
	}

	template<typename T, typename V> static void ForAllExclusiveBits(const StorageWordType* StorageA, const StorageWordType* StorageB, const uint32 WordCount, const uint32 BitCount, T&& FunctorA, V&& FunctorB)
	{
		constexpr auto AlwaysProcessPredicate = [](uint32 Index)
			{ 
				return true; 
			};
		
		return ForAllExclusiveBitsByPredicate(StorageA, StorageB, WordCount, BitCount, FunctorA, FunctorB, AlwaysProcessPredicate);
	}

};

namespace Private
{

//*************************************************************************************************
// FNetBitArrayRangedForConstIterator
//*************************************************************************************************
class FNetBitArrayRangedForConstIterator
{
public:
	FNetBitArrayRangedForConstIterator();

	FNetBitArrayRangedForConstIterator& operator++();

	bool operator!=(const FNetBitArrayRangedForConstIterator& It) const;

	/** Returns the bit index the iterator is pointing at. */
	uint32 operator*() const;

private:
	friend FNetBitArray;
	friend FNetBitArrayView;

	enum class ERangeStart : unsigned
	{
		Begin,
		End
	};

	FNetBitArrayRangedForConstIterator(const FNetBitArrayBase::StorageWordType* InData UE_LIFETIMEBOUND, uint32 BitCount, ERangeStart RangeStart);

	void AdvanceToNextSetBit();

	const FNetBitArrayBase::StorageWordType* Data = nullptr;
	uint32 BitCount = 0;
	uint32 WordCount = 0;
	uint32 CurrentBitIndex = 0;
	uint32 CurrentWord = 0;
};

} // end namespace Private

//*************************************************************************************************
// FNetBitArray Implementation
//*************************************************************************************************
inline FNetBitArray::FNetBitArray()
: BitCount(0)
{
}

inline bool FNetBitArray::operator==(const FNetBitArray& Other) const
{
	if (BitCount != Other.BitCount)
	{
		return false;
	}

	return FNetBitArrayHelper::IsEqual(this->GetData(), Other.GetData(), GetNumWords());
}

inline FNetBitArray::FNetBitArray(uint32 InBitCount)
: BitCount(InBitCount)
{
	Init(InBitCount);
}

inline FNetBitArray::FNetBitArray(uint32 InBitCount, const FNetBitArrayBase::ENoResetNoValidateType)
: BitCount(InBitCount)
{
	Reserve(BitCount);
}

inline void FNetBitArray::Init(uint32 InBitCount)
{
	BitCount = InBitCount;

	const uint32 WordCount = (InBitCount + WordBitCount - 1U)/WordBitCount;
	Storage.Reset(WordCount);
	Storage.AddZeroed(WordCount);
}

inline void FNetBitArray::Reserve(uint32 InBitCount)
{
	BitCount = InBitCount;

	const uint32 WordCount = (InBitCount + WordBitCount - 1U) / WordBitCount;
	Storage.Reset(WordCount);
	Storage.AddUninitialized(WordCount);
}

inline void FNetBitArray::InitAndCopy(const FNetBitArray& Source)
{
	Reserve(Source.GetNumBits());

	Copy(Source);
}

inline void FNetBitArray::InitAndCopy(const FNetBitArrayView& Source)
{
	Reserve(Source.GetNumBits());

	Copy(Source);
}

inline void FNetBitArray::Empty()
{
	BitCount = 0U;
	Storage.Empty();
}

inline void FNetBitArray::SetNumBits(uint32 InBitCount)
{
	BitCount = InBitCount;

	const uint32 WordCount = (InBitCount + WordBitCount - 1U)/WordBitCount;
	Storage.SetNumZeroed(WordCount, EAllowShrinking::No);
	ClearPaddingBits();
}

inline void FNetBitArray::AddBits(uint32 AdditionalBitCount)
{
	SetNumBits(BitCount + AdditionalBitCount);
}

inline uint32 FNetBitArray::GetNumBits() const
{
	return BitCount;
}

inline uint32 FNetBitArray::GetNumWords() const
{
	return static_cast<uint32>(Storage.Num());
}

inline void FNetBitArray::ClearAllBits()
{
	FPlatformMemory::Memset(Storage.GetData(), 0, Storage.Num()*sizeof(StorageWordType));
}

inline void FNetBitArray::SetAllBits()
{
	FPlatformMemory::Memset(Storage.GetData(), 0xff, Storage.Num()*sizeof(StorageWordType));
	ClearPaddingBits();
}

inline bool FNetBitArray::IsAnyBitSet() const
{
	return FNetBitArrayHelper::IsAnyBitSet(Storage.GetData(), Storage.Num());
}

inline bool FNetBitArray::IsAnyBitSet(uint32 StartIndex, uint32 Count) const
{
	return ((Count == 1U) ? GetBit(StartIndex) : FNetBitArrayHelper::IsAnyBitSet(Storage.GetData(), BitCount, StartIndex, Count));
}

inline bool FNetBitArray::IsNoBitSet() const
{
	return !IsAnyBitSet();
}

inline void FNetBitArray::SetBit(uint32 Index)
{
	FNetBitArrayHelper::SetBit(Storage.GetData(), BitCount, Index);
}

inline void FNetBitArray::SetBitValue(uint32 Index, bool bValue)
{
	FNetBitArrayHelper::SetBitValue(Storage.GetData(), BitCount, Index, bValue);
}

inline void FNetBitArray::SetBits(uint32 StartIndex, uint32 Count)
{
	FNetBitArrayHelper::SetBits(Storage.GetData(), BitCount, StartIndex, Count);
}

inline void FNetBitArray::ClearBits(uint32 StartIndex, uint32 Count)
{
	FNetBitArrayHelper::ClearBits(Storage.GetData(), BitCount, StartIndex, Count);
}

inline void FNetBitArray::ClearBit(uint32 Index)
{
	FNetBitArrayHelper::ClearBit(Storage.GetData(), BitCount, Index);
}

inline bool FNetBitArray::GetBit(uint32 Index) const
{
	return FNetBitArrayHelper::GetBit(Storage.GetData(), BitCount, Index);
}

inline void FNetBitArray::OrBitValue(uint32 Index, bool bValue)
{
	FNetBitArrayHelper::OrBitValue(Storage.GetData(), BitCount, Index, bValue);
}

inline void FNetBitArray::AndBitValue(uint32 Index, bool bValue)
{
	FNetBitArrayHelper::AndBitValue(Storage.GetData(), BitCount, Index, bValue);
}

inline uint32 FNetBitArray::FindFirstZero() const
{
	return FNetBitArrayHelper::FindFirstZero(Storage.GetData(), Storage.Num(), BitCount);
}

inline uint32 FNetBitArray::FindFirstOne() const
{
	return FNetBitArrayHelper::FindFirstOne(Storage.GetData(), Storage.Num(), BitCount);
}

inline uint32 FNetBitArray::FindFirstZero(uint32 StartIndex) const
{
	return FNetBitArrayHelper::FindFirstZero(Storage.GetData(), Storage.Num(), BitCount, StartIndex);
}

inline uint32 FNetBitArray::FindFirstOne(uint32 StartIndex) const
{
	return FNetBitArrayHelper::FindFirstOne(Storage.GetData(), Storage.Num(), BitCount, StartIndex);
}

inline uint32 FNetBitArray::FindLastZero() const
{
	return FNetBitArrayHelper::FindLastZero(Storage.GetData(), Storage.Num(), BitCount);
}

inline uint32 FNetBitArray::FindLastOne() const
{
	return FNetBitArrayHelper::FindLastOne(Storage.GetData(), Storage.Num(), BitCount);
}

inline uint32 FNetBitArray::GetSetBitIndices(uint32 StartIndex, uint32 Count, uint32* OutIndices, uint32 OutIndicesCapacity) const
{
	return FNetBitArrayHelper::GetSetBitIndices(Storage.GetData(), BitCount, StartIndex, Count, OutIndices, OutIndicesCapacity);
}

inline uint32 FNetBitArray::CountSetBits(uint32 StartIndex, uint32 Count) const
{
	return FNetBitArrayHelper::CountSetBits(Storage.GetData(), BitCount, StartIndex, Count);
}

inline void FNetBitArray::Copy(const FNetBitArray& Other)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, Other);
	if (GetNumWords() > 0 && Other.GetNumWords() > 0)
	{
		// Intentionally doing a memory copy instead of using assignment operator as we know the bit counts are the same and don't need extra checks.
		FPlatformMemory::Memcpy(GetData(), Other.GetData(), GetNumWords() * sizeof(StorageWordType));
	}
}

inline void FNetBitArray::Copy(const FNetBitArrayView& Other)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, Other);
	if (GetNumWords() > 0 && Other.GetNumWords() > 0)
	{
		// Intentionally doing a memory copy instead of using assignment operator as we know the bit counts are the same and don't need extra checks.
		FPlatformMemory::Memcpy(GetData(), Other.GetData(), GetNumWords() * sizeof(StorageWordType));
	}
}

template<typename WordOpFunctor> inline void FNetBitArray::Set(const FNetBitArray& First, WordOpFunctor&& WordOp, const FNetBitArray& Second)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, First);
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, Second);
	FNetBitArrayHelper::Set(GetData(), First.GetData(), Second.GetData(), GetNumWords(), WordOp);
}

template<typename WordOpFunctor> inline void FNetBitArray::Combine(const FNetBitArray& Other, WordOpFunctor&& WordOp)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, Other);
	FNetBitArrayHelper::Combine(Storage.GetData(), Other.Storage.GetData(), Storage.Num(), WordOp);
}

template<typename WordOpFunctor1, typename WordOpFunctor2> inline void FNetBitArray::CombineMultiple(WordOpFunctor1&& Op, const FNetBitArray& ArrayA, WordOpFunctor2&& Op2, const FNetBitArray& ArrayB)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, ArrayA);
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, ArrayB);
	FNetBitArrayHelper::CombineMultiple(Storage.GetData(), ArrayA.Storage.GetData(), ArrayB.Storage.GetData(), Storage.Num(), Op, Op2);
}

template<typename T> inline void FNetBitArray::ForAllSetBits(T&& Functor) const
{
	FNetBitArrayHelper::ForAllSetBits(Storage.GetData(), Storage.Num(), BitCount, Functor);
}

template<typename T> inline void FNetBitArray::ForAllSetBitsInRange(const uint32 StartBit, const uint32 EndBit, T&& Functor) const
{
	FNetBitArrayHelper::ForAllSetBitsInRange(Storage.GetData(), Storage.Num(), BitCount, StartBit, EndBit, Functor);
}

template<typename T> inline void FNetBitArray::ForAllUnsetBits(T&& Functor) const
{
	FNetBitArrayHelper::ForAllUnsetBits(Storage.GetData(), Storage.Num(), BitCount, Functor);
}

template <typename T, typename V> inline void FNetBitArray::ForAllSetBits(const FNetBitArray& A, const FNetBitArray& B, T&& WordOpFunctor, V&& Functor)
{
	UE_NETBITARRAY_CHECK(A.GetNumBits() == B.GetNumBits());
	FNetBitArrayHelper::ForAllSetBits(A.Storage.GetData(), B.Storage.GetData(), A.Storage.Num(), A.BitCount, WordOpFunctor, Functor);
}

template<typename WordOpFunctor1, typename WordOpFunctor2, typename UserFunctor> inline void FNetBitArray::ForAllSetBitsMultiple(const FNetBitArray& A, WordOpFunctor1&& Op1, const FNetBitArray& B, WordOpFunctor2&& Op2, const FNetBitArray& C, UserFunctor&& Functor)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(A, B);
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(B, C);
	FNetBitArrayHelper::ForAllSetBitsMultiple(A.Storage.GetData(), B.Storage.GetData(), C.Storage.GetData(), A.Storage.Num(), A.BitCount, Op1, Op2, Functor);
}

template <typename T, typename V> inline void FNetBitArray::ForAllExclusiveBits(const FNetBitArray& A, const FNetBitArray& B, T&& FunctorA, V&& FunctorB)
{
	UE_NETBITARRAY_CHECK(A.GetNumBits() == B.GetNumBits());
	FNetBitArrayHelper::ForAllExclusiveBits(A.Storage.GetData(), B.Storage.GetData(), A.Storage.Num(), A.BitCount, FunctorA, FunctorB);
}

template <typename T, typename V, typename Predicate> inline void FNetBitArray::ForAllExclusiveBitsByPredicate(const FNetBitArray& A, const FNetBitArray& B, T&& FunctorA, V&& FunctorB, Predicate&& Pred)
{
	UE_NETBITARRAY_CHECK(A.GetNumBits() == B.GetNumBits());
	FNetBitArrayHelper::ForAllExclusiveBitsByPredicate(A.Storage.GetData(), B.Storage.GetData(), A.Storage.Num(), A.BitCount, FunctorA, FunctorB, Pred);
}

inline void FNetBitArray::ClearPaddingBits()
{
	if (BitCount > 0)
	{
		StorageWordType& LastWord = Storage.GetData()[Storage.Num() - 1];
		LastWord &= GetLastWordMask();
	}
}

//*************************************************************************************************
// FNetBitArrayRangedForConstIterator implementation
//*************************************************************************************************
inline Private::FNetBitArrayRangedForConstIterator FNetBitArray::begin() const
{
	return Private::FNetBitArrayRangedForConstIterator(Storage.GetData(), BitCount, Private::FNetBitArrayRangedForConstIterator::ERangeStart::Begin);
}

inline Private::FNetBitArrayRangedForConstIterator FNetBitArray::end() const
{
	return Private::FNetBitArrayRangedForConstIterator(Storage.GetData(), BitCount, Private::FNetBitArrayRangedForConstIterator::ERangeStart::End);
}


//*************************************************************************************************
// NetBitArrayView implementation
//*************************************************************************************************
constexpr uint32 FNetBitArrayView::CalculateRequiredWordCount(uint32 BitCount)
{ 
	return (BitCount + WordBitCount - 1) / WordBitCount; 
}

FNetBitArrayView::FNetBitArrayView()
: Storage(&BitCount)
, WordCount(0)
, BitCount(0)
{
}

FNetBitArrayView::FNetBitArrayView(StorageWordType* StorageIn, uint32 BitCountIn, const ENoResetNoValidateType)
: Storage(StorageIn)
, WordCount(CalculateRequiredWordCount(BitCountIn))
, BitCount(BitCountIn)
{
	if (BitCountIn == 0)
	{
		Storage = const_cast<uint32*>(&BitCount);
	}
}

FNetBitArrayView::FNetBitArrayView(StorageWordType* StorageIn, uint32 BitCountIn, const EResetOnInitType)
: FNetBitArrayView(StorageIn, BitCountIn, NoResetNoValidate)
{
	ClearAllBits();
}

FNetBitArrayView::FNetBitArrayView(StorageWordType* StorageIn, uint32 BitCountIn)
: FNetBitArrayView(StorageIn, BitCountIn, NoResetNoValidate)
{
	UE_NETBITARRAY_CHECK(Storage != nullptr);
	UE_NETBITARRAY_CHECK((BitCount == 0) || ((Storage[WordCount - 1] & ~GetLastWordMask()) == 0u));
}

inline bool FNetBitArrayView::operator==(const FNetBitArrayView& Other) const
{
	if (BitCount != Other.BitCount)
	{
		return false;
	}

	return FNetBitArrayHelper::IsEqual(this->GetData(), Other.GetData(), GetNumWords());
}

bool FNetBitArrayView::IsAnyBitSet() const
{
	return FNetBitArrayHelper::IsAnyBitSet(Storage, WordCount);
}

bool FNetBitArrayView::IsAnyBitSet(uint32 StartIndex, uint32 Count) const
{
	return ((Count == 1U) ? GetBit(StartIndex) : FNetBitArrayHelper::IsAnyBitSet(GetData(), BitCount, StartIndex, Count));
}

bool FNetBitArrayView::IsNoBitSet() const
{
	return !IsAnyBitSet();
}

void FNetBitArrayView::ClearAllBits()
{
	FPlatformMemory::Memset(&Storage[0], 0, WordCount * sizeof(StorageWordType));
}

void FNetBitArrayView::ClearPaddingBits()
{
	if (BitCount > 0)
	{
		Storage[WordCount - 1] = Storage[WordCount - 1] & GetLastWordMask();
	}
}

void FNetBitArrayView::SetAllBits()
{
	FPlatformMemory::Memset(&Storage[0], 0xff, WordCount * sizeof(StorageWordType));
	ClearPaddingBits();
}

void FNetBitArrayView::Copy(const FNetBitArrayView& Other)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, Other);
	if (GetNumWords() > 0 && Other.GetNumWords() > 0 )
	{
		FPlatformMemory::Memcpy(&Storage[0], &Other.Storage[0], WordCount * sizeof(StorageWordType));
	}
}

void FNetBitArrayView::Copy(const FNetBitArray& Other)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, Other);
	if (GetNumWords() > 0 && Other.GetNumWords() > 0)
	{
		FPlatformMemory::Memcpy(&Storage[0], Other.GetData(), WordCount * sizeof(StorageWordType));
	}
}

void FNetBitArrayView::SetBit(uint32 Index)
{
	FNetBitArrayHelper::SetBit(Storage, BitCount, Index);
}

void FNetBitArrayView::SetBitValue(uint32 Index, bool bValue)
{
	FNetBitArrayHelper::SetBitValue(Storage, BitCount, Index, bValue);
}

void FNetBitArrayView::SetBits(uint32 StartIndex, uint32 Count)
{
	FNetBitArrayHelper::SetBits(Storage, BitCount, StartIndex, Count);
}

void FNetBitArrayView::ClearBits(uint32 StartIndex, uint32 Count)
{
	FNetBitArrayHelper::ClearBits(Storage, BitCount, StartIndex, Count);
}

void FNetBitArrayView::ClearBit(uint32 Index) const
{
	FNetBitArrayHelper::ClearBit(Storage, BitCount, Index);
}

bool FNetBitArrayView::GetBit(uint32 Index) const
{
	return FNetBitArrayHelper::GetBit(Storage, BitCount, Index);
}

void FNetBitArrayView::OrBitValue(uint32 Index, bool bValue)
{
	FNetBitArrayHelper::OrBitValue(Storage, BitCount, Index, bValue);
}

void FNetBitArrayView::AndBitValue(uint32 Index, bool bValue)
{
	FNetBitArrayHelper::AndBitValue(Storage, BitCount, Index, bValue);
}

uint32 FNetBitArrayView::FindFirstZero() const
{
	return FNetBitArrayHelper::FindFirstZero(Storage, WordCount, BitCount);
}

uint32 FNetBitArrayView::FindFirstOne() const
{
	return FNetBitArrayHelper::FindFirstOne(Storage, WordCount, BitCount);
}

uint32 FNetBitArrayView::FindFirstZero(uint32 StartIndex) const
{
	return FNetBitArrayHelper::FindFirstZero(Storage, WordCount, BitCount, StartIndex);
}

uint32 FNetBitArrayView::FindFirstOne(uint32 StartIndex) const
{
	return FNetBitArrayHelper::FindFirstOne(Storage, WordCount, BitCount, StartIndex);
}

inline uint32 FNetBitArrayView::FindLastZero() const
{
	return FNetBitArrayHelper::FindLastZero(Storage, WordCount, BitCount);
}

inline uint32 FNetBitArrayView::FindLastOne() const
{
	return FNetBitArrayHelper::FindLastOne(Storage, WordCount, BitCount);
}

uint32 FNetBitArrayView::GetNumWords() const
{
	return WordCount;
}

uint32 FNetBitArrayView::GetNumBits() const
{
	return BitCount;
}

inline uint32 FNetBitArrayView::GetSetBitIndices(uint32 StartIndex, uint32 Count, uint32* OutIndices, uint32 OutIndicesCapacity) const
{
	return FNetBitArrayHelper::GetSetBitIndices(Storage, BitCount, StartIndex, Count, OutIndices, OutIndicesCapacity);
}

inline uint32 FNetBitArrayView::CountSetBits(uint32 StartIndex, uint32 Count) const
{
	return FNetBitArrayHelper::CountSetBits(Storage, BitCount, StartIndex, Count);
}

template<typename WordOpFunctor> inline void FNetBitArrayView::Set(const FNetBitArrayView& First, WordOpFunctor&& WordOp, const FNetBitArrayView& Second)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, First);
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, Second);
	FNetBitArrayHelper::Set(GetData(), First.GetData(), Second.GetData(), GetNumWords(), WordOp);
}

template<typename WordOpFunctor> inline void FNetBitArrayView::Combine(const FNetBitArrayView& Other, WordOpFunctor&& WordOp)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, Other);
	FNetBitArrayHelper::Combine(Storage, Other.Storage, WordCount, WordOp);
}

template<typename WordOpFunctor1, typename WordOpFunctor2> inline void FNetBitArrayView::CombineMultiple(WordOpFunctor1&& Op, const FNetBitArrayView& ArrayA, WordOpFunctor2&& Op2, const FNetBitArrayView& ArrayB)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, ArrayA);
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(*this, ArrayB);
	FNetBitArrayHelper::CombineMultiple(Storage, ArrayA.Storage, ArrayB.Storage, WordCount, Op, Op2);
}

template<typename T> inline void FNetBitArrayView::ForAllSetBits(T&& Functor) const
{
	FNetBitArrayHelper::ForAllSetBits(Storage, WordCount, BitCount, Functor);
}

template<typename T> inline void FNetBitArrayView::ForAllSetBitsInRange(const uint32 StartBit, const uint32 EndBit, T&& Functor) const
{
	FNetBitArrayHelper::ForAllSetBitsInRange(Storage, WordCount, BitCount, StartBit, EndBit, Functor);
}

template<typename T> inline void FNetBitArrayView::ForAllUnsetBits(T&& Functor) const
{
	FNetBitArrayHelper::ForAllUnsetBits(Storage, WordCount, BitCount, Functor);
}

template <typename T, typename V> 
inline void FNetBitArrayView::ForAllSetBits(const FNetBitArrayView& A, const FNetBitArrayView& B, T&& WordOpFunctor, V&& Functor)
{
	UE_NETBITARRAY_CHECK(A.GetNumBits() == B.GetNumBits());
	FNetBitArrayHelper::ForAllSetBits(A.Storage, B.Storage, A.WordCount, A.BitCount, WordOpFunctor, Functor);
}

template <typename WordOpFunctor1, typename WordOpFunctor2, typename UserFunctor> 
inline void FNetBitArrayView::ForAllSetBitsMultiple(const FNetBitArrayView& A, WordOpFunctor1&& Op1, const FNetBitArrayView& B, WordOpFunctor2&& Op2, const FNetBitArrayView& C, UserFunctor&& Functor)
{
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(A, B);
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(B, C);
	FNetBitArrayHelper::ForAllSetBitsMultiple(A.Storage, B.Storage, C.Storage, A.WordCount, A.BitCount, Op1, Op2, Functor);
}

template <typename T, typename V>
void FNetBitArrayView::ForAllExclusiveBits(const FNetBitArrayView& A, const FNetBitArrayView& B, T&& FunctorA, V&& FunctorB)
{
	UE_NETBITARRAY_CHECK(A.GetNumBits() == B.GetNumBits());
	FNetBitArrayHelper::ForAllExclusiveBits(A.Storage, B.Storage, A.WordCount, A.BitCount, FunctorA, FunctorB);
}

template <typename T, typename V, typename Predicate>
void FNetBitArrayView::ForAllExclusiveBitsByPredicate(const FNetBitArrayView& A, const FNetBitArrayView& B, T&& FunctorA, V&& FunctorB, Predicate&& Pred)
{
	UE_NETBITARRAY_CHECK(A.GetNumBits() == B.GetNumBits());
	FNetBitArrayHelper::ForAllExclusiveBitsByPredicate(A.Storage, B.Storage, A.WordCount, A.BitCount, FunctorA, FunctorB, Pred);
}

inline Private::FNetBitArrayRangedForConstIterator FNetBitArrayView::begin() const
{
	return Private::FNetBitArrayRangedForConstIterator(Storage, BitCount, Private::FNetBitArrayRangedForConstIterator::ERangeStart::Begin);
}

inline Private::FNetBitArrayRangedForConstIterator FNetBitArrayView::end() const
{
	return Private::FNetBitArrayRangedForConstIterator(Storage, BitCount, Private::FNetBitArrayRangedForConstIterator::ERangeStart::End);
}

/** Transform any raw buffer into a FNetBitArrayView */
inline FNetBitArrayView MakeNetBitArrayView(const FNetBitArrayView::StorageWordType* Storage, uint32 BitCount)
{
	return FNetBitArrayView(const_cast<FNetBitArrayView::StorageWordType*>(Storage), BitCount);
}

inline FNetBitArrayView MakeNetBitArrayView(const FNetBitArrayView::StorageWordType* Storage, uint32 BitCount, const FNetBitArrayBase::ENoResetNoValidateType)
{
	return FNetBitArrayView(const_cast<FNetBitArrayView::StorageWordType*>(Storage), BitCount, FNetBitArrayBase::NoResetNoValidate);
}

/** Transform a FNetBitArray into a FNetBitArrayView */
inline FNetBitArrayView MakeNetBitArrayView(const FNetBitArray& BitArray)
{
	return FNetBitArrayView(const_cast<FNetBitArrayView::StorageWordType*>(BitArray.GetData()), BitArray.GetNumBits());
}

/** Transform a FNetBitArray into a FNetBitArrayView without validating the contents of the bit array received */
inline FNetBitArrayView MakeNetBitArrayView(FNetBitArray& BitArray, const FNetBitArrayBase::ENoResetNoValidateType)
{
	return FNetBitArrayView(const_cast<FNetBitArrayView::StorageWordType*>(BitArray.GetData()), BitArray.GetNumBits(), FNetBitArrayBase::NoResetNoValidate);
}


} // end namespace UE::Net

namespace UE::Net::Private
{


//*************************************************************************************************
// FNetBitArrayRangedForConstIterator 
// Implementation
//*************************************************************************************************

inline FNetBitArrayRangedForConstIterator::FNetBitArrayRangedForConstIterator()
{
}

inline FNetBitArrayRangedForConstIterator::FNetBitArrayRangedForConstIterator(const FNetBitArrayBase::StorageWordType* InData UE_LIFETIMEBOUND, uint32 InBitCount, FNetBitArrayRangedForConstIterator::ERangeStart RangeStart)
: Data(InData)
, BitCount(InBitCount)
, WordCount(FNetBitArrayView::CalculateRequiredWordCount(BitCount))
, CurrentBitIndex(RangeStart == ERangeStart::Begin ? 0 : BitCount)
{
	UE_NETBITARRAY_CHECK(InData != nullptr || InBitCount == 0);
	CurrentWord = (CurrentBitIndex < BitCount ? Data[0] : FNetBitArrayBase::StorageWordType(0));
	AdvanceToNextSetBit();
}

inline FNetBitArrayRangedForConstIterator& FNetBitArrayRangedForConstIterator::operator++()
{
	AdvanceToNextSetBit();
	return *this;
}

inline uint32 FNetBitArrayRangedForConstIterator::operator*() const
{
	return CurrentBitIndex;
}

inline bool FNetBitArrayRangedForConstIterator::operator!=(const FNetBitArrayRangedForConstIterator& It) const
{
	UE_NETBITARRAY_CHECK(this->Data == It.Data);
	return this->CurrentBitIndex != It.CurrentBitIndex;
}

inline void FNetBitArrayRangedForConstIterator::AdvanceToNextSetBit()
{
	using SignedWordType = typename std::make_signed<FNetBitArrayBase::StorageWordType>::type;

	uint32 WordIt = CurrentBitIndex/FNetBitArrayBase::WordBitCount;
	while (CurrentWord == 0U)
	{
		++WordIt;
		if (WordIt >= WordCount)
		{
			CurrentBitIndex = BitCount;
			return;
		}

		CurrentWord = Data[WordIt];
		// It's ok to not adjust the bit index to start on a word boundary. It's handled outside the loop.
		CurrentBitIndex += FNetBitArrayBase::WordBitCount;
	}

	const uint32 NewBitIndex = (CurrentBitIndex & ~(FNetBitArrayBase::WordBitCount - 1U)) + FPlatformMath::CountTrailingZeros(CurrentWord);
	CurrentBitIndex = FPlatformMath::Min(NewBitIndex, BitCount);

	// Clear least significant bit from CurrentWord. It doesn't matter if we're at the end if the bit array as the CurrentWord will be ignored in that case.
	const FNetBitArrayBase::StorageWordType LeastSignificantBit = CurrentWord & FNetBitArrayBase::StorageWordType(-SignedWordType(CurrentWord));
	CurrentWord ^= LeastSignificantBit;
}

} // end namespace UE::Net::Private
