// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/NetBitArray.h"
#include "Containers/UnrealString.h"

namespace UE::Net
{

/**
 * FNetBitArrayPrinter utility class
 * Collection of helper functions that can describe any given BitArray into a string for logging or debugging purposes
 */
class FNetBitArrayPrinter
{
public:

	/** Print the number of set bits in the bitarray */
	template <class T>
	static FString PrintSetSummary(const T& BitArray);

	/** Print the number of zero bits int he bitarray */
	template <class T>
	static FString PrintZeroSummary(const T& BitArray);

	/** Print the number of set bits in the bitarray and the value of each word with set bits in them */
	template <class T>
	static FString PrintSetBits(const T& BitArray);

	/** Print the number of zero bits in the bitarray and the value of each word with zero bits in them */
	template <class T>
	static FString PrintZeroBits(const T& BitArray);

	/** Print the number of bits different between each array */
	template <class T, class U>
	static FString PrintDeltaSummary(const T& BitArrayA, const U& BitArrayB);

	/** Print the number of bits different between each array and the value of each word with different bits in them*/
	template <class T, class U>
	static FString PrintDeltaBits(const T& BitArrayA, const U& BitArrayB);

private:

	FNetBitArrayPrinter() = delete;
	~FNetBitArrayPrinter() = delete;
};


//*************************************************************************************************
// FNetBitArrayPrinter 
// Implementation
//*************************************************************************************************

template <class T>
FString FNetBitArrayPrinter::PrintSetSummary(const T& BitArray)
{
	return FString::Printf(TEXT("%u bits set"), BitArray.CountSetBits());
}

template <class T>
FString FNetBitArrayPrinter::PrintZeroSummary(const T& BitArray)
{
	return FString::Printf(TEXT("%u bits zero"), BitArray.GetNumBits() - BitArray.CountSetBits());
}

template <class T>
FString FNetBitArrayPrinter::PrintSetBits(const T& BitArray)
{
	using WordType = typename T::StorageWordType;

	FString Buffer;
	Buffer.Reserve(1024);

	const uint32 BitEndIndex = BitArray.GetNumBits() - 1U;
	const uint32 WordEndIndex = BitEndIndex / T::WordBitCount;
	const WordType EndWordMask = (~WordType(0) >> (~BitEndIndex & (T::WordBitCount - 1U)));

	const WordType* BitWordData = BitArray.GetData();

	int32 TotalSetBits = 0;
	for (uint32 WordIndex = 0; WordIndex < BitArray.GetNumWords(); ++WordIndex)
	{
		const bool bIsEndWord = (WordIndex == WordEndIndex);
		const WordType WordMask = bIsEndWord ? EndWordMask : ~WordType(0);

		const WordType BitWord = BitWordData[WordIndex];

		const int32 NumSetBits = FPlatformMath::CountBits(BitWord & WordMask);

		if (NumSetBits > 0)
		{
			TotalSetBits += NumSetBits;

			Buffer += FString::Printf(TEXT(" (0x%.2x)[%#.8x]"), WordIndex, BitWord);
		}
	}

	if (Buffer.IsEmpty())
	{
		Buffer = TEXT(" none");
	}

	return FString::Printf(TEXT("%d bits set:%s"), TotalSetBits, *Buffer);
}

template <class T>
FString FNetBitArrayPrinter::PrintZeroBits(const T& BitArray)
{
	using WordType = typename T::StorageWordType;

	FString Buffer;
	Buffer.Reserve(1024);

	const uint32 BitEndIndex = BitArray.GetNumBits() - 1U;
	const uint32 WordEndIndex = BitEndIndex / T::WordBitCount;
	const WordType EndWordMask = (~WordType(0) >> (~BitEndIndex & (T::WordBitCount - 1U)));

	const WordType* BitWordData = BitArray.GetData();

	int32 TotalZeroBits = 0;
	for (uint32 WordIndex = 0; WordIndex < BitArray.GetNumWords(); ++WordIndex)
	{
		const bool bIsEndWord = (WordIndex == WordEndIndex);
		const WordType WordMask = bIsEndWord ? EndWordMask : ~WordType(0);

		const WordType BitWord = BitWordData[WordIndex];
		const WordType FlipBitWord = ~(BitWord & WordMask);

		const int32 NumZeroBits = FPlatformMath::CountBits(FlipBitWord);

		if (NumZeroBits > 0)
		{
			TotalZeroBits += NumZeroBits;

			Buffer += FString::Printf(TEXT(" (0x%.2x)[%#.8x]"), WordIndex, BitWord);
		}
	}

	if (Buffer.IsEmpty())
	{
		Buffer = TEXT(" none");
	}

	return FString::Printf(TEXT("%d bits zero:%s"), TotalZeroBits, *Buffer);
}

template <class T, class U>
FString FNetBitArrayPrinter::PrintDeltaSummary(const T& BitArrayA, const U& BitArrayB)
{
	using WordTypeT = typename T::StorageWordType;
	using WordTypeU = typename U::StorageWordType;
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(BitArrayA, BitArrayB);

	const WordTypeT* BitWordDataA = BitArrayA.GetData();
	const WordTypeU* BitWordDataB = BitArrayB.GetData();

	int32 DiffNumBits = 0;
	for (uint32 WordIndex = 0; WordIndex < BitArrayA.GetNumWords(); ++WordIndex)
	{
		const WordTypeT BitWordA = BitWordDataA[WordIndex];
		const WordTypeU BitWordB = BitWordDataB[WordIndex];

		// XOR both words
		const WordTypeT DeltaBits = BitWordA ^ BitWordB;

		DiffNumBits += FPlatformMath::CountBits(DeltaBits);
	}

	return FString::Printf(TEXT("%d bits differ"), DiffNumBits);
}

template <class T, class U>
FString FNetBitArrayPrinter::PrintDeltaBits(const T& BitArrayA, const U& BitArrayB)
{
	using WordTypeT = typename T::StorageWordType;
	using WordTypeU = typename U::StorageWordType;
	UE_NETBITARRAY_VALIDATE_BOTH_COMPATIBLE(BitArrayA, BitArrayB);

	FString Buffer;
	Buffer.Reserve(1024);

	const WordTypeT* BitWordDataA = BitArrayA.GetData();
	const WordTypeU* BitWordDataB = BitArrayB.GetData();

	int32 DiffNumBits = 0;
	for (uint32 WordIndex = 0; WordIndex < BitArrayA.GetNumWords(); ++WordIndex)
	{
		const WordTypeT BitWordA = BitWordDataA[WordIndex];
		const WordTypeU BitWordB = BitWordDataB[WordIndex];

		const WordTypeT DeltaBits = BitWordA ^ BitWordB;

		const int32 DiffBits = FPlatformMath::CountBits(DeltaBits);

		if (DiffBits > 0)
		{
			DiffNumBits += DiffBits;

			Buffer += FString::Printf(TEXT(" (0x%.2x)[%#.8x]vs[%#.8x]"), WordIndex, BitWordA, BitWordB);
		}
	}

	if (Buffer.IsEmpty())
	{
		Buffer = TEXT(" none");
	}

	return FString::Printf(TEXT("%d bits differ:%s"), DiffNumBits, *Buffer);
}

} // end namespace UE::Net