// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/TypeHash.h"
#include "UObject/ObjectKey.h"

namespace UE::Net
{
	class FNetHandle;
	class FNetHandleManager;
	namespace Private
	{
		class FNetPushObjectHandle;
	}
}

NETCORE_API FAnsiStringBuilderBase& operator<<(FAnsiStringBuilderBase& Builder, const UE::Net::FNetHandle& NetHandle);
NETCORE_API FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, const UE::Net::FNetHandle& NetHandle);
NETCORE_API FWideStringBuilderBase& operator<<(FWideStringBuilderBase& Builder, const UE::Net::FNetHandle& NetHandle);

namespace UE::Net
{

/**
 * FNetHandle can be used to uniquely identify a replicated object for the lifetime of the application.
 */

class FNetHandle
{
public:
#if UE_WITH_REMOTE_OBJECT_HANDLE
	static constexpr uint64 IdBits = (REMOTE_OBJECT_SERVER_ID_BIT_SIZE + REMOTE_OBJECT_SERIAL_NUMBER_BIT_SIZE);
#else
	static constexpr uint64 IdBits = 30;
#endif

	static_assert(IdBits <= 64, "The number of bits representing the FNetHandle id cannot be greater than 64-bits");

	FNetHandle();
	FNetHandle(const FNetHandle& Other);
	FNetHandle(FNetHandle&& Other);

	FNetHandle& operator=(const FNetHandle& Other);
	FNetHandle& operator=(FNetHandle&& Other);

	uint64 GetId() const;
	bool IsValid() const;

	bool IsCompleteHandle() const;

	bool operator==(FNetHandle Other) const;
	bool operator<(FNetHandle Other) const;
	bool operator!=(FNetHandle Other) const;

	NETCORE_API FString ToString() const;

private:
	friend FNetHandleManager;
	friend Private::FNetPushObjectHandle;

	friend uint32 GetTypeHash(FNetHandle Handle);

	struct FInternalValue
	{
		uint32 Id;
		uint32 Epoch;
	};

	FNetHandle(FObjectKey ObjectKey);
	explicit FNetHandle(FInternalValue InInternalValue);
	explicit FNetHandle(uint64 InInternalValue);

	uint64 GetInternalValue() const;

	union
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		FRemoteObjectId Value;
#else
		FObjectKey Value;
#endif
		FInternalValue InternalValue;
	};
};

inline FNetHandle::FNetHandle()
: Value()
{
}

inline FNetHandle::FNetHandle(const	FNetHandle& Other)
: Value(Other.Value)
{
}

inline FNetHandle::FNetHandle(FNetHandle&& Other)
: Value(Other.Value)
{
}

inline FNetHandle::FNetHandle(FObjectKey ObjectKey)
#if UE_WITH_REMOTE_OBJECT_HANDLE
: Value(ObjectKey.GetRemoteId())
#else
: Value(ObjectKey)
#endif
{
}

inline FNetHandle::FNetHandle(FInternalValue InInternalValue)
#if UE_WITH_REMOTE_OBJECT_HANDLE
: Value(BitCast<FRemoteObjectId>(InInternalValue))
#else
: Value(BitCast<FObjectKey>(InInternalValue))
#endif
{
}

inline FNetHandle::FNetHandle(uint64 InInternalValue)
#if UE_WITH_REMOTE_OBJECT_HANDLE
: Value(BitCast<FRemoteObjectId>(InInternalValue))
#else
: Value(BitCast<FObjectKey>(InInternalValue))
#endif
{
}

inline uint64 FNetHandle::GetInternalValue() const
{
	return BitCast<uint64>(Value);
}

inline FNetHandle& FNetHandle::operator=(const FNetHandle& Other)
{
	Value = Other.Value;
	return *this;
}

inline FNetHandle& FNetHandle::operator=(FNetHandle&& Other)
{
	Value = Other.Value;
	return *this;
}

inline uint64 FNetHandle::GetId() const
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	return (Value.GetIdNumber() >> REMOTE_OBJECT_RESERVED_BIT_SIZE);
#else
	return BitCast<FInternalValue>(Value).Id;
#endif
}

inline bool FNetHandle::IsValid() const
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	return Value != FRemoteObjectId();
#else
	return Value != FObjectKey();
#endif
}

inline bool FNetHandle::IsCompleteHandle() const
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	return Value != FRemoteObjectId();
#else
	return Value != FObjectKey();
#endif
}

inline bool FNetHandle::operator==(FNetHandle Other) const
{
	return GetId() == Other.GetId();
}

inline bool FNetHandle::operator<(FNetHandle Other) const
{
	return GetId() < Other.GetId();
}

inline bool FNetHandle::operator!=(FNetHandle Other) const
{
	return GetId() != Other.GetId();
}

inline uint32 GetTypeHash(FNetHandle Handle)
{
	return GetTypeHash(Handle.Value);
}

}

