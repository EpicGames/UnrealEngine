// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/NetHandle/NetHandle.h"

class FNetCoreModule;
class UReplicationBridge;

namespace UE::Net
{

/** This class restricts access to DestroyNetHandle functionality as early destruction prevents further dirty state tracking. */
class FNetHandleDestroyer
{
private:
	/** Calls FNetHandleManager::DestroyNetHandle(Handle). */
	static void DestroyNetHandle(FNetHandle Handle);

private:
	/** Friends */
	friend UReplicationBridge;
};

/** Manages the association between a UObject and a FNetHandle, which can be used to uniquely identify a replicated object for the lifetime of the application. */
class FNetHandleManager
{
public:
	/** Returns the NetHandle for an object if it's present. Assumes Init() has been called. */
	NETCORE_API static FNetHandle GetNetHandle(const UObject*);

	/** Returns an existing NetHandle for an object if it's present or creates one if it's not. Assumes Init() has been called. */
	NETCORE_API static FNetHandle GetOrCreateNetHandle(const UObject*);

	/** Returns a NetHandle given an Id that has previously been extracted from a valid NetHandle. The behavior is implementation defined if the Id was not extracted from a valid NetHandle. */
	NETCORE_API static FNetHandle MakeNetHandleFromId(uint32 Id);

private:
	friend FNetHandleDestroyer;

	/** Destroys a NetHandle and removes the association between it and the object such that GetObject(Handle) will return nullptr. Assumes Init() has been called. */
	NETCORE_API static void DestroyNetHandle(FNetHandle Handle);

protected:
	friend FNetCoreModule;

	/** Creates a new instance that is used by functions that need it. Checks that no instance exists. */
	static void Init();

	/** Destroys the instance if it exists. */
	static void Deinit();

private:
	class FPimpl;
	static FPimpl* Instance;
};

inline void FNetHandleDestroyer::DestroyNetHandle(FNetHandle Handle)
{
	FNetHandleManager::DestroyNetHandle(Handle);
}

}
