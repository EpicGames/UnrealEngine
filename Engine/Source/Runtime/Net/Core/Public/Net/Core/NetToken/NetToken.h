// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Templates/TypeHash.h"
#include "Logging/LogMacros.h"

NETCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogNetToken, Log, All);

namespace UE::Net
{
	class FNetTokenStore;
	class FNetTokenStoreState;
}

namespace UE::Net
{

class FNetToken
{
public:
	typedef uint32 FTypeId;

	enum : uint32 { Invalid = 0U };

	enum : uint32 { InvalidTokenTypeId = ~FTypeId(0) };
	enum : uint32 { InvalidTokenIndex = 0U };

	/** How many bits we use to express the TypeId for NetTokens, Increasing this value will break network compatibility and might need versioning for replays. */
	enum : uint32 { TokenTypeIdBits = 3U };

	/** How many bits we use to express the index part of NetTokens */
	enum : uint32 { TokenBits = 20U };

	enum : uint32 { MaxTypeIdCount = 1U << TokenTypeIdBits };
	enum : uint32 { MaxNetTokenCount = 1U << TokenBits };

	enum class ENetTokenAuthority
	{
		None,
		Authority,
	};

public:	
	FNetToken()
	: Value(Invalid)
	{
	}

	inline bool IsValid() const
	{
		return Index != InvalidTokenIndex;
	}

	bool IsAssignedByAuthority() const
	{
		return bIsAssignedByAuthority != 0U;
	}

	uint32 GetIndex() const
	{
		return Index;
	}

	FTypeId GetTypeId() const
	{
		return TypeId;
	}

	bool operator==(const FNetToken& Other) const
	{
		return Value == Other.Value;
	}

	FString ToString() const;

	friend uint32 GetTypeHash(const FNetToken& Token)
	{
		return ::GetTypeHash(Token.Value);
	}

private:
	friend class UE::Net::FNetTokenStore;

	explicit FNetToken(uint32 InTypeId, uint32 InIndex, ENetTokenAuthority Authority)
	{
		Padding = 0U;
		TypeId = InTypeId;
		Index = InIndex;
		bIsAssignedByAuthority = Authority == ENetTokenAuthority::Authority ? 1U : 0U;
	}

private:

	union 
	{
		struct
		{
			uint32 Index : TokenBits;
			uint32 TypeId : TokenTypeIdBits;
			uint32 bIsAssignedByAuthority : 1U;
			uint32 Padding : 32 - TokenTypeIdBits - TokenBits - 1U;
		};
		uint32 Value;
	};
};

// Contains necessary context to resolve NetTokens
class FNetTokenResolveContext
{
public:
	FNetTokenStore* NetTokenStore = nullptr;
	const FNetTokenStoreState* RemoteNetTokenStoreState = nullptr;
};

inline FString FNetToken::ToString() const
{
	FString Result;
	Result = FString::Printf(TEXT("NetToken (Auth:%u TypeId=%u Index=%u)"), IsAssignedByAuthority(), TypeId, Index);
	return Result;
}

}

template <> struct TIsPODType<UE::Net::FNetToken> { enum { Value = true }; };
