// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "NetToken.h"

class FNetBitWriter;

namespace UE::Net
{
	class FNetTokenStore;
	class FNetTokenStoreState;
}

namespace UE::Net
{

// Contains necessary context to export NetTokens
class FNetTokenExportContext
{
public:
	typedef TArray<UE::Net::FNetToken, TInlineAllocator<4>> FNetTokenExports;

	inline FNetTokenStore* GetNetTokenStore();		
	NETCORE_API void AddNetTokenPendingExport(UE::Net::FNetToken NetToken);
	NETCORE_API void AppendNetTokensPendingExport(TArrayView<const UE::Net::FNetToken> NetTokens);
	NETCORE_API TArrayView<const UE::Net::FNetToken> GetNetTokensPendingExport() const;
	NETCORE_API static FNetTokenExportContext* GetNetTokenExportContext(FArchive& Ar);

private:
	friend class FNetTokenExportScope;

	FNetTokenExportContext(FNetTokenStore* InNetTokenStore, FNetTokenExports* InNetTokensPendingExport);

	// We can only export if we have a NetTokenStore
	FNetTokenStore* NetTokenStore = nullptr;

	// We also need a target where to store exports.
	FNetTokenExports* NetTokensPendingExport = nullptr;
};

// Simple scope to make sure we set the correct ExportContext and restore the old one when we exit the scope
class FNetTokenExportScope
{
public:
	NETCORE_API FNetTokenExportScope(FNetBitWriter& InNetBitWriter, UE::Net::FNetTokenStore* InNetTokenStore, FNetTokenExportContext::FNetTokenExports& TargetExports, const char* InDebugName = "None");
	NETCORE_API ~FNetTokenExportScope();

	FNetTokenExportContext ExportContext;
	FNetBitWriter& NetBitWriter;
	FNetTokenExportContext* OldExportContext;
	const char* DebugName;
};

FNetTokenStore* FNetTokenExportContext::GetNetTokenStore()
{ 
	return NetTokenStore; 
}

}

