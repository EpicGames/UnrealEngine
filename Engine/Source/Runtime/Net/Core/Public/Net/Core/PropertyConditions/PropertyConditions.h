// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/CoreNet.h"
#include "UObject/ObjectKey.h"

class FOutputDevice;
class FRepChangedPropertyTracker;

namespace UE::Net::Private
{

class FNetPropertyConditionManager
{
public:
	NETCORE_API FNetPropertyConditionManager();
	NETCORE_API ~FNetPropertyConditionManager();

	static NETCORE_API FNetPropertyConditionManager& Get();

	NETCORE_API void SetPropertyActive(const UObject* Object, const uint16 RepIndex, const bool bActive);
	NETCORE_API void SetPropertyActiveOverride(const UObject* Object, const uint16 RepIndex, const bool bIsActive);
	NETCORE_API void SetPropertyDynamicCondition(const UObject* Object, const uint16 RepIndex, const ELifetimeCondition Condition);
	
	NETCORE_API void NotifyObjectDestroyed(const FObjectKey ObjectKey);

	NETCORE_API TSharedPtr<FRepChangedPropertyTracker> FindOrCreatePropertyTracker(const FObjectKey ObjectKey);
	NETCORE_API TSharedPtr<FRepChangedPropertyTracker> FindPropertyTracker(const FObjectKey ObjectKey) const;

	NETCORE_API void LogMemory(FOutputDevice& Ar);

	class FAllowCreateTrackerFromSetPropertyActiveOverrideScope
	{
	public:
		FAllowCreateTrackerFromSetPropertyActiveOverrideScope(FNetPropertyConditionManager& ConditionManager)
		: GuardValue(ConditionManager.bAllowCreateTrackerFromSetPropertyActiveOverride, true) {};
	private:
		TGuardValue<bool> GuardValue;
	};

private:
	friend class FAllowCreateTrackerFromSetPropertyActiveOverrideScope;

	void PostGarbageCollect();

	FDelegateHandle PostGarbageCollectHandle;

	TMap<FObjectKey, TSharedPtr<FRepChangedPropertyTracker>> PropertyTrackerMap;

	FObjectKey LastFoundTrackerKey;
	FRepChangedPropertyTracker* LastFoundTracker = nullptr;
	bool bAllowCreateTrackerFromSetPropertyActiveOverride = false;
};

}; // UE::Net::Private
