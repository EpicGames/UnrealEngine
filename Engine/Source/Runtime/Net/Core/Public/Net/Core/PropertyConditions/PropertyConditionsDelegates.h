// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

enum ELifetimeCondition : int;

namespace UE::Net::Private
{

class FPropertyConditionDelegates
{
public:
	using FOnPropertyCustomConditionChanged = TMulticastDelegate<void(const UObject* OwningObject, const uint16 RepIndex, const bool bActive)>;
	using FOnPropertyDynamicConditionChanged = TMulticastDelegate<void(const UObject* OwningObject, const uint16 RepIndex, const ELifetimeCondition Condition)>;
	
	static FOnPropertyCustomConditionChanged& GetOnPropertyCustomConditionChangedDelegate() { return OnPropertyCustomConditionChangedDelegate; }
	static FOnPropertyDynamicConditionChanged& GetOnPropertyDynamicConditionChangedDelegate() { return OnPropertyDynamicConditionChangedDelegate; }

private:
	static NETCORE_API FOnPropertyCustomConditionChanged OnPropertyCustomConditionChangedDelegate;
	static NETCORE_API FOnPropertyDynamicConditionChanged OnPropertyDynamicConditionChangedDelegate;
};

} // UE::Net::Private

