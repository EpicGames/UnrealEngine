// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "UObject/CoreNet.h"
#include "Delegates/Delegate.h"

class FArchive;
class UObject;

/**
 * This class is used to store meta data about properties that is shared between connections,
 * including whether or not a given property is Conditional, Active, and any external data
 * that may be needed for Replays.
 *
 * TODO: This class (and arguably IRepChangedPropertyTracker) should be renamed to reflect
 *			what they actually do now.
 */
class FRepChangedPropertyTracker
{
public:
	FRepChangedPropertyTracker() = delete;
	FRepChangedPropertyTracker(FCustomPropertyConditionState&& InActiveState);

	~FRepChangedPropertyTracker() = default;
	
	void CountBytes(FArchive& Ar) const;

	bool IsParentActive(uint16 ParentIndex) const
	{
		return ActiveState.GetActiveState(ParentIndex);
	}

	int32 GetParentCount() const
	{
		return ActiveState.GetNumProperties();
	}

	ELifetimeCondition GetDynamicCondition(uint16 ParentIndex) const
	{
		return ActiveState.GetDynamicCondition(ParentIndex);
	}

	uint32 GetDynamicConditionChangeCounter() const
	{
		return ActiveState.GetDynamicConditionChangeCounter();
	}

private:
	friend UE::Net::Private::FNetPropertyConditionManager;

	// Called from FNetPropertyConditionManager 
	void SetDynamicCondition(const UObject* OwningObject, const uint16 RepIndex, const ELifetimeCondition Condition);
	void SetCustomIsActiveOverride(const UObject* OwningObject, const uint16 RepIndex, const bool bIsActive);

	/** Activation data for top level Properties on the given Actor / Object. */
	FCustomPropertyConditionState ActiveState;
};
