// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PushModelMacros.h"

#if WITH_PUSH_MODEL

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "UObject/ObjectKey.h"

/**
 * Push Model Support for networking.
 *
 * Note: While the theoretical gains for Push Model are good, in practice the gains seen haven't been as good as expected.
 *			More work may be needed in order to optimize comparisons, and have other systems buy in / use Push Model in order
 *			to increase gains.
 *
 ****************************************************************
 * Rationale
 ****************************************************************
 *
 * The intention of Push Model is to provide a fundamental trade off in the way UE property replication works, without changing
 * the underlying replication machinery.
 *
 * Traditionally, UE Networking works by having coders and designers mark properties as replicated, and then having the network
 * system periodically compare the values of those properties to their last sent state. When we detect a change in the state,
 * then we will send that data to clients.
 *
 * This system works great from the perspective that if there's a change, clients are going to receive it with minimal effort from
 * designers and coders.
 *
 * Where the system starts to fall down is in terms of performance.
 *
 * Because the engine has no way to know if a given Object is "dirty", that means there's no choice but to compare every property
 * on every object. This sort of overhead isn't a big deal while games are relatively small, but as they start to scale up, this
 * can become a bottleneck.
 *
 * To abate this, there's a bunch of machinery UE provides, things like:
 *
 *	Net Frequency
 *		Gives devs the ability to tune how often actors are considered for replication.
 *
 *	Net Dormancy, Replication Keys, Replication Pausing, and Net Relevancy
 *		Gives devs the ability to completely stop replication of Actors and Objects to all connections, or subsets of connections.
 *
 *	Property Conditions, Fast Array Serialization, and Rep Changed Trackers
 *		Gives devs the ability to completely stop replication of specific Properties to all connections, or subsets of connections.
 *
 *	Rep Graph
 *		Gives devs the ability to fine tune when and how Actors are replicated that is much more specific to a game or mode.
 *
 * The main focus of all of these systems is to reduce the number of Actors, Objects, and Properties we consider in an efficient way.
 * The tradeoff all of these things have in common is that it forces devs to be more cognizant of *how* and *when* things need
 * to replicate.
 *
 * Fast Arrays, Replication Keys, and Dormancy are great examples of this, and they have a very similar mechanic to Push Model.
 *
 * With Fast Arrays (see NetSerialization.h), no changes will be replicated unless devs explicitly mark a given Array Property
 * as being dirty. Beyond that, individual elements won't be sent unless they are marked dirty. This means that the replication
 * system only needs to check a few flags / keys, and if they aren't different, it doesn't need to do anything.
 *
 * With Replication Keys (see ActorChannel.h, KeyNeedsToReplicate / ReplicateSubobject), we go up a step and handle changes
 * at the Subobject level. Devs can assign an arbitrary ID and Key to a specific subobject. As they make changes to the
 * Subobject, it's their responsibility to increment or otherwise update the Key. Then, when they go to replicate the
 * subobject (via a call to ReplicateSubobject), they can check the Key for the object to see if it's changed.
 *
 * With Dormancy (see ActorChannel.h, NetDriver.h, NetConnection.h), we again go up a step and handle changes at the Actor
 * level. Devs can decide whether or not a given Actor supports Dormancy, and at what dormant level an actor starts at
 * (i.e., whether it's considered initially "active" in the game, or "inactive" in the game). When an Actor becomes Active
 * (woken from Dormancy), it will replicate one more time to all connections. After that happens, it will return to its
 * dormant state. Once Dormant, it is completely ignored by the networking system until Devs explicitly wake it via a call
 * to something like AActor::ForceNetUpdate.
 *
 * The other benefit to PushModel is that it works at the Object Level, meaning any dirty state can be pushed to / shared
 * across all Net Drivers. This means that we're potentially eliminating multiple property comparisons in cases of
 * things like Server Side Replays.
 *
 ****************************************************************
 * How does it work?
 ****************************************************************
 *
 * With Push Model, we provide a very limited and straightforward interface for allowing Devs to notify the Networking
 * System when they change the value of a property. There are some hooks to help do this automatically for Blueprints,
 * but if they are using things like Blueprint Setters, passing values around by reference, or setting values in Native
 * C++, then it is completely up to them to call the necessary methods.
 *
 * UFUNCTIONS or Blueprint Defined Functions that take Properties by reference will check for Networked Properties and
 * should add nodes automatically (see UNetPushModelHelpers). This is **only** true when they are called from Blueprints!!
 *
 * Devs don't need to care *if* a given object is actually networked or is actively replicating, Push Model handles
 * that sort of tracking internally. As such, we don't expect users to ever need to know or have a reference to
 * any information other than A) The Object that's being dirtied and B) The Property that's being dirtied.
 * Whenever a Property's value is changed, a call to one of the MARK_PROPERTY_DIRTY macros should be added.
 * This will forward the request to Push Model, and it will handle the rest.
 *
 * The preferred way of achieving this is by putting any Replicated Properties behind Getters and Setters.
 * That way, these changes and be made in a single or very limited subset of implementation code, and callers
 * never have to know what's going on.
 *
 * Behind the scenes, Push Model does very simple book keeping, and will forward these dirty notifications on to the
 * networking systems as necessary. The next time an object is replicated, we can simply check a few flags to determine
 * whether or not a given property is dirty. If it is, then we can compare and replicate it normally, otherwise we
 * can quickly skip it.
 *
 * One important thing to note is that at this time, Push Model only supports Top Level properties of Objects
 * (in RepLayout parlance, Parent Commands).
 *
 * So, if changes are made to Structs, Containers, or nested properties therein, the owning Struct or Container
 * property must be marked dirty.
 *
 * This also means that we *must* compare a property when it's dirty to make sure we know exactly what's changed.
 * Optimizations could be made for "network primtiive" types where we skip comparisons if desired, at the potential
 * expense of extra bandwidth for properties that didn't actually change.
 *
 ****************************************************************
 * Examples
 ****************************************************************
 *
 * NOTE: Struct and Object markup excluded to prevent issues from UHT.
 */

#if 0
USTRUCT()
struct FMyAwesomeStruct
{
	UPROPERTY()
	int32 SomeReplicatedProperty;

	UPROPERTY()
	bool bSomeOtherReplicatedProperty;
}

UCLASS()
class AMyAwesomeActor : public AActor
{
	GENERATED_BODY()

	/** Convenience methods to make sure we always capture changes to property state. */
	void SetMyReplicatedBool(const bool bNewValue)
	{
		bMyReplicatedBool = bNewValue;
		MARK_PROPERTY_DIRTY_FROM_NAME(AMyAwesomeActor, bMyReplicatedBool, this);
	}

	/**
	 * Typically, calls to Set <Property> in blueprint will be handled automatically.
	 * However, since this is just an exposed Blueprint Function (especially since it is callable
	 * from native code), we should make the property dirty.
	 */
	UFUNCTION(BlueprintCallable)
	void SetMyBlueprintProperty(const int32 NewValue)
	{
		MyBlueprintProperty = NewValue;
		MARK_PROPERTY_DIRTY_FROM_NAME(AMyAwesomeActor, MyBlueprintProperty, this);
	}

	/**
	 * Since we have an explicit Blueprint Setter, this won't automatically be handled
	 * anymore, and we need to mark it dirty.
	 */
	void MySetter(const FString& NewValue)
	{
		MyBlueprintSetterProperty = NewValue;
		MARK_PROPERTY_DIRTY_FROM_NAME(AMyAwesomeActor, MyBlueprintSetterProperty, this);
	}

	/** For static arrays, each element will have its own RepIndex, and so we need to pass that along. */
	void SetStaticArrayValue(const int32 NewValue, const int32 Index)
	{
		MyStaticArray[Index] = NewValue;
		MARK_PROPERTY_DIRTY_FROM_NAME_STATIC_ARRAY_INDEX(AMyAwesomeActor, MyStaticArray, Index, this);
	}

	void SetEntireStaticArray(const int32 NewArray[4])
	{
		MyStaticArray = NewArray;
		MARK_PROPERTY_DIRTY_FROM_NAME_STATIC_ARRAY(AMyAwesomeActor, MyStaticArray, this);
	}

	/**
	 * Currently, there is no support for marking individual properties within structs or dynamic array indices as dirty.
	 * So, we just need to mark the top level Struct or Array as dirty.
	 * This will cause us to compare the entire struct / array next replication.
	 */
	void UpdateMyStruct(int32 NewValue)
	{
		MyStruct.SomeReplicatedProperty = NewValue;
		MARK_PROPERTY_DIRTY_FROM_NAME(AMyAwesomeActor, MyStruct, this);
	}

	void UpdateMyStruct(bool bNewValue)
	{
		MyStruct.bSomeOtherReplicatedProperty = bNewValue;
		MARK_PROPERTY_DIRTY_FROM_NAME(AMyAwesomeActor, MyStruct, this);
	}

	/**
	 *!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	 *!!!!!!!!!!!!!! Warning !!!!!!!!!!!!!!!!!!
	 *!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	 *
	 * Always avoid returning references to Push Model Properties.
	 * When absolutely necessary, the property should be marked dirty before returning it
	 * (just assume values will be changed if you need a reference).
	 *
	 * Absolutely **never** hold onto references to Push Model Properties outside of short scopes.
	 * Holding onto references and changing values *without* marking the property dirty will cause
	 * those changes to be skipped until it is marked dirty again.
	 *
	 * If there are properties that need to have references held onto by other managers (this is not
	 * recommended in any case), then it's safest to not make them push model based.
	 */
	FMyAwesomeStruct& GetMyStruct_Mutable()
	{
		MARK_PROPERTY_DIRTY_FROM_NAME(AMyAwesomeActor, MyStruct, this);
		return MyStruct;
	}

	virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override
	{
		Super::GetLifetimeReplicatedProps(OutLifetimeProps);

		FDoRepLifetimeParams Params;

		Params.bIsPushBased = true;
		DOREPLIFETIME_WITH_PARAMS_FAST(AMyAwesomeActor, bMyReplicatedBool, Params);
		DOREPLIFETIME_WITH_PARAMS_FAST(AMyAwesomeActor, MyBlueprintProperty, Params);
		DOREPLIFETIME_WITH_PARAMS_FAST(AMyAwesomeActor, MyBlueprintSetterProperty, Params);
		DOREPLIFETIME_WITH_PARAMS_FAST(AMyAwesomeActor, MyStruct, Params);
		DOREPLIFETIME_WITH_PARAMS_FAST(AMyAwesomeActor, MyStaticArray, Params);

		Params.bIsPushBased = false;
		DOREPLIFETIME_WITH_PARAMS_FAST(AMyAwesomeActor, NonPushModelProperty, Params);
	}

private:

	/** This is a standard property that can only be set natively */
	UPROPERTY(Replicated)
	bool bMyReplicatedBool;

	/**
	 * Properties marked as BlueprintReadWrite will automatically be marked dirty when set is called.
	 * This won't work if the value is passed by reference.
	 *
	 * Note: AllowPrivateAccess is only necessary here because this property is private.
	 */
	UPROPERTY(Replicated, BlueprintReadWrite, Meta=(AllowPrivateAccess="true"))
	int32 MyBlueprintProperty;

	/**
	 * Properties with custom BlueprintSetters won't automatically be marked dirty when set is called.
	 * This means our setter will need to call it manually.
	 */
	UPROPERTY(Replicated, BlueprintSetter=SetMyBlueprintSetterProperty, Meta = (AllowPrivateAccess = "true"))
	FString MyBlueprintSetterProperty;

	UPROPERTY(Replicated)
	FMyAwesomeStruct MyStruct;

	/**
	 * Static Arrays are broken into separate replicated properties for each index, so we can flag individual
	 * indices for comparison, or the entire array.
	 */
	UPROPERTY(Replicated)
	int32 MyStaticArray[4];

	UPROPERTY(Replicated)
	int32 NonPushModelProperty;
};
#endif

// DO NOT USE METHODS IN THIS NAMESPACE DIRECTLY
// Use the Macros instead, as they respect conditional compilation.
// See PushModelMacros.h
namespace UEPushModelPrivate
{
	//~ Using int32 isn't very forward looking, but for now GUObjectArray also uses int32
	//~ so we're probably safe.

	using FNetPushPerNetDriverId = int32;
	using FNetLegacyPushObjectId = int32;
	using FNetIrisPushObjectId = uint64;

	// The ID need to be able to handle both Iris and non-Iris replication.
	struct FNetPushObjectId
	{
		FNetPushObjectId() : Value(0xFFFFFFFFFFFFFFFFULL) {}
		explicit FNetPushObjectId(FNetLegacyPushObjectId Id) { Value = 0xFFFFFFFF00000000ULL | uint32(Id); }
		explicit FNetPushObjectId(FNetIrisPushObjectId Id) { Value = Id; }

		bool IsValid() const { return Value != 0xFFFFFFFFFFFFFFFFULL; }
		bool IsIrisId() const { return uint32(Value >> 32U) != 0xFFFFFFFFU; }
		uint64 GetValue() const { return Value; }

		FNetLegacyPushObjectId GetLegacyPushObjectId() const { return FNetLegacyPushObjectId(Value & 0xFFFFFFFFU);}
		FNetIrisPushObjectId GetIrisPushObjectId() const { return Value;}

	private:
		uint64 Value;
	};

	/** A handle that can be used to refer to a Push Model Object State owned by a specific Net Driver. */
	struct FPushModelPerNetDriverHandle
	{
		FPushModelPerNetDriverHandle(const FNetPushPerNetDriverId InNetDriverId, const FNetLegacyPushObjectId InObjectId)
			: NetDriverId(InNetDriverId)
			, ObjectId(InObjectId)
		{
			// They should either both be valid, or both invalid.
			check((INDEX_NONE == ObjectId) == (INDEX_NONE == NetDriverId));
		}

		//! Used by Push Model code to find a specific Net Driver state.
		//! Note, this **does not** have a global mapping to a given NetDriver.
		//! That is, two FPushModelPerNetDriver handles that identify Push Model Object's that are both tracked
		//! by the same Net Driver may report different NetDriver IDs.
		const FNetPushPerNetDriverId NetDriverId;
		
		//! The "globally unique" ID that is used to refer to the Push Model Object.
		const FNetLegacyPushObjectId ObjectId;

		const bool IsValid() const
		{
			return INDEX_NONE != ObjectId;
		}

		static FPushModelPerNetDriverHandle MakeInvalidHandle()
		{
			return FPushModelPerNetDriverHandle(INDEX_NONE, INDEX_NONE);
		}
	};

	extern NETCORE_API bool bIsPushModelEnabled;
	extern NETCORE_API bool bMakeBpPropertiesPushModel;

	/** @return Whether or not PushModel is currently enabled. See "net.IsPushModelEnabled." */
	static const bool IsPushModelEnabled()
	{
		return bIsPushModelEnabled;
	}

	/** @return Whether or not Blueprint Properties will use Push Model. See "net.MakeBpPropertiesPushModel." */
	static const bool MakeBpPropertiesPushModel()
	{
		return bMakeBpPropertiesPushModel;
	}

	/** Are we allowed to create new push model handles */
	NETCORE_API bool IsHandleCreationAllowed();

	/** Control if we are allowed to create pushmodel handles */
	NETCORE_API void SetHandleCreationAllowed(bool bAllow);
	
	static FString ToString(const FNetPushObjectId Id)
	{
		return FString::Printf(TEXT("0x%" UINT64_X_FMT), Id.GetValue());
	}

	/**
	 * Marks a specified property as dirty, causing it to be compared the next time the owner is considered
	 * for Replication.
	 *
	 * Note, C-Style Array Properties (Static Array is the name used everywhere else here), will
	 * have a RepIndex equal to UProperty::RepIndex + ArrayIndex.
	 *
	 * MARK_PROPERTY_DIRTY_FROM_NAME and MARK_PROPERTY_DIRTY_FROM_NAME_STATIC_ARRAY are preferred over
	 * MARK_PROPERTY_DIRTY and MARK_PROPERTY_DIRTY_STATIC_ARRAY, because they offer compile time warnings
	 * for things like invalid Class, Property, Static Array Index, etc., and they'll use compile time
	 * constants instead of relying on UProperty::RepIndex.
	 *
	 * @param Object	The object we're marking dirty.
	 * @param ObjectId	The push ID for the object we're marking dirty.
	 * @param RepIndex	The index of the property we're marking dirty. UProperty::RepIndex.
	 */
	NETCORE_API void MarkPropertyDirty(const UObject* Object, const FNetPushObjectId ObjectId, const int32 RepIndex);

	/**
	 * Marks a range of properties as dirty, causing it to be compared the next time the owner is considered
	 * for Replication.
	 *
	 * This is primarily used to mark entire Static Arrays 
	 *
	 * MARK_PROPERTY_DIRTY_FROM_NAME and MARK_PROPERTY_DIRTY_FROM_NAME_STATIC_ARRAY are preferred over
	 * MARK_PROPERTY_DIRTY and MARK_PROPERTY_DIRTY_STATIC_ARRAY, because they offer compile time warnings
	 * for things like invalid Class, Property, Static Array Index, etc., and they'll use compile time
	 * constants instead of relying on UProperty::RepIndex.
	 *
	 * @param Object	The object we're marking dirty.
	 * @param ObjectId	The push ID for the object we're marking dirty.
	 * @param RepIndex	The index of the property we're marking dirty. UProperty::RepIndex.
	 */
	NETCORE_API void MarkPropertyDirty(const UObject* Object, const FNetPushObjectId ObjectId, const int32 StartRepIndex, const int32 EndRepIndex);

	//~ As the comments above state, none of the methods in this namespace should be invoked directly.
	//~ Particularly, the methods below are **only** needed by internal systems.
	//~ When Push Model is enabled, registration is handled automatically by the networking system, so no
	//~ extra dev work is required.

	NETCORE_API const FPushModelPerNetDriverHandle AddPushModelObject(const FObjectKey ObjectId, const uint16 NumberOfReplicatedProperties);
	NETCORE_API void RemovePushModelObject(const FPushModelPerNetDriverHandle Handle);

	NETCORE_API class FPushModelPerNetDriverState* GetPerNetDriverState(const FPushModelPerNetDriverHandle Handle);

	NETCORE_API bool DoesHaveDirtyPropertiesOrRecentlyCollectedGarbage(const FPushModelPerNetDriverHandle Handle);

	NETCORE_API bool ValidateObjectIdReassignment(FNetLegacyPushObjectId CurrentId, FNetLegacyPushObjectId NewId);

	NETCORE_API void LogMemory(FOutputDevice& Ar);

	// For internal use only.
	DECLARE_DELEGATE_ThreeParams(FIrisMarkPropertyDirty, const UObject*, UEPushModelPrivate::FNetIrisPushObjectId, const int32);
	DECLARE_DELEGATE_FourParams(FIrisMarkPropertiesDirty, const UObject*, UEPushModelPrivate::FNetIrisPushObjectId, const int32, const int32);

	NETCORE_API void SetIrisMarkPropertyDirtyDelegate(const FIrisMarkPropertyDirty& Delegate);
	NETCORE_API void SetIrisMarkPropertiesDirtyDelegate(const FIrisMarkPropertiesDirty& Delegate);
}


#define CONDITIONAL_ON_PUSH_MODEL(Work) if (UEPushModelPrivate::IsPushModelEnabled()) { Work; }
#define IS_PUSH_MODEL_ENABLED() UEPushModelPrivate::IsPushModelEnabled()
#define PUSH_MAKE_BP_PROPERTIES_PUSH_MODEL() (UEPushModelPrivate::IsPushModelEnabled() && UEPushModelPrivate::MakeBpPropertiesPushModel())

#define GET_PROPERTY_REP_INDEX(ClassName, PropertyName) (int32)ClassName::ENetFields_Private::PropertyName
#define GET_PROPERTY_REP_INDEX_STATIC_ARRAY_START(ClassName, PropertyName) ((int32)ClassName::ENetFields_Private::PropertyName ## _STATIC_ARRAY)
#define GET_PROPERTY_REP_INDEX_STATIC_ARRAY_END(ClassName, PropertyName) ((int32)ClassName::ENetFields_Private::PropertyName ## _STATIC_ARRAY_END)
#define GET_PROPERTY_REP_INDEX_STATIC_ARRAY_INDEX(ClassName, PropertyName, ArrayIndex) (GET_PROPERTY_REP_INDEX_STATIC_ARRAY_START(ClassName, PropertyName) + ArrayIndex)

#define IS_PROPERTY_REPLICATED(Property) (0 != (EPropertyFlags::CPF_Net & Property->PropertyFlags))

#define CONDITIONAL_ON_OBJECT_NET_ID(Object, Work) { const UEPushModelPrivate::FNetPushObjectId PrivatePushId(Object->GetNetPushId()); Work; }
#define CONDITIONAL_ON_OBJECT_NET_ID_DYNAMIC(Object, Work) { const UEPushModelPrivate::FNetPushObjectId PrivatePushId(Object->GetNetPushIdDynamic()); Work; }
#define CONDITIONAL_ON_REP_INDEX_AND_OBJECT_NET_ID(Object, Property, Work) if (IS_PROPERTY_REPLICATED(Property)) { const UEPushModelPrivate::FNetPushObjectId PrivatePushId(Object->GetNetPushIdDynamic()); Work; }

//~ For these macros, we won't bother checking if Push Model is enabled. Instead, we'll just check to see whether or not the Custom ID is valid.

// Marks a property dirty by RepIndex without doing additional rep index validation.
#define MARK_PROPERTY_DIRTY_UNSAFE(Object, RepIndex) CONDITIONAL_ON_OBJECT_NET_ID_DYNAMIC(Object, UEPushModelPrivate::MarkPropertyDirty(Object, PrivatePushId, RepIndex))

// Marks a property dirty by UProperty*, validating that it's actually a replicated property.
#define MARK_PROPERTY_DIRTY(Object, Property) CONDITIONAL_ON_REP_INDEX_AND_OBJECT_NET_ID(Object, Property, UEPushModelPrivate::MarkPropertyDirty(Object, PrivatePushId, Property->RepIndex))


// Marks a static array property dirty given, the Object, UProperty*, and Index.
#define MARK_PROPERTY_DIRTY_STATIC_ARRAY_INDEX(Object, Property, ArrayIndex) CONDITIONAL_ON_REP_INDEX_AND_OBJECT_NET_ID(Object, Property, UEPushModelPrivate::MarkPropertyDirty(Object, PrivatePushId, Property->RepIndex + ArrayIndex))

// Marks all elements of a static array property dirty, given the Object and UProperty*
#define MARK_PROPERTY_DIRTY_STATIC_ARRAY(Object, Property) CONDITIONAL_ON_REP_INDEX_AND_OBJECT_NET_ID(Object, Property, UEPushModelPrivate::MarkPropertyDirty(Object, PrivatePushId, Property->RepIndex, Property->RepIndex + Property->ArrayDim - 1))


// Marks a property dirty, given the Class Name, Property Name, and Object. This will fail to compile if the Property or Class aren't valid.
#define MARK_PROPERTY_DIRTY_FROM_NAME(ClassName, PropertyName, Object) CONDITIONAL_ON_OBJECT_NET_ID(Object, UEPushModelPrivate::MarkPropertyDirty(Object, PrivatePushId, GET_PROPERTY_REP_INDEX(ClassName, PropertyName)))

// Marks a static array property dirty, given the Class Name, Property Name, Index, and Object. This will fail to compile if the Property and Class aren't valid. Callers are responsible for validating the index.
#define MARK_PROPERTY_DIRTY_FROM_NAME_STATIC_ARRAY_INDEX(ClassName, PropertyName, ArrayIndex, Object) CONDITIONAL_ON_OBJECT_NET_ID(Object, UEPushModelPrivate::MarkPropertyDirty(Object, PrivatePushId, GET_PROPERTY_REP_INDEX_STATIC_ARRAY_INDEX(ClassName, PropertyName, ArrayIndex)))

// Marks an entire static array property dirty, given the Class Name, Property Name, and Object. This will fail to compile if the Property or Class aren't valid.
#define MARK_PROPERTY_DIRTY_FROM_NAME_STATIC_ARRAY(ClassName, PropertyName, Object) CONDITIONAL_ON_OBJECT_NET_ID(Object, UEPushModelPrivate::MarkPropertyDirty(Object, PrivatePushId, GET_PROPERTY_REP_INDEX_STATIC_ARRAY_START(ClassName, PropertyName), GET_PROPERTY_REP_INDEX_STATIC_ARRAY_END(ClassName, PropertyName)))

/**
 * This is used to reduce lines of code (mostly in setters) by doing the comparison check before changing the value and marking dirty only if we have changed the value. 
 * This is best if used for things like ints, floats, pointers, but not something like a struct because of the memcmp cost. 
 */
#define COMPARE_ASSIGN_AND_MARK_PROPERTY_DIRTY(ClassName, PropertyName, NewValue, Object, ...) \
	do { \
		if (NewValue != PropertyName) \
		{ \
			PropertyName = NewValue; \
			MARK_PROPERTY_DIRTY_FROM_NAME(ClassName, PropertyName, Object); \
			__VA_ARGS__ \
		} \
	} while (0)


#define UE_NET_SET_IRIS_MARK_PROPERTY_DIRTY_DELEGATE(Delegate) UEPushModelPrivate::SetIrisMarkPropertyDirtyDelegate(Delegate)
#define UE_NET_SET_IRIS_MARK_PROPERTIES_DIRTY_DELEGATE(Delegate) UEPushModelPrivate::SetIrisMarkPropertiesDirtyDelegate(Delegate)

#else // WITH_PUSH_MODEL

#define MARK_PROPERTY_DIRTY_UNSAFE(Object, RepIndex)
#define MARK_PROPERTY_DIRTY(Object, Property) 
#define MARK_PROPERTY_DIRTY_STATIC_ARRAY_INDEX(Object, RepIndex, ArrayIndex) 
#define MARK_PROPERTY_DIRTY_STATIC_ARRAY(Object, RepIndex, ArrayIndex) 

#define MARK_PROPERTY_DIRTY_FROM_NAME(ClassName, PropertyName, Object) 
#define MARK_PROPERTY_DIRTY_FROM_NAME_STATIC_ARRAY_INDEX(ClassName, PropertyName, ArrayIndex, Object) 
#define MARK_PROPERTY_DIRTY_FROM_NAME_STATIC_ARRAY(ClassName, PropertyName, ArrayIndex, Object) 


#define GET_PROPERTY_REP_INDEX(ClassName, PropertyName) INDEX_NONE
#define GET_PROPERTY_REP_INDEX_STATIC_ARRAY(ClassName, PropertyName, ArrayIndex) INDEX_NONE

#define IS_PUSH_MODEL_ENABLED() false
#define PUSH_MAKE_BP_PROPERTIES_PUSH_MODEL() false

#define COMPARE_ASSIGN_AND_MARK_PROPERTY_DIRTY(ClassName, PropertyName, NewValue, Object, ...) \
	do { \
		if (NewValue != PropertyName) \
		{ \
			PropertyName = NewValue; \
			__VA_ARGS__ \
		} \
	} while(0)

#define UE_NET_SET_IRIS_MARK_PROPERTY_DIRTY_DELEGATE(...)
#define UE_NET_SET_IRIS_MARK_PROPERTIES_DIRTY_DELEGATE(...)

#endif
