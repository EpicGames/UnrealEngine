// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef WITH_PUSH_MODEL
#define WITH_PUSH_MODEL 0
#endif

#define DISABLE_PUSH_MODEL_VALIDATION 0
#define WITH_PUSH_VALIDATION_SUPPORT (WITH_PUSH_MODEL && !(UE_BUILD_SHIPPING || UE_BUILD_TEST || DISABLE_PUSH_MODEL_VALIDATION))

#if WITH_PUSH_MODEL

// This macro could be moved somewhere else and be made more generic if we want.

#define REPLICATED_BASE_CLASS(ClassName) \
private: \
uint64 NetPushId_Internal = uint64(int64(INDEX_NONE)); \
virtual void SetNetPushIdDynamic(const uint64 InNetPushId) override final { NetPushId_Internal = InNetPushId; } \
public: \
virtual uint64 GetNetPushIdDynamic() const override final { return GetNetPushId(); } \
uint64 GetNetPushId() const { return NetPushId_Internal; }

#else

#define REPLICATED_BASE_CLASS(ClassName)

#endif // WITH_PUSH_MODEL