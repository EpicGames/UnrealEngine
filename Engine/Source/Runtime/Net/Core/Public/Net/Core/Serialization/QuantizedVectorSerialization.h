// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "Serialization/Archive.h"

namespace UE::Net
{

/* Scales the supplied vector and then rounds to an integer value which is serialized.
 * If scaling cannot improve the precision of any of the components the value will not be scaled before rounding.
 * If scaling produces a value that would cause any of the components to be clamped then serialization
 * will fallback to send full precision components.
 * N.B. The Vector3f and Vector3d variants are bit compatible with eachother, but it's discouraged to mix types
 * when writing and reading.
 */
NETCORE_API bool WriteQuantizedVector(const int32 Scale, const FVector3d& Value, FArchive& Ar);
NETCORE_API bool WriteQuantizedVector(const int32 Scale, const FVector3f& Value, FArchive& Ar);

/* Reads a quantized vector that was written using WriteQuantizedVector with the same scale. */
NETCORE_API bool ReadQuantizedVector(const int32 Scale, FVector3d& Value, FArchive& Ar);
NETCORE_API bool ReadQuantizedVector(const int32 Scale, FVector3f& Value, FArchive& Ar);

/* Quantize a vector using the same quantization as WriteQuantizedVector followed by ReadQuantizedVector. */
NETCORE_API FVector3d QuantizeVector(const int32 Scale, const FVector3d& Value);
NETCORE_API FVector3f QuantizeVector(const int32 Scale, const FVector3f& Value);

template<int32 Scale>
bool SerializeQuantizedVector(FVector& Value, FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		return ReadQuantizedVector(Scale, Value, Ar);
	}
	else
	{
		return WriteQuantizedVector(Scale, Value, Ar);
	}
}

}
