// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/CString.h"

namespace UE::Net
{

/**
 * NetDebugName, carries both string pointer and a debugname id assigned by the persistent debug storage. DebugNameId is used to be able to avoid hashing already seen NetDebugNames
 */
typedef uint32 FNetDebugNameId;

struct FNetDebugName
{
	const TCHAR* Name = nullptr;
	mutable FNetDebugNameId DebugNameId = 0;
};


NETCORE_API const FNetDebugName* CreatePersistentNetDebugName(const TCHAR* Name, uint32 NameLen);
inline const FNetDebugName* CreatePersistentNetDebugName(const TCHAR* Name) { return CreatePersistentNetDebugName(Name, FCString::Strlen(Name)); }

NETCORE_API FNetDebugName const* GetReliabilityNetDebugName(bool bIsReliable);

}

inline const TCHAR* ToCStr(const UE::Net::FNetDebugName* DebugName)
{
	if (DebugName && DebugName->Name)
	{
		return DebugName->Name;
	}
	else
	{
		return TEXT("N/A");
	}
}


