// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Private/NetTraceInternal.h"

#if UE_NET_TRACE_ENABLED

/** Create New Collector */
#define UE_NET_TRACE_CREATE_COLLECTOR(Verbosity) UE_NET_TRACE_INTERNAL_CREATE_COLLECTOR(Verbosity)

/** Create Destroy Collector */
#define UE_NET_TRACE_DESTROY_COLLECTOR(Collector) FNetTrace::DestroyTraceCollector(Collector)

/** 
	NET_TRACE_SCOPE macros are used to measure serialized bits during the lifetime of the Scope and report the result to the provided collector
*/

/** Simple NetTraceScope with a static name */
#define UE_NET_TRACE_SCOPE(Name, Stream, Collector, Verbosity) UE_NET_TRACE_INTERNAL_SCOPE(Name, Stream, Collector, Verbosity)

/** NetTraceScope that uses NetHandle or NetGUID to identify the scope */
#define UE_NET_TRACE_OBJECT_SCOPE(HandleOrNetGUID, Stream, Collector, Verbosity) UE_NET_TRACE_INTERNAL_OBJECT_SCOPE(HandleOrNetGUID, Stream, Collector, Verbosity)

/** NetTraceScope with dynamic name */
#define UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Name, Stream, Collector, Verbosity) UE_NET_TRACE_INTERNAL_DYNAMIC_NAME_SCOPE(Name, Stream, Collector, Verbosity)

/**
 * UE_NET_TRACE_NAMED_SCOPE:macros`s are similar to the normal scopes but they have an explicit name which allows the scope to be modified
 * This is useful for example when we are reading data and do not yet have any context on what we are reading.
*/

/** Named NetTraceScope */
#define UE_NET_TRACE_NAMED_SCOPE(ScopeName, EventName, Stream, Collector, Verbosity) UE_NET_TRACE_INTERNAL_NAMED_SCOPE(ScopeName, EventName, Stream, Collector, Verbosity)

/** Named Object NetTraceScope */
#define UE_NET_TRACE_NAMED_OBJECT_SCOPE(ScopeName, HandleOrNetGUID, Stream, Collector, Verbosity) UE_NET_TRACE_INTERNAL_NAMED_OBJECT_SCOPE(ScopeName, HandleOrNetGUID, Stream, Collector, Verbosity)

/** Named dynamic name NetTraceScope */
#define UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(ScopeName, EventName, Stream, Collector, Verbosity) UE_NET_TRACE_INTERNAL_NAMED_DYNAMIC_NAME_SCOPE(ScopeName, EventName, Stream, Collector, Verbosity)

/** Set the EventName of a named scope */
#define UE_NET_TRACE_SET_SCOPE_NAME(ScopeName, EventName) UE_NET_TRACE_INTERNAL_SET_SCOPE_NAME(ScopeName, EventName)

/** Set the ObjectId of named scope */
#define UE_NET_TRACE_SET_SCOPE_OBJECTID(ScopeName, HandleOrNetGUID) UE_NET_TRACE_INTERNAL_SET_SCOPE_OBJECTID(ScopeName, HandleOrNetGUID)

/** Early out of a named scope*/
#define UE_NET_TRACE_EXIT_NAMED_SCOPE(ScopeName) UE_NET_TRACE_INTERNAL_EXIT_NAMED_SCOPE(ScopeName)

/**
 During the lifetime of an offset scope all reported content events are offset by the provided offset
 Supports nesting, offsets of nested scopes are accumulated
*/
#define UE_NET_TRACE_OFFSET_SCOPE(Offset, Collector) UE_NET_TRACE_INTERNAL_OFFSET_SCOPE(Offset, Collector)

/** Trace event information for a static name */
#define UE_NET_TRACE(Name, Collector, StartPos, EndPos, Verbosity) UE_NET_TRACE_INTERNAL(Name, Collector, StartPos, EndPos, Verbosity)

/** Trace event information with a dynamic name */
#define UE_NET_TRACE_DYNAMIC_NAME(Name, Collector, StartPos, EndPos, Verbosity) UE_NET_TRACE_INTERNAL_DYNAMIC_NAME(Name, Collector, StartPos, EndPos, Verbosity)

/** Report collected events for the current packet */
#define UE_NET_TRACE_FLUSH_COLLECTOR(Collector, ...) UE_NET_TRACE_INTERNAL_FLUSH_COLLECTOR(Collector, __VA_ARGS__)

/** Mark the beginning of a new bunch */
#define UE_NET_TRACE_BEGIN_BUNCH(Collector)	UE_NET_TRACE_INTERNAL_BEGIN_BUNCH(Collector)

/** Discard pending bunch */
#define UE_NET_TRACE_DISCARD_BUNCH(Collector) UE_NET_TRACE_INTERNAL_DISCARD_BUNCH(Collector)

/** Pop the last send bunch, used by merging bunch logic */
#define UE_NET_TRACE_POP_SEND_BUNCH(Collector) UE_NET_TRACE_INTERNAL_POP_SEND_BUNCH(Collector)

/** Append events from SrcCollector to Collector */
#define UE_NET_TRACE_EVENTS(DstCollector, SrcCollector, Stream) UE_NET_TRACE_INTERNAL_EVENTS(DstCollector, SrcCollector, Stream)

/** Mark the end of a bunch */
#define UE_NET_TRACE_END_BUNCH(Collector, Bunch, ...) UE_NET_TRACE_INTERNAL_END_BUNCH(Collector, Bunch, __VA_ARGS__)				

/** Scope to track begin/end bunch */
#define UE_NET_TRACE_BUNCH_SCOPE(Collector, Bunch, ...) UE_NET_TRACE_INTERNAL_BUNCH_SCOPE(Collector, Bunch, __VA_ARGS__)

/** Trace Assigned NetGUID */
#define UE_NET_TRACE_ASSIGNED_GUID(GameInstanceId, NetGUID, PathName, OwnerId) UE_NET_TRACE_INTERNAL_ASSIGNED_GUID(GameInstanceId, NetGUID, PathName, OwnerId)

/** Trace that we have started to replicate a handle */
#define UE_NET_TRACE_NETHANDLE_CREATED(Handle, DebugName, ProtocolId, OwnerId) UE_NET_TRACE_INTERNAL_NETHANDLE_CREATED(Handle, DebugName, ProtocolId, OwnerId)

/** Trace that we stopped replicating a handle */
#define UE_NET_TRACE_NETHANDLE_DESTROYED(Handle) UE_NET_TRACE_INTERNAL_NETHANDLE_DESTROYED(Handle)

/** Trace that a new connection has been created */
#define UE_NET_TRACE_CONNECTION_CREATED(...) UE_NET_TRACE_INTERNAL_CONNECTION_CREATED(__VA_ARGS__)

/** Trace that the state of this connection has been set */
#define UE_NET_TRACE_CONNECTION_STATE_UPDATED(...) UE_NET_TRACE_INTERNAL_CONNECTION_STATE_UPDATED(__VA_ARGS__)

/** Trace additional information about the given connection */
#define UE_NET_TRACE_CONNECTION_UPDATED(...) UE_NET_TRACE_INTERNAL_CONNECTION_UPDATED(__VA_ARGS__)

/** Trace StatsCounter associated with next reported packet */
#define UE_NET_TRACE_PACKET_STATSCOUNTER(GameInstanceId, ConnectionId, Name, StatValue, Verbosity) UE_NET_TRACE_INTERNAL_PACKET_STATSCOUNTER(GameInstanceId, ConnectionId, Name, StatValue, Verbosity)

/** Trace StatsCounter associated with current frame */
#define UE_NET_TRACE_FRAME_STATSCOUNTER(GameInstanceId, Name, StatValue, Verbosity) UE_NET_TRACE_INTERNAL_FRAME_STATSCOUNTER(GameInstanceId, Name, StatValue, Verbosity)

/** That that a connection has been closed */
#define UE_NET_TRACE_CONNECTION_CLOSED(...) UE_NET_TRACE_INTERNAL_CONNECTION_CLOSED(__VA_ARGS__)

/** Trace that we have dropped a packet */
#define UE_NET_TRACE_PACKET_DROPPED(...)  UE_NET_TRACE_INTERNAL_PACKET_DROPPED(__VA_ARGS__)

/** Trace that we have sent a packet */
#define UE_NET_TRACE_PACKET_SEND(...) UE_NET_TRACE_INTERNAL_PACKET_SEND(__VA_ARGS__)

/** Trace that we have received a packet */
#define UE_NET_TRACE_PACKET_RECV(...) UE_NET_TRACE_INTERNAL_PACKET_RECV(__VA_ARGS__)

/** Trace that the session has ended */
#define UE_NET_TRACE_END_SESSION(GameInstanceId) UE_NET_TRACE_INTERNAL_END_SESSION(GameInstanceId)

/** Trace additional information about the given game instance */
#define UE_NET_TRACE_UPDATE_INSTANCE(...) UE_NET_TRACE_INTERNAL_UPDATE_INSTANCE(__VA_ARGS__)

#define UE_NET_TRACE_POLL_OBJECT_SCOPE(...) UE_NET_TRACE_INTERNAL_POLL_OBJECT_SCOPE(__VA_ARGS__)
#define UE_NET_TRACE_POLL_OBJECT_IS_WASTE() UE_NET_TRACE_INTERNAL_POLL_OBJECT_IS_WASTE
#define UE_NET_TRACE_QUANTIZE_OBJECT_SCOPE(...) UE_NET_TRACE_INTERNAL_QUANTIZE_OBJECT_SCOPE(__VA_ARGS__)
#define UE_NET_TRACE_WRITE_OBJECT_SCOPE(...) UE_NET_TRACE_INTERNAL_WRITE_OBJECT_SCOPE(__VA_ARGS__)

#define UE_NET_TRACE_SUBOBJECT(ParentHandle, SubObjectHandle) UE_NET_TRACE_INTERNAL_SUBOBJECT(ParentHandle, SubObjectHandle)

#else

#define UE_NET_TRACE_SCOPE(...)
#define UE_NET_TRACE_OBJECT_SCOPE(...)
#define UE_NET_TRACE_DYNAMIC_NAME_SCOPE(...)
#define UE_NET_TRACE_NAMED_SCOPE(...)
#define UE_NET_TRACE_SET_SCOPE_NAME(...)
#define UE_NET_TRACE_SET_SCOPE_OBJECTID(...)
#define UE_NET_TRACE_EXIT_NAMED_SCOPE(...)
#define UE_NET_TRACE(...)
#define UE_NET_TRACE_DYNAMIC_NAME(...)
#define UE_NET_TRACE_CREATE_COLLECTOR(...) nullptr
#define UE_NET_TRACE_DESTROY_COLLECTOR(...)

#define UE_NET_TRACE_FLUSH_COLLECTOR(...)
#define UE_NET_TRACE_BEGIN_BUNCH(...)
#define UE_NET_TRACE_DISCARD_BUNCH(...)
#define UE_NET_TRACE_POP_SEND_BUNCH(...)
#define UE_NET_TRACE_EVENTS(...)
#define UE_NET_TRACE_END_BUNCH(...)
#define UE_NET_TRACE_BUNCH_SCOPE(...)
#define UE_NET_TRACE_OFFSET_SCOPE(...)

#define UE_NET_TRACE_ASSIGNED_GUID(...)
#define UE_NET_TRACE_NETHANDLE_CREATED(...)
#define UE_NET_TRACE_NETHANDLE_DESTROYED(...)
#define UE_NET_TRACE_CONNECTION_CREATED(...)
#define UE_NET_TRACE_CONNECTION_STATE_UPDATED(...)
#define UE_NET_TRACE_CONNECTION_UPDATED(...)
#define UE_NET_TRACE_CONNECTION_CLOSED(...)
#define UE_NET_TRACE_PACKET_DROPPED(...)
#define UE_NET_TRACE_NAMED_OBJECT_SCOPE(...)
#define UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(...)

#define UE_NET_TRACE_PACKET_STATSCOUNTER(...)
#define UE_NET_TRACE_FRAME_STATSCOUNTER(...)

#define UE_NET_TRACE_PACKET_DROPPED(...)
#define UE_NET_TRACE_PACKET_SEND(...)
#define UE_NET_TRACE_PACKET_RECV(...)

#define UE_NET_TRACE_END_SESSION(...)

#define UE_NET_TRACE_UPDATE_INSTANCE(...)

#define UE_NET_TRACE_POLL_OBJECT_SCOPE(...)
#define UE_NET_TRACE_POLL_OBJECT_IS_WASTE()
#define UE_NET_TRACE_QUANTIZE_OBJECT_SCOPE(...)
#define UE_NET_TRACE_WRITE_OBJECT_SCOPE(...)

#define UE_NET_TRACE_SUBOBJECT(...)

#endif // UE_NET_TRACE_ENABLED
