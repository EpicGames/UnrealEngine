// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"

// Forward declarations
class FNetTraceCollector;

#if !defined(UE_NET_TRACE_ENABLED)
#	if !(UE_BUILD_SHIPPING) && UE_TRACE_ENABLED
#		define UE_NET_TRACE_ENABLED 1
#	else
#		define UE_NET_TRACE_ENABLED 0
#	endif
#endif

namespace ENetTraceVerbosity
{
	enum Type : uint32
	{
		None = 0,
		Trace,
		Verbose,
		VeryVerbose,
	};
}

inline const uint32 NetTraceInvalidGameInstanceId = ~0U;

#if UE_NET_TRACE_ENABLED
#	ifndef UE_NET_TRACE_COMPILETIME_VERBOSITY
#		if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#			define UE_NET_TRACE_COMPILETIME_VERBOSITY ENetTraceVerbosity::VeryVerbose
#		else
#			define UE_NET_TRACE_COMPILETIME_VERBOSITY ENetTraceVerbosity::Verbose
#		endif
#	else
#		define UE_NET_TRACE_COMPILETIME_VERBOSITY ENetTraceVerbosity::None
#	endif
#endif