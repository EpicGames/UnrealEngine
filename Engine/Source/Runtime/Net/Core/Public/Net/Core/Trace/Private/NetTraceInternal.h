// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/Trace/NetTraceConfig.h"
#include "HAL/PlatformTime.h"

namespace UE::Net::Private
{
	class FNetTimer
	{
	public:
		FNetTimer(bool bEnabled)
		: StartCycle(bEnabled ? FPlatformTime::Cycles64() : uint64(0)) 
		{
		}

		uint64 GetCyclesSinceStart() const { return FPlatformTime::Cycles64() - StartCycle; }
	
	private:
		uint64 StartCycle;
	};
}


#if UE_NET_TRACE_ENABLED

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/NetworkGuid.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "UObject/NameTypes.h"
#include "ProfilingDebugging/TraceAuxiliary.h"

class FNetTraceCollector;

// $TODO: Implement pooling of TraceCollectors
// $TODO: Add disconnect reason to connection
// $TODO: Add reporting of final packet sizes after packethandlers

struct FBitReader;
struct FBitWriter;
namespace UE::Net
{
	class FNetBitStreamReader;
	class FNetBitStreamWriter;
}

extern uint32 GetBitStreamPositionForNetTrace(const UE::Net::FNetBitStreamReader& Stream);
extern uint32 GetBitStreamPositionForNetTrace(const UE::Net::FNetBitStreamWriter& Stream);
extern uint32 GetBitStreamPositionForNetTrace(const FBitReader& Stream);
extern uint32 GetBitStreamPositionForNetTrace(const FBitWriter& Stream);

extern NETCORE_API uint32 GNetTraceRuntimeVerbosity;

enum class ENetTracePacketContentEventType : uint8
{
	Object = 0,
	NameId = 1,
	BunchEvent = 2,
	BunchHeaderEvent = 3,
};

enum class ENetTracePacketType : uint8
{
	Outgoing = 0,
	Incoming,
};

enum class ENetTraceStatsCounterType : uint8
{
	Packet = 0,
	Frame = 1,
};

struct FNetTracePacketInfo
{
	uint32 GameInstanceId;
	uint32 PacketSequenceNumber;
	uint16 ConnectionId;
	ENetTracePacketType PacketType;
};

// Any changes to this struct must also be reflected on the NetTraceAnalyzer side and versioned
union FNetTraceBunchInfo
{
	struct
	{
		uint64 ChannelIndex : 20;
		uint64 Seq : 12;
		uint64 ChannelCloseReason : 4;
		uint64 bPartial : 1;
		uint64 bPartialInitial : 1;
		uint64 bPartialCustomExportsFinal : 1;
		uint64 bPartialFinal : 1;
		UE_DEPRECATED(5.3, "Replication pausing is deprecated")
		uint64 bIsReplicationPaused : 1;
		uint64 bOpen : 1;
		uint64 bClose : 1;
		uint64 bReliable : 1;
		uint64 bHasPackageMapExports : 1;
		uint64 bHasMustBeMappedGUIDs : 1;
		uint64 Padding : 18;
	};
	uint64 Value;
};

static_assert(sizeof(FNetTraceBunchInfo) == sizeof(uint64), "Unexpected size of FNetTraceBunchInfo.");

struct FNetTracePacketContentEvent
{
	union
	{
		UE::Net::FNetDebugNameId DebugNameId;
		uint64 ObjectId;
		FNetTraceBunchInfo BunchInfo;
	};
	struct
	{
		uint32 StartPos : 24;
		uint32 EventType : 8;
		uint32 EndPos : 24;
		uint32 NestingLevel : 8;
	};
};

static_assert(sizeof(FNetTracePacketContentEvent) == 16U, "Unexpected size of PacketContentEvent");

/** 
* Scope used to track data read from or written to a bitstream, supports nesting
* In order to support instrumenting different types of bitstreams using the same set of macros, a loose function with the signature below can be provided
* uint32 GetBitStreamPositionForNetTrace(const T& Stream)
*/
template <typename T>
class FNetTraceEventScope
{
	UE_NONCOPYABLE(FNetTraceEventScope)
public:

	FNetTraceEventScope(uint64 ObjectId, T& InStream, FNetTraceCollector* InCollector, uint32 Verbosity);

	// When a scope is disabled due to verbosity we want to deffer the tracing of names
	// We do this by using a lambda, this also allows us to complete eliminate statics if a scope is compiled out due to compile time verbosity
	template <typename TraceNameFunctor>
	FNetTraceEventScope(TraceNameFunctor&& TraceNameFunc, T& InStream, FNetTraceCollector* InCollector, uint32 Verbosity);

	~FNetTraceEventScope();

	void SetEventName(UE::Net::FNetDebugNameId NetTraceNameId);
	void SetEventObjectId(uint64 ObjectId);
	void ExitScope();
	bool IsValid() const;

private:

	FNetTraceCollector* Collector;
	T& Stream;
	uint32 EventIndex;
};

/** 
 * Null implementation of NetTraceEventScope used for compile-time verbosity
 */
template <typename T>
class FNetTraceNullEventScope
{
	UE_NONCOPYABLE(FNetTraceNullEventScope)
public:
	template <typename SinkT>
	FNetTraceNullEventScope(SinkT, T& InStream, FNetTraceCollector* InCollector, uint32 Verbosity)  {};
	~FNetTraceNullEventScope() = default;

	void SetEventName(UE::Net::FNetDebugNameId NetTraceNameId) {};
	void SetEventObjectId(uint64 ObjectId) {};
	void ExitScope() {};
	bool IsValid() const { return false; }
};

struct FNetTrace
{
	enum { InvalidEventIndex = ~0U };

	/** Returns if tracing is enabled or not */
	static bool IsEnabled() { return GetTraceVerbosity() > 0U; }

	/** Sets the runtime trace verbosity level, Verbosity == 0 means that it is disabled */
	NETCORE_API static void SetTraceVerbosity(uint32 Verbosity);

	static void OnTraceConnected();

	static void OnTraceStopped(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);

	DECLARE_TS_MULTICAST_DELEGATE(FOnNetTraceStarted);
	static NETCORE_API FOnNetTraceStarted OnNetTraceStarted;

	/** Trace end of session for a GameInstance */
	NETCORE_API static void TraceEndSession(uint32 GameInstanceId);

	/** Trace that information about the current GameInstance has been updated */
	NETCORE_API static void TraceInstanceUpdated(uint32 GameInstanceId, bool bIsServer, const TCHAR* Name);

	/** Get the current trace verbosity */
	inline static uint32 GetTraceVerbosity() { return GNetTraceRuntimeVerbosity; }
	
	/** Create a new trace collector if tracing is enabled */
	NETCORE_API static FNetTraceCollector* CreateTraceCollector();

	/** Create a new trace collector if tracing is enabled */
	inline static FNetTraceCollector* CreateTraceCollector(uint32 Verbosity);

	/** Destroy trace collector without reporting the collected events */
	NETCORE_API static void DestroyTraceCollector(FNetTraceCollector* Collector);

	/** Folds events into DstCollector */
	NETCORE_API static void FoldTraceCollector(FNetTraceCollector* DstCollector, const FNetTraceCollector* SrcCollector, uint32 Offset, uint32 LevelOffset = 0U);
	
	/** Folds bunch into DstCollector, if the src and dst are the same bunch it is assumed that a bunch already is open and that it will be closed */
	NETCORE_API static void FoldBunchCollector(FNetTraceCollector* DstCollector, const FNetTraceCollector* SrcCollector);

	/** Events will be offset by the provided offset, useful when doing partial reads using separate bitstreams, Does not support nesting */
	NETCORE_API static void PushStreamOffset(FNetTraceCollector* Collector, uint32 Offset);
	NETCORE_API static void PopStreamOffset(FNetTraceCollector* Collector);

	/** Begin a new PacketContentEvent */
	NETCORE_API static uint32 BeginPacketContentEvent(FNetTraceCollector& Collector, ENetTracePacketContentEventType EventType, uint32 Pos);

	/** End a PacketContent event */
	NETCORE_API static void EndPacketContentEvent(FNetTraceCollector& Collector, uint32 EventIndex, uint32 Pos);

	/** Trace PacketContent event */
	NETCORE_API static void TracePacketContentEvent(FNetTraceCollector& Collector, UE::Net::FNetDebugNameId InNetTraceNameId, uint32 StartPos, uint32 EndPos, uint32 Verbosity);

	/** Mark the beginning of a new Bunch */
	NETCORE_API static void BeginBunch(FNetTraceCollector& Collector);

	/** Discard any events Collected since BeginBunch */
	NETCORE_API static void DiscardBunch(FNetTraceCollector& Collector);

	/** Trace end of a bunch, all events recorded between BeginBunch/EndBunch will be part of the bunch */
	NETCORE_API static void EndBunch(FNetTraceCollector& DstCollector, UE::Net::FNetDebugNameId BunchName, uint32 StartPos, uint32 HeaderBits, uint32 BunchBits, const FNetTraceBunchInfo& BunchInfo);

	/** TraceBunch, Commits all events in BunchCollector to DstCollector, if DstCollector == BunchCollector it is assumed that we are Ending the bunch */
	NETCORE_API static void TraceBunch(FNetTraceCollector& DstCollector, const FNetTraceBunchInfo& BunchInfo, FName BunchName, uint32 StartPos, uint32 HeaderBits, uint32 BunchBits, const FNetTraceCollector* BunchCollector);

	/** TraceBunch, Commits all events in BunchCollector to DstCollector, if DstCollector == BunchCollector it is assumed that we are Ending the bunch */
	NETCORE_API static void TraceBunch(FNetTraceCollector& DstCollector, const FNetTraceBunchInfo& BunchInfo, const TCHAR* BunchName, uint32 StartPos, uint32 HeaderBits, uint32 BunchBits, const FNetTraceCollector* BunchCollector);

	/** PopSendBunch, last send bunch is considered to be merged with the next one, mark this by zeroing out the BunchHeaderSize */
	NETCORE_API static void PopSendBunch(FNetTraceCollector& Collector);

	/** Trace all events in the bunch */
	NETCORE_API static void TraceCollectedEvents(FNetTraceCollector& Collector, uint32 GameInstanceId, uint32 ConnectionId, ENetTracePacketType PacketType);

	/** Trace if the packet was dropped */
 	NETCORE_API static void TracePacketDropped(uint32 GameInstanceId, uint32 ConnectionId, uint32 PacketSequenceNumber, ENetTracePacketType PacketType);

	/** Trace incoming or outgoing packet */
 	NETCORE_API static void TracePacket(uint32 GameInstanceId, uint32 ConnectionId, uint32 PacketSequenceNumber, uint32 PacketBits, ENetTracePacketType PacketType);

	/** Trace that we have created a new object with the given ObjectId */
	NETCORE_API static void TraceObjectCreated(uint32 GameInstanceId, uint64 NetObjectId, const UE::Net::FNetDebugName* Name, uint64 TypeIdentifier, uint32 OwnerId);

	/** Trace that we have created a new object with the given ObjectId */
	NETCORE_API static void TraceObjectCreated(uint32 GameInstanceId, uint64 NetObjectId, const FName Name, uint64 TypeIdentifier, uint32 OwnerId);

	/** Trace that a handle has been destroyed, the handle contains required info */
	NETCORE_API static void TraceObjectDestroyed(uint32 GameInstanceId, uint64 NetObjectId);

	/** Trace that we have added a new connection for the given GameInstanceId */
	NETCORE_API static void TraceConnectionCreated(uint32 GameInstanceId, uint32 ConnectionId);

	/** Trace that the Connection State the given connection has been set */
	NETCORE_API static void TraceConnectionStateUpdated(uint32 GameInstanceId, uint32 ConnectionId, uint8 ConnectionStateValue);

	/** Trace additional information related to the given connection */
	NETCORE_API static void TraceConnectionUpdated(uint32 GameInstanceId, uint32 ConnectionId, const TCHAR* AddressString, const TCHAR* OwningActor);

	/** Trace that we have removed a connection for the given GameInstanceId */
	NETCORE_API static void TraceConnectionClosed(uint32 GameInstanceId, uint32 ConnectionId);

	/** Trace stats counters for networking */
	NETCORE_API static void TracePacketStatsCounter(uint32 GameInstanceId, uint32 ConnectionId, UE::Net::FNetDebugNameId CounterNameId, uint32 StatValue);
	NETCORE_API static void TracePacketStatsCounter(uint32 GameInstanceId, uint32 ConnectionId, const UE::Net::FNetDebugName* CounterName, uint32 StatValue);
	NETCORE_API static void TraceFrameStatsCounter(uint32 GameInstanceId, UE::Net::FNetDebugNameId CounterNameId,  uint32 StatValue);
	NETCORE_API static void TraceFrameStatsCounter(uint32 GameInstanceId, const UE::Net::FNetDebugName* CounterNameId,  uint32 StatValue);
	
	/** Trace the name */
	NETCORE_API static UE::Net::FNetDebugNameId TraceName(const TCHAR* Name);

	/** Trace the FName */
	NETCORE_API static UE::Net::FNetDebugNameId TraceName(FName Name);

	/** Trace the FNetDebugName */
	NETCORE_API static UE::Net::FNetDebugNameId TraceName(const UE::Net::FNetDebugName* Name);

	/** Trace */
	inline static UE::Net::FNetDebugNameId TraceName(const FString& Name) { return TraceName(ToCStr(Name)); };

	NETCORE_API static void TraceObjectPolled(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles, bool bIsDirty);

	NETCORE_API static void TraceObjectQuantized(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles);

	NETCORE_API static void TraceObjectWritten(uint32 GameInstanceId, uint64 NetObjectId, uint64 Cycles);

	NETCORE_API static void TraceSubObject(uint32 GameInstanceId, uint64 ParentNetObjectId, uint64 SubObjectNetObjectId);

	NETCORE_API static void SetupInstanceForNetTraceId(uint32 InNetTraceId);

	NETCORE_API static void FinishConnectionTick(const uint32 GameInstanceId);

	static bool constexpr GetNetTraceVerbosityEnabled(const uint32 V) { return (uint32)UE_NET_TRACE_COMPILETIME_VERBOSITY >= V; }

	inline static FNetTraceCollector* GetCollectorAtVerbosity(FNetTraceCollector* Collector, uint32 CollectorVerbosity);

	template<uint32 Verbosity, typename StreamType>
	struct TChooseTraceEventScope
	{
		typedef std::conditional_t<FNetTrace::GetNetTraceVerbosityEnabled(Verbosity), FNetTraceEventScope<StreamType>, FNetTraceNullEventScope<StreamType> > Type;
	};
};

template<typename T>
uint64 GetObjectIdForNetTrace(const T&)
{
	static_assert(sizeof(T) == 0, "Not supported type for NetTraceObjectID, implement uint64 GetObjectIdForNetTrace(const T&)");
	return 0;
}

inline uint64 GetObjectIdForNetTrace(const FNetworkGUID& NetGUID)
{
	return NetGUID.ObjectId;
}

template<typename T>
FNetTraceBunchInfo MakeBunchInfo(const T& Bunch)
{
	FNetTraceBunchInfo BunchInfo;
	BunchInfo.Value = uint64(0);
	BunchInfo.ChannelIndex = Bunch.ChIndex;
	BunchInfo.Seq = Bunch.ChSequence;
	BunchInfo.ChannelCloseReason = uint64(Bunch.CloseReason);
	BunchInfo.bPartial = Bunch.bPartial;
	BunchInfo.bPartialInitial = Bunch.bPartialInitial;
	BunchInfo.bPartialCustomExportsFinal = Bunch.bPartialCustomExportsFinal;
	BunchInfo.bPartialFinal = Bunch.bPartialFinal;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	BunchInfo.bIsReplicationPaused = Bunch.bIsReplicationPaused;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	BunchInfo.bOpen = Bunch.bOpen;
	BunchInfo.bClose = Bunch.bClose;
	BunchInfo.bReliable = Bunch.bReliable;
	BunchInfo.bHasPackageMapExports = Bunch.bHasPackageMapExports;
	BunchInfo.bHasMustBeMappedGUIDs = Bunch.bHasMustBeMappedGUIDs;
	BunchInfo.Padding = 0;

	return BunchInfo;
}

// The purpose of a NetTraceCollector is to allow tracing of full packet content data with as low overhead as possible.
class FNetTraceCollector
{
	UE_NONCOPYABLE(FNetTraceCollector)
public:
	// constants
	enum { MaxNestingLevel = 32 };
	enum { InlinedEventCount = 256 };
	enum { InlinedObjectIdCount = 128 };

	explicit FNetTraceCollector(uint32 InitialEventCount) { Events.SetNumUninitialized(InitialEventCount); Reset(); }
	FNetTraceCollector() : FNetTraceCollector(InlinedEventCount) {}; 

	inline void Reset();

	inline FNetTracePacketContentEvent& EditEvent(uint32 Index) { return Events[Index]; }
	inline FNetTracePacketContentEvent& EditEventNoCheck(uint32 Index) { return Events.GetData()[Index]; }
	inline void AddReferencedObjectId(uint64 ObjectId) { ReferencedObjectIds.Add(ObjectId); }
	inline bool CanBeginPacketContentEvent() const { return CurrentNestingLevel < (MaxNestingLevel - 1); }
	
private:
	friend struct FNetTrace;

	// Bookkeeping
	uint32 NestingStack[MaxNestingLevel];
	uint32 OffsetStack[MaxNestingLevel];
	uint32 EventCount;
	uint32 CurrentNestingLevel : 16;
	uint32 OffsetStackLevel : 16;	

	// Bunch tracking
	uint32 LastBunchEventIndex;
	uint32 BunchEventCount;
	uint32 PendingBunchEventIndex;

	// Events
	typedef TArray<FNetTracePacketContentEvent, TInlineAllocator<InlinedEventCount> > FEventArray;
	FEventArray Events;

	// Object Ids referenced by events
	TArray<uint64, TInlineAllocator<InlinedObjectIdCount>> ReferencedObjectIds;
};

/**
 * Used to trace events generated while reading or writing a bunch
 */
template <typename T>
class FNetTraceBunchScope
{
	UE_NONCOPYABLE(FNetTraceBunchScope)
public:
	FNetTraceBunchScope(T& InBunch, uint32 InStartPos, uint32 InHeaderBits, FNetTraceCollector* InCollector);
	~FNetTraceBunchScope();

private:
	FNetTraceCollector* Collector;
	T& Bunch;
	const uint32 StartPos;
	const uint32 HeaderBits;
};

/**
 * Used to apply a offset to events collected when the scope is active, used to avoid instantiate multiple collectors
 */
class FNetTraceOffsetScope
{
	UE_NONCOPYABLE(FNetTraceOffsetScope)
public:
	FNetTraceOffsetScope(uint32 Offset, FNetTraceCollector* InCollector);
	~FNetTraceOffsetScope();

private:
	FNetTraceCollector* Collector;
};

struct FNetTraceObjectTimer
{
	FNetTraceObjectTimer(const UE::Net::Private::FNetTimer& InTimer, uint32 InGameInstanceId, uint64 InNetObjectId)
	: Timer(InTimer)
	, NetObjectId(InNetObjectId)
	, GameInstanceId(InGameInstanceId)
	{}

	const UE::Net::Private::FNetTimer& Timer;
	uint64 NetObjectId;
	uint32 GameInstanceId;
};

struct FNetTraceObjectPollTimer : public FNetTraceObjectTimer
{
	FNetTraceObjectPollTimer(const UE::Net::Private::FNetTimer& InTimer, uint32 InGameInstanceId, uint64 InNetObjectId) 
	: FNetTraceObjectTimer(InTimer, InGameInstanceId, InNetObjectId)
	, bIsWaste(false) {}

	~FNetTraceObjectPollTimer() 
	{ 
		if (GameInstanceId != -1)
		{
			FNetTrace::TraceObjectPolled(GameInstanceId, NetObjectId, Timer.GetCyclesSinceStart(), bIsWaste);
		}
	}
	
	bool bIsWaste;
};

struct FNetTraceObjectQuantizeTimer : public FNetTraceObjectTimer
{
	FNetTraceObjectQuantizeTimer(const UE::Net::Private::FNetTimer& InTimer, uint32 InGameInstanceId, uint64 InNetObjectId) 
	: FNetTraceObjectTimer(InTimer, InGameInstanceId, InNetObjectId)
	{}

	~FNetTraceObjectQuantizeTimer() 
	{ 
		if (GameInstanceId != -1)
		{
			FNetTrace::TraceObjectQuantized(GameInstanceId, NetObjectId, Timer.GetCyclesSinceStart());
		}
	}
};

struct FNetTraceObjectWriteTimer : public FNetTraceObjectTimer
{
	FNetTraceObjectWriteTimer(const UE::Net::Private::FNetTimer& InTimer, uint32 InGameInstanceId, uint64 InNetObjectId) 
	: FNetTraceObjectTimer(InTimer, InGameInstanceId, InNetObjectId)
	{}

	~FNetTraceObjectWriteTimer()
	{
		if (GameInstanceId != -1)
		{
			FNetTrace::TraceObjectWritten(GameInstanceId, NetObjectId, Timer.GetCyclesSinceStart());
		}
	}
};

// FNetTrace implementation
FNetTraceCollector* FNetTrace::CreateTraceCollector(uint32 Verbosity)
{
	if (FNetTrace::GetNetTraceVerbosityEnabled(Verbosity))
	{
		return GNetTraceRuntimeVerbosity >= Verbosity ? FNetTrace::CreateTraceCollector() : nullptr;
	}
	else
	{
		return nullptr;
	}
}

FNetTraceCollector* FNetTrace::GetCollectorAtVerbosity(FNetTraceCollector* Collector, uint32 CollectorVerbosity)
{ 
	if (GetNetTraceVerbosityEnabled(CollectorVerbosity))
	{
		return (FNetTrace::GetTraceVerbosity() >= CollectorVerbosity) ? Collector : nullptr;
	}
	else
	{
		return nullptr;
	}
}

// FNetTraceEventScope implementation
template <typename T>
FNetTraceEventScope<T>::FNetTraceEventScope(uint64 InObjectId, T& InStream, FNetTraceCollector* InCollector, uint32 Verbosity)
	: Collector(FNetTrace::GetCollectorAtVerbosity(InCollector, Verbosity))
	, Stream(InStream)
	, EventIndex(FNetTrace::InvalidEventIndex)
{
	if (Collector && Collector->CanBeginPacketContentEvent())
	{
		EventIndex = FNetTrace::BeginPacketContentEvent(*Collector, ENetTracePacketContentEventType::Object, GetBitStreamPositionForNetTrace(Stream));
		Collector->EditEventNoCheck(EventIndex).ObjectId = InObjectId;
		if (InObjectId != 0)
		{
			Collector->AddReferencedObjectId(InObjectId);
		}
	}
}

template <typename T>
template <typename TraceNameFunctor>
FNetTraceEventScope<T>::FNetTraceEventScope(TraceNameFunctor&& TraceNameFunc, T& InStream, FNetTraceCollector* InCollector, uint32 Verbosity)
: Collector(FNetTrace::GetCollectorAtVerbosity(InCollector, Verbosity))
, Stream(InStream)
, EventIndex(FNetTrace::InvalidEventIndex)
{
	if (Collector && Collector->CanBeginPacketContentEvent())
	{
		EventIndex = FNetTrace::BeginPacketContentEvent(*Collector, ENetTracePacketContentEventType::NameId, GetBitStreamPositionForNetTrace(Stream));
		Collector->EditEventNoCheck(EventIndex).DebugNameId = TraceNameFunc();
	}
}

template <typename T>
FNetTraceEventScope<T>::~FNetTraceEventScope()
{
	if (EventIndex != FNetTrace::InvalidEventIndex)
	{
		FNetTrace::EndPacketContentEvent(*Collector, EventIndex, GetBitStreamPositionForNetTrace(Stream));
	}
}

template <typename T>
void FNetTraceEventScope<T>::ExitScope()
{
	if (EventIndex != FNetTrace::InvalidEventIndex)
	{
		FNetTrace::EndPacketContentEvent(*Collector, EventIndex, GetBitStreamPositionForNetTrace(Stream));
		EventIndex = FNetTrace::InvalidEventIndex;
	}
}

template <typename T>
void FNetTraceEventScope<T>::SetEventName(UE::Net::FNetDebugNameId NetTraceNameId)
{
	if (EventIndex != FNetTrace::InvalidEventIndex)
	{
		Collector->EditEventNoCheck(EventIndex).DebugNameId = NetTraceNameId;
	}
}

template <typename T>
void FNetTraceEventScope<T>::SetEventObjectId(uint64 ObjectId)
{
	if (EventIndex != FNetTrace::InvalidEventIndex)
	{
		Collector->EditEventNoCheck(EventIndex).ObjectId = ObjectId;
		Collector->AddReferencedObjectId(ObjectId);
	}
}

template <typename T>
bool FNetTraceEventScope<T>::IsValid() const
{
	return EventIndex != FNetTrace::InvalidEventIndex;
}

// FNetTraceBunchScope implementation
template <typename T>
FNetTraceBunchScope<T>::FNetTraceBunchScope(T& InBunch, uint32 InStartPos, uint32 InHeaderBits, FNetTraceCollector* InCollector)
: Collector(InCollector)
, Bunch(InBunch)
, StartPos(InStartPos)
, HeaderBits(InHeaderBits)
{
	if (InCollector)
	{
		FNetTrace::BeginBunch(*InCollector);
	}
}

template <typename T>
FNetTraceBunchScope<T>::~FNetTraceBunchScope()
{
	// Only report if we have a valid collector
	if (Collector)
	{
		FNetTrace::EndBunch(*Collector, FNetTrace::TraceName(Bunch.ChName), StartPos, HeaderBits, (uint32)Bunch.GetNumBits(), MakeBunchInfo(Bunch));	// LWC_TODO: Precision loss. FNetTrace::EndBunch et al. should take int64 BunchBits?
	}
}

// FNetTraceOffsetScope implementation
inline FNetTraceOffsetScope::FNetTraceOffsetScope(uint32 Offset, FNetTraceCollector* InCollector)
: Collector(InCollector)
{
	if (Collector)
	{
		FNetTrace::PushStreamOffset(Collector, Offset);
	}
}

inline FNetTraceOffsetScope::~FNetTraceOffsetScope()
{
	if (Collector)
	{
		FNetTrace::PopStreamOffset(Collector);
	}
}

// FNetTraceCollector implementation
void FNetTraceCollector::Reset()
{
	EventCount = 0U;
	CurrentNestingLevel = 0U;
	LastBunchEventIndex = 0U;
	BunchEventCount = 0U;
	PendingBunchEventIndex = ~0U;
	OffsetStackLevel = 0U;
	OffsetStack[0] = 0U;
	ReferencedObjectIds.Reset();
}

#define UE_NET_TRACE_DO_IF(Cond, x) do { if (Cond) { x; } } while (0)

// Internal macros wrapping all operations
#define UE_NET_TRACE_INTERNAL_CREATE_COLLECTOR(Verbosity) (GNetTraceRuntimeVerbosity ? FNetTrace::CreateTraceCollector(Verbosity) : nullptr)
#define UE_NET_TRACE_INTERNAL_DESTROY_COLLECTOR(Collector) UE_NET_TRACE_DO_IF(Collector, FNetTrace::DestroyTraceCollector(Collector))
#define UE_NET_TRACE_INTERNAL_FLUSH_COLLECTOR(Collector, ...) UE_NET_TRACE_DO_IF(Collector, FNetTrace::TraceCollectedEvents(*Collector, __VA_ARGS__))
#define UE_NET_TRACE_INTERNAL_BEGIN_BUNCH(Collector) UE_NET_TRACE_DO_IF(Collector, FNetTrace::BeginBunch(*Collector))
#define UE_NET_TRACE_INTERNAL_DISCARD_BUNCH(Collector) UE_NET_TRACE_DO_IF(Collector, FNetTrace::DiscardBunch(*Collector))
#define UE_NET_TRACE_INTERNAL_POP_SEND_BUNCH(Collector) UE_NET_TRACE_DO_IF(Collector, FNetTrace::PopSendBunch(*Collector))
#define UE_NET_TRACE_INTERNAL_EVENTS(Collector, SrcCollector, Stream) UE_NET_TRACE_DO_IF(Collector, FNetTrace::FoldTraceCollector(Collector, SrcCollector, GetBitStreamPositionForNetTrace(Stream)))
#define UE_NET_TRACE_INTERNAL_END_BUNCH(Collector, Bunch, ...) UE_NET_TRACE_DO_IF(Collector, FNetTrace::TraceBunch(*Collector, MakeBunchInfo(Bunch), __VA_ARGS__))
#define UE_NET_TRACE_INTERNAL_BUNCH_SCOPE(Collector, Bunch, ...) FNetTraceBunchScope<decltype(Bunch)> UE_JOIN(NetTraceBunchScope, __LINE__)(Bunch, __VA_ARGS__, Collector)

#define UE_NET_TRACE_INTERNAL_SCOPE(Name, Stream, Collector, Verbosity) \
	auto UE_JOIN(NetTraceNameFunc_, __LINE__) = []() { static const UE::Net::FNetDebugName* Name = UE::Net::CreatePersistentNetDebugName(TEXT(#Name)); return FNetTrace::TraceName(Name); }; \
	FNetTrace::TChooseTraceEventScope<Verbosity, decltype(Stream)>::Type UE_JOIN(NetTraceScope, __LINE__)(UE_JOIN(NetTraceNameFunc_, __LINE__), Stream, Collector, Verbosity)

#define UE_NET_TRACE_INTERNAL_OBJECT_SCOPE(HandleOrNetGUID, Stream, Collector, Verbosity) \
	FNetTrace::TChooseTraceEventScope<Verbosity, decltype(Stream)>::Type UE_JOIN(NetTraceScope, __LINE__)(GetObjectIdForNetTrace(HandleOrNetGUID), Stream, Collector, Verbosity)

#define UE_NET_TRACE_INTERNAL_DYNAMIC_NAME_SCOPE(Name, Stream, Collector, Verbosity) \
	auto UE_JOIN(NetTraceDynamicNameFunc_, __LINE__) = [&]() { return FNetTrace::TraceName(Name); }; \
	FNetTrace::TChooseTraceEventScope<Verbosity, decltype(Stream)>::Type UE_JOIN(NetTraceScope, __LINE__)(UE_JOIN(NetTraceDynamicNameFunc_, __LINE__), Stream, Collector, Verbosity)

#define UE_NET_TRACE_INTERNAL_NAMED_SCOPE(ScopeName, EventName, Stream, Collector, Verbosity) \
	auto UE_JOIN(NetTraceNameFunc_, __LINE__) = []() { static const UE::Net::FNetDebugName* Name = UE::Net::CreatePersistentNetDebugName(TEXT(#EventName)); return FNetTrace::TraceName(Name); }; \
	FNetTrace::TChooseTraceEventScope<Verbosity, decltype(Stream)>::Type ScopeName(UE_JOIN(NetTraceNameFunc_, __LINE__), Stream, Collector, Verbosity)

#define UE_NET_TRACE_INTERNAL_NAMED_OBJECT_SCOPE(ScopeName, HandleOrNetGUID, Stream, Collector, Verbosity) \
	FNetTrace::TChooseTraceEventScope<Verbosity, decltype(Stream)>::Type ScopeName(GetObjectIdForNetTrace(HandleOrNetGUID), Stream, Collector, Verbosity)

#define UE_NET_TRACE_INTERNAL_NAMED_DYNAMIC_NAME_SCOPE(ScopeName, EventName, Stream, Collector, Verbosity) \
	auto UE_JOIN(NetTraceDynamicNameFunc_, __LINE__) = [&]() { UE::Net::FNetDebugNameId NameId = FNetTrace::TraceName(EventName); return NameId; }; \
	FNetTrace::TChooseTraceEventScope<Verbosity, decltype(Stream)>::Type ScopeName(UE_JOIN(NetTraceDynamicNameFunc_, __LINE__), Stream, Collector, Verbosity)

#define UE_NET_TRACE_INTERNAL_SET_SCOPE_NAME(ScopeName, EventName) UE_NET_TRACE_DO_IF(ScopeName.IsValid(), ScopeName.SetEventName(FNetTrace::TraceName(EventName)))
#define UE_NET_TRACE_INTERNAL_SET_SCOPE_OBJECTID(ScopeName, HandleOrNetGUID) UE_NET_TRACE_DO_IF(ScopeName.IsValid(), ScopeName.SetEventObjectId(GetObjectIdForNetTrace(HandleOrNetGUID)))
#define UE_NET_TRACE_INTERNAL_EXIT_NAMED_SCOPE(ScopeName) ScopeName.ExitScope()
#define UE_NET_TRACE_INTERNAL_OFFSET_SCOPE(Offset, Collector) FNetTraceOffsetScope UE_JOIN(NetTraceOffsetScope, __LINE__)(Offset, Collector);

#define UE_NET_TRACE_INTERNAL(Name, Collector, StartPos, EndPos, Verbosity) \
	do { if (FNetTrace::GetCollectorAtVerbosity(Collector, Verbosity)) { \
			static const UE::Net::FNetDebugName* UE_JOIN(NetTraceNameId_, __LINE__) = UE::Net::CreatePersistentNetDebugName(TEXT(#Name)); \
			FNetTrace::TracePacketContentEvent(*Collector, FNetTrace::TraceName(UE_JOIN(NetTraceNameId_, __LINE__)), StartPos, EndPos, Verbosity); \
	} } while (0)

#define UE_NET_TRACE_INTERNAL_DYNAMIC_NAME(Name, Collector, StartPos, EndPos, Verbosity) UE_NET_TRACE_DO_IF(FNetTrace::GetCollectorAtVerbosity(Collector, Verbosity), FNetTrace::TracePacketContentEvent(*Collector, FNetTrace::TraceName(Name), StartPos, EndPos, Verbosity))
#define UE_NET_TRACE_INTERNAL_ASSIGNED_GUID(GameInstanceId, NetGUID, PathName, OwnerId) UE_NET_TRACE_DO_IF(GNetTraceRuntimeVerbosity, FNetTrace::TraceObjectCreated(GameInstanceId, GetObjectIdForNetTrace(NetGUID), PathName, 0U, OwnerId))
#define UE_NET_TRACE_INTERNAL_NETHANDLE_CREATED(Handle, DebugName, ProtocolId, OwnerId) FNetTrace::TraceObjectCreated(Handle.GetReplicationSystemId(), GetObjectIdForNetTrace(Handle), DebugName, ProtocolId, OwnerId)
#define UE_NET_TRACE_INTERNAL_NETHANDLE_DESTROYED(Handle) FNetTrace::TraceObjectDestroyed(Handle.GetReplicationSystemId(), GetObjectIdForNetTrace(Handle))
#define UE_NET_TRACE_INTERNAL_CONNECTION_CREATED(...) FNetTrace::TraceConnectionCreated(__VA_ARGS__)
#define UE_NET_TRACE_INTERNAL_CONNECTION_STATE_UPDATED(...) FNetTrace::TraceConnectionStateUpdated(__VA_ARGS__)
#define UE_NET_TRACE_INTERNAL_CONNECTION_UPDATED(...) FNetTrace::TraceConnectionUpdated(__VA_ARGS__)
#define UE_NET_TRACE_INTERNAL_CONNECTION_CLOSED(...) FNetTrace::TraceConnectionClosed(__VA_ARGS__)
#define UE_NET_TRACE_INTERNAL_PACKET_DROPPED(...) UE_NET_TRACE_DO_IF(GNetTraceRuntimeVerbosity, FNetTrace::TracePacketDropped(__VA_ARGS__))
#define UE_NET_TRACE_INTERNAL_PACKET_SEND(...) UE_NET_TRACE_DO_IF(GNetTraceRuntimeVerbosity, FNetTrace::TracePacket(__VA_ARGS__, ENetTracePacketType::Outgoing))
#define UE_NET_TRACE_INTERNAL_PACKET_RECV(...) UE_NET_TRACE_DO_IF(GNetTraceRuntimeVerbosity, FNetTrace::TracePacket(__VA_ARGS__, ENetTracePacketType::Incoming))

#define UE_NET_TRACE_INTERNAL_PACKET_STATSCOUNTER(GameInstanceId, ConnectionId, Name, StatValue, Verbosity) \
	do { if (FNetTrace::GetNetTraceVerbosityEnabled(Verbosity)) { \
			static const UE::Net::FNetDebugName* UE_JOIN(NetTraceNameId_, __LINE__) = UE::Net::CreatePersistentNetDebugName(TEXT(#Name)); \
			FNetTrace::TracePacketStatsCounter(GameInstanceId, ConnectionId, FNetTrace::TraceName(UE_JOIN(NetTraceNameId_, __LINE__)), StatValue); \
	} } while (0)

#define UE_NET_TRACE_INTERNAL_FRAME_STATSCOUNTER(GameInstanceId, Name, StatValue, Verbosity) \
	do { if (FNetTrace::GetNetTraceVerbosityEnabled(Verbosity)) { \
			static const UE::Net::FNetDebugName* UE_JOIN(NetTraceNameId_, __LINE__) = UE::Net::CreatePersistentNetDebugName(TEXT(#Name)); \
			FNetTrace::TraceFrameStatsCounter(GameInstanceId, FNetTrace::TraceName(UE_JOIN(NetTraceNameId_, __LINE__)), StatValue); \
	} } while (0)

#define UE_NET_TRACE_INTERNAL_END_SESSION(GameInstanceId) FNetTrace::TraceEndSession(GameInstanceId);
#define UE_NET_TRACE_INTERNAL_UPDATE_INSTANCE(...) FNetTrace::TraceInstanceUpdated(__VA_ARGS__)

	#define UE_NET_TRACE_INTERNAL_POLL_OBJECT_SCOPE(Handle, Timer) FNetTraceObjectPollTimer __NetTracePollTimer(Timer, (GNetTraceRuntimeVerbosity >= ENetTraceVerbosity::VeryVerbose && Handle.IsValid()) ? Handle.GetReplicationSystemId() : -1, Handle.GetId());
#define UE_NET_TRACE_INTERNAL_POLL_OBJECT_IS_WASTE __NetTracePollTimer.bIsWaste = true;
#define UE_NET_TRACE_INTERNAL_QUANTIZE_OBJECT_SCOPE(Handle, Timer) FNetTraceObjectQuantizeTimer UE_JOIN(__NetTraceQuantizeTimer, __LINE__)(Timer, (GNetTraceRuntimeVerbosity >= ENetTraceVerbosity::VeryVerbose && Handle.IsValid()) ? Handle.GetReplicationSystemId() : -1, Handle.GetId());
#define UE_NET_TRACE_INTERNAL_WRITE_OBJECT_SCOPE(Handle, Timer) FNetTraceObjectWriteTimer UE_JOIN(__NetTraceWriteTimer, __LINE__)(Timer, (GNetTraceRuntimeVerbosity >= ENetTraceVerbosity::VeryVerbose && Handle.IsValid()) ? Handle.GetReplicationSystemId() : -1, Handle.GetId());	
	
#define UE_NET_TRACE_INTERNAL_SUBOBJECT(ParentHandle, SubObjectHandle) FNetTrace::TraceSubObject(ParentHandle.GetReplicationSystemId(), GetObjectIdForNetTrace(ParentHandle), GetObjectIdForNetTrace(SubObjectHandle))
					
#endif // UE_NET_TRACE_ENABLED
