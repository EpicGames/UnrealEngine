// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class IrisCore : ModuleRules
	{
		public IrisCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"NetCore",
					"TraceLog"
				}
			);

			// To be removed once UE_WITH_IRIS has been removed in code.
			PublicDefinitions.Add("UE_WITH_IRIS=1");

			//Only compile in support for Iris Parallel thread safety if not Client,Program and not disabled on the build command line 
			if (!Target.bExcludeParallelIrisSupport && !(Target.Type == TargetType.Client || Target.Type == TargetType.Program))
			{
				PublicDefinitions.Add("UE_SUPPORT_PARALLEL_IRIS=1");
			}

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ReturnTypeWarningLevel = WarningLevel.Error;
		}
	}
}
