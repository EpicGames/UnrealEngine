<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

	<!-- // Copyright Epic Games, Inc. All Rights Reserved. -->
	<!-- Iris visualizers -->

	<Type Name="UE::Net::FReplicationStateIdentifier">
		<DisplayString>{Value}</DisplayString>
	</Type>

  <Type Name="UE::Net::FNetRefHandle">
      <Intrinsic Name="_GetId" Expression="(Fields.Serial &lt;&lt; 1 | Fields.Static &amp; 1)"></Intrinsic>
      <DisplayString Condition="this->_GetId()==0">Invalid</DisplayString>
      <DisplayString Condition="this->_GetId() !=0 &amp;&amp; Fields.ReplicationSystemId==0">Id={this->_GetId()} Static={Fields.Static} ReplicationSystemId=Unknown</DisplayString>
      <DisplayString Condition="this->_GetId() !=0">Id={this->_GetId()} Static={Fields.Static} ReplicationSystemId={Fields.ReplicationSystemId - 1}</DisplayString>
  </Type>

	<Type Name="UE::Net::FNetSerializer">
		<DisplayString>SerializerName = {Name, sub}</DisplayString>
	</Type>

  <Type Name="UE::Net::FNetDebugName">
    <DisplayString>{Name, sub}</DisplayString>
  </Type>

  <Type Name="UE::Net::FReplicationStateMemberSerializerDescriptor">
		<DisplayString>SerializerName = {Serializer->Name, sub}</DisplayString>
		<Expand>
			<Item Name="SerializerConfig">SerializerConfig</Item>
		</Expand>
	</Type>

  <!-- Debugging tip: When debugging a breakpoint and you have access to a tracefile on the sending side it can be useful to add an offset to the BitOffset to align the trace with ReadJournal -->
  <Type Name="UE::Net::FNetJournal::FJournalEntry">
    <DisplayString>Offset: {BitOffset} {Name, sub} Handle: {NetRefHandle}</DisplayString>
  </Type>

  <Type Name="UE::Net::FNetJournal">
    <DisplayString>StartIndex:{(NumEntries - JournalSize) &amp; JournalMask}</DisplayString>
    <Expand>
      <IndexListItems Condition="NumEntries &lt; JournalSize">
        <Size>NumEntries</Size>
        <ValueNode> (Entries[($i) &amp; JournalMask])</ValueNode>
      </IndexListItems>
      <IndexListItems Condition="NumEntries &gt;= JournalSize">
        <Size>JournalSize</Size>
        <ValueNode> (Entries[(NumEntries - JournalSize + $i) &amp; JournalMask])</ValueNode>
      </IndexListItems>
    </Expand>
  </Type>

  <!-- FNetBitArray visualizer -->
  <Type Name="UE::Net::FNetBitArray">
    <DisplayString>BitCount={BitCount}</DisplayString>
    <Expand>
      <IndexListItems Condition="BitCount &gt; 0">
        <Size>BitCount</Size>
        <ValueNode Condition="(reinterpret_cast&lt;uint32*&gt;(Storage.AllocatorInstance.Data)[$i/32]&gt;&gt;($i%32) &amp; 1) != 0">1</ValueNode>
        <ValueNode Condition="(reinterpret_cast&lt;uint32*&gt;(Storage.AllocatorInstance.Data)[$i/32]&gt;&gt;($i%32) &amp; 1) == 0">0</ValueNode>
      </IndexListItems>
    </Expand>
  </Type>

  <!-- FNetBitArray visualizer -->
  <Type Name="UE::Net::FNetBitArrayView">
    <DisplayString>BitCount={BitCount}</DisplayString>
    <Expand>
      <IndexListItems Condition="BitCount &gt; 0">
        <Size>BitCount</Size>
        <ValueNode Condition="(reinterpret_cast&lt;uint32*&gt;(Storage)[$i/32]&gt;&gt;($i%32) &amp; 1) != 0">1</ValueNode>
        <ValueNode Condition="(reinterpret_cast&lt;uint32*&gt;(Storage)[$i/32]&gt;&gt;($i%32) &amp; 1) == 0">0</ValueNode>
      </IndexListItems>
    </Expand>
  </Type>

  <!-- FReplicationInfo visualizer -->
  <Type Name="UE::Net::Private::FReplicationWriter::FReplicationInfo">
    <DisplayString>State={(FReplicationWriter::EReplicatedObjectState)State} IsSubObject={IsSubObject} IsDestructionInfo={IsDestructionInfo} HasAttachments={HasAttachments} HasDirtyChangeMask={HasDirtyChangeMask} HasDirtySubObjects={HasDirtySubObjects}</DisplayString>
    <Expand>
      <Item Name="State">(FReplicationWriter::EReplicatedObjectState)State</Item>
      <Item Name="HasDirtyChangeMask">HasDirtyChangeMask</Item>
      <Item Name="IsSubObject">IsSubObject</Item>
      <Item Name="HasDirtySubObjects">HasDirtySubObjects</Item>
      <Item Name="HasAttachments">HasAttachments</Item>
      <Item Name="HasChangemaskFilter">HasChangemaskFilter</Item>
      <Item Name="IsDestructionInfo">IsDestructionInfo</Item>
      <Item Name="IsCreationConfirmed">IsCreationConfirmed</Item>
      <Item Name="TearOff">TearOff</Item>
      <Item Name="SubObjectPendingDestroy">SubObjectPendingDestroy</Item>
      <Item Name="IsDeltaCompressionEnabled">IsDeltaCompressionEnabled</Item>
      <Item Name="LastAckedBaselineIndex">LastAckedBaselineIndex</Item>
      <Item Name="PendingBaselineIndex">PendingBaselineIndex</Item>
      <Item Name="FlushFlags">FlushFlags</Item>
      <Item Name="HasDirtyConditionals">HasDirtyConditionals</Item>
      <Item Name="HasCannotSendInfo">HasCannotSendInfo</Item>
      <Item Name="IsDebugObject">IsDebugObject</Item>
      <Item Name="IsFilteredOutSubObjectPendingCreate">IsFilteredOutSubObjectPendingCreate</Item>
      <Item Name="IsFilteredOutSubObjectTearOff">IsFilteredOutSubObjectTearOff</Item>
      <Item Name="ChangeMaskBitCount">ChangeMaskBitCount</Item>
      <Item Name="Inlined ChangeMask BitCount" Condition="ChangeMaskBitCount &lt;= 64">ChangeMaskBitCount</Item>
      <IndexListItems Condition="ChangeMaskBitCount &lt;= 64">
        <Size>ChangeMaskBitCount</Size>
        <ValueNode Condition="(reinterpret_cast&lt;uint64*&gt;(&amp;ChangeMaskOrPtr.ChangeMaskOrPointer)[$i/64]&gt;&gt;($i%64) &amp; 1) != 0">1</ValueNode>
        <ValueNode Condition="(reinterpret_cast&lt;uint64*&gt;(&amp;ChangeMaskOrPtr.ChangeMaskOrPointer)[$i/64]&gt;&gt;($i%64) &amp; 1) == 0">0</ValueNode>
      </IndexListItems>
      <Item Name="External ChangeMask" Condition="ChangeMaskBitCount &gt; 64">ChangeMaskOrPtr</Item>
    </Expand>
  </Type>

  <!-- FReplicationStateDescriptor visualizer -->
	<Type Name="UE::Net::FReplicationStateDescriptor">
		<DisplayString>Name={DebugName->Name, sub} Id={DescriptorIdentifier,X} MemberCount={MemberCount}</DisplayString>
		<Expand>
			<!-- Traits -->
			<Item Name="InitOnly"                           Condition="Traits &amp; (EReplicationStateTraits::InitOnly)">true</Item>
			<Item Name="HasLifetimeConditionals"            Condition="Traits &amp; (EReplicationStateTraits::HasLifetimeConditionals)">true</Item>
			<Item Name="HasObjectReference"                 Condition="Traits &amp; (EReplicationStateTraits::HasObjectReference)">true</Item>
			<Item Name="NeedsRefCount"			            Condition="Traits &amp; (EReplicationStateTraits::NeedsRefCount)">true</Item>
			<Item Name="HasRepNotifies"			            Condition="Traits &amp; (EReplicationStateTraits::HasRepNotifies)">true</Item>
			<Item Name="KeepPreviousState"		            Condition="Traits &amp; (EReplicationStateTraits::KeepPreviousState)">true</Item>
			<Item Name="HasDynamicState"		            Condition="Traits &amp; (EReplicationStateTraits::HasDynamicState)">true</Item>
            <Item Name="IsSourceTriviallyConstructible"     Condition="Traits &amp; (EReplicationStateTraits::IsSourceTriviallyConstructible)">true</Item>
			<Item Name="IsSourceTriviallyDestructible"      Condition="Traits &amp; (EReplicationStateTraits::IsSourceTriviallyDestructible)">true</Item>
			<Item Name="AllMembersAreReplicated"            Condition="Traits &amp; (EReplicationStateTraits::AllMembersAreReplicated)">true</Item>
			<Item Name="IsFastArrayReplicationState"        Condition="Traits &amp; (EReplicationStateTraits::IsFastArrayReplicationState)">true</Item>
            <Item Name="IsNativeFastArrayReplicationState"  Condition="Traits &amp; (EReplicationStateTraits::IsNativeFastArrayReplicationState)">true</Item>
			<Item Name="HasConnectionSpecificSerialization" Condition="Traits &amp; (EReplicationStateTraits::HasConnectionSpecificSerialization)">true</Item>
            <Item Name="HasPushBasedDirtiness"              Condition="Traits &amp; (EReplicationStateTraits::HasPushBasedDirtiness)">true</Item>
            <Item Name="HasFullPushBasedDirtiness"          Condition="Traits &amp; (EReplicationStateTraits::HasFullPushBasedDirtiness)">true</Item>
            <Item Name="SupportsDeltaCompression"           Condition="Traits &amp; (EReplicationStateTraits::SupportsDeltaCompression)">true</Item>
            <Item Name="UseSerializerIsEqual"               Condition="Traits &amp; (EReplicationStateTraits::UseSerializerIsEqual)">true</Item>
            <Item Name="IsDerivedFromStructWithCustomSerializer" Condition="Traits &amp; (EReplicationStateTraits::IsDerivedFromStructWithCustomSerializer)">true</Item>
            <Item Name="IsStructWithCustomSerializer" Condition="Traits &amp; (EReplicationStateTraits::IsStructWithCustomSerializer)">true</Item>
			<Item Name="ExternalSize">ExternalSize</Item>
			<Item Name="InternalSize">InternalSize</Item>
			<Item Name="MemberCount">MemberCount</Item>
			<IndexListItems>
				<Size>MemberCount</Size>
				<ValueNode>(MemberSerializerDescriptors[$i])</ValueNode>
			</IndexListItems>
			<ArrayItems>
				<Size>MemberCount</Size>
				<ValuePointer>MemberDescriptors</ValuePointer>
			</ArrayItems>
			<ArrayItems Condition="MemberProperties != nullptr">
				<Size>MemberCount</Size>
				<ValuePointer>MemberProperties</ValuePointer>
			</ArrayItems>
		</Expand>
	</Type>

	<!-- FReplicationStateMemberLifetimeConditionDescriptor visualizer -->
	<Type Name="UE::Net::FReplicationStateMemberLifetimeConditionDescriptor">
		<DisplayString>Condition={(ELifetimeCondition)Condition}</DisplayString>
    </Type>

    <!-- FReplicationProtocol visualizer -->
	<Type Name="UE::Net::FReplicationProtocol">
		<DisplayString>ProtocolName = {DebugName->Name} ProtocolId={ProtocolIdentifier} StateCount={ReplicationStateCount}</DisplayString>
		<Expand>
			<Item Name="InternalTotalSize">InternalTotalSize</Item>
			<Item Name="InternalTotalAlignment">InternalTotalAlignment</Item>
			<Item Name="MemberCount">ReplicationStateCount</Item>
			<IndexListItems>
				<Size>ReplicationStateCount</Size>
				<ValueNode>*(ReplicationStateDescriptors[$i])</ValueNode>
			</IndexListItems>
		</Expand>
	</Type>

	<!-- FInstanceProtocol visualizer -->
	<Type Name="UE::Net::FReplicationInstanceProtocol">
		<DisplayString>FragmentCount={FragmentCount}</DisplayString>
		<Expand>
			<Item Name="FragmentCount">FragmentCount</Item>
			<IndexListItems>
				<Size>FragmentCount</Size>
				<ValueNode>*(Fragments[$i])</ValueNode>
			</IndexListItems>
		</Expand>
	</Type>

</AutoVisualizer>
