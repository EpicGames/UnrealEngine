// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Config/IrisDynamicConfig.h"

#include "CoreGlobals.h" // for IsEngineExitRequested()
#include "Containers/Map.h"
#include "Iris/Core/IrisLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

namespace UE::Net::Private
{

class FIrisDynamicConfigImpl
{
public:
	void RegisterDynamicConfig(FName UniqueConfigName, const FString& FilePath);
	void RegisterDynamicConfigBuffer(FName UniqueConfigName, const FString& VirtualFilePath, const FString& ConfigContents);
	void RegisterDynamicConfigBuffer(FName UniqueConfigName, const FString& VirtualFilePath, FUtf8StringView ConfigContents);
	void UnregisterDynamicConfig(FName UniqueConfigName);

	void GetSection(const TCHAR* SectionName, TArray<FString>& OutKeyValuePairs);
	void GetArray(const TCHAR* SectionName, const TCHAR* ArrayName, TArray<FString>& OutValues);

	FIrisDynamicConfig::FOnIrisDynamicConfigChange OnIrisDynamicConfigChange;

private:
	void OnAppPreExit();
	FConfigBranch* GetOrCreateConfig();

	// The config system uses three different strings types to retrieve things...
	inline static FString VirtualBranchFileName = "Iris.ini";

	// The cache is only needed to be able to construct the FConfigBranch.
	TOptional<FConfigCacheIni> ConfigCache;
	FConfigBranch* IrisConfig = nullptr;
	TCompactMap<FName, FString> ConfigNameToFilePath;
};

static FIrisDynamicConfigImpl IrisDynamicConfig;

}

namespace UE::Net
{

void FIrisDynamicConfig::RegisterDynamicConfig(FName UniqueConfigName, const FString& FilePath)
{
	Private::IrisDynamicConfig.RegisterDynamicConfig(UniqueConfigName, FilePath);
}

void FIrisDynamicConfig::RegisterDynamicConfigBuffer(FName UniqueConfigName, const FString& VirtualFilePath, const FString& ConfigContents)
{
	Private::IrisDynamicConfig.RegisterDynamicConfigBuffer(UniqueConfigName, VirtualFilePath, ConfigContents);
}

void FIrisDynamicConfig::RegisterDynamicConfigBuffer(FName UniqueConfigName, const FString& VirtualFilePath, FUtf8StringView ConfigContents)
{
	Private::IrisDynamicConfig.RegisterDynamicConfigBuffer(UniqueConfigName, VirtualFilePath, ConfigContents);
}

void FIrisDynamicConfig::UnregisterDynamicConfig(FName UniqueConfigName)
{
	Private::IrisDynamicConfig.UnregisterDynamicConfig(UniqueConfigName);
}

FIrisDynamicConfig::FOnIrisDynamicConfigChange::RegistrationType& FIrisDynamicConfig::OnIrisDynamicConfigChange()
{
	return Private::IrisDynamicConfig.OnIrisDynamicConfigChange;
}

void FIrisDynamicConfig::GetSection(const TCHAR* SectionName, TArray<FString>& OutKeyValuePairs)
{
	Private::IrisDynamicConfig.GetSection(SectionName, OutKeyValuePairs);
}

void FIrisDynamicConfig::GetArray(const TCHAR* SectionName, const TCHAR* ArrayName, TArray<FString>& OutValues)
{
	Private::IrisDynamicConfig.GetArray(SectionName, ArrayName, OutValues);
}

}

// FIrisDynamicConfigImpl
namespace UE::Net::Private
{

void FIrisDynamicConfigImpl::OnAppPreExit()
{
	IrisConfig = nullptr;
	ConfigCache.Reset();
}

FConfigBranch* FIrisDynamicConfigImpl::GetOrCreateConfig()
{
	if (IrisConfig == nullptr)
	{
		ConfigCache.Emplace(EConfigCacheType::Temporary);
		IrisConfig = &ConfigCache->AddNewBranch(VirtualBranchFileName);

		FCoreDelegates::OnPreExit.AddRaw(this, &FIrisDynamicConfigImpl::OnAppPreExit);
	}

	UE_ASSUME(IrisConfig != nullptr);
	return IrisConfig;
}

void FIrisDynamicConfigImpl::RegisterDynamicConfig(FName UniqueConfigName, const FString& FilePath)
{
	if (IsEngineExitRequested())
	{
		return;
	}

	if (!ensure(IsInGameThread()))
	{
		UE_LOGF(LogIris, Error, "FIrisDynamicConfig::RegisterDynamicConfig called from non-game thread. IGNORING dynamic config named '%ls'.", ToCStr(UniqueConfigName.ToString()));
		return;
	}
	
	if (ensureMsgf(!ConfigNameToFilePath.Find(UniqueConfigName), TEXT("FIrisDynamicConfig::RegisterDynamicConfig A dynamic config named '%s' has already been registered. IGNORING it."), ToCStr(UniqueConfigName.ToString())))
	{
		ConfigNameToFilePath.Add(UniqueConfigName, FilePath);

		// Add config name to map and load it
		FConfigModificationTracker ModTracker;
		GetOrCreateConfig()->AddDynamicLayerToHierarchy(FilePath, &ModTracker);

		// We're only modifying our branch so any modified section would be in there.
		if (const TPair<FName, TSet<FString>>* ModifiedSections = ModTracker.ModifiedSectionsPerBranch.FindArbitraryElement())
		{
			OnIrisDynamicConfigChange.Broadcast(ModifiedSections->Value);
		}
	}
}

void FIrisDynamicConfigImpl::RegisterDynamicConfigBuffer(FName UniqueConfigName, const FString& VirtualFilePath, const FString& ConfigContents)
{
	if (IsEngineExitRequested())
	{
		return;
	}

	if (!ensure(IsInGameThread()))
	{
		UE_LOGF(LogIris, Error, "FIrisDynamicConfig::RegisterDynamicConfigBuffer called from non-game thread. IGNORING dynamic config named '%ls'.", ToCStr(UniqueConfigName.ToString()));
		return;
	}

	if (ConfigContents.IsEmpty())
	{
		UE_LOGF(LogIris, Verbose, "FIrisDynamicConfig::RegisterDynamicConfigBuffer Ignoring empty config named '%ls'", ToCStr(UniqueConfigName.ToString()));
		return;
	}

	if (ensureMsgf(!ConfigNameToFilePath.Find(UniqueConfigName), TEXT("FIrisDynamicConfig::RegisterDynamicConfigBuffer A dynamic config named '%s' has already been registered. IGNORING it."), ToCStr(UniqueConfigName.ToString())))
	{
		ConfigNameToFilePath.Add(UniqueConfigName, VirtualFilePath);

		// Add config name to map and process the config contents
		constexpr EName Tag = NAME_None;
		constexpr DynamicLayerPriority LayerPriority = DynamicLayerPriority::Unknown;
		FConfigModificationTracker ModTracker;
		GetOrCreateConfig()->AddDynamicLayerStringToHierarchy(VirtualFilePath, ConfigContents, Tag, LayerPriority, &ModTracker);

		// We're only modifying our branch so any modified section would be in there.
		if (const TPair<FName, TSet<FString>>* ModifiedSections = ModTracker.ModifiedSectionsPerBranch.FindArbitraryElement())
		{
			OnIrisDynamicConfigChange.Broadcast(ModifiedSections->Value);
		}
	}
}

void FIrisDynamicConfigImpl::RegisterDynamicConfigBuffer(FName UniqueConfigName, const FString& VirtualFilePath, FUtf8StringView ConfigContents)
{
	const FString ContentsAsString(ConfigContents);
	return RegisterDynamicConfigBuffer(UniqueConfigName, VirtualFilePath, ContentsAsString);
}

void FIrisDynamicConfigImpl::UnregisterDynamicConfig(FName UniqueConfigName)
{
	if (IsEngineExitRequested())
	{
		return;
	}

	if (!ensure(IsInGameThread()))
	{
		UE_LOGF(LogIris, Error, "FIrisDynamicConfig::UnregisterDynamicConfig called from non-game thread. IGNORING unregistering config named '%ls'.", ToCStr(UniqueConfigName.ToString()));
		return;
	}

	FString FilePath;
	if (!ConfigNameToFilePath.RemoveAndCopyValue(UniqueConfigName, FilePath))
	{
		return;
	}

	FConfigModificationTracker ModTracker;
	GetOrCreateConfig()->RemoveDynamicLayerFromHierarchy(FilePath, &ModTracker);

	// We're only modifying our branch so any modified section would be in there.
	if (const TPair<FName,TSet<FString>>* ModifiedSections = ModTracker.ModifiedSectionsPerBranch.FindArbitraryElement())
	{
		OnIrisDynamicConfigChange.Broadcast(ModifiedSections->Value);
	}
}

void FIrisDynamicConfigImpl::GetSection(const TCHAR* SectionName, TArray<FString>& OutKeyValuePairs)
{
	if (FConfigCacheIni* Config = ConfigCache.GetPtrOrNull())
	{
		Config->GetSection(SectionName, OutKeyValuePairs, VirtualBranchFileName);
	}
}

void FIrisDynamicConfigImpl::GetArray(const TCHAR* SectionName, const TCHAR* ArrayName, TArray<FString>& OutValues)
{
	if (FConfigCacheIni* Config = ConfigCache.GetPtrOrNull())
	{
		Config->GetArray(SectionName, ArrayName, OutValues, VirtualBranchFileName);
	}
}

}
