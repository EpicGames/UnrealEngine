// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Core/IrisDebugging.h"

#if !UE_BUILD_SHIPPING

#include "UObject/Field.h"
#include "HAL/IConsoleManager.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationProtocolManager.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Containers/StringFwd.h"

namespace UE::Net::IrisDebugHelper
{

namespace IrisDebugHelperInternal
{
	static FString GIrisDebugName;
	FAutoConsoleVariableRef NetIrisDebugName(
		TEXT("Net.Iris.DebugName"),
		GIrisDebugName,
		TEXT("Set a class name or object name to break on."),
		ECVF_Default);

	static FString GIrisDebugRPCName;
	FAutoConsoleVariableRef NetRPCSetDebugRPCName(
		TEXT("Net.Iris.DebugRPCName"),
		GIrisDebugRPCName,
		TEXT("Set the name of an RPC to break on."),
		ECVF_Default);

	static FNetRefHandle GIrisDebugNetRefHandle;
	static FAutoConsoleCommand NetIrisDebugNetRefHandle(
		TEXT("Net.Iris.DebugNetRefHandle"), 
		TEXT("Specify a NetRefHandle ID that we will break on (or none to turn off)."), 
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (Args.Num() > 0)
			{
				uint32 NetId = 0;
				LexFromString(NetId, *Args[0]);
				GIrisDebugNetRefHandle = Private::FNetRefHandleManager::MakeNetRefHandleFromId(NetId);
			}
			else
			{
				GIrisDebugNetRefHandle = FNetRefHandle();
			}
		}));

	static UE::Net::FInternalNetRefIndex GIrisDebugInternalIndex = InvalidInternalNetRefIndex;
	static FAutoConsoleCommand NetIrisDebugNetInternalIndex(
		TEXT("Net.Iris.DebugNetInternalIndex"),
		TEXT("Specify an internal index that we will break on (or none to turn off)."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				if (Args.Num() > 0)
				{
					uint32 InternalIndex = 0;
					LexFromString(InternalIndex, *Args[0]);
					GIrisDebugInternalIndex = InternalIndex;
				}
				else
				{
					GIrisDebugInternalIndex = 0;
				}
			}));
}; // namespace IrisDebugHelperInternal

bool BreakOnObjectName(UObject* Object)
{
	if (IrisDebugHelperInternal::GIrisDebugName.IsEmpty() == false && GetNameSafe(Object).Contains(IrisDebugHelperInternal::GIrisDebugName))
	{
		UE_DEBUG_BREAK();
		return true;
	}

	return false;
}

bool FilterDebuggedObject(UObject* Object)
{
	if (IrisDebugHelperInternal::GIrisDebugName.IsEmpty() || GetNameSafe(Object).Contains(IrisDebugHelperInternal::GIrisDebugName))
	{
		return true;
	}

	return false;
}

bool BreakOnNetRefHandle(FNetRefHandle NetRefHandle)
{
	if (IrisDebugHelperInternal::GIrisDebugNetRefHandle.IsValid() && IrisDebugHelperInternal::GIrisDebugNetRefHandle == NetRefHandle)
	{
		UE_DEBUG_BREAK();
		return true;
	}

	return false;
}

bool BreakOnRPCName(FName RPCName)
{
	if (IrisDebugHelperInternal::GIrisDebugRPCName.IsEmpty() == false && RPCName.GetPlainNameString().Contains(IrisDebugHelperInternal::GIrisDebugRPCName))
	{
		UE_DEBUG_BREAK();
		return true;
	}

	return false;
}


bool BreakOnInternalNetRefIndex(UE::Net::FInternalNetRefIndex InternalIndex)
{
	if (IrisDebugHelperInternal::GIrisDebugInternalIndex != 0 && IrisDebugHelperInternal::GIrisDebugInternalIndex == InternalIndex)
	{
		UE_DEBUG_BREAK();
		return true;
	}

	return false;
}

UE::Net::FInternalNetRefIndex GetDebugInternalNetRefIndex()
{
	return IrisDebugHelperInternal::GIrisDebugInternalIndex;
}

FNetRefHandle GetDebugNetRefHandle()
{
	return IrisDebugHelperInternal::GIrisDebugNetRefHandle;
}

void SetIrisDebugObjectName(const ANSICHAR* NameBuffer)
{
	if (NameBuffer)
	{
		IrisDebugHelperInternal::GIrisDebugName = ANSI_TO_TCHAR(NameBuffer);
	}
	else
	{
		IrisDebugHelperInternal::GIrisDebugName = FString();
	}
}

void SetIrisDebugNetRefHandle(uint64 NetRefHandleId)
{
	IrisDebugHelperInternal::GIrisDebugNetRefHandle = Private::FNetRefHandleManager::MakeNetRefHandleFromId(NetRefHandleId);
}

void SetIrisDebugRPCName(const ANSICHAR* NameBuffer)
{
	if (NameBuffer)
	{
		IrisDebugHelperInternal::GIrisDebugRPCName = ANSI_TO_TCHAR(NameBuffer);
	}
	else
	{
		IrisDebugHelperInternal::GIrisDebugRPCName = FString();
	}
}

void SetIrisDebugInternalNetRefIndex(UE::Net::FInternalNetRefIndex InternalIndex)
{
	IrisDebugHelperInternal::GIrisDebugInternalIndex = InternalIndex;
}

void SetIrisDebugInternalNetRefIndexViaNetHandle(FNetRefHandle RefHandle)
{
	using namespace UE::Net::Private;

	if (!RefHandle.IsValid())
	{
		IrisDebugHelperInternal::GIrisDebugNetRefHandle = RefHandle;
		IrisDebugHelperInternal::GIrisDebugInternalIndex = InvalidInternalNetRefIndex;
		return;
	}

	UReplicationSystem* ReplicationSystem = GetReplicationSystem(RefHandle.GetReplicationSystemId());
	if (!ReplicationSystem)
	{
		return;
	}

	FReplicationSystemInternal* ReplicationSystemInternal = ReplicationSystem->GetReplicationSystemInternal();
	const FNetRefHandleManager& NetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();

	const FInternalNetRefIndex InternalNetRefIndex = NetRefHandleManager.GetInternalIndex(RefHandle);
	if (!InternalNetRefIndex)
	{
		return;
	}

	IrisDebugHelperInternal::GIrisDebugNetRefHandle = RefHandle;
	IrisDebugHelperInternal::GIrisDebugInternalIndex = InternalNetRefIndex;
}

void SetIrisDebugInternalNetRefIndexViaObject(UObject* Instance)
{
	using namespace UE::Net::Private;

	if (!Instance)
	{
		IrisDebugHelperInternal::GIrisDebugNetRefHandle = FNetRefHandle();
		IrisDebugHelperInternal::GIrisDebugInternalIndex = InvalidInternalNetRefIndex;
		return;
	}

	// See if we can find the instance in any replication system
	for (const UReplicationSystem* ReplicationSystem : FReplicationSystemFactory::GetAllReplicationSystems())
	{
		if (ReplicationSystem)
		{
			if (const UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UObjectReplicationBridge>())
			{
				FNetRefHandle Handle = Bridge->GetReplicatedRefHandle(Instance);
				if (Handle.IsValid())
				{
					const FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
					FInternalNetRefIndex InternalIndex = NetRefHandleManager.GetInternalIndex(Handle);

					IrisDebugHelperInternal::GIrisDebugNetRefHandle = Handle;
					IrisDebugHelperInternal::GIrisDebugInternalIndex = InternalIndex;
					break;
				}
			}
		}
	}
}


UReplicationSystem* GetReplicationSystemForDebug(uint32 Id)
{
	return GetReplicationSystem(Id);
}

#define UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(FunctionName) FunctionReferenceAccumulator += uint64(&FunctionName);

uint64 Init()
{
	uint64 FunctionReferenceAccumulator = uint64(0);

	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(GetReplicationSystemForDebug);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(DebugOutputNetObjectState);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(DebugNetObjectStateToString);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(DebugOutputNetObjectProtocolReferences);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(DebugNetObject);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(DebugNetObjectById);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(DebugNetRefHandle);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(DebugNetRefHandleById);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(DebugInternalNetRefIndex);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(DebugNetObjectProtocolReferencesToString);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(SetIrisDebugObjectName);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(SetIrisDebugNetRefHandle);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(SetIrisDebugInternalNetRefIndex);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(SetIrisDebugInternalNetRefIndexViaNetHandle);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(SetIrisDebugInternalNetRefIndexViaObject);
	UE_NET_FORCE_REFERENCE_DEBUGFUNCTION(SetIrisDebugRPCName);
	
	return FunctionReferenceAccumulator;
}

#undef UE_NET_FORCE_REFERENCE_DEBUGFUNCTION

void NetObjectStateToString(FStringBuilderBase& StringBuilder, FNetRefHandle RefHandle)
{
	using namespace UE::Net::Private;

	if (!RefHandle.IsValid())
	{
		return;
	}

	UReplicationSystem* ReplicationSystem = GetReplicationSystem(RefHandle.GetReplicationSystemId());
	if (!ReplicationSystem)
	{
		return;
	}

	FReplicationSystemInternal* ReplicationSystemInternal = ReplicationSystem->GetReplicationSystemInternal();
	const FNetRefHandleManager& NetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();

	const FInternalNetRefIndex InternalNetRefIndex = NetRefHandleManager.GetInternalIndex(RefHandle);
	if (!InternalNetRefIndex)
	{
		return;
	}

	const bool bIsServer = ReplicationSystem->IsServer();
	const FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = NetRefHandleManager.GetReplicatedObjectDataNoCheck(InternalNetRefIndex);	
	const uint8* InternalStateBuffer = bIsServer ? NetRefHandleManager.GetReplicatedObjectStateBufferNoCheck(InternalNetRefIndex) : ReplicatedObjectData.ReceiveStateBuffer;
	if (ReplicatedObjectData.InstanceProtocol == nullptr || InternalStateBuffer == nullptr)
	{
		return;
	}

	const FReplicationProtocol* Protocol = ReplicatedObjectData.Protocol;

	// In order to be able to output object references we need the TokenStoreState, for the server we just use the local one but if we are a client we must use the remote token store state
	// If this is a client handle we assume that we only have a single connections and use the first valid connection to get the remote token store.
	FReplicationConnections& Connections = ReplicationSystemInternal->GetConnections();
	const uint32 FirstValidConnectionId = Connections.GetValidConnections().FindFirstOne();

	// Setup Context
	FInternalNetSerializationContext InternalContext;
	FInternalNetSerializationContext::FInitParameters InternalContextInitParams;
	InternalContextInitParams.ReplicationSystem = ReplicationSystem;
	InternalContextInitParams.PackageMap = ReplicationSystemInternal->GetIrisObjectReferencePackageMap();
	InternalContextInitParams.ObjectResolveContext.RemoteNetTokenStoreState = FirstValidConnectionId == FNetBitArray::InvalidIndex ? nullptr :ReplicationSystem->GetNetTokenStore()->GetRemoteNetTokenStoreState(FirstValidConnectionId);
	InternalContextInitParams.ObjectResolveContext.ConnectionId = (FirstValidConnectionId == FNetBitArray::InvalidIndex ? InvalidConnectionId : FirstValidConnectionId);
	InternalContext.Init(InternalContextInitParams);

	FNetSerializationContext NetSerializationContext;
	NetSerializationContext.SetInternalContext(&InternalContext);
	NetSerializationContext.SetLocalConnectionId(InternalContextInitParams.ObjectResolveContext.ConnectionId);

	StringBuilder << TEXT("State for ") << RefHandle;
	StringBuilder.Appendf(TEXT(" of type %s\n"), Protocol->DebugName->Name);

	FReplicationInstanceOperations::OutputInternalStateToString(NetSerializationContext, StringBuilder, nullptr, InternalStateBuffer, ReplicatedObjectData.InstanceProtocol, ReplicatedObjectData.Protocol);
}

void DebugOutputNetObjectState(uint64 NetRefHandleId, uint32 ReplicationSystemId)
{
	const FNetRefHandle RefHandle = Private::FNetRefHandleManager::MakeNetRefHandle(NetRefHandleId, ReplicationSystemId);

	TStringBuilder<4096> StringBuilder;

	NetObjectStateToString(StringBuilder, RefHandle);

	FPlatformMisc::LowLevelOutputDebugString(StringBuilder.ToString());
}

const TCHAR* DebugNetObjectStateToString(uint32 NetRefHandleId, uint32 ReplicationSystemId)
{
	static TStringBuilder<4096> StringBuilder;

	StringBuilder.Reset();
	const FNetRefHandle RefHandle = Private::FNetRefHandleManager::MakeNetRefHandle(NetRefHandleId, ReplicationSystemId);
	NetObjectStateToString(StringBuilder, RefHandle);

	return StringBuilder.ToString();
}

FNetReplicatedObjectDebugInfo DebugNetObject(UObject* Instance)
{
	using namespace UE::Net::Private;
	FNetReplicatedObjectDebugInfo Info = {};

	// See if we can find the instance in any replication system
	for (const UReplicationSystem* ReplicationSystem : FReplicationSystemFactory::GetAllReplicationSystems())
	{
		if (ReplicationSystem)
		{
			if (const UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UObjectReplicationBridge>())
			{
				FNetRefHandle Handle = Bridge->GetReplicatedRefHandle(Instance);
				if (Handle.IsValid())
				{
					const FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
					Info.InternalNetRefIndex = NetRefHandleManager.GetInternalIndex(Handle);

					const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager.GetReplicatedObjectDataNoCheck(Info.InternalNetRefIndex);				

					Info.RefHandle = &ObjectData.RefHandle;
					Info.Protocol = ObjectData.Protocol;
					Info.InstanceProtocol = ObjectData.InstanceProtocol;
					Info.ReplicationSystem = ReplicationSystem;
					Info.Object = NetRefHandleManager.GetReplicatedInstances()[Info.InternalNetRefIndex];

					return Info;
				}
			}
		}
	}

	return Info;
}

FNetReplicatedObjectDebugInfo DebugNetObjectById(UObject* Instance, uint32 ReplicationSystemId)
{
	using namespace UE::Net::Private;
	FNetReplicatedObjectDebugInfo Info = {};

	const UReplicationSystem* ReplicationSystem = GetReplicationSystemForDebug(ReplicationSystemId);
	if (ReplicationSystem)
	{
		if (const UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UObjectReplicationBridge>())
		{
			FNetRefHandle Handle = Bridge->GetReplicatedRefHandle(Instance);
			if (Handle.IsValid())
			{
				const FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
				Info.InternalNetRefIndex = NetRefHandleManager.GetInternalIndex(Handle);

				const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager.GetReplicatedObjectDataNoCheck(Info.InternalNetRefIndex);

				Info.RefHandle = &ObjectData.RefHandle;
				Info.Protocol = ObjectData.Protocol;
				Info.InstanceProtocol = ObjectData.InstanceProtocol;
				Info.ReplicationSystem = ReplicationSystem;
				Info.Object = NetRefHandleManager.GetReplicatedInstances()[Info.InternalNetRefIndex];

				return Info;
			}
		}
	}

	return Info;
}

FNetReplicatedObjectDebugInfo DebugNetRefHandle(FNetRefHandle Handle)
{
	return DebugNetRefHandleById(Handle.GetId(), Handle.GetReplicationSystemId());
}

FNetReplicatedObjectDebugInfo DebugNetRefHandleById(uint64 NetRefHandleId, uint32 ReplicationSystemId)
{
	using namespace UE::Net::Private;
	FNetReplicatedObjectDebugInfo Info = {};

	// See if we can find the instance in any replication system
	{
		const UReplicationSystem* ReplicationSystem = GetReplicationSystemForDebug(ReplicationSystemId);
		if (ReplicationSystem)
		{
			const FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
			FNetRefHandle IncompleteHandle = FNetRefHandleManager::MakeNetRefHandle(NetRefHandleId, ReplicationSystemId);
			const uint32 InternalNetRefIndex = Info.InternalNetRefIndex = NetRefHandleManager.GetInternalIndex(IncompleteHandle);
			if (InternalNetRefIndex != InvalidInternalNetRefIndex)
			{
				const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager.GetReplicatedObjectDataNoCheck(InternalNetRefIndex);
				if (ObjectData.RefHandle == IncompleteHandle)
				{
					Info.RefHandle = &ObjectData.RefHandle;
					Info.InternalNetRefIndex = InternalNetRefIndex;
					Info.Protocol = ObjectData.Protocol;
					Info.InstanceProtocol = ObjectData.InstanceProtocol;
					Info.ReplicationSystem = ReplicationSystem;
					Info.Object = NetRefHandleManager.GetReplicatedInstances()[InternalNetRefIndex];

					return Info;
				}
			}
		}
	}

	return Info;
}

FNetReplicatedObjectDebugInfo DebugInternalNetRefIndex(uint32 InternalIndex, uint32 ReplicationSystemId)
{
	using namespace UE::Net::Private;
	FNetReplicatedObjectDebugInfo Info = {};

	// See if we can find the instance in any replication system
	{
		const UReplicationSystem* ReplicationSystem = GetReplicationSystemForDebug(ReplicationSystemId);
		if (ReplicationSystem)
		{
			const FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
			if (InternalIndex != InvalidInternalNetRefIndex)
			{
				FNetRefHandle ObjectHandle = NetRefHandleManager.GetNetRefHandleFromInternalIndex(InternalIndex);
				const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager.GetReplicatedObjectDataNoCheck(InternalIndex);
				if (ObjectData.RefHandle == ObjectHandle)
				{
					Info.RefHandle = &ObjectData.RefHandle;
					Info.InternalNetRefIndex = InternalIndex;
					Info.Protocol = ObjectData.Protocol;
					Info.InstanceProtocol = ObjectData.InstanceProtocol;
					Info.ReplicationSystem = ReplicationSystem;
					Info.Object = NetRefHandleManager.GetReplicatedInstances()[InternalIndex];

					return Info;
				}
			}
		}
	}

	return Info;
}

void NetObjectProtocolReferencesToString(FStringBuilderBase& StringBuilder, FReplicationProtocolIdentifier ProtocolId, uint32 ReplicationSystemId)
{
	using namespace UE::Net::Private;

	UReplicationSystem* ReplicationSystem = GetReplicationSystem(ReplicationSystemId);
	if (!ReplicationSystem)
	{
		return;
	}

	FReplicationSystemInternal* ReplicationSystemInternal = ReplicationSystem->GetReplicationSystemInternal();
	const FNetRefHandleManager& NetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();
	const FReplicationProtocolManager& ProtocolManager = ReplicationSystemInternal->GetReplicationProtocolManager();

	// As there might be multiple protocols sharing the same identifier
	ProtocolManager.ForEachProtocol(ProtocolId, [&StringBuilder, &NetRefHandleManager](const FReplicationProtocol* Protocol, const FObjectKey TemplateKey)
		{
			StringBuilder << TEXT("Protocol: ") << ToCStr(Protocol->DebugName);
			StringBuilder.Appendf(TEXT("Id: 0x%x Created From : 0x%p"), Protocol->ProtocolIdentifier, TemplateKey.ResolveObjectPtrEvenIfGarbage()) << TEXT(" Used by : \n");

			auto FindMatchingProtocol = [&StringBuilder, &Protocol, &NetRefHandleManager](uint32 InternalObjectIndex)
			{
				const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager.GetReplicatedObjectDataNoCheck(InternalObjectIndex);
				if (ObjectData.Protocol == Protocol)
				{
					StringBuilder << ObjectData.RefHandle << TEXT(" ( InternalIndex: ") << InternalObjectIndex << TEXT(", )\n");
				}
			};
			
			NetRefHandleManager.GetAssignedInternalIndicesView().ForAllSetBits(FindMatchingProtocol);
		}
	);
}

void DebugOutputNetObjectProtocolReferences(FReplicationProtocolIdentifier ProtocolId, uint32 ReplicationSystemId)
{
	TStringBuilder<4096> StringBuilder;
	StringBuilder.Reset();

	NetObjectProtocolReferencesToString(StringBuilder, ProtocolId, ReplicationSystemId);

	FPlatformMisc::LowLevelOutputDebugString(StringBuilder.ToString());
}

const TCHAR* DebugNetObjectProtocolReferencesToString(FReplicationProtocolIdentifier ProtocolId, uint32 ReplicationSystemId)
{
	static TStringBuilder<4096> StringBuilder;
	StringBuilder.Reset();

	NetObjectProtocolReferencesToString(StringBuilder, ProtocolId, ReplicationSystemId);

	return StringBuilder.ToString();
}

} // End of namespaces

#endif