// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Core/IrisLogUtils.h"
#include "UObject/NameTypes.h"
#include "Templates/TypeHash.h"

namespace UE::Net
{

bool FIrisLogOnceTracker::ShouldLog(uint32 Hash) const
{
	bool bAlreadySet = false;
	LoggedHashes.Emplace(Hash, &bAlreadySet);
	return !bAlreadySet;
}

bool FIrisLogOnceTracker::ShouldLog(const FName& Name) const
{
	return ShouldLog(::GetTypeHashHelper(Name));
}

}
