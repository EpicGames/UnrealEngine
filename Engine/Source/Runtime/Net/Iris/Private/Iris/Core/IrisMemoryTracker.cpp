// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Core/IrisMemoryTracker.h"
#include "HAL/LowLevelMemStats.h"

DECLARE_LLM_MEMORY_STAT(TEXT("Iris"), STAT_IrisLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("IrisState"), STAT_IrisStateLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("IrisInitialization"), STAT_IrisInitializationLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("IrisConnection"), STAT_IrisConnectionLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("NetTokenStructState"), STAT_NetTokenStructStateLLM, STATGROUP_LLMFULL);

// Tag name, display name, parent tag name, stat name, summary stat name
LLM_DEFINE_TAG(Iris, NAME_None, NAME_None, GET_STATFNAME(STAT_IrisLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));
LLM_DEFINE_TAG(IrisState, "State", "Iris", GET_STATFNAME(STAT_IrisStateLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));
LLM_DEFINE_TAG(IrisInitialization, "Initialization", "Iris", GET_STATFNAME(STAT_IrisInitializationLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));
LLM_DEFINE_TAG(IrisConnection, "Connection", "Iris", GET_STATFNAME(STAT_IrisConnectionLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));
LLM_DEFINE_TAG(NetTokenStructState, "NetTokenStructState", "Iris", GET_STATFNAME(STAT_NetTokenStructStateLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));