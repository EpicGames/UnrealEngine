// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Core/IrisProfiler.h"
#include "HAL/IConsoleManager.h"

CSV_DEFINE_CATEGORY(IrisClient, IRIS_CLIENT_PROFILER_ENABLE);
CSV_DEFINE_CATEGORY(IrisClientDetailObjectCreate, IRIS_CLIENT_PROFILER_ENABLE);
CSV_DEFINE_CATEGORY(IrisClientDetailRepNotify, IRIS_CLIENT_PROFILER_ENABLE);
CSV_DEFINE_CATEGORY(IrisClientDetailRPC, IRIS_CLIENT_PROFILER_ENABLE);
CSV_DEFINE_CATEGORY(IrisClientBlockedByAsyncLoading, IRIS_CLIENT_PROFILER_ENABLE);

// Detailed CSV stats are disabled in shipping by default.
#ifndef IRIS_CLIENT_PROFILER_DETAILED
#	define IRIS_CLIENT_PROFILER_DETAILED (!UE_BUILD_SHIPPING)
#endif

namespace UE::Net::Private
{

#if IRIS_CLIENT_PROFILER_DETAILED
static bool bEnableDetailedClientProfiler = 0;
static FAutoConsoleVariableRef CVarEnableDetailedClientProfilerRef(TEXT("net.Iris.EnableDetailedClientProfiler"), bEnableDetailedClientProfiler, TEXT("Generates detailed CSV Iris stats (client only)."), ECVF_Default );
#endif

}

namespace UE::Net
{

#if IRIS_CLIENT_PROFILER_ENABLE

void FClientProfiler::RecordObjectCreate(FName ObjectName, bool bIsSubObject)
{
	FCsvProfiler::RecordCustomStat("ClientObjectCreate", CSV_CATEGORY_INDEX(IrisClient), 1, ECsvCustomStatOp::Accumulate);
	if (!bIsSubObject)
	{
		FCsvProfiler::RecordCustomStat("ClientObjectCreateRoot", CSV_CATEGORY_INDEX(IrisClient), 1, ECsvCustomStatOp::Accumulate);
	}

#if IRIS_CLIENT_PROFILER_DETAILED
	if (UE::Net::Private::bEnableDetailedClientProfiler)
	{
		if (!bIsSubObject)
		{
			FCsvProfiler::RecordCustomStat(ObjectName, CSV_CATEGORY_INDEX(IrisClientDetailObjectCreate), 1, ECsvCustomStatOp::Accumulate);
		}
	}
#endif
}

void FClientProfiler::RecordRepNotify(FName RepNotifyName)
{
	FCsvProfiler::RecordCustomStat("RepNotifyCount", CSV_CATEGORY_INDEX(IrisClient), 1, ECsvCustomStatOp::Accumulate);

#if IRIS_CLIENT_PROFILER_DETAILED
	if (UE::Net::Private::bEnableDetailedClientProfiler)
	{
		FCsvProfiler::RecordCustomStat(RepNotifyName, CSV_CATEGORY_INDEX(IrisClientDetailRepNotify), 1, ECsvCustomStatOp::Accumulate);
	}
#endif
}

void FClientProfiler::RecordRPC(FName RPCName)
{
	FCsvProfiler::RecordCustomStat("CallCountRPC", CSV_CATEGORY_INDEX(IrisClient), 1, ECsvCustomStatOp::Accumulate);

#if IRIS_CLIENT_PROFILER_DETAILED
	if (UE::Net::Private::bEnableDetailedClientProfiler)
	{
		FCsvProfiler::RecordCustomStat(RPCName, CSV_CATEGORY_INDEX(IrisClientDetailRPC), 1, ECsvCustomStatOp::Accumulate);
	}
#endif
}

void FClientProfiler::RecordBlockedReplication(const TCHAR* BlockedObject, int32 NumBlockedAssets, float BlockedTime)
{
	FCsvProfiler::RecordEventf(CSV_CATEGORY_INDEX(IrisClientBlockedByAsyncLoading), TEXT("ReplicatedObject: %s has been blocked waiting for async loading of %d assets for %f secs"), BlockedObject, NumBlockedAssets, BlockedTime);
}

bool FClientProfiler::IsCapturing()
{
	return FCsvProfiler::Get()->IsCapturing();
}

#endif

}