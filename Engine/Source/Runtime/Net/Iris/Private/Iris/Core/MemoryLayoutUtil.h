// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMathUtility.h"
#include "Templates/AlignmentTemplates.h"

/** Helper to deal with manually setting up a block of memory containing multiple allocations. */
struct FMemoryLayoutUtil
{
	struct FLayout
	{
		SIZE_T CurrentOffset = 0;
		SIZE_T MaxAlignment = 1;
	};

	struct FOffsetAndSize
	{
		SIZE_T Offset;
		SIZE_T Size;
	};

	struct FSizeAndAlignment
	{
		SIZE_T Size = 0;
		SIZE_T Alignment = 0;
	};

	/** Get the current total size in bytes, including alignment, of the layout. */
	static SIZE_T GetTotalSizeIncludingAlignment(const FLayout& Layout) { return Align(Layout.CurrentOffset, Layout.MaxAlignment); }

	/** Add an entry to the Layout with the specified size and alignment in bytes. */
	static void AddToLayout(FLayout& Layout, FOffsetAndSize& OutOffsetAndSize, SIZE_T ItemSize, SIZE_T ItemAlignment, SIZE_T ItemCount)
	{
		const SIZE_T CurrentAlignedOffset = Align(Layout.CurrentOffset, ItemAlignment);
		const SIZE_T TotalItemSize = ItemSize * ItemCount;

		OutOffsetAndSize.Offset = CurrentAlignedOffset;
		OutOffsetAndSize.Size = TotalItemSize;

		Layout.CurrentOffset = CurrentAlignedOffset + TotalItemSize;
		Layout.MaxAlignment = FMath::Max<SIZE_T>(ItemAlignment, Layout.MaxAlignment);
	}

	/** Add a typed entry to the Layout. */
	template<typename T>
	static void AddToLayout(FLayout& Layout, FOffsetAndSize& OutOffsetAndSize, SIZE_T Count) { AddToLayout(Layout, OutOffsetAndSize, sizeof(T), alignof(T), Count); }
};