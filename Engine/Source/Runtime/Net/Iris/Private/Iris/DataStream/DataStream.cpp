// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/DataStream/DataStream.h"
#include "Iris/DataStream/DataStreamManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataStream)

UDataStream::~UDataStream()
{
}

UDataStream::EWriteResult UDataStream::BeginWrite(const FBeginWriteParameters& Params)
{
	return EWriteResult::HasMoreData;
}

void UDataStream::EndWrite()
{
}

void UDataStream::Init(const FInitParameters& Params)
{
	DataStreamInitParameters = Params;
}

void UDataStream::Deinit()
{
}

void UDataStream::Update(const FUpdateParameters& Params)
{
}

bool UDataStream::HasDebugDataToSend() const
{
	return false;
}

const TCHAR* LexToString(const UDataStream::EDataStreamState State)
{
	static const TCHAR* Names[] = {
		TEXT("Invalid"),
		TEXT("PendingCreate"),
		TEXT("WaitOnCreateConfirmation"),
		TEXT("Open"),
		TEXT("PendingClose"),
		TEXT("WaitOnCloseConfirmation"),
	};
	static_assert(UE_ARRAY_COUNT(Names) == uint32(UDataStream::EDataStreamState::Count), "Missing names for one or more values of EDataStreamState.");

	return State < UDataStream::EDataStreamState::Count ? Names[(uint32)State] : TEXT("");
}

const UDataStream::EDataStreamState UDataStream::GetState() const
{
	if (DataStreamInitParameters.DataStreamManager)
	{
		return DataStreamInitParameters.DataStreamManager->GetStreamState(GetDataStreamName());
	}
	else
	{
		return EDataStreamState::Invalid;
	}
}

void UDataStream::RequestClose()
{
	if (DataStreamInitParameters.DataStreamManager)
	{
		DataStreamInitParameters.DataStreamManager->CloseStream(GetDataStreamName());
	}
}
