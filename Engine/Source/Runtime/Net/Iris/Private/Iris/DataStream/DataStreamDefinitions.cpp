// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/DataStream/DataStreamDefinitions.h"
#include "Iris/DataStream/DataStream.h"
#include "Iris/Core/IrisLog.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataStreamDefinitions)

UDataStreamDefinitions::UDataStreamDefinitions()
: bFixupComplete(false)
{
}

void UDataStreamDefinitions::FixupDefinitions()
{
	if (bFixupComplete)
	{
		return;
	}

	int32 CurrentStreamIndex = 0;
	for (FDataStreamDefinition& Definition : DataStreamDefinitions)
	{
		UE_CLOGF(DataStreamDefinitions.ContainsByPredicate([Name = Definition.DataStreamName, &Definition](const FDataStreamDefinition& ExistingDefinition) { return Name == ExistingDefinition.DataStreamName && &Definition != &ExistingDefinition; }), LogIris, Error, "DataStream name is defined multiple times: %ls.", *Definition.DataStreamName.GetPlainNameString());
		UE_CLOGF(!StaticEnum<EDataStreamSendStatus>()->IsValidEnumValue(int8(Definition.DefaultSendStatus)), LogIris, Error, "Invalid DataStreamSendStatus %u for DataStream %ls.", unsigned(Definition.DefaultSendStatus), *Definition.DataStreamName.GetPlainNameString());

		Definition.Class = StaticLoadClass(UDataStream::StaticClass(), nullptr, *Definition.ClassName.ToString(), nullptr, LOAD_Quiet);

		UE_CLOGF(Definition.Class == nullptr, LogIris, Error, "DataStream class could not be loaded: %ls", *Definition.ClassName.GetPlainNameString());

		Definition.StreamIndex = CurrentStreamIndex++;
	}

	bFixupComplete = true;
}

int32 UDataStreamDefinitions::GetStreamIndex(const FDataStreamDefinition& Definition)
{
	return Definition.StreamIndex;
}

const FDataStreamDefinition* UDataStreamDefinitions::FindDefinition(const FName Name) const
{
	return DataStreamDefinitions.FindByPredicate([Name](const FDataStreamDefinition& Definition) { return Name == Definition.DataStreamName; });
}

const FDataStreamDefinition* UDataStreamDefinitions::FindDefinition(int32 StreamIndex) const
{
	return DataStreamDefinitions.FindByPredicate([StreamIndex](const FDataStreamDefinition& Definition) { return StreamIndex == Definition.StreamIndex; });
}

void UDataStreamDefinitions::GetStreamNamesToAutoCreateOrRegister(TArray<FName>& OutStreamNames) const
{
	for (const FDataStreamDefinition& Definition : DataStreamDefinitions)
	{
		if (Definition.bAutoCreate || Definition.bDynamicCreate)
		{
			OutStreamNames.Add(Definition.DataStreamName);
		}
	}
}
