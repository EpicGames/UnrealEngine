// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/DataStream/DataStream.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "DataStreamDefinitions.generated.h"

USTRUCT()
struct FDataStreamDefinition
{
	GENERATED_BODY()

	// Data stream identifier
	UPROPERTY()
	FName DataStreamName;

	// UClass name used to create the UDataStream
	UPROPERTY()
	FName ClassName;		

	// UClass used to create the UDataStream
	UPROPERTY()
	TObjectPtr<UClass> Class = nullptr;

	// Default send status when created.
	UPROPERTY()
	EDataStreamSendStatus DefaultSendStatus = EDataStreamSendStatus::Send;

	// Whether the DataStream should be auto created for each connection. If not then CreateStream need be called manually.
	UPROPERTY()
	bool bAutoCreate = false;

	// If bDynamicCreate is set to true we will reserve a slot for the stream allowing it to be openened and closed on demand
	UPROPERTY()
	bool bDynamicCreate = false;

	// Get the assigned stream index
	int32 GetStreamIndex() const;

private:
	friend class UDataStreamDefinitions;
	int32 StreamIndex = -1;
};

UCLASS(transient, config=Engine)
class UDataStreamDefinitions : public UObject
{
	GENERATED_BODY()

protected:
	UDataStreamDefinitions();

protected:
	friend class UDataStreamManager;

	void FixupDefinitions();
	const FDataStreamDefinition* FindDefinition(const FName Name) const;
	const FDataStreamDefinition* FindDefinition(int32 StreamIndex) const;
	static int32 GetStreamIndex(const FDataStreamDefinition& Definition);
	void GetStreamNamesToAutoCreateOrRegister(TArray<FName>& OutStreamNames) const;

private:
	UPROPERTY(Config)
	TArray<FDataStreamDefinition> DataStreamDefinitions;

	bool bFixupComplete;

// For testing purposes only
#if WITH_AUTOMATION_WORKER
public:
	inline TArray<FDataStreamDefinition>& ReadWriteDataStreamDefinitions() { return DataStreamDefinitions; }
	inline bool& ReadWriteFixupComplete() { return bFixupComplete; }
#endif
};
