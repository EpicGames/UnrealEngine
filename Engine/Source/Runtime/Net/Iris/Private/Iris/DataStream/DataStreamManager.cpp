// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/DataStream/DataStreamManager.h"
#include "Iris/DataStream/DataStreamDefinitions.h"
#include "Iris/Core/IrisLog.h"
#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "Iris/PacketControl/PacketNotification.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/ReplicationSystem/NetExports.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataStreamManager)

#define UE_LOG_DATASTREAM_CONN(Verbosity, Format, ...)  UE_LOG(LogIris, Verbosity, TEXT("DataStreamManager: R:%u :C%u ") Format, InitParameters.ReplicationSystemId, InitParameters.ConnectionId, ##__VA_ARGS__)

class UDataStreamManager::FImpl
{
public:
	using EDataStreamState = UDataStream::EDataStreamState;

	enum class ECreateDataStreamFlags : uint8
	{
		// Create stream
		None,
		// Streams marked with bDynamicCreate will only be registered if this flag is set.
		RegisterIfStreamIsDynamic,
	};
	FRIEND_ENUM_CLASS_FLAGS(ECreateDataStreamFlags);

public:
	FImpl();
	~FImpl();

	void Init(const UDataStream::FInitParameters& InitParams);
	void Deinit();

	void Update(const FUpdateParameters& Params);

	EWriteResult BeginWrite(const UDataStream::FBeginWriteParameters& Params);
	UDataStream::EWriteResult WriteData(UE::Net::FNetSerializationContext& context, FDataStreamRecord const*& OutRecord);
	void EndWrite();
	void ReadData(UE::Net::FNetSerializationContext& context);
	void ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record);
	bool HasAcknowledgedAllReliableData() const;
	bool HasDebugDataToSend() const;
	
	ECreateDataStreamResult CreateStream(const FName StreamName, ECreateDataStreamFlags Flags = ECreateDataStreamFlags::None);
	void CloseStream(const FName StreamName);

	const UDataStream* GetStream(const FName StreamName) const;
	UDataStream* GetStream(const FName StreamName);

	void SetSendStatus(const FName StreamName, EDataStreamSendStatus Status);
	EDataStreamSendStatus GetSendStatus(const FName StreamName) const;

	EDataStreamState GetStreamState(const FName StreamName) const;

	UE::Net::Private::FNetExports& GetNetExports();

	void AddReferencedObjects(FReferenceCollector& Collector);

private:
	struct FindStreamByName
	{
		inline FindStreamByName(FName InName) : Name(InName) {}
		
		inline bool operator()(const UDataStream* Stream) const { return Stream && Name.IsEqual(Stream->GetFName(), ENameCase::IgnoreCase, false); }
		inline bool operator()(const FDataStreamDefinition& Definition) const { return Name == Definition.DataStreamName; }

	private:
		FName Name;
	};

	struct FRecord : public FDataStreamRecord
	{
		TArray<const FDataStreamRecord*, TInlineAllocator<8>> DataStreamRecords;
		uint32 DataStreamMask;
		// What streams carried state changes in last record
		uint32 DataStreamStateMask;
	};

	void InitRecordStorage();
	void InitStream(UDataStream* Stream, FName StreamName);
	void InitStreams();
	void DestroyStream(uint32 StreamIndex);
	void SetStreamState(uint32 StreamIndex, EDataStreamState NewState);
	EDataStreamState GetStreamState(uint32 StreamIndex) const;
	ECreateDataStreamResult CreateStreamFromDefinition(const FDataStreamDefinition& Definition, ECreateDataStreamFlags Flags);
	ECreateDataStreamResult CreateStreamFromIndex(int32 StreamIndex);
	void MarkStreamStateDirty(uint32 StreamIndex);
	void HandleReceivedStreamState(UE::Net::FNetSerializationContext& Context, uint32 StreamIndex, EDataStreamState RecvdState);

private:
	static constexpr uint32 MaxStreamCount = 32U;
	static constexpr uint32 StreamCountBitCount = 5U; // Enough for 32 streams
	static constexpr uint32 StreamStateBitCount = 4U; // Enough for 16 states

	UE::Net::Private::FNetExports NetExports;

	// We can afford reserving space for a few pointers It's unlikely we will create anything close to 16 streams.
	TArray<TObjectPtr<UDataStream>> Streams;
	TArray<EDataStreamSendStatus> StreamSendStatus;
	TArray<EDataStreamState> StreamState;
	TArray<FRecord> RecordStorage;
	TResizableCircularQueue<FRecord*> Records;
	UDataStream::FInitParameters InitParameters;
	uint32 DirtyStreamsMask = 0U;
};

static_assert((uint32)(UDataStream::EDataStreamState::Count) <= 15U, "EDataStreamState must fit in 4 bits.");

UDataStreamManager::UDataStreamManager()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		Impl = MakePimpl<FImpl>();
	}
}

UDataStreamManager::~UDataStreamManager()
{
}

void UDataStreamManager::Init(const UDataStream::FInitParameters& InitParams)
{
	UDataStream::FInitParameters Params(this, InitParams);
	Impl->Init(Params);
}

void UDataStreamManager::Deinit()
{
	Impl->Deinit();
}

void UDataStreamManager::Update(const FUpdateParameters& Params)
{
	Impl->Update(Params);
}

UDataStream::EWriteResult UDataStreamManager::BeginWrite(const UDataStream::FBeginWriteParameters& Params)
{
	return Impl->BeginWrite(Params);
}

void UDataStreamManager::EndWrite()
{
	Impl->EndWrite();
}

UDataStream::EWriteResult UDataStreamManager::WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord)
{
	return Impl->WriteData(Context, OutRecord);
}

void UDataStreamManager::ReadData(UE::Net::FNetSerializationContext& Context)
{
	return Impl->ReadData(Context);
}

void UDataStreamManager::ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record)
{
	return Impl->ProcessPacketDeliveryStatus(Status, Record);
}

bool UDataStreamManager::HasAcknowledgedAllReliableData() const
{
	return Impl->HasAcknowledgedAllReliableData();
}

bool UDataStreamManager::HasDebugDataToSend() const
{
	return Impl->HasDebugDataToSend();
}

bool UDataStreamManager::IsKnownStreamDefinition(const FName StreamName)
{
	const UDataStreamDefinitions* StreamDefinitions = GetDefault<UDataStreamDefinitions>();
	return StreamDefinitions->FindDefinition(StreamName) != nullptr;
}

ECreateDataStreamResult UDataStreamManager::CreateStream(const FName StreamName)
{
	return Impl->CreateStream(StreamName);
}

UDataStream* UDataStreamManager::GetStream(const FName StreamName)
{
	return Impl->GetStream(StreamName);
}

const UDataStream* UDataStreamManager::GetStream(const FName StreamName) const
{
	return Impl->GetStream(StreamName);
}

void UDataStreamManager::CloseStream(const FName StreamName)
{
	Impl->CloseStream(StreamName);
}

UDataStream::EDataStreamState UDataStreamManager::GetStreamState(const FName StreamName) const
{
	return Impl->GetStreamState(StreamName);
}

void UDataStreamManager::SetSendStatus(const FName StreamName, EDataStreamSendStatus Status)
{
	return Impl->SetSendStatus(StreamName, Status);
}

EDataStreamSendStatus UDataStreamManager::GetSendStatus(const FName StreamName) const
{
	return Impl->GetSendStatus(StreamName);
}

UE::Net::Private::FNetExports& UDataStreamManager::GetNetExports()
{
	return Impl->GetNetExports();
}

void UDataStreamManager::AddReferencedObjects(UObject* Object, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Object, Collector);

	UDataStreamManager* StreamManager = CastChecked<UDataStreamManager>(Object);
	if (FImpl* Impl = StreamManager->Impl.Get())
	{
		Impl->AddReferencedObjects(Collector);
	}
}

// FImpl
UDataStreamManager::FImpl::FImpl()
{
}

UDataStreamManager::FImpl::~FImpl()
{
}

void UDataStreamManager::FImpl::Init(const UDataStreamManager::FInitParameters& InitParams)
{
	InitParameters = InitParams;
	InitParameters.NetExports = &NetExports;

	InitRecordStorage();
	InitStreams();
}

void UDataStreamManager::FImpl::Deinit()
{
	// Discard all records
	for (SIZE_T RecordIt = 0, RecordEndIt = Records.Count(); RecordIt != RecordEndIt; ++RecordIt)
	{
		const FDataStreamRecord* const Record = Records.Peek();
		ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus::Discard, Record);
	}

	for (UDataStream* Stream : Streams)
	{
		if (IsValid(Stream))
		{
			Stream->Deinit();
			Stream->MarkAsGarbage();
		}
	}

	Streams.Reset();
	StreamSendStatus.Reset();
	StreamState.Reset();
}

void UDataStreamManager::FImpl::Update(const FUpdateParameters& Params)
{
	for (UDataStream* Stream : Streams)
	{
		if (IsValid(Stream))
		{
			Stream->Update(Params);
		}
	}
}

void UDataStreamManager::FImpl::DestroyStream(uint32 StreamIndex)
{
	UDataStream* Stream = Streams[StreamIndex];
	if (IsValid(Stream))
	{
		Stream->Deinit();
		Stream->MarkAsGarbage();
		Streams[StreamIndex] = nullptr;
		StreamState[StreamIndex] = EDataStreamState::Invalid;
		StreamSendStatus[StreamIndex] = EDataStreamSendStatus::Pause;
	}
}

UDataStreamManager::EWriteResult UDataStreamManager::FImpl::BeginWrite(const UDataStream::FBeginWriteParameters& Params)
{
	const SIZE_T StreamCount = Streams.Num();
	if (StreamCount == 0)
	{
		return EWriteResult::NoData;
	}

	TObjectPtr<UDataStream>* StreamData = Streams.GetData();
	const EDataStreamSendStatus* SendStatusData = StreamSendStatus.GetData();
	const EDataStreamState* StreamStateData = StreamState.GetData();

	UDataStream::EWriteResult CombinedWriteResult = DirtyStreamsMask == 0U ? UDataStream::EWriteResult::NoData : UDataStream::EWriteResult::HasMoreData;

	for (SIZE_T StreamIt = 0, StreamEndIt = StreamCount; StreamIt != StreamEndIt; ++StreamIt)
	{
		UDataStream* Stream = StreamData[StreamIt];
		const EDataStreamSendStatus SendStatus = SendStatusData[StreamIt];
		if (SendStatus == EDataStreamSendStatus::Pause)
		{
			continue;
		}

		const EWriteResult WriteResult = Stream->BeginWrite(Params);
		CombinedWriteResult = (CombinedWriteResult == EWriteResult::HasMoreData || WriteResult == EWriteResult::HasMoreData) ? EWriteResult::HasMoreData : WriteResult;
	}

	return CombinedWriteResult;
}

void UDataStreamManager::FImpl::EndWrite()
{
	const SIZE_T StreamCount = Streams.Num();

	TObjectPtr<UDataStream>* StreamData = Streams.GetData();
	const EDataStreamSendStatus* SendStatusData = StreamSendStatus.GetData();

	for (SIZE_T StreamIt = 0, StreamEndIt = StreamCount; StreamIt != StreamEndIt; ++StreamIt)
	{
		UDataStream* Stream = StreamData[StreamIt];
		const EDataStreamSendStatus SendStatus = SendStatusData[StreamIt];
		if (SendStatus == EDataStreamSendStatus::Pause)
		{
			continue;
		}

		Stream->EndWrite();
	}
}

UDataStreamManager::EWriteResult UDataStreamManager::FImpl::WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const int32 StreamCount = Streams.Num();
	if (StreamCount <= 0)
	{
		return EWriteResult::NoData;
	}

	// Is the packet window full? Unexpected.
	if (Records.Count() == Records.AllocatedCapacity())
	{
		ensureMsgf(false, TEXT("DataStreamManager record storage is full."));
		return EWriteResult::NoData;
	}

	FRecord TempRecord;
	TempRecord.DataStreamRecords.SetNumZeroed(StreamCount);
	FDataStreamRecord const** TempStreamRecords = TempRecord.DataStreamRecords.GetData();

	FNetBitStreamWriter ManagerStream = Context.GetBitStreamWriter()->CreateSubstream();
	// This will write the number of bits required for the StreamBitCount
	ManagerStream.WriteBits(0U, StreamCountBitCount);
	// Will be rewritten later to contain a bit mask for all streams that have written data.
	ManagerStream.WriteBits(0U, StreamCount);

	const bool bHasStreamsWithDirtyState = DirtyStreamsMask != 0U;
	if (ManagerStream.WriteBool(bHasStreamsWithDirtyState))
	{
		ManagerStream.WriteBits(DirtyStreamsMask, StreamCount);
	}

	TObjectPtr<UDataStream>* StreamData = Streams.GetData();
	const EDataStreamSendStatus* SendStatusData = StreamSendStatus.GetData();

	UDataStream::EWriteResult CombinedWriteResult = bHasStreamsWithDirtyState ? UDataStream::EWriteResult::Ok : UDataStream::EWriteResult::NoData;

	// Write rare datastream state changes
	if (bHasStreamsWithDirtyState)
	{
		TArray<EDataStreamState> OldStreamState = StreamState;
		for (uint32 StreamIt = 0U, StreamEndIt = StreamCount, CurrentStreamMask = 1U; StreamIt != StreamEndIt; ++StreamIt, CurrentStreamMask += CurrentStreamMask)
		{
			if (DirtyStreamsMask & CurrentStreamMask)
			{
				const EDataStreamState State = GetStreamState(StreamIt);
				// Write state
				ManagerStream.WriteBits((uint32)State, StreamStateBitCount);
				UE_LOG_DATASTREAM_CONN(Verbose, TEXT("WriteStreamState for StreamIndex: %u, State: %s"), StreamIt, LexToString(State));
				switch (State)
				{
					case EDataStreamState::PendingCreate:
					{
						// If we would like to add more data for create, this would be the spot.
						SetStreamState(StreamIt, EDataStreamState::WaitOnCreateConfirmation);
						break;
					}
					case EDataStreamState::PendingClose:
					{
						// For now, if we have no data to flush we can go directly to WaitOnCloseConfirmation
						UDataStream* Stream = Streams[StreamIt];
						if (Stream->HasAcknowledgedAllReliableData())
						{
							SetStreamState(StreamIt, EDataStreamState::WaitOnCloseConfirmation);
						}
						break;
					}
					default:
					{
						break;
					}
				}
			}
		}
	}

	// If we can't fit our header we can't fit anything else either.
	if (ManagerStream.IsOverflown())
	{
		Context.GetBitStreamWriter()->DiscardSubstream(ManagerStream);
		return EWriteResult::NoData;
	}

	{
		// Init export record
		NetExports.InitExportRecordForPacket();

		// Setup export context for this packet
		FNetExportContext::FBatchExports CurrentPacketBatchExports;
		FNetExports::FExportScope ExportScope = NetExports.MakeExportScope(Context, CurrentPacketBatchExports);

		uint32 DataStreamMask = 0;
		for (uint32 StreamIt = 0U, StreamEndIt = StreamCount, CurrentStreamMask = 1U; StreamIt != StreamEndIt; ++StreamIt, CurrentStreamMask += CurrentStreamMask)
		{
			UDataStream* Stream = StreamData[StreamIt];
			const EDataStreamSendStatus SendStatus = SendStatusData[StreamIt];
			if (SendStatus == EDataStreamSendStatus::Pause)
			{
				continue;
			}

			// We only write stream data of the stream is considered is open
			const EDataStreamState State = StreamState[StreamIt];
			if (!(State == EDataStreamState::Open || State == EDataStreamState::PendingClose))
			{
				continue;
			}

			FNetBitStreamWriter SubBitStream = ManagerStream.CreateSubstream();
			FNetSerializationContext SubContext = Context.MakeSubContext(&SubBitStream);

			const FDataStreamRecord* SubRecord = nullptr;
			const EWriteResult WriteResult = Stream->WriteData(SubContext, SubRecord);
			if (WriteResult == EWriteResult::NoData || SubContext.HasError())
			{
				checkf(SubRecord == nullptr, TEXT("DataStream '%s' provided a record despite errors or returning NoData."), ToCStr(Stream->GetFName().GetPlainNameString()));
				ManagerStream.DiscardSubstream(SubBitStream);

				if (SubContext.HasError())
				{
					Context.SetError(SubContext.GetError(), false);
					break;
				}
				else
				{
					continue;
				}
			}

			// Only update DataStreamMask if data was written. 
			if (SubBitStream.GetPosBits() > 0U)
			{
				DataStreamMask |= CurrentStreamMask;		
				TempStreamRecords[StreamIt] = SubRecord;
			}
			else
			{
				ensureMsgf(SubRecord == nullptr, TEXT("DataStream '%s' provided a record despite not writing any data."), ToCStr(Stream->GetFName().GetPlainNameString()));
			}

			ManagerStream.CommitSubstream(SubBitStream);

			// Set CombinedWriteResult to HasMoreData if any of the result variables is HasMoreData, otherwise take the WriteResult, which will be 'Ok'.
			CombinedWriteResult = (CombinedWriteResult == EWriteResult::HasMoreData || WriteResult == EWriteResult::HasMoreData) ? EWriteResult::HasMoreData : WriteResult;
		}

		if (!(DataStreamMask || bHasStreamsWithDirtyState))
		{
			Context.GetBitStreamWriter()->DiscardSubstream(ManagerStream);
			// Technically we could also return EWriteResult::HasMoreData
			CombinedWriteResult = EWriteResult::NoData;
		}
		else
		{
			// Fixup manager header
			{
				const uint32 CurrentBitPos = ManagerStream.GetPosBits();
				ManagerStream.Seek(0);
				ManagerStream.WriteBits(StreamCount - 1U, StreamCountBitCount);
				ManagerStream.WriteBits(DataStreamMask, StreamCount);
				ManagerStream.Seek(CurrentBitPos);
				Context.GetBitStreamWriter()->CommitSubstream(ManagerStream);
			}

			// Fixup and store record
			TempRecord.DataStreamMask = DataStreamMask;
			TempRecord.DataStreamStateMask = DirtyStreamsMask;
			DirtyStreamsMask = 0U;

			FRecord*& Record = Records.Enqueue_GetRef();
			*Record = MoveTemp(TempRecord);

			OutRecord = Record;

			// Push exports and update export record
			NetExports.CommitExportsToRecord(ExportScope);
			NetExports.PushExportRecordForPacket();
		}
	}

	// Suppress clang analyzer false positive about Context retaining reference to ExportScope's member.
	// The FExportScope destructor restores the original ExportContext before this return.
	UE_CLANG_SUPPRESS
	return CombinedWriteResult;
}

void UDataStreamManager::FImpl::MarkStreamStateDirty(uint32 StreamIndex)
{
	DirtyStreamsMask |= 1U << StreamIndex;
}

UDataStream::EDataStreamState UDataStreamManager::FImpl::GetStreamState(uint32 StreamIndex) const
{
	return StreamState[StreamIndex];
}

void UDataStreamManager::FImpl::SetStreamState(uint32 StreamIndex, EDataStreamState NewState)
{
	const EDataStreamState CurrentState = StreamState[StreamIndex];

	switch (CurrentState)
	{
		case EDataStreamState::Invalid:
		{
			if (NewState == EDataStreamState::Invalid)
			{
				// This means that we have rejected a PendingCreate request
				goto AcceptStateChange;
			}
			else if (NewState == EDataStreamState::PendingCreate)
			{
				// Next write will respond with PendingCreate to confirm
				goto AcceptStateChange;
			}
		}
		break;
		
		case EDataStreamState::PendingCreate:
		{
			if (NewState == EDataStreamState::Invalid)
			{
				// Local request to close stream that has not yet been sent.
				goto AcceptStateChange;
			}
			else if (NewState == EDataStreamState::WaitOnCreateConfirmation)			
			{
				// We have sent PendingCreate request
				goto AcceptStateChange;
			}
		}
		break;
		
		case EDataStreamState::WaitOnCreateConfirmation:
		{
			if (NewState == EDataStreamState::PendingCreate)
			{
				// We have dropped WaitOnCreateConfirmation (or PendingCreate)
				goto AcceptStateChange;
			}
			else if (NewState == EDataStreamState::PendingClose)
			{
				// API call to close
				goto AcceptStateChange;
			}
			else if (NewState == EDataStreamState::Open)
			{
				// Received PendingCreate/Open from other side to confirm
				goto AcceptStateChange;

			}
			else if (NewState == EDataStreamState::Invalid)
			{
				// Received reject from other side.
				goto AcceptStateChange;
			}
		}
		break;

		case EDataStreamState::Open:
		{
			if (NewState == EDataStreamState::PendingClose)
			{
				// API call to close or request from other side to close
				goto AcceptStateChange;
			}
			else if (NewState == EDataStreamState::Open)
			{
				// Received PendingCreate/Open from other side to confirm
				// nothing should be done.
				return;
			}
		}
		break;

		case EDataStreamState::PendingClose:
		{
			if (NewState == EDataStreamState::PendingClose)
			{
				// API call to close or request from other side to close
				goto AcceptStateChange;
			}
			else if (NewState == EDataStreamState::WaitOnCloseConfirmation)
			{
				// We are flushed and can cleanup
				goto AcceptStateChange;
			}
			else if (NewState == EDataStreamState::Open)
			{
				// Received PendingCreate/Open from other side to confirm
				// nothing should be done.
				return;
			}
		}
		break;

		case EDataStreamState::WaitOnCloseConfirmation:
		{
			if (NewState == EDataStreamState::PendingClose)
			{
				// We dropped PendingClose
				goto AcceptStateChange;				
			}
			else if (NewState == EDataStreamState::WaitOnCloseConfirmation)
			{
				// Other side has confirmed close
				goto AcceptStateChange;
			}
			else if (NewState == EDataStreamState::Invalid)
			{
				// We can cleanup
				goto AcceptStateChange;
			}
		}
		break;

		default:
		break;
	};

	UE_LOG_DATASTREAM_CONN(Verbose, TEXT("SetDataStreamState Reject: for StreamIndex: %u, CurrentState: %s, NewState: %s"), StreamIndex, LexToString(CurrentState), LexToString(NewState));
	ensure(false);
	return;

AcceptStateChange:
	UE_LOG_DATASTREAM_CONN(Verbose, TEXT("SetDataStreamState Accept: for StreamIndex: %u, CurrentState: %s, NewState: %s"), StreamIndex, LexToString(CurrentState), LexToString(NewState));
	StreamState[StreamIndex] = NewState;
	MarkStreamStateDirty(StreamIndex);
	return;
}

void UDataStreamManager::FImpl::HandleReceivedStreamState(UE::Net::FNetSerializationContext& Context, uint32 StreamIndex, EDataStreamState RecvdState)
{
	UDataStream* DataStream = Streams[StreamIndex];
	const EDataStreamState CurrentState = GetStreamState(StreamIndex);

	auto HandleUnexpectedState = [&CurrentState, &RecvdState, &StreamIndex, &Context, this]()
	{
		UE_LOG_DATASTREAM_CONN(Error, TEXT("Received invalid DataStream State: %s for StreamIndex: %u, while in State: %s"), LexToString(RecvdState), StreamIndex, LexToString(CurrentState));
		Context.SetError(TEXT("Invalid DataStreamState"));
		// Just for log attention
		ensure(false);
	};

	switch (RecvdState)
	{
		case EDataStreamState::PendingCreate:
		{
			// PendingCreate is received to request or confirm open/create
			if (CurrentState == EDataStreamState::Invalid)
			{
				// Create stream
				if (CreateStreamFromIndex(StreamIndex) != ECreateDataStreamResult::Success)
				{
					// If we fail, we set state as Invalid and send that to server.
					SetStreamState(StreamIndex, EDataStreamState::Invalid);
				}
			}
			else if (CurrentState == EDataStreamState::WaitOnCreateConfirmation)
			{
				// Other side have now confirmed open
				SetStreamState(StreamIndex, EDataStreamState::Open);
			}
			else
			{
				HandleUnexpectedState();
			}
		}
		break;

		case EDataStreamState::Open:
		{
			// Open is received when other side has accepted stream
			if (CurrentState == EDataStreamState::WaitOnCreateConfirmation)
			{
				// We have now completed open handshake and can send data.
				SetStreamState(StreamIndex, EDataStreamState::Open);
			}
			else if (CurrentState == EDataStreamState::Open)
			{
				// We are already open, nothing to do
			}
			else
			{
				HandleUnexpectedState();
			}
		}
		break;
		
		case EDataStreamState::PendingClose:
		{						
			if (CurrentState == EDataStreamState::PendingCreate)
			{
				// Received Pending close while we have yet to acknowledge or sent create
				SetStreamState(StreamIndex, EDataStreamState::WaitOnCreateConfirmation);
				SetStreamState(StreamIndex, EDataStreamState::PendingClose);
			} 
			else if (CurrentState == EDataStreamState::WaitOnCreateConfirmation || CurrentState == EDataStreamState::Open)
			{
				// Pending close is received when other side has started to close the connection
				// there might still be data to be flushed but no new data should be written
				SetStreamState(StreamIndex, EDataStreamState::PendingClose);
			}
			else if (CurrentState == EDataStreamState::PendingClose)
			{
				UDataStream* Stream = Streams[StreamIndex];
				if (Stream->HasAcknowledgedAllReliableData())
				{
					SetStreamState(StreamIndex, EDataStreamState::WaitOnCloseConfirmation);
				}
				else
				{
					UE_LOG_DATASTREAM_CONN(Verbose, TEXT("Flushing DataStream StreamIndex: %u in State: %s"), StreamIndex, LexToString(CurrentState));
					MarkStreamStateDirty(StreamIndex);
				}
			}
			else if (CurrentState == EDataStreamState::WaitOnCloseConfirmation)
			{
				// Trigger update of state machine as other side might still be flushing
				MarkStreamStateDirty(StreamIndex);				
			}
			else
			{
				HandleUnexpectedState();
			}
		}
		break;

		case EDataStreamState::WaitOnCloseConfirmation:
		{
			if (CurrentState == EDataStreamState::WaitOnCloseConfirmation)
			{
				SetStreamState(StreamIndex, EDataStreamState::Invalid);
				DestroyStream(StreamIndex);
			}
			else if (CurrentState == EDataStreamState::PendingClose)
			{
				if (DataStream->HasAcknowledgedAllReliableData())
				{
					SetStreamState(StreamIndex, EDataStreamState::WaitOnCloseConfirmation);
				}
				else
				{
					// Trigger update of state machine as other side might still be flushing
					UE_LOG_DATASTREAM_CONN(Verbose, TEXT("Flushing DataStream StreamIndex: %u in State: %s"), StreamIndex, LexToString(CurrentState));
					MarkStreamStateDirty(StreamIndex);
				}
			}
			else
			{
				HandleUnexpectedState();
			}
		}
		break;

		case EDataStreamState::Invalid:
		{
			// Sent when a stream is invalidated
			if (CurrentState == EDataStreamState::Invalid)
			{
				// Do nothing
			}
			else if (CurrentState == EDataStreamState::WaitOnCreateConfirmation)
			{
				// Report error and close stream
				SetStreamState(StreamIndex, EDataStreamState::PendingClose);
				SetStreamState(StreamIndex, EDataStreamState::WaitOnCloseConfirmation);

				SetStreamState(StreamIndex, EDataStreamState::Invalid);
				DestroyStream(StreamIndex);
			}
			else if (CurrentState == EDataStreamState::WaitOnCloseConfirmation)
			{
				// Ready to destroy
				SetStreamState(StreamIndex, EDataStreamState::Invalid);
				DestroyStream(StreamIndex);
			}
			else
			{
				HandleUnexpectedState();
			}
		}
		break;

		default:
			HandleUnexpectedState();
			break;
	}
}

void UDataStreamManager::FImpl::ReadData(UE::Net::FNetSerializationContext& Context)
{
	using namespace UE::Net;

	FNetBitStreamReader* Stream = Context.GetBitStreamReader();
	const uint32 StreamCount = 1U + Stream->ReadBits(StreamCountBitCount);
	const uint32 DataStreamMask = Stream->ReadBits(StreamCount);

	// Read and apply stream state changes
	const bool bHasDataStreamStateChanges = Stream->ReadBool();
	const uint32 DataStreamsWithChangedStateMask = bHasDataStreamStateChanges ? Stream->ReadBits(StreamCount) : 0U;

	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	// Validate the received information
	if (StreamCount > uint32(Streams.Num()) || (DataStreamMask == 0U && !bHasDataStreamStateChanges))
	{
		Context.SetError(GNetError_BitStreamError);
		return;
	}

	TObjectPtr<UDataStream>* StreamData = Streams.GetData();

	if (bHasDataStreamStateChanges)
	{
		for (uint32 StreamIt = 0, StreamEndIt = StreamCount, Mask = 1U; StreamIt != StreamEndIt; ++StreamIt, Mask += Mask)
		{
			if (DataStreamsWithChangedStateMask & Mask)
			{
				// Read Stream state
				EDataStreamState RcvdState = (EDataStreamState)Stream->ReadBits(StreamStateBitCount);

				// If something went wrong we should stop deserializing immediately.
				if (Context.HasErrorOrOverflow())
				{
					return;
				}

				UE_LOG_DATASTREAM_CONN(Verbose, TEXT("ReadStreamState for StreamIndex: %u, State: %s"), StreamIt, LexToString(RcvdState));
				HandleReceivedStreamState(Context, StreamIt, RcvdState);
			}
		}
	}

	for (uint32 StreamIt = 0, StreamEndIt = StreamCount, Mask = 1U; StreamIt != StreamEndIt; ++StreamIt, Mask += Mask)
	{
		if (DataStreamMask & Mask)
		{
			// We should always have a DataStream here.
			UDataStream* DataStream = StreamData[StreamIt];
			if (ensure(DataStream))
			{
				DataStream->ReadData(Context);
			}
			// If something went wrong we should stop deserializing immediately.
			if (Context.HasErrorOrOverflow())
			{
				break;
			}
		}
	}
}

void UDataStreamManager::FImpl::ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* InRecord)
{
	const FRecord* Record = Records.Peek();
	check(Record == InRecord);

	// Process delivery notifications for our NetExports
	NetExports.ProcessPacketDeliveryStatus(Status);

	// Forward the call to each DataStream that was included in the record.
	const uint32 DataStreamMask = Record->DataStreamMask;
	const uint32 DataStreamStateMask = Record->DataStreamStateMask;
	const uint32 StreamCount = Streams.Num();

	TObjectPtr<UDataStream>* StreamData = Streams.GetData();
	for (uint32 StreamIt = 0, StreamEndIt = StreamCount, Mask = 1U; StreamIt != StreamEndIt; ++StreamIt, Mask += Mask)
	{
		if (DataStreamStateMask & Mask)
		{
			// State transitions are driven explicitly, but if we drop a transition we must dirty the StreamState to trigger a write.			
			if  (Status == UE::Net::EPacketDeliveryStatus::Lost)
			{
				UE_LOG_DATASTREAM_CONN(Verbose, TEXT("ProcessPacketDeliveryStatus Handle Lost DataStream State for StreamIndex: %u CurrentState: %s "), StreamIt, LexToString(StreamState[StreamIt]));

				// Note: As we do not store actual lost state in record, we are pessimistic for pendingcreate/close
				if (StreamState[StreamIt] == EDataStreamState::WaitOnCreateConfirmation)
				{
					SetStreamState(StreamIt, EDataStreamState::PendingCreate);
				}
				else if (StreamState[StreamIt] == EDataStreamState::WaitOnCloseConfirmation)
				{
					SetStreamState(StreamIt, EDataStreamState::PendingClose);
				}
				MarkStreamStateDirty(StreamIt);
			}
		}

		if (DataStreamMask & Mask)
		{
			UDataStream* DataStream = StreamData[StreamIt];
			// We should always have a DataStream here.
			if (ensure(DataStream))
			{
				DataStream->ProcessPacketDeliveryStatus(Status, Record->DataStreamRecords[StreamIt]);
			}
		}
	}

	Records.Pop();
}

bool UDataStreamManager::FImpl::HasDebugDataToSend() const
{
	for (TObjectPtr<const UDataStream> Stream : Streams)
	{
		if (Stream && Stream->HasDebugDataToSend())
		{
			return true;
		}
	}

	return false;
}

bool UDataStreamManager::FImpl::HasAcknowledgedAllReliableData() const
{
	for (TObjectPtr<const UDataStream> Stream : Streams)
	{
		if (Stream && !Stream->HasAcknowledgedAllReliableData())
		{
			return false;
		}
	}

	return true;
}

void UDataStreamManager::FImpl::CloseStream(const FName StreamName)
{
	// Find index
	const UDataStreamDefinitions* StreamDefinitions = GetDefault<UDataStreamDefinitions>();
	const FDataStreamDefinition* Definition = StreamDefinitions->FindDefinition(StreamName);

	if (!Definition)
	{
		UE_LOG_DATASTREAM_CONN(Warning, TEXT("UDataStreamManager::FImpl::CloseStream No DataStreamDefinition exists for name '%s' exists."), ToCStr(StreamName.GetPlainNameString()));
		return;
	}

	if (!Definition->bDynamicCreate)
	{
		UE_LOG_DATASTREAM_CONN(Warning, TEXT("UDataStreamManager::FImpl::CloseStream cannot request DataStream'%s' to be closed as it is not marked as bDynamicCreate."), ToCStr(StreamName.GetPlainNameString()));
		return;
	}

	if (!Streams.ContainsByPredicate(FindStreamByName(StreamName)))
	{
		UE_LOG_DATASTREAM_CONN(Warning, TEXT("UDataStreamManager::FImpl::CloseStream No DataStream with name '%s' exists."), ToCStr(StreamName.GetPlainNameString()));
		return;
	}

	const uint32 StreamIndex = StreamDefinitions->GetStreamIndex(*Definition);
	const EDataStreamState CurrentState = GetStreamState(StreamIndex);
	switch (CurrentState)
	{
		case EDataStreamState::PendingCreate:
		{
			// If we are in PendingCreate it means that we either have not yet sent create request and can go back to invalid and release the stream
			SetStreamState(StreamIndex, EDataStreamState::Invalid);
			DestroyStream(StreamIndex);
			break;
		}
		case EDataStreamState::WaitOnCreateConfirmation:
		{
			// If we are closed while waiting for create confirmation.
			SetStreamState(StreamIndex, EDataStreamState::PendingClose);
			break;
		}
		case EDataStreamState::Open:
		{
			SetStreamState(StreamIndex, EDataStreamState::PendingClose);
			break;
		}
		default:
		{
			break;
		}
	};
}

ECreateDataStreamResult UDataStreamManager::FImpl::CreateStreamFromDefinition(const FDataStreamDefinition& Definition, ECreateDataStreamFlags Flags)
{
	if (Definition.Class == nullptr)
	{
		return ECreateDataStreamResult::Error_InvalidDefinition;
	}

	const int32 WantedStreamIndex = UDataStreamDefinitions::GetStreamIndex(Definition);
	if (WantedStreamIndex == -1)
	{
		return ECreateDataStreamResult::Error_InvalidDefinition;
	}

	// Bumping MaxStreamCount may require modifying the FRecord and WriteData/ReadData.
	if (WantedStreamIndex >= MaxStreamCount)
	{
		return ECreateDataStreamResult::Error_TooManyStreams;
	}

	// Make room
	const int32 RequiredStreamCount = WantedStreamIndex + 1;
	if (Streams.Num() < RequiredStreamCount)
	{
		Streams.SetNum(RequiredStreamCount, EAllowShrinking::No);
		StreamSendStatus.SetNumZeroed(RequiredStreamCount, EAllowShrinking::No);
		StreamState.SetNumZeroed(RequiredStreamCount, EAllowShrinking::No);
	}

	UDataStream* Stream = nullptr;

	bool bIsDynamic = Definition.bDynamicCreate;
	bool bShouldCreate = !bIsDynamic || !EnumHasAnyFlags(Flags, ECreateDataStreamFlags::RegisterIfStreamIsDynamic);
	if (bShouldCreate)
	{
		Stream = NewObject<UDataStream>(GetTransientPackage(), ToRawPtr(Definition.Class), MakeUniqueObjectName(nullptr, Definition.Class, Definition.DataStreamName));

		Streams[WantedStreamIndex] = Stream;
		StreamSendStatus[WantedStreamIndex] = Definition.DefaultSendStatus;

		// Auto created streams are always considered to be opened
		StreamState[WantedStreamIndex] = bIsDynamic ? EDataStreamState::PendingCreate : EDataStreamState::Open;
		if (bIsDynamic)
		{
			MarkStreamStateDirty(WantedStreamIndex);
		}
		UE_LOG_DATASTREAM_CONN(Verbose, TEXT("Created DataStream with name '%s' with streamindex: %d State:%s"), ToCStr(Definition.DataStreamName.ToString()), WantedStreamIndex, LexToString(StreamState[WantedStreamIndex]));
	}
	else
	{
		Streams[WantedStreamIndex] = nullptr;
		StreamSendStatus[WantedStreamIndex] = EDataStreamSendStatus::Pause;
		StreamState[WantedStreamIndex] = EDataStreamState::Invalid;

		UE_LOG_DATASTREAM_CONN(Verbose, TEXT("Registered DataStream with name '%s' with streamindex: %d State:%s"), ToCStr(Definition.DataStreamName.ToString()), WantedStreamIndex, LexToString(StreamState[WantedStreamIndex]));
	}

	// Init stream
	InitStream(Stream, Definition.DataStreamName);

	return ECreateDataStreamResult::Success;
}

ECreateDataStreamResult UDataStreamManager::FImpl::CreateStream(const FName StreamName, ECreateDataStreamFlags Flags)
{
	if (Streams.ContainsByPredicate(FindStreamByName(StreamName)))
	{
		UE_LOG_DATASTREAM_CONN(Warning, TEXT("A DataStream with name '%s' already exists."), ToCStr(StreamName.GetPlainNameString()));
		return ECreateDataStreamResult::Error_Duplicate;
	}
	
	const UDataStreamDefinitions* StreamDefinitions = GetDefault<UDataStreamDefinitions>();
	if (const FDataStreamDefinition* Definition = StreamDefinitions->FindDefinition(StreamName))
	{
		return CreateStreamFromDefinition(*Definition, Flags);
	}

	return ECreateDataStreamResult::Error_MissingDefinition;
}

ECreateDataStreamResult UDataStreamManager::FImpl::CreateStreamFromIndex(int32 StreamIndex)
{	
	const UDataStreamDefinitions* StreamDefinitions = GetDefault<UDataStreamDefinitions>();
	if (!StreamDefinitions->bFixupComplete)
	{
		UE_LOG_DATASTREAM_CONN(Warning, TEXT("Cannot create datastream by index if DataStreamDefinitions are not FixedUp."));
		return ECreateDataStreamResult::Error_MissingDefinition;
	}

	if (const FDataStreamDefinition* Definition = StreamDefinitions->FindDefinition(StreamIndex))
	{
		return CreateStreamFromDefinition(*Definition, ECreateDataStreamFlags::None);
	}

	return ECreateDataStreamResult::Error_MissingDefinition;
}

inline const UDataStream* UDataStreamManager::FImpl::GetStream(const FName StreamName) const
{
	const TObjectPtr<UDataStream>* Stream = Streams.FindByPredicate(FindStreamByName(StreamName));
	return Stream != nullptr ? *Stream : nullptr;
}

inline UDataStream* UDataStreamManager::FImpl::GetStream(const FName StreamName)
{
	TObjectPtr<UDataStream>* Stream = Streams.FindByPredicate(FindStreamByName(StreamName));
	return Stream != nullptr ? *Stream : nullptr;
}

void UDataStreamManager::FImpl::SetSendStatus(const FName StreamName, EDataStreamSendStatus Status)
{
	const int32 Index = Streams.IndexOfByPredicate(FindStreamByName(StreamName));
	if (Index == INDEX_NONE)
	{
		UE_LOG_DATASTREAM_CONN(Display, TEXT("Cannot set send status for DataStream '%s' that hasn't been created."), ToCStr(StreamName.GetPlainNameString()));
		return;
	}

	StreamSendStatus[Index] = Status;
}

EDataStreamSendStatus UDataStreamManager::FImpl::GetSendStatus(const FName StreamName) const
{
	const int32 Index = Streams.IndexOfByPredicate(FindStreamByName(StreamName));
	if (Index == INDEX_NONE)
	{
		UE_LOG_DATASTREAM_CONN(Display, TEXT("Cannot retrieve send status for DataStream '%s' that hasn't been created. Returning Pause."), ToCStr(StreamName.GetPlainNameString()));
		return EDataStreamSendStatus::Pause;
	}

	return StreamSendStatus[Index];
}

UDataStream::EDataStreamState UDataStreamManager::FImpl::GetStreamState(const FName StreamName) const
{
	const int32 Index = Streams.IndexOfByPredicate(FindStreamByName(StreamName));
	if (Index == INDEX_NONE)
	{
		return UDataStream::EDataStreamState::Invalid;
	}
	return GetStreamState(Index);
}

void UDataStreamManager::FImpl::InitRecordStorage()
{
	const uint32 PacketWindowSize = InitParameters.PacketWindowSize;
	RecordStorage.SetNum(PacketWindowSize);

	Records = TResizableCircularQueue<FRecord*>(PacketWindowSize);
	for (uint32 It = 0, EndIt = PacketWindowSize; It != EndIt; ++It)
	{
		FRecord*& Record = Records.Enqueue();
		Record = &RecordStorage[It];
	}

	// Note: The circular queue will not modify the contents of its storage for POD types.
	Records.Reset();
}

void UDataStreamManager::FImpl::InitStream(UDataStream* Stream, FName DataStreamName)
{
	if (IsValid(Stream))
	{
		UDataStream::FInitParameters StreamInitParameters(InitParameters);
		StreamInitParameters.Name = DataStreamName;

		Stream->Init(StreamInitParameters);

		// Catch if DataStream does not call Super::Init.
		ensureMsgf(Stream->GetDataStreamName() == DataStreamName, TEXT("DataStream %s did not call Super::Init"), *DataStreamName.ToString());
	}
}

void UDataStreamManager::FImpl::InitStreams()
{
	UDataStreamDefinitions* StreamDefinitions = GetMutableDefault<UDataStreamDefinitions>();
	StreamDefinitions->FixupDefinitions();

	TArray<FName> StreamsToAutoCreateOrRegister;
	StreamsToAutoCreateOrRegister.Reserve(MaxStreamCount);
	StreamDefinitions->GetStreamNamesToAutoCreateOrRegister(StreamsToAutoCreateOrRegister);

	for (const FName& StreamName : StreamsToAutoCreateOrRegister)
	{
		CreateStream(StreamName, ECreateDataStreamFlags::RegisterIfStreamIsDynamic);
	}
}

UE::Net::Private::FNetExports& UDataStreamManager::FImpl::GetNetExports()
{
	return NetExports;
}

void UDataStreamManager::FImpl::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(Streams);
}
