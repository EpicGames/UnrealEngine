// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/IrisConfig.h"
#include "Iris/Core/IrisCsv.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

namespace UE::Net
{

namespace Private::CVars
{

static int32 GUseIrisReplication = 0;
static FAutoConsoleVariableRef CVarUseIrisReplicationRef(TEXT("net.Iris.UseIrisReplication"), GUseIrisReplication, TEXT("Enables Iris replication system. 0 will fallback to legacy replicationsystem."), ECVF_Default );

static bool bScheduleCreationDependenciesFirst = true;
static FAutoConsoleVariableRef CVarScheduleCreationDependenciesFirst(TEXT("net.Iris.ScheduleCreationDependenciesFirst"), bScheduleCreationDependenciesFirst, TEXT("If enabled we will schedule an objects creation dependencies before the object if they are not confirmed as created."));

}

bool ShouldUseIrisReplication()
{
	return Private::CVars::GUseIrisReplication > 0;
}

void SetUseIrisReplication(bool EnableIrisReplication)
{
	Private::CVars::GUseIrisReplication = EnableIrisReplication ? 1 : 0;
}

EReplicationSystem GetUseIrisReplicationCmdlineValue()
{
	int32 UseIrisReplication=0;
	if (FParse::Value(FCommandLine::Get(), TEXT("-UseIrisReplication="), UseIrisReplication))
	{
		// Try to force the requested system if the cmdline is present
		return UseIrisReplication > 0 ? EReplicationSystem::Iris : EReplicationSystem::Generic;
	}

	// Use the default engine value
	return EReplicationSystem::Default;
}

bool IsSchedulingCreationDependenciesFirst()
{
	return Private::CVars::bScheduleCreationDependenciesFirst;
}

}


// Enable Iris category by default on servers
CSV_DEFINE_CATEGORY(Iris, WITH_SERVER_CODE);
