// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/IrisConfig.h"

// Log config for Iris subsystems
#ifndef UE_NET_ENABLE_REPLICATIONREADER_LOG
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#	define UE_NET_ENABLE_REPLICATIONREADER_LOG 0
#else
#	define UE_NET_ENABLE_REPLICATIONREADER_LOG 0
#endif
#endif


#ifndef UE_NET_ENABLE_REPLICATIONWRITER_LOG
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#	define UE_NET_ENABLE_REPLICATIONWRITER_LOG 0
#else
#	define UE_NET_ENABLE_REPLICATIONWRITER_LOG 1
#endif
#endif

// Misc config
#ifndef UE_NET_VALIDATE_REPLICATION_RECORD
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#	define UE_NET_VALIDATE_REPLICATION_RECORD 0
#else
#	define UE_NET_VALIDATE_REPLICATION_RECORD 1
#endif 
#endif

#ifndef UE_NET_VALIDATE_DC_BASELINES
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#	define UE_NET_VALIDATE_DC_BASELINES 0
#else
#	define UE_NET_VALIDATE_DC_BASELINES 1
#endif 
#endif

#ifndef UE_NET_ENABLE_REPLICATIONWRITER_CANNOT_SEND_WARNING
#if (UE_BUILD_SHIPPING)
#	define UE_NET_ENABLE_REPLICATIONWRITER_CANNOT_SEND_WARNING 1
#else
#	define UE_NET_ENABLE_REPLICATIONWRITER_CANNOT_SEND_WARNING 1
#endif 
#endif

#ifndef UE_NET_WITH_VERBOSE_POLL_PROFILING
#   define UE_NET_WITH_VERBOSE_POLL_PROFILING (!UE_BUILD_SHIPPING)
#endif
