// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/Ticker.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"

#include "Net/Core/Connection/NetEnums.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"

#include "Iris/Config/IrisDynamicConfig.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationState/DefaultPropertyNetSerializerInfos.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationSystem/IrisCreationFlowLog.h"
#include "Iris/ReplicationSystem/LegacyPushModel.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/Serialization/InternalNetSerializerDelegates.h"
#include "Iris/IrisConfigInternal.h"


class FIrisCoreModule : public IModuleInterface
{
private:

	void RegisterPropertyNetSerializerSelectorTypes()
	{
		using namespace UE::Net;
		using namespace UE::Net::Private;

		FPropertyNetSerializerInfoRegistry::Reset();

		FInternalNetSerializerDelegates::BroadcastPreFreezeNetSerializerRegistry();
		RegisterDefaultPropertyNetSerializerInfos();

		FPropertyNetSerializerInfoRegistry::Freeze();
		FInternalNetSerializerDelegates::BroadcastPostFreezeNetSerializerRegistry();
	}

	virtual void StartupModule() override
	{
		// Iris requires NetCore
		FModuleManager::LoadModuleChecked<IModuleInterface>("NetCore");

		FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddRaw(this, &FIrisCoreModule::OnAllModuleLoadingPhasesComplete);

		// Check command line for whether we should override the net.Iris.UseIrisReplication cvar, as we need to do that early
		const EReplicationSystem CmdlineRepSystem = UE::Net::GetUseIrisReplicationCmdlineValue();
		if (CmdlineRepSystem != EReplicationSystem::Default)
		{
			const bool bEnableIris = CmdlineRepSystem == EReplicationSystem::Iris;
			UE::Net::SetUseIrisReplication(bEnableIris);
		}

		RegisterPropertyNetSerializerSelectorTypes();

		UE_NET_IRIS_INIT_LEGACY_PUSH_MODEL();

#if UE_NET_ENABLE_IRISCREATIONFLOWLOG
		UE::Net::CreationFlowLog::Init();
#endif // UE_NET_ENABLE_IRISCREATIONFLOWLOG

		ModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FIrisCoreModule::OnModulesChanged);
		RepSysCreatedHandle = UE::Net::FReplicationSystemFactory::GetReplicationSystemCreatedDelegate().AddRaw(this, &FIrisCoreModule::OnRepSystemCreated);
		RepSysDestroyedHandle = UE::Net::FReplicationSystemFactory::GetReplicationSystemDestroyedDelegate().AddRaw(this, &FIrisCoreModule::OnRepSystemDestroyed);

		// Figure out how many ReplicationSystems there are so we start on a correct balance prior to getting callbacks.
		for (TArrayView<UReplicationSystem*> RepSystems = UE::Net::FReplicationSystemFactory::GetAllReplicationSystems(); const UReplicationSystem* RepSystem : RepSystems)
		{
			RepSystemCount += (RepSystem != nullptr ? 1 : 0);
		}
	}

	virtual void ShutdownModule() override
	{
		if (ModulesChangedHandle.IsValid())
		{
			FModuleManager::Get().OnModulesChanged().Remove(ModulesChangedHandle);
			ModulesChangedHandle.Reset();
		}

	
		UE::Net::FReplicationSystemFactory::GetReplicationSystemCreatedDelegate().Remove(RepSysCreatedHandle);
		UE::Net::FReplicationSystemFactory::GetReplicationSystemDestroyedDelegate().Remove(RepSysDestroyedHandle);
		RepSysCreatedHandle.Reset();
		RepSysDestroyedHandle.Reset();

#if UE_NET_ENABLE_IRISCREATIONFLOWLOG
		UE::Net::CreationFlowLog::Shutdown();
#endif // UE_NET_ENABLE_IRISCREATIONFLOWLOG

		UE_NET_IRIS_SHUTDOWN_LEGACY_PUSH_MODEL();
	}

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
	 
	void OnAllModuleLoadingPhasesComplete()
	{
		FCoreDelegates::OnAllModuleLoadingPhasesComplete.RemoveAll(this);

		bAllowLoadedModulesUpdatedCallback = true;
		if (RepSystemCount > 0 && ShouldBroadcastLoadedModulesUpdated())
		{
			ForceBroadcastLoadedModulesUpdated();
		}
	}

	void OnModulesChanged(FName ModuleThatChanged, EModuleChangeReason ReasonForChange)
	{
		switch (ReasonForChange)
		{
			case EModuleChangeReason::ModuleLoaded:
			{
				++LoadedModulesCount;
				if (bAllowLoadedModulesUpdatedCallback && (RepSystemCount > 0))
				{
					if (!BroadcastModulesUpdatedHandle.IsValid())
					{
						LoadedModulesCountAtTickerCreation = LoadedModulesCount;
						BroadcastModulesUpdatedHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FIrisCoreModule::BroadcastLoadedModulesUpdated));
					}
				}

				UE::Net::Private::FInternalNetSerializerDelegates::BroadcastPreFreezeNetSerializerRegistry();
				UE::Net::Private::FInternalNetSerializerDelegates::BroadcastPostFreezeNetSerializerRegistry();
			}
			break;

			default:
			{
			}
			break;
		}
	}

	bool BroadcastLoadedModulesUpdated(float /* DeltaTime */)
	{
		// If we're still loading modules check again next frame
		if (LoadedModulesCountAtTickerCreation != LoadedModulesCount)
		{
			LoadedModulesCountAtTickerCreation = LoadedModulesCount;
			return true;
		}
		else
		{
			if (RepSystemCount > 0)
			{
				UE_LOGF(LogIris, Warning, "FInternalNetSerializerDelegates::BroadcastLoadedModulesUpdated() called while there are %d active ReplicationSystems. If polymorphic types are registered we may have corrupt data. A restart of the ReplicationSystem or NetDriver is recommended. Total loaded modules: %u.", RepSystemCount, LoadedModulesCount);
			}
			else
			{
				UE_LOGF(LogIris, Display, "FInternalNetSerializerDelegates::BroadcastLoadedModulesUpdated() called while there are %d active ReplicationSystems. This is %ls. Total loaded modules: %u.", RepSystemCount, (RepSystemCount == 0 ? TEXT("good") : TEXT("unexpected")), LoadedModulesCount);
			}

			BroadcastModulesUpdatedHandle.Reset();
			UE::Net::Private::FInternalNetSerializerDelegates::BroadcastLoadedModulesUpdated();
			return false;
		}
	}

	void ForceBroadcastLoadedModulesUpdated()
	{
		ResetBroadcastLoadedModulesTicker();
		LoadedModulesCountAtTickerCreation = LoadedModulesCount;
		BroadcastLoadedModulesUpdated(0.0f);
	}

	bool ShouldBroadcastLoadedModulesUpdated() const
	{
		return BroadcastModulesUpdatedHandle.IsValid() || (LoadedModulesCountAtTickerCreation != LoadedModulesCount);
	}

	void OnRepSystemCreated(UReplicationSystem*)
	{
		if (ShouldBroadcastLoadedModulesUpdated())
		{
			ForceBroadcastLoadedModulesUpdated();
		}
		else if (RepSystemCount == 0)
		{
			UE_LOGF(LogIris, Display, "FInternalNetSerializerDelegates::BroadcastLoadedModulesUpdated() not called when creating ReplicationSystem since no additional modules have been loaded since last broadcast. This is good. Total loaded modules: %u.", LoadedModulesCount);
		}

		// Update RepSystemCount after broadcasting such that we get logging ideally saying there weren't any active replication systems.
		++RepSystemCount;
	}

	void OnRepSystemDestroyed(UReplicationSystem*)
	{
		--RepSystemCount;
		ensure(RepSystemCount >= 0);
	}

	void ResetBroadcastLoadedModulesTicker()
	{
		if (BroadcastModulesUpdatedHandle.IsValid())
		{
			FTSTicker::RemoveTicker(BroadcastModulesUpdatedHandle);
			BroadcastModulesUpdatedHandle.Reset();
		}
	}

private:
	FDelegateHandle ModulesChangedHandle;
	FDelegateHandle RepSysCreatedHandle;
	FDelegateHandle RepSysDestroyedHandle;
	FTSTicker::FDelegateHandle BroadcastModulesUpdatedHandle;
	int32 RepSystemCount = 0;
	uint32 LoadedModulesCount = 0;
	uint32 LoadedModulesCountAtTickerCreation = 0;
	bool bAllowLoadedModulesUpdatedCallback = false;
};
IMPLEMENT_MODULE(FIrisCoreModule, IrisCore);
