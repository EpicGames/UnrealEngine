// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/DefaultPropertyNetSerializerInfos.h"
#include "Iris/ReplicationState/EnumPropertyNetSerializerInfo.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/DateTimeNetSerializer.h"
#include "Iris/Serialization/RemoteObjectReferenceNetSerializer.h"
#include "Iris/Serialization/RemoteServerIdNetSerializer.h"
#include "Iris/Serialization/RemoteObjectIdNetSerializer.h"
#include "UObject/TextProperty.h"

namespace UE::Net::Private
{

/**
 *	Primitive types defaults, remember to bind them as well in RegisterDefaultPropertyNetSerializerInfos
 */
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FInt8Property, FInt8NetSerializer);
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FInt16Property, FInt16NetSerializer);
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FIntProperty, FPackedInt32NetSerializer);
//UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(IntProperty, FInt32NetSerializer);
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FInt64Property, FInt64NetSerializer);
// ByteProperty is special as it also handles EnumAsByte
//UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FByteProperty, FUint8NetSerializer);
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FUInt16Property, FUint16NetSerializer);
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FUInt32Property, FPackedUint32NetSerializer);
//UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FUInt32Property, FUint32NetSerializer);
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FUInt64Property, FUint64NetSerializer);
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FFloatProperty, FFloatNetSerializer);
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FDoubleProperty, FDoubleNetSerializer);

// Objects and fields
static const FName PropertyNetSerializerRegistry_NAME_SoftObjectPath("SoftObjectPath");
static const FName PropertyNetSerializerRegistry_NAME_SoftClassPath("SoftClassPath");
static const FName PropertyNetSerializerRegistry_NAME_RemoteObjectReference("RemoteObjectReference");
static const FName PropertyNetSerializerRegistry_NAME_RemoteServerId("RemoteServerId");
static const FName PropertyNetSerializerRegistry_NAME_RemoteObjectId("RemoteObjectId");

// Info struct for raw object pointers, can be used in RPC parameters
struct FObjectPropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const override
	{
		return FObjectProperty::StaticClass();
	}

	virtual bool IsSupported(const FProperty* Property) const
	{
		// Only use this info for raw UObject pointers.
		return !Property->HasAnyPropertyFlags(CPF_TObjectPtr);
	}

	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override
	{
		return &UE_NET_GET_SERIALIZER(FObjectNetSerializer);
	}

	virtual bool CanUseDefaultConfig(const FProperty*) const override { return false; }

	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override
	{
		FObjectNetSerializerConfig* Config = new (NetSerializerConfigBuffer) FObjectNetSerializerConfig();
		const FObjectPropertyBase* ObjectPtrPropertyBase = static_cast<const FObjectPropertyBase*>(Property);
		Config->PropertyClass = ObjectPtrPropertyBase->PropertyClass;
		return Config;
	}
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FObjectPropertyNetSerializerInfo);

// Info struct for TObjectPtr properties
struct FObjectPtrPropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const override
	{
		return FObjectProperty::StaticClass();
}

	virtual bool IsSupported(const FProperty* Property) const
	{
		// Only use this info for TObjectPtr properties.
		return Property->HasAnyPropertyFlags(CPF_TObjectPtr);
	}

	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override
	{
		return &UE_NET_GET_SERIALIZER(FObjectPtrNetSerializer);
	}

	virtual bool CanUseDefaultConfig(const FProperty*) const override { return false; }

	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override
	{
		FObjectPtrNetSerializerConfig* Config = new (NetSerializerConfigBuffer) FObjectPtrNetSerializerConfig();
		const FObjectPropertyBase* ObjectPtrPropertyBase = static_cast<const FObjectPropertyBase*>(Property);
		Config->PropertyClass = ObjectPtrPropertyBase->PropertyClass;
		return Config;
	}
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FObjectPtrPropertyNetSerializerInfo);

struct FWeakObjectPtrPropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const override
	{
		return FWeakObjectProperty::StaticClass();
	}

	virtual bool IsSupported(const FProperty* Property) const
	{
		return true;
	}

	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override
	{
		return &UE_NET_GET_SERIALIZER(FWeakObjectNetSerializer);
	}

	virtual bool CanUseDefaultConfig(const FProperty*) const override { return false; }

	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override
	{
		FWeakObjectNetSerializerConfig* Config = new (NetSerializerConfigBuffer) FWeakObjectNetSerializerConfig();
		const FObjectPropertyBase* ObjectPtrPropertyBase = static_cast<const FObjectPropertyBase*>(Property);
		Config->PropertyClass = ObjectPtrPropertyBase->PropertyClass;
		return Config;
	}
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FWeakObjectPtrPropertyNetSerializerInfo);


UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FSoftObjectProperty, FSoftObjectNetSerializer);
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FSoftClassProperty, FSoftObjectNetSerializer);
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FFieldPathProperty, FFieldPathNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_SoftObjectPath, FSoftObjectPathNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_SoftClassPath, FSoftClassPathNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_RemoteObjectReference, FRemoteObjectReferenceNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_RemoteServerId, FRemoteServerIdNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_RemoteObjectId, FRemoteObjectIdNetSerializer);

// Strings
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FStrProperty, FStringNetSerializer);
//UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FNameProperty, FNameNetSerializer);
// Use NetTokens instead of strings when serializing FNames
UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(FNameProperty, FNameAsNetTokenNetSerializer);

// Named structs with specific serializers
static const FName PropertyNetSerializerRegistry_NAME_Guid("Guid");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_Guid, FGuidNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NAME_Vector, FVectorNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NAME_Vector3f, FVector3fNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NAME_Vector3d, FVector3dNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NAME_Rotator, FRotatorNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NAME_Rotator3f, FRotator3fNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NAME_Rotator3d, FRotator3dNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NAME_Quat, FUnitQuatNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NAME_Quat4f, FUnitQuat4fNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NAME_Quat4d, FUnitQuat4dNetSerializer);

// Vector types. These are structs with custom serialization.
static const FName PropertyNetSerializerRegistry_NAME_Vector_NetQuantize100("Vector_NetQuantize100");
static const FName PropertyNetSerializerRegistry_NAME_Vector_NetQuantize10("Vector_NetQuantize10");
static const FName PropertyNetSerializerRegistry_NAME_Vector_NetQuantize("Vector_NetQuantize");
static const FName PropertyNetSerializerRegistry_NAME_Vector_NetQuantizeNormal("Vector_NetQuantizeNormal");

static const FName PropertyNetSerializerRegistry_NAME_DateTime("DateTime");

UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_Vector_NetQuantize100, FVectorNetQuantize100NetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_Vector_NetQuantize10, FVectorNetQuantize10NetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_Vector_NetQuantize, FVectorNetQuantizeNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_Vector_NetQuantizeNormal, FVectorNetQuantizeNormalNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_DateTime, FDateTimeNetSerializer);

/**
 * ByteProperty when backed by a uint8
 */
struct FUint8PropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const override { return FByteProperty::StaticClass(); }
	virtual bool IsSupported(const FProperty* Property) const override
	{ 
		const FByteProperty* ByteProperty = CastFieldChecked<FByteProperty>(Property); 
		return !ByteProperty->IsEnum();
	}
	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override { return &UE_NET_GET_SERIALIZER(FUint8NetSerializer); }
	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override { checkf(false, TEXT("%s"), TEXT("Internal error. uint8s should use default NetSerializerConfig")); return nullptr; }
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FUint8PropertyNetSerializerInfo);

/**
 * Native bool
 */
struct FNativeBoolPropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const override { return FBoolProperty::StaticClass(); }
	virtual bool IsSupported(const FProperty* Property) const override { const FBoolProperty* BoolProperty = CastFieldChecked<const FBoolProperty>(Property); return BoolProperty->IsNativeBool(); }
	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override { return &UE_NET_GET_SERIALIZER(FBoolNetSerializer); }
	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override { checkf(false, TEXT("%s"), TEXT("Internal error. Bools should use default NetSerializerConfig")); return nullptr; }
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FNativeBoolPropertyNetSerializerInfo);

/**
 * Bool as bitfield
 */
struct FBitFieldPropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const override { return FBoolProperty::StaticClass(); }
	virtual bool IsSupported(const FProperty* Property) const override { const FBoolProperty* BoolProperty = CastFieldChecked<const FBoolProperty>(Property); return !BoolProperty->IsNativeBool(); }
	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override {return &UE_NET_GET_SERIALIZER(FBitfieldNetSerializer); }
	virtual bool CanUseDefaultConfig(const FProperty* Property) const override { return false; }
	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override
	{
		FBitfieldNetSerializerConfig* Config = new (NetSerializerConfigBuffer) FBitfieldNetSerializerConfig();

		InitBitfieldNetSerializerConfigFromProperty(*Config, CastFieldChecked<FBoolProperty>(Property));
	
		return Config;
	}
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FBitFieldPropertyNetSerializerInfo);

/**
 * ScriptInterface NetSerializerInfo
 */
struct FScriptInterfacePropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const override { return FInterfaceProperty::StaticClass(); }
	virtual const FNetSerializer* GetNetSerializer(const FProperty*) const override { return &UE_NET_GET_SERIALIZER(FScriptInterfaceNetSerializer); }
	virtual bool CanUseDefaultConfig(const FProperty*) const override { return false; }
	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override
	{
		FScriptInterfaceNetSerializerConfig* Config = new (NetSerializerConfigBuffer) FScriptInterfaceNetSerializerConfig();
		const FInterfaceProperty* InterfaceProperty = static_cast<const FInterfaceProperty*>(Property);
		Config->PropertyClass = InterfaceProperty->InterfaceClass;
		return Config;
	}
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FScriptInterfacePropertyNetSerializerInfo);

/**
 * FText NetSerializerInfo, special forwarding to LastResortNetSerializer excluding default state hash.
 */
struct FTextPropertyNetSerializerInfo : public FLastResortPropertyNetSerializerInfo
{
	FTextPropertyNetSerializerInfo() : FLastResortPropertyNetSerializerInfo()
	{
		bExcludeFromDefaultStateHash = true;
	}

	virtual const FFieldClass* GetPropertyTypeClass() const override
	{
		return FTextProperty::StaticClass();
	}
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FTextPropertyNetSerializerInfo);

// $TODO: Investigate automatic registration by binding all infos in static chain and auto-register when we freeze the registry
/**
 * Apart from the specialized serializer we register here we support the following:
 * Structs. Structs without specialized serializer will use the StructNetSerializer.
 * UArrayProperty will use ArrayPropertyNetSerializer.
 * As a last resort for properties we use LastResortPropertyNetSerializer. It's by no means ideal.
 */
void RegisterDefaultPropertyNetSerializerInfos()
{
	// Register supported types
	// Integer types
	UE_NET_REGISTER_NETSERIALIZER_INFO(FInt8Property);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FInt16Property);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FIntProperty);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FInt64Property);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FUint8PropertyNetSerializerInfo);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FUInt16Property);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FUInt32Property);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FUInt64Property);

	// Enum types
	UE_NET_REGISTER_NETSERIALIZER_INFO(FEnumAsBytePropertyNetSerializerInfo);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FEnumPropertyNetSerializerInfo);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FNetRoleNetSerializerInfo);

	// Float types
	UE_NET_REGISTER_NETSERIALIZER_INFO(FFloatProperty);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FDoubleProperty);

	// Object and field types
	UE_NET_REGISTER_NETSERIALIZER_INFO(FObjectPropertyNetSerializerInfo);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FObjectPtrPropertyNetSerializerInfo);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FWeakObjectPtrPropertyNetSerializerInfo);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FScriptInterfacePropertyNetSerializerInfo);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FSoftObjectProperty);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FFieldPathProperty);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_SoftObjectPath);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_SoftClassPath);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_RemoteObjectReference);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_RemoteServerId);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_RemoteObjectId);

	// String types
	UE_NET_REGISTER_NETSERIALIZER_INFO(FNameProperty);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FStrProperty);

	// FTextProperty is using a special variant of LastResortNetSerializer that does not serialize defaultstatehash.
	UE_NET_REGISTER_NETSERIALIZER_INFO(FTextPropertyNetSerializerInfo);

	// Special types
	UE_NET_REGISTER_NETSERIALIZER_INFO(FNativeBoolPropertyNetSerializerInfo);
	UE_NET_REGISTER_NETSERIALIZER_INFO(FBitFieldPropertyNetSerializerInfo);

	// Named structs that we support
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_Guid);
	UE_NET_REGISTER_NETSERIALIZER_INFO(NAME_Vector);
	UE_NET_REGISTER_NETSERIALIZER_INFO(NAME_Vector3f);
	UE_NET_REGISTER_NETSERIALIZER_INFO(NAME_Vector3d);
	UE_NET_REGISTER_NETSERIALIZER_INFO(NAME_Rotator);
	UE_NET_REGISTER_NETSERIALIZER_INFO(NAME_Rotator3f);
	UE_NET_REGISTER_NETSERIALIZER_INFO(NAME_Rotator3d);
	UE_NET_REGISTER_NETSERIALIZER_INFO(NAME_Quat);
	UE_NET_REGISTER_NETSERIALIZER_INFO(NAME_Quat4f);
	UE_NET_REGISTER_NETSERIALIZER_INFO(NAME_Quat4d);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_Vector_NetQuantize100);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_Vector_NetQuantize10);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_Vector_NetQuantize);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_Vector_NetQuantizeNormal);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_DateTime);
}

}

