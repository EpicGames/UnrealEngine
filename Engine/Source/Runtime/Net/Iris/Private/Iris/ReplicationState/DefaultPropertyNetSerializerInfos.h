// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::Net::Private
{

/**
 * Register supported property types and their default NetSerializers
 */
void RegisterDefaultPropertyNetSerializerInfos();

}
