// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/EnumPropertyNetSerializerInfo.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/InternalEnumNetSerializers.h"

namespace UE::Net::Private
{

static const FName PropertyNetSerializerRegistry_NAME_NetRole("ENetRole");

static bool PropertyNetSerializerInfo_ShouldUseNetRoleSerializer(const FByteProperty* Property)
{
	// Only ENetRole named Role and RemoteRole are supported
	const FName PropertyName = Property->GetFName();
	if (PropertyName != NAME_Role && PropertyName != NAME_RemoteRole)
	{
		return false;
	}

	// Verify this is the right enum.
	const UEnum* Enum = Property->Enum;
	if (Enum == nullptr)
	{
		return false;
	}

	if (Enum->GetFName() != PropertyNetSerializerRegistry_NAME_NetRole)
	{
		return false;
	}

	// Only NetRoles in classes is expected to need the role swapping
	if (const UClass* Class = Cast<UClass>(Property->GetOwner<UObject>()))
	{
		return true;
	}

	return false;
}

/**
 * ByteProperty when backed by EnumAsByte
 */
struct FEnumAsBytePropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const override { return FByteProperty::StaticClass(); }
	
	virtual bool IsSupported(const FProperty* Property) const override
	{ 
		const FByteProperty* ByteProperty = CastFieldChecked<FByteProperty>(Property);
		if (PropertyNetSerializerInfo_ShouldUseNetRoleSerializer(ByteProperty))
		{
			return false;
		}

		return ByteProperty->IsEnum(); 
	}

	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override { return &UE_NET_GET_SERIALIZER(FEnumUint8NetSerializer); }

	virtual bool CanUseDefaultConfig(const FProperty* Property) const override { return false; }

	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override
	{ 
		FEnumUint8NetSerializerConfig* Config = new (NetSerializerConfigBuffer) FEnumUint8NetSerializerConfig();

		Private::InitEnumNetSerializerConfig(*Config, CastFieldChecked<FByteProperty>(Property)->Enum);

		return Config;
	}
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FEnumAsBytePropertyNetSerializerInfo);

/**
 * ByteProperty when used on ENetRole
 */
struct FNetRoleNetSerializerInfo : public FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const override { return FByteProperty::StaticClass(); }
	
	virtual bool IsSupported(const FProperty* Property) const override
	{ 
		const FByteProperty* ByteProperty = CastFieldChecked<FByteProperty>(Property);
		return PropertyNetSerializerInfo_ShouldUseNetRoleSerializer(ByteProperty);
	}

	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override { return &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer); }

	virtual bool CanUseDefaultConfig(const FProperty* Property) const override { return false; }

	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override
	{ 
		FNetRoleNetSerializerConfig* Config = new (NetSerializerConfigBuffer) FNetRoleNetSerializerConfig();

		PartialInitNetRoleSerializerConfig(*Config, CastFieldChecked<FByteProperty>(Property)->Enum);

		return Config;
	}
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FNetRoleNetSerializerInfo);

/**
 * This is supposed to support all enum serializers, both signed and unsigned.
 */
class FEnumPropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
public:
	FEnumPropertyNetSerializerInfo() : FPropertyNetSerializerInfo()
	{
		Serializers[0] = &UE_NET_GET_SERIALIZER(FEnumUint8NetSerializer);
		Serializers[1] = &UE_NET_GET_SERIALIZER(FEnumUint16NetSerializer);
		Serializers[2] = &UE_NET_GET_SERIALIZER(FEnumUint32NetSerializer);
		Serializers[3] = &UE_NET_GET_SERIALIZER(FEnumUint64NetSerializer);
		Serializers[4] = &UE_NET_GET_SERIALIZER(FEnumInt8NetSerializer);
		Serializers[5] = &UE_NET_GET_SERIALIZER(FEnumInt16NetSerializer);
		Serializers[6] = &UE_NET_GET_SERIALIZER(FEnumInt32NetSerializer);
		Serializers[7] = &UE_NET_GET_SERIALIZER(FEnumInt64NetSerializer);
	}

private:
	virtual const FFieldClass* GetPropertyTypeClass() const override { return FEnumProperty::StaticClass(); }
	
	virtual bool IsSupported(const FProperty* Property) const override
	{ 
		const FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(Property);
		const FProperty* UnderlyingType = EnumProperty->GetUnderlyingProperty();
		const int32 SerializerIndex = PropertyToSerializerIndex(UnderlyingType);
		return SerializerIndex >= 0;
	}

	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override
	{
		const FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(Property);
		const FProperty* UnderlyingType = EnumProperty->GetUnderlyingProperty();
		const int32 SerializerIndex = PropertyToSerializerIndex(UnderlyingType);
		return Serializers[SerializerIndex];
	}

	virtual bool CanUseDefaultConfig(const FProperty* Property) const override { return false; }

	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override
	{
		using namespace UE::Net::Private;

		const FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(Property);
		const FProperty* UnderlyingType = EnumProperty->GetUnderlyingProperty();
		const int32 SerializerIndex = PropertyToSerializerIndex(UnderlyingType);

		switch (SerializerIndex)
		{
		case 0:
		{
			FEnumUint8NetSerializerConfig* Config = new (NetSerializerConfigBuffer) FEnumUint8NetSerializerConfig();
			InitEnumNetSerializerConfig(*Config, EnumProperty->GetEnum());
			return Config;
		}
		case 1:
		{
			FEnumUint16NetSerializerConfig* Config = new (NetSerializerConfigBuffer) FEnumUint16NetSerializerConfig();
			InitEnumNetSerializerConfig(*Config, EnumProperty->GetEnum());
			return Config;
		}
		case 2:
		{
			FEnumUint32NetSerializerConfig* Config = new (NetSerializerConfigBuffer) FEnumUint32NetSerializerConfig();
			InitEnumNetSerializerConfig(*Config, EnumProperty->GetEnum());
			return Config;
		}
		case 3:
		{
			FEnumUint64NetSerializerConfig* Config = new (NetSerializerConfigBuffer) FEnumUint64NetSerializerConfig();
			InitEnumNetSerializerConfig(*Config, EnumProperty->GetEnum());
			return Config;
		}
		case 4:
		{
			FEnumInt8NetSerializerConfig* Config = new (NetSerializerConfigBuffer) FEnumInt8NetSerializerConfig();
			InitEnumNetSerializerConfig(*Config, EnumProperty->GetEnum());
			return Config;
		}
		case 5:
		{
			FEnumInt16NetSerializerConfig* Config = new (NetSerializerConfigBuffer) FEnumInt16NetSerializerConfig();
			InitEnumNetSerializerConfig(*Config, EnumProperty->GetEnum());
			return Config;
		}
		case 6:
		{
			FEnumInt32NetSerializerConfig* Config = new (NetSerializerConfigBuffer) FEnumInt32NetSerializerConfig();
			InitEnumNetSerializerConfig(*Config, EnumProperty->GetEnum());
			return Config;
		}
		case 7:
		{
			FEnumInt64NetSerializerConfig* Config = new (NetSerializerConfigBuffer) FEnumInt64NetSerializerConfig();
			InitEnumNetSerializerConfig(*Config, EnumProperty->GetEnum());
			return Config;
		}

		default:
			break;
		};


		return nullptr;
	}

	int32 PropertyToSerializerIndex(const FProperty* Property) const
	{
		if (CastField<FByteProperty>(Property) != nullptr)
		{
			return 0;
		}
		if (CastField<FUInt16Property>(Property) != nullptr)
		{
			return 1;
		}
		if (CastField<FUInt32Property>(Property) != nullptr)
		{
			return 2;
		}
		if (CastField<FUInt64Property>(Property) != nullptr)
		{
			return 3;
		}
		if (CastField<FInt8Property>(Property) != nullptr)
		{
			return 4;
		}
		if (CastField<FInt16Property>(Property) != nullptr)
		{
			return 5;
		}
		if (CastField<FIntProperty>(Property) != nullptr)
		{
			return 6;
		}
		if (CastField<FInt64Property>(Property) != nullptr)
		{
			return 7;
		}

		return -1;
	}

	const FNetSerializer* Serializers[8];

};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FEnumPropertyNetSerializerInfo);

}
