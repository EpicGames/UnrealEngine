// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"

namespace UE::Net::Private
{

UE_NET_DECLARE_NETSERIALIZER_INFO(FEnumAsBytePropertyNetSerializerInfo)
UE_NET_DECLARE_NETSERIALIZER_INFO(FEnumPropertyNetSerializerInfo);
UE_NET_DECLARE_NETSERIALIZER_INFO(FNetRoleNetSerializerInfo);

}
