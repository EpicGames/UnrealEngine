// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/InternalPropertyReplicationState.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationState/InternalReplicationStateDescriptorUtils.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Net/Core/NetBitArray.h"
#include "UObject/UnrealType.h"

namespace UE::Net::Private
{

void InitReplicationStateInternals(uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	// Init internal data
	new (StateBuffer) FReplicationStateHeader();

	// init dirty state tracking
	FNetBitArrayView DirtyStates = GetMemberChangeMask(StateBuffer, Descriptor);
	DirtyStates.ClearAllBits();

	// Init optional conditionals
	if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals))
	{
		FNetBitArrayView ConditionalChangeMask = GetMemberConditionalChangeMask(StateBuffer, Descriptor);
		ConditionalChangeMask.SetAllBits();
	}
}

void CopyPropertyReplicationStateInternals(uint8* RESTRICT DstStateBuffer, uint8* RESTRICT SrcStateBuffer, const FReplicationStateDescriptor* Descriptor, bool bOverwriteChangeMask = true)
{
	FNetBitArrayView DstChangeMask = GetMemberChangeMask(DstStateBuffer, Descriptor);
	FNetBitArrayView SrcChangeMask = GetMemberChangeMask(SrcStateBuffer, Descriptor);

	if (bOverwriteChangeMask)
	{
		DstChangeMask.Copy(SrcChangeMask);
	}
	else
	{
		DstChangeMask.Combine(SrcChangeMask, FNetBitArrayView::OrOp);
	}

	// Copy optional conditional changemask
	if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals))
	{
		FNetBitArrayView DstConditionalChangeMask = GetMemberConditionalChangeMask(DstStateBuffer, Descriptor);
		FNetBitArrayView SrcConditionalChangeMask = GetMemberConditionalChangeMask(SrcStateBuffer, Descriptor);

		if (bOverwriteChangeMask)
		{
			DstConditionalChangeMask.Copy(SrcConditionalChangeMask);
		}
		else
		{
			DstConditionalChangeMask.Combine(SrcConditionalChangeMask, FNetBitArrayView::OrOp);
		}
	}
}

void ConstructPropertyReplicationState(uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	check(IsAligned(StateBuffer, Descriptor->ExternalAlignment));
	
	InitReplicationStateInternals(StateBuffer, Descriptor);

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FProperty** MemberProperties = Descriptor->MemberProperties;
	const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
		
		// InitializeValue operates on the entire static array so make sure not to call it other than for the first element.
		if (MemberPropertyDescriptor.ArrayIndex == 0)
		{
			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const FProperty* Property = MemberProperties[MemberIt];
			Property->InitializeValue(StateBuffer + MemberDescriptor.ExternalMemberOffset);
		}
	}
}

void DestructPropertyReplicationState(uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	check(IsAligned(StateBuffer, Descriptor->ExternalAlignment));
	if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::IsSourceTriviallyDestructible))
	{
		return;
	}

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FProperty** MemberProperties = Descriptor->MemberProperties;
	const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
		
		// DestroyValue operates on the entire static array so make sure not to call it other than for the first element.
		if (MemberPropertyDescriptor.ArrayIndex == 0)
		{
			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const FProperty* Property = MemberProperties[MemberIt];
			Property->DestroyValue(StateBuffer + MemberDescriptor.ExternalMemberOffset);
		}
	}
}

void CopyPropertyReplicationState(uint8* RESTRICT DstStateBuffer, uint8* RESTRICT SrcStateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	check(IsAligned(DstStateBuffer, Descriptor->ExternalAlignment) && IsAligned(SrcStateBuffer, Descriptor->ExternalAlignment));

	// Copy changemasks
	CopyPropertyReplicationStateInternals(DstStateBuffer, SrcStateBuffer, Descriptor);

	// copy statedata
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FProperty** MemberProperties = Descriptor->MemberProperties;
	const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
		if (MemberPropertyDescriptor.ArrayIndex == 0)
		{
			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const FProperty* Property = MemberProperties[MemberIt];
			Property->CopyCompleteValue(DstStateBuffer + MemberDescriptor.ExternalMemberOffset, SrcStateBuffer + MemberDescriptor.ExternalMemberOffset);
		}
	}
}

void CopyDirtyMembers(uint8* RESTRICT DstStateBuffer, uint8* RESTRICT SrcStateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	check(IsAligned(DstStateBuffer, Descriptor->ExternalAlignment) && IsAligned(SrcStateBuffer, Descriptor->ExternalAlignment));

	// Merge changemasks
	const bool bOverwriteChangeMask = false;
	CopyPropertyReplicationStateInternals(DstStateBuffer, SrcStateBuffer, Descriptor, bOverwriteChangeMask);

	// copy dirty members
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FProperty** MemberProperties = Descriptor->MemberProperties;
	const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
	FNetBitArrayView DirtyStates = GetMemberChangeMask(SrcStateBuffer, Descriptor);
	
	const uint32 MemberCount = Descriptor->MemberCount;

	const bool bIsInitState = Descriptor->IsInitState();

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
		const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskInfo = Descriptor->MemberChangeMaskDescriptors[MemberIt];

		const bool bShouldCopyProperty = bIsInitState || DirtyStates.IsAnyBitSet(ChangeMaskInfo.BitOffset, ChangeMaskInfo.BitCount);
		if (bShouldCopyProperty && MemberPropertyDescriptor.ArrayIndex == 0)
		{
			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const FProperty* Property = MemberProperties[MemberIt];
			Property->CopyCompleteValue(DstStateBuffer + MemberDescriptor.ExternalMemberOffset, SrcStateBuffer + MemberDescriptor.ExternalMemberOffset);
		}
	}
}

/**
 * Structs and Arrays need to be handled carefully. Our intermediate representation need 
 * to match the actual native representation perfectly. We are careful not
 * to copy anything that isn't replicated and not consider a struct dirty
 * in case a non-replicated member was modified. Rep notifies that require
 * the previous value as parameter will only get the replicated values
 * updated- non-replicated members will be the same as the default state.
 */
bool InternalCompareMember(const FReplicationStateDescriptor* Descriptor, uint32 MemberIndex, const void* RESTRICT ValueA, const void* RESTRICT ValueB)
{
	const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[MemberIndex];

	if (EnumHasAnyFlags(MemberSerializerDescriptor.Serializer->Traits, ENetSerializerTraits::UseSerializerIsEqual))
	{
		FNetSerializationContext Context;
		FNetIsEqualArgs Args;
		Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		Args.Source0 = NetSerializerValuePointer(ValueA);
		Args.Source1 = NetSerializerValuePointer(ValueB);
		Args.bStateIsQuantized = false;

		return MemberSerializerDescriptor.Serializer->IsEqual(Context, Args);
	}
	else if (IsUsingStructNetSerializer(MemberSerializerDescriptor))
	{
		const FStructNetSerializerConfig* StructConfig = static_cast<const FStructNetSerializerConfig*>(MemberSerializerDescriptor.SerializerConfig);
		const FReplicationStateDescriptor* StructDescriptor = StructConfig->StateDescriptor;
		if (!EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated))
		{
			return InternalCompareStructProperty(StructDescriptor, ValueA, ValueB);
		}
	}
	else if (IsUsingArrayPropertyNetSerializer(MemberSerializerDescriptor))
	{
		const FArrayPropertyNetSerializerConfig* ArrayConfig = static_cast<const FArrayPropertyNetSerializerConfig*>(MemberSerializerDescriptor.SerializerConfig);
		const FReplicationStateDescriptor* StructDescriptor = ArrayConfig->StateDescriptor;
		if (!EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated))
		{
			FNetSerializationContext Context;
			FNetIsEqualArgs Args;
			Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			Args.Source0 = NetSerializerValuePointer(ValueA);
			Args.Source1 = NetSerializerValuePointer(ValueB);
			Args.bStateIsQuantized = false;

			return MemberSerializerDescriptor.Serializer->IsEqual(Context, Args);
		}
	}

	// Default handling
	const FProperty* Property = Descriptor->MemberProperties[MemberIndex];
	return Property->Identical(ValueA, ValueB);
}

bool InternalCompareStructProperty(const FReplicationStateDescriptor* StructDescriptor, const void* RESTRICT ValueA, const void* RESTRICT ValueB)
{
	uint32 FirstStructMemberForCompare = 0U;

	const FReplicationStateMemberDescriptor* MemberDescriptors = StructDescriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = StructDescriptor->MemberSerializerDescriptors;
	const FProperty** MemberProperties = StructDescriptor->MemberProperties;
	const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = StructDescriptor->MemberPropertyDescriptors;

	// For derived structs the first property is a NetSerializer for some super struct. Its property will be null so we can't use standard iteration over properties. Let's use serializer equal for it. 
	if (EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::IsDerivedFromStructWithCustomSerializer))
	{
		FirstStructMemberForCompare = 1U;

		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[0];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[0];

		FNetSerializationContext Context;
		FNetIsEqualArgs IsEqualArgs;
		IsEqualArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		IsEqualArgs.Source0 = NetSerializerValuePointer(ValueA) + MemberDescriptor.ExternalMemberOffset;
		IsEqualArgs.Source1 = NetSerializerValuePointer(ValueB) + MemberDescriptor.ExternalMemberOffset;
		IsEqualArgs.bStateIsQuantized = false;
		if (!MemberSerializerDescriptor.Serializer->IsEqual(Context, IsEqualArgs))
		{
			return false;
		}
	}

	for (uint32 StructMemberIt = FirstStructMemberForCompare, StructMemberEndIt = StructDescriptor->MemberCount; StructMemberIt < StructMemberEndIt; ++StructMemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[StructMemberIt];
		const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[StructMemberIt];
		const FProperty* MemberProperty = MemberProperties[StructMemberIt];
		const SIZE_T MemberOffset = MemberProperty->GetOffset_ForGC() + MemberProperty->GetElementSize()*MemberPropertyDescriptor.ArrayIndex;
		if (!InternalCompareMember(StructDescriptor, StructMemberIt, static_cast<const uint8*>(ValueA) + MemberOffset, static_cast<const uint8*>(ValueB) + MemberOffset))
		{
			return false;
		}
	}

	return true;
}

void InternalApplyStructProperty(const FReplicationStateDescriptor* StructDescriptor, void* RESTRICT Dst, const void* RESTRICT Src)
{
	uint32 FirstStructMemberForApply = 0U;
	if (EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::IsStructWithCustomSerializer | EReplicationStateTraits::IsDerivedFromStructWithCustomSerializer))
	{
		// The first member in a derived struct descriptor is the base struct. We can skip after we've applied the value.
		FirstStructMemberForApply = 1U;

		// If the custom serializer has an Apply method we need to use it.
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = StructDescriptor->MemberSerializerDescriptors[0];
		if (EnumHasAnyFlags(MemberSerializerDescriptor.Serializer->Traits, ENetSerializerTraits::HasApply))
		{
			FNetSerializationContext Context;
			FNetApplyArgs ApplyArgs;
			ApplyArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			ApplyArgs.Source = NetSerializerValuePointer(Src);
			ApplyArgs.Target = NetSerializerValuePointer(Dst);
			MemberSerializerDescriptor.Serializer->Apply(Context, ApplyArgs);
		}
		else
		{
			// For derived structs the base struct is the closest parent with a custom serializer. For root structs it's the actual struct.
			const UScriptStruct* BaseStruct = StructDescriptor->BaseStruct;
			BaseStruct->CopyScriptStruct(Dst, Src, 1);
		}
	}

	const FReplicationStateMemberDescriptor* MemberDescriptors = StructDescriptor->MemberDescriptors;
	const FProperty** MemberProperties = StructDescriptor->MemberProperties;
	const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = StructDescriptor->MemberPropertyDescriptors;
	for (uint32 StructMemberIt = FirstStructMemberForApply, StructMemberEndIt = StructDescriptor->MemberCount; StructMemberIt < StructMemberEndIt; ++StructMemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[StructMemberIt];
		const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[StructMemberIt];
		const FProperty* MemberProperty = MemberProperties[StructMemberIt];
		const SIZE_T MemberOffset = MemberProperty->GetOffset_ForGC() + MemberProperty->GetElementSize()*MemberPropertyDescriptor.ArrayIndex;
		InternalApplyPropertyValue(StructDescriptor, StructMemberIt, static_cast<uint8*>(Dst) + MemberOffset, static_cast<const uint8*>(Src) + MemberOffset);
	}
}

void InternalApplyPropertyValue(const FReplicationStateDescriptor* Descriptor, uint32 MemberIndex, void* RESTRICT Dst, const void* RESTRICT Src)
{
	const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[MemberIndex];
	
	// If member has serializer with Apply then use it.
	if (EnumHasAnyFlags(MemberSerializerDescriptor.Serializer->Traits, ENetSerializerTraits::HasApply))
	{
		FNetSerializationContext Context;
		FNetApplyArgs ApplyArgs;
		ApplyArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		ApplyArgs.Source = NetSerializerValuePointer(Src);
		ApplyArgs.Target = NetSerializerValuePointer(Dst);
		MemberSerializerDescriptor.Serializer->Apply(Context, ApplyArgs);
		return;
	}

	if (IsUsingStructNetSerializer(MemberSerializerDescriptor))
	{
		const FStructNetSerializerConfig* StructConfig = static_cast<const FStructNetSerializerConfig*>(MemberSerializerDescriptor.SerializerConfig);
		const FReplicationStateDescriptor* StructDescriptor = StructConfig->StateDescriptor;
		if (!EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated))
		{
			InternalApplyStructProperty(StructDescriptor, Dst, Src);
			return;
		}
	}
	else if (IsUsingArrayPropertyNetSerializer(MemberSerializerDescriptor))
	{
		const FArrayPropertyNetSerializerConfig* ArrayConfig = static_cast<const FArrayPropertyNetSerializerConfig*>(MemberSerializerDescriptor.SerializerConfig);
		const FReplicationStateDescriptor* ElementStateDescriptor = ArrayConfig->StateDescriptor;
		const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

		const bool bSerializerHasApply = EnumHasAnyFlags(ElementSerializerDescriptor.Serializer->Traits, ENetSerializerTraits::HasApply);
		const bool bIsStructWithNotReplicatedProps = IsUsingStructNetSerializer(ElementSerializerDescriptor) && !EnumHasAnyFlags(ElementStateDescriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated);
		if (bSerializerHasApply || bIsStructWithNotReplicatedProps)
		{
			const FReplicationStateDescriptor* StructDescriptor = static_cast<const FStructNetSerializerConfig*>(ElementSerializerDescriptor.SerializerConfig)->StateDescriptor;
	
			// Need to explicitly iterate over array members to be able to only copy data that we should copy
			FScriptArrayHelper ScriptArrayHelperSrc(ArrayConfig->Property.Get(), reinterpret_cast<const void*>(Src));
			FScriptArrayHelper ScriptArrayHelperDst(ArrayConfig->Property.Get(), reinterpret_cast<void*>(Dst));

			// First we must resize target to match size of source data
			const uint32 ElementCount = ScriptArrayHelperSrc.Num();
			ScriptArrayHelperDst.Resize(ElementCount);
		
			// Iterate over array entries and copy the values
			if (bSerializerHasApply)
			{
				for (uint32 ElementIt = 0, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
				{
					const uint8* ArraySrc = ScriptArrayHelperSrc.GetRawPtr(ElementIt);
					uint8* ArrayDst = ScriptArrayHelperDst.GetRawPtr(ElementIt);

					FNetSerializationContext Context;
					FNetApplyArgs ApplyArgs;
					ApplyArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
					ApplyArgs.Source = NetSerializerValuePointer(ArraySrc);
					ApplyArgs.Target = NetSerializerValuePointer(ArrayDst);
					ElementSerializerDescriptor.Serializer->Apply(Context, ApplyArgs);
				}
			}
			else
			{
				for (uint32 ElementIt = 0, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
				{
					const uint8* ArraySrc = ScriptArrayHelperSrc.GetRawPtr(ElementIt);
					uint8* ArrayDst = ScriptArrayHelperDst.GetRawPtr(ElementIt);

					InternalApplyStructProperty(StructDescriptor, ArrayDst, ArraySrc);
				}
			}

			return;
		}
	}

	// Default handling for all properties except for structs and arrays that have some non-replicated members or a serializer with custom Apply.
	const FProperty* Property = Descriptor->MemberProperties[MemberIndex];
	Property->CopySingleValue(Dst, Src);
}

void InternalCopyStructProperty(const FReplicationStateDescriptor* StructDescriptor, void* RESTRICT Dst, const void* RESTRICT Src)
{
	uint32 FirstStructMemberForCopy = 0U;
	if (EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::IsDerivedFromStructWithCustomSerializer))
	{
		// The base struct is the closest parent with a custom serializer, for which we always copy the entire state.
		const UScriptStruct* BaseStruct = StructDescriptor->BaseStruct;
		BaseStruct->CopyScriptStruct(Dst, Src, 1);

		// The first member in a derived struct descriptor is the base struct. We can skip it now that we've copied the value.
		FirstStructMemberForCopy = 1U;
	}

	const FReplicationStateMemberDescriptor* MemberDescriptors = StructDescriptor->MemberDescriptors;
	const FProperty** MemberProperties = StructDescriptor->MemberProperties;
	const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = StructDescriptor->MemberPropertyDescriptors;

	for (uint32 StructMemberIt = FirstStructMemberForCopy, StructMemberEndIt = StructDescriptor->MemberCount; StructMemberIt < StructMemberEndIt; ++StructMemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[StructMemberIt];
		const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[StructMemberIt];
		const FProperty* MemberProperty = MemberProperties[StructMemberIt];
		const SIZE_T MemberOffset = MemberProperty->GetOffset_ForGC() + MemberProperty->GetElementSize()*MemberPropertyDescriptor.ArrayIndex;
		InternalCopyPropertyValue(StructDescriptor, StructMemberIt, static_cast<uint8*>(Dst) + MemberOffset, static_cast<const uint8*>(Src) + MemberOffset);
	}
}

void InternalCopyPropertyValue(const FReplicationStateDescriptor* Descriptor, uint32 MemberIndex, void* RESTRICT Dst, const void* RESTRICT Src)
{
	const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[MemberIndex];
	if (IsUsingStructNetSerializer(MemberSerializerDescriptor))
	{
		const FStructNetSerializerConfig* StructConfig = static_cast<const FStructNetSerializerConfig*>(MemberSerializerDescriptor.SerializerConfig);
		const FReplicationStateDescriptor* StructDescriptor = StructConfig->StateDescriptor;
		if (!EnumHasAnyFlags(StructDescriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated))
		{
			InternalCopyStructProperty(StructDescriptor, Dst, Src);
			return;
		}
	}
	else if (IsUsingArrayPropertyNetSerializer(MemberSerializerDescriptor))
	{
		const FArrayPropertyNetSerializerConfig* ArrayConfig = static_cast<const FArrayPropertyNetSerializerConfig*>(MemberSerializerDescriptor.SerializerConfig);
		const FReplicationStateDescriptor* ElementStateDescriptor = ArrayConfig->StateDescriptor;
		const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

		if (IsUsingStructNetSerializer(ElementSerializerDescriptor) && !EnumHasAnyFlags(ElementStateDescriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated))
		{
			const FReplicationStateDescriptor* StructDescriptor = static_cast<const FStructNetSerializerConfig*>(ElementSerializerDescriptor.SerializerConfig)->StateDescriptor;
	
			// Need to explicitly iterate over array members to be able to only copy data that we should copy
			FScriptArrayHelper ScriptArrayHelperSrc(ArrayConfig->Property.Get(), reinterpret_cast<const void*>(Src));
			FScriptArrayHelper ScriptArrayHelperDst(ArrayConfig->Property.Get(), reinterpret_cast<void*>(Dst));

			// First we must resize target to match size of source data
			const uint32 ElementCount = ScriptArrayHelperSrc.Num();
			ScriptArrayHelperDst.Resize(ElementCount);
		
			// Iterate over array entries and copy the statedata using internal data
			for (uint32 ElementIt = 0, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
			{
				const uint8*  ArraySrc = ScriptArrayHelperSrc.GetRawPtr(ElementIt);
				uint8* ArrayDst = ScriptArrayHelperDst.GetRawPtr(ElementIt);

				InternalCopyStructProperty(StructDescriptor, ArrayDst, ArraySrc);
			}

			return;
		}
	}

	// Default handling for all properties except for structs and arrays that have some non-replicated members
	const FProperty* Property = Descriptor->MemberProperties[MemberIndex];
	Property->CopySingleValue(Dst, Src);
}

bool InternalCompareAndCopyArrayWithElementChangeMask(const FReplicationStateDescriptor* Descriptor, uint32 MemberIndex, const void* RESTRICT DstArray, const void* RESTRICT SrcArray, UE::Net::FNetBitArrayView& ChangeMask)
{
	bool bArrayIsEqual = true;

	const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskInfo = Descriptor->MemberChangeMaskDescriptors[MemberIndex];

	const FReplicationStateMemberSerializerDescriptor& ArrayDescriptor = Descriptor->MemberSerializerDescriptors[MemberIndex];
	const FArrayPropertyNetSerializerConfig* ArrayConfig = static_cast<const FArrayPropertyNetSerializerConfig*>(ArrayDescriptor.SerializerConfig);

	FScriptArrayHelper SrcScriptArray(ArrayConfig->Property.Get(), SrcArray);
	FScriptArrayHelper DstScriptArray(ArrayConfig->Property.Get(), DstArray);

	// Detect size change and adjust the destination array size as needed and clear bits that don't relate to any elements.
	const uint32 SrcElementCount = SrcScriptArray.Num();
	const uint32 DstElementCount = DstScriptArray.Num();
	if (SrcElementCount != DstElementCount)
	{
		bArrayIsEqual = false;

		DstScriptArray.Resize(SrcElementCount);

		// If array has shrunk then mask off bits in the changemask pertaining to elements that no longer exist. For a growing array we skip compare and set it to dirty in the element loop.
		if (SrcElementCount < DstElementCount)
		{
			if (SrcElementCount + 1U < ChangeMaskInfo.BitCount)
			{
				const uint32 BitOffsetToClear = ChangeMaskInfo.BitOffset + FPropertyReplicationState::TArrayElementChangeMaskBitOffset + SrcElementCount;
				const uint32 BitCountToClear = ChangeMaskInfo.BitCount - 1U - SrcElementCount;
				ChangeMask.ClearBits(BitOffsetToClear, BitCountToClear);
			}
		}
	}

	const FReplicationStateDescriptor* ElementDescriptor = ArrayConfig->StateDescriptor;
	const uint32 ElementChangeMaskBitOffset = ChangeMaskInfo.BitOffset + FPropertyReplicationState::TArrayElementChangeMaskBitOffset;
	for (uint32 ElementIt = 0, ElementEndIt = SrcElementCount; ElementIt < ElementEndIt; ++ElementIt)
	{
		const uint8* SrcElement = SrcScriptArray.GetRawPtr(ElementIt);
		uint8* DstElement = DstScriptArray.GetRawPtr(ElementIt);

		// Compare elements up to the previous element count, i.e. at most DstElementCount. New elements will be considered different and always copied.
		const bool bIsNewElement = ElementIt >= DstElementCount;
		if (bIsNewElement || !InternalCompareStructProperty(ElementDescriptor, DstElement, SrcElement))
		{
			bArrayIsEqual = false;
			const uint32 ElementBitOffset = ElementChangeMaskBitOffset + (ElementIt % FPropertyReplicationState::TArrayElementChangeMaskBits);
			ChangeMask.SetBit(ElementBitOffset);
			InternalCopyStructProperty(ElementDescriptor, DstElement, SrcElement);
		}
	}

	return bArrayIsEqual;
}

}
