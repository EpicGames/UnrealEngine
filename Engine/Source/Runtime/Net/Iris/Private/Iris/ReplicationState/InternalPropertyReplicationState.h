// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"

namespace UE::Net
{
	class FNetBitArrayView;
}

namespace UE::Net::Private
{

/** Construct the external state described by the descriptor in the given Buffer */
IRISCORE_API void ConstructPropertyReplicationState(uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor);

/** Destruct the external state described by the descriptor in the given Buffer */
IRISCORE_API void DestructPropertyReplicationState(uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor);

/** Copy external state in SrcStateBuffer to external state DstStateBuffer including changemask */
IRISCORE_API void CopyPropertyReplicationState(uint8* RESTRICT DstStateBuffer, uint8* RESTRICT SrcStateBuffer, const FReplicationStateDescriptor* Descriptor);

/** Copy only properties from external state in SrcStateBuffer to external state DstStateBufferif dirty, destination changemask will also be updated */
IRISCORE_API void CopyDirtyMembers(uint8* RESTRICT DstStateBuffer, uint8* RESTRICT SrcStateBuffer, const FReplicationStateDescriptor* Descriptor);

/** Copy property value to target. If the property is fully replicated we use the properties copy function, otherwise we do a per member copy of all replicated data. Members with NetSerializers with an Apply() function will call that. */
IRISCORE_API void InternalApplyPropertyValue(const FReplicationStateDescriptor* Descriptor, uint32 MemberIndex, void* RESTRICT Dst, const void* RESTRICT Src);

/** Copy struct members to target using InternalApplyPropertyValue */
IRISCORE_API void InternalApplyStructProperty(const FReplicationStateDescriptor* StructDescriptor, void* RESTRICT Dst, const void* RESTRICT Src);

/** Copy property value, if the property is fully replicated we use the properties copy function, otherwise we do a per member copy of all replicated data */
IRISCORE_API void InternalCopyPropertyValue(const FReplicationStateDescriptor* Descriptor, uint32 MemberIndex, void* RESTRICT Dst, const void* RESTRICT Src);

/** Copy struct members using InternalCopyPropertyValue */
IRISCORE_API void InternalCopyStructProperty(const FReplicationStateDescriptor* StructDescriptor, void* RESTRICT Dst, const void* RESTRICT Src);

/** Compare struct members using InternalCompareMember, this function will only compare replicated members of the struct */
IRISCORE_API bool InternalCompareStructProperty(const FReplicationStateDescriptor* StructDescriptor, const void* RESTRICT ValueA, const void* RESTRICT ValueB);

/** CompareMembers, if data is not fully replicated we will use per member compare for structs and arrays */
IRISCORE_API bool InternalCompareMember(const FReplicationStateDescriptor* Descriptor, uint32 MemberIndex, const void* RESTRICT ValueA, const void* RESTRICT ValueB);

/**
 * Compare array elements and mark dirty indices in the changemask. The resulting DstArray will be identical to the SrcArray after calling this function.
 * @param ChangeMask needs to be big enough for the entire descriptor. The member's change mask descriptor will be modified for dirty array indices. The bit which indicates the array is dirty is left unmodified.
 * @return false if the DstArray was modified in anyway, including size changes, and true if the arrays were already identical.
 */
IRISCORE_API bool InternalCompareAndCopyArrayWithElementChangeMask(const FReplicationStateDescriptor* Descriptor, uint32 MemberIndex, const void* RESTRICT DstArray, const void* RESTRICT SrcArray, UE::Net::FNetBitArrayView& ChangeMask);

}
