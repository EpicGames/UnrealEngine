// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/Serialization/InternalNetSerializerUtils.h"

namespace UE::Net
{

bool IsUsingStructNetSerializer(const FReplicationStateMemberSerializerDescriptor& MemberDescriptor);
bool IsUsingArrayPropertyNetSerializer(const FReplicationStateMemberSerializerDescriptor& MemberDescriptor);

}

namespace UE::Net
{

inline bool IsUsingStructNetSerializer(const FReplicationStateMemberSerializerDescriptor& MemberDescriptor)
{
	return IsStructNetSerializer(MemberDescriptor.Serializer);
}

inline bool IsUsingArrayPropertyNetSerializer(const FReplicationStateMemberSerializerDescriptor& MemberDescriptor)
{
	return IsArrayPropertyNetSerializer(MemberDescriptor.Serializer);
}

}
