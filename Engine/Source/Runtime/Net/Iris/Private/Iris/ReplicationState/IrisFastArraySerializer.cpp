// Copyright Epic Games, Inc. All Rights Reserved.
#include "Iris/ReplicationState/IrisFastArraySerializer.h"
#include "Iris/ReplicationState/Private/IrisFastArraySerializerInternal.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Net/Core/NetBitArray.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IrisFastArraySerializer)

FIrisFastArraySerializer::FIrisFastArraySerializer()
: FFastArraySerializer()
{
	static_assert(offsetof(FIrisFastArraySerializer, ChangeMaskStorage) - sizeof(UE::Net::FReplicationStateHeader) == offsetof(FIrisFastArraySerializer, ReplicationStateHeader), "GetReplicationStateHeader is no longer compatible with FIrisFastArraySerializer");
	InitChangeMask();
}

FIrisFastArraySerializer::FIrisFastArraySerializer(const FIrisFastArraySerializer& Other)
: FFastArraySerializer(Other)
{
	InitChangeMask();
}

FIrisFastArraySerializer::FIrisFastArraySerializer(const FIrisFastArraySerializer&& Other)
: FFastArraySerializer(Other)
{
	InitChangeMask();
}

FIrisFastArraySerializer& FIrisFastArraySerializer::operator=(const FIrisFastArraySerializer& Other)
{
	if (this != &Other)
	{
		Super::operator=(Other);
		// Currently we are pessimistic and marks everything as dirty
		if (ReplicationStateHeader.IsBound())
		{
			InternalMarkAllItemsChanged();
		}
	}

	return *this;
}

FIrisFastArraySerializer& FIrisFastArraySerializer::operator=(FIrisFastArraySerializer&& Other)
{
	if (this != &Other)
	{
		Super::operator=(Other);
		// Currently we are pessimistic and marks everything as dirty
		if (ReplicationStateHeader.IsBound())
		{
			InternalMarkAllItemsChanged();
		}
	}
	return *this;
}

void FIrisFastArraySerializer::InitChangeMask()
{
	// Init changemask and conditional changemask
	for (uint32& Storage : ChangeMaskStorage)
	{
		const SIZE_T Index = &Storage - &ChangeMaskStorage[0];
		if (Index < ConditionalChangeMaskStorageIndex)
		{
			Storage = 0U;
		}
		else
		{
			Storage = 0xFFFFFFFFU;
		}
	}
}

void FIrisFastArraySerializer::InternalMarkAllItemsChanged()
{
	checkSlow(ReplicationStateHeader.IsBound());
	UE::Net::Private::FIrisFastArraySerializerPrivateAccessor::MarkAllArrayItemsDirty(*this);
}

void FIrisFastArraySerializer::InternalMarkArrayAsDirty()
{
	checkSlow(ReplicationStateHeader.IsBound());
	UE::Net::Private::FIrisFastArraySerializerPrivateAccessor::MarkArrayDirty(*this);
}

void FIrisFastArraySerializer::InternalMarkItemChanged(int32 ItemIdx)
{
	checkSlow(ReplicationStateHeader.IsBound());
	UE::Net::Private::FIrisFastArraySerializerPrivateAccessor::MarkArrayItemDirty(*this, ItemIdx);
}

namespace UE::Net::Private
{

FNetBitArrayView FIrisFastArraySerializerPrivateAccessor::GetChangeMask(FIrisFastArraySerializer& Array)
{
	return MakeNetBitArrayView(&Array.ChangeMaskStorage[FIrisFastArraySerializer::MemberChangeMaskStorageIndex], FIrisFastArraySerializer::IrisFastArrayChangeMaskBits + 1U);
}

FNetBitArrayView FIrisFastArraySerializerPrivateAccessor::GetConditionalChangeMask(FIrisFastArraySerializer& Array)
{
	return MakeNetBitArrayView(&Array.ChangeMaskStorage[FIrisFastArraySerializer::ConditionalChangeMaskStorageIndex], FIrisFastArraySerializer::IrisFastArrayChangeMaskBits + 1U);
}

void FIrisFastArraySerializerPrivateAccessor::MarkAllArrayItemsDirty(FIrisFastArraySerializer& Array, uint32 StartingIndex)
{
	checkSlow(Array.ReplicationStateHeader.IsBound());

	FNetBitArrayView MemberChangeMask = UE::Net::Private::FIrisFastArraySerializerPrivateAccessor::GetChangeMask(Array);
	if (!MemberChangeMask.GetBit(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex))
	{
		MarkNetObjectStateHeaderDirty(Array.ReplicationStateHeader);
	}
	if (StartingIndex == 0)
	{
		MemberChangeMask.SetAllBits();
	}
	else
	{
		MemberChangeMask.SetBit(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex);
		MemberChangeMask.SetBits(FIrisFastArraySerializer::IrisFastArrayChangeMaskBitOffset + StartingIndex, FIrisFastArraySerializer::IrisFastArrayChangeMaskBits - StartingIndex);
	}
}

void FIrisFastArraySerializerPrivateAccessor::MarkArrayDirty(FIrisFastArraySerializer& Array)
{
	checkSlow(Array.ReplicationStateHeader.IsBound());

	FNetBitArrayView MemberChangeMask = UE::Net::Private::FIrisFastArraySerializerPrivateAccessor::GetChangeMask(Array);

	// Dirty object unless already dirty, we only use the array bit for this purpose
	if (!MemberChangeMask.GetBit(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex))
	{
		MemberChangeMask.SetBit(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex);
		MarkNetObjectStateHeaderDirty(Array.ReplicationStateHeader);
	}
}

void FIrisFastArraySerializerPrivateAccessor::MarkArrayItemDirty(FIrisFastArraySerializer& Array, int32 ItemIdx)
{
	checkSlow(Array.ReplicationStateHeader.IsBound());

	// Mark changemask dirty for this item
	// We are using a modulo scheme for dirtiness
	FNetBitArrayView MemberChangeMask = UE::Net::Private::FIrisFastArraySerializerPrivateAccessor::GetChangeMask(Array);
	MemberChangeMask.SetBit((ItemIdx % FIrisFastArraySerializer::IrisFastArrayChangeMaskBits) + FIrisFastArraySerializer::IrisFastArrayChangeMaskBitOffset);

	// Dirty object unless already dirty, we only use the array bit for this purpose
	if (!MemberChangeMask.GetBit(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex))
	{
		MemberChangeMask.SetBit(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex);
		MarkNetObjectStateHeaderDirty(Array.ReplicationStateHeader);
	}
}

}
