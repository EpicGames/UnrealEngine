// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/EnumPropertyNetSerializerInfo.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Templates/Sorting.h"
#include "UObject/UnrealType.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/InternalNetSerializers.h"

namespace UE::Net::Private
{
	struct FNopNetSerializerInfo : public FPropertyNetSerializerInfo
	{
		virtual const FFieldClass* GetPropertyTypeClass() const { return nullptr; }
		virtual bool IsSupported(const FProperty* Property) const { return true; }
		virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const { return &UE_NET_GET_SERIALIZER(FNopNetSerializer); }
		virtual bool CanUseDefaultConfig(const FProperty* Property) const { return true; }
		virtual const FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const { return nullptr; }
	} static const NopNetSerializerInfo;

}

namespace UE::Net
{

FPropertyNetSerializerInfoRegistry::FNetSerializerInfoRegistry FPropertyNetSerializerInfoRegistry::Registry;
bool FPropertyNetSerializerInfoRegistry::bRegistryIsDirty = false;

void FPropertyNetSerializerInfoRegistry::Register(const FPropertyNetSerializerInfo* Info)
{
	check(Info);

	const FFieldClass* PropertyClass = Info->GetPropertyTypeClass();

	check(PropertyClass);

	Registry.AddUnique(MakeTuple<>(PropertyClass, Info));

	bRegistryIsDirty = true;
}

void FPropertyNetSerializerInfoRegistry::Unregister(const FPropertyNetSerializerInfo* Info)
{
	Registry.RemoveAll([Info](const FRegistryEntry& Entry) { return Entry.Get<1>() == Info; });
	bRegistryIsDirty = true;
}

void FPropertyNetSerializerInfoRegistry::Reset()
{
	Registry.Empty();
	bRegistryIsDirty = false;
}

void FPropertyNetSerializerInfoRegistry::Freeze()
{
	if (bRegistryIsDirty)
	{
		// Sort by StaticClass pointer
		Registry.Sort([](const FRegistryEntry& A, const FRegistryEntry& B) { return A.Get<0>() < B.Get<0>();} );
		bRegistryIsDirty = false;
	}
}

/** Default struct Serializer info */
struct FDefaultStructPropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{		
	virtual const FFieldClass* GetPropertyTypeClass() const override { return FStructProperty::StaticClass(); }
	virtual bool IsSupported(const FProperty* Property) const override { return true; }
	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override { return &UE_NET_GET_SERIALIZER(FStructNetSerializer); }
	virtual bool CanUseDefaultConfig(const FProperty* Property) const override { return false; }
	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override { return nullptr; }
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FDefaultStructPropertyNetSerializerInfo);

/** Default dynamic array Serializer info */
struct FDefaultArrayPropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{		
	virtual const FFieldClass* GetPropertyTypeClass() const override { return FArrayProperty::StaticClass(); }
	virtual bool IsSupported(const FProperty* Property) const override { return FPropertyNetSerializerInfoRegistry::FindSerializerInfo(CastFieldChecked<FArrayProperty>(Property)->Inner) != nullptr; }
	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override { return &UE_NET_GET_SERIALIZER(FArrayPropertyNetSerializer); }
	virtual bool CanUseDefaultConfig(const FProperty* Property) const override { return false; }
	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override { return nullptr; }
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FDefaultArrayPropertyNetSerializerInfo);

struct FDefaultOptionalPropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const override { return FOptionalProperty::StaticClass(); }
	virtual bool IsSupported(const FProperty* Property) const override { return FPropertyNetSerializerInfoRegistry::FindSerializerInfo(CastFieldChecked<FOptionalProperty>(Property)->GetValueProperty()) != nullptr; }
	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override { return &UE_NET_GET_SERIALIZER(FOptionalPropertyNetSerializer); }
	virtual bool CanUseDefaultConfig(const FProperty* Property) const override { return false; }
	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override { return nullptr; }
};
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FDefaultOptionalPropertyNetSerializerInfo);

/** Last resort for properties that have no specialized serializer. An example is TextProperty. */
FLastResortPropertyNetSerializerInfo::FLastResortPropertyNetSerializerInfo(const FLastResortPropertyNetSerializerUserOptions& UserOptions)
: FPropertyNetSerializerInfo()
, CreateAndRegisterReplicationFragmentFunction(UserOptions.CreateAndRegisterReplicationFragmentFunction)
, MaxQuantizedSizeBits(UserOptions.MaxQuantizedSizeBits)
, bExcludeFromDefaultStateHash(UserOptions.bExcludeFromDefaultStateHash)
{
}

const FFieldClass* FLastResortPropertyNetSerializerInfo::GetPropertyTypeClass() const
{
	return FProperty::StaticClass();
}

bool FLastResortPropertyNetSerializerInfo::IsSupported(const FProperty* Property) const
{ 
	return true; 
}

const FNetSerializer* FLastResortPropertyNetSerializerInfo::GetNetSerializer(const FProperty* Property) const
{
	return &UE_NET_GET_SERIALIZER(FLastResortPropertyNetSerializer);
}

bool FLastResortPropertyNetSerializerInfo::CanUseDefaultConfig(const FProperty* Property) const
{
	return false; 
}

FNetSerializerConfig* FLastResortPropertyNetSerializerInfo::BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const
{
	FLastResortPropertyNetSerializerConfig* Config = new (NetSerializerConfigBuffer) FLastResortPropertyNetSerializerConfig();
	InitLastResortPropertyNetSerializerConfigFromProperty(*Config, Property);
	Config->bExcludeFromDefaultStateHash = bExcludeFromDefaultStateHash;
	Config->MaxQuantizedSizeBits = MaxQuantizedSizeBits > 0U ? MaxQuantizedSizeBits : FLastResortPropertyNetSerializerConfig::DefaultMaxQuantizedSizeBits;

	return Config;
}
UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FLastResortPropertyNetSerializerInfo);

const FFieldClass* FNamedStructLastResortPropertyNetSerializerInfo::GetPropertyTypeClass() const
{
	return FStructProperty::StaticClass();
}

bool FNamedStructLastResortPropertyNetSerializerInfo::IsSupported(const FProperty* Property) const
{
	const FStructProperty* StructProp = CastFieldChecked<const FStructProperty>(Property);
	return IsSupportedStruct(StructProp->Struct->GetFName());
}

const FPropertyNetSerializerInfo* FPropertyNetSerializerInfoRegistry::FindSerializerInfo(const FProperty* Property)
{
	// Make sure that the registry is sorted to avoid doing IsA multiple times for the same property type
	Freeze();

	// first find Index containing a info matching the type of the property
	int32 Index = Registry.IndexOfByPredicate([Property](const FRegistryEntry& Element) { return Property->IsA(Element.Get<0>());} );
	
	if (Index != INDEX_NONE)
	{
		// Check if we got a match
		// as we allow multiple infos for the same class type we need to check if the info supports the Property
		const FFieldClass* ClassType = Registry[Index].Get<0>();
		for (; Index < Registry.Num() && Registry[Index].Get<0>() == ClassType; ++Index)
		{
			const FPropertyNetSerializerInfo* Info = Registry[Index].Get<1>();
			if (Info->IsSupported(Property))
			{
				return Info;
			}
		}
	}
		
	// We did not find a match, handle default cases.
	
	// Check if this is a struct and fall back on default.
	if (Property->IsA(FStructProperty::StaticClass()))
	{
		return &GetPropertyNetSerializerInfo_FDefaultStructPropertyNetSerializerInfo();
	}

	// Check if this is a dynamic array and the serializer supports it.
	if (Property->IsA(FArrayProperty::StaticClass()))
	{
		const FPropertyNetSerializerInfo& Info = GetPropertyNetSerializerInfo_FDefaultArrayPropertyNetSerializerInfo();
		if (Info.IsSupported(Property))
		{
			return &Info;
		}
	}

	// Special handling for optional properties required
	if (Property->IsA(FOptionalProperty::StaticClass()))
	{
		const FPropertyNetSerializerInfo& Info = GetPropertyNetSerializerInfo_FDefaultOptionalPropertyNetSerializerInfo();
		if (Info.IsSupported(Property))
		{
			return &Info;
		}
	}

	// Last resort!
	return &GetPropertyNetSerializerInfo_FLastResortPropertyNetSerializerInfo();
}

const FPropertyNetSerializerInfo* FPropertyNetSerializerInfoRegistry::FindStructSerializerInfo(const FName Name)
{
	// Make sure that the registry is sorted to avoid doing IsA multiple times for the same property type
	Freeze();

	const FFieldClass* ClassType = FStructProperty::StaticClass();

	// first find Index containing a info matching the type of the property
	int32 Index = Registry.IndexOfByPredicate([ClassType](const FRegistryEntry& Element) { return Element.Get<0>() == ClassType; } );
	
	if (Index != INDEX_NONE)
	{
		// Check if we got a match on the name
		// as we allow multiple infos for the same class type we need to check if the info supports the Property
		for (; Index < Registry.Num() && Registry[Index].Get<0>() == ClassType; ++Index)
		{
			const FPropertyNetSerializerInfo* Info = Registry[Index].Get<1>();
			if (Info->IsSupportedStruct(Name))
			{
				return Info;
			}
		}
	}
		
	return nullptr;
}

const FPropertyNetSerializerInfo* FPropertyNetSerializerInfoRegistry::GetNopNetSerializerInfo()
{
	return &Private::NopNetSerializerInfo;
}

void ValidateForwardingNetSerializerTraits(const FNetSerializer* Serializer, EReplicationStateTraits UsedReplicationStateTraits)
{
	const ENetSerializerTraits SerializerTraits = Serializer->Traits;
	if (EnumHasAnyFlags(UsedReplicationStateTraits, EReplicationStateTraits::HasDynamicState) && !EnumHasAnyFlags(SerializerTraits, ENetSerializerTraits::HasDynamicState))
	{
		LowLevelFatalError(TEXT("FNetSerializer: %s is using serializer(s) that HasDynamicState without setting trait: static constexpr bool bHasDynamicState = true; in the serializer declaration."), Serializer->Name);
	}
	if (EnumHasAnyFlags(UsedReplicationStateTraits, EReplicationStateTraits::HasObjectReference) && !EnumHasAnyFlags(SerializerTraits, ENetSerializerTraits::HasCustomNetReference))
	{
		LowLevelFatalError(TEXT("FNetSerializer: %s is using serializer(s) that has trait HasCustomNetReference without setting trait: static constexpr bool bHasCustomNetReference = true; in the serializer declaration."), Serializer->Name);
	}
	if (EnumHasAnyFlags(UsedReplicationStateTraits, EReplicationStateTraits::HasConnectionSpecificSerialization) && !EnumHasAnyFlags(SerializerTraits, ENetSerializerTraits::HasConnectionSpecificSerialization))
	{
		LowLevelFatalError(TEXT("FNetSerializer: %s is using serializer(s) that has HasConnectionSpecificSerialization without setting trait: static constexpr bool bHasConnectionSpecificSerialization = true; in the serializer declaration."), Serializer->Name);
	}
}

}
