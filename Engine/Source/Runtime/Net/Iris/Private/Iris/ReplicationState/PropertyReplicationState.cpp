// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/Core/IrisDebugging.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/ReplicationState/InternalPropertyReplicationState.h"
#include "Iris/ReplicationState/InternalReplicationStateDescriptorUtils.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "CoreTypes.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "Containers/StringFwd.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Iris/IrisConfigInternal.h"

DEFINE_LOG_CATEGORY_STATIC(LogIrisRepNotify, Warning, All);

namespace UE::Net
{

FPropertyReplicationState::FPropertyReplicationState(const FReplicationStateDescriptor* Descriptor, uint8* InStateBuffer)
: ReplicationStateDescriptor(Descriptor)
, StateBuffer(nullptr)
, bOwnState(0u)
{
	check(InStateBuffer);
	InjectState(Descriptor, InStateBuffer);
}

FPropertyReplicationState::FPropertyReplicationState(const FReplicationStateDescriptor* Descriptor)
: ReplicationStateDescriptor(Descriptor)
, StateBuffer(nullptr)
, bOwnState(1u)
{
	ConstructStateInternal();
}

/** Copy constructor, will not copy internal data */
FPropertyReplicationState::FPropertyReplicationState(const FPropertyReplicationState& Other)
: ReplicationStateDescriptor(Other.ReplicationStateDescriptor)
, StateBuffer(nullptr)
, bOwnState(1u)
{
	ConstructStateInternal();

	// Invoke assignment operator
	*this = Other;
}

FPropertyReplicationState& 
FPropertyReplicationState::operator=(const FPropertyReplicationState& Other)
{
	check(this != &Other && IsValid());
	check(ReplicationStateDescriptor.GetReference() == Other.ReplicationStateDescriptor.GetReference());

	if (!Private::IsReplicationStateBound(StateBuffer, ReplicationStateDescriptor.GetReference()))
	{
		Private::CopyPropertyReplicationState(StateBuffer, Other.StateBuffer, ReplicationStateDescriptor.GetReference());
	}
	else
	{
		Set(Other);
	}

	return *this;
}

FPropertyReplicationState::~FPropertyReplicationState()
{
	DestructStateInternal();
}

bool FPropertyReplicationState::IsValid() const
{
	return StateBuffer && ReplicationStateDescriptor;
}

bool FPropertyReplicationState::IsInitState() const
{
	return IsValid() && ReplicationStateDescriptor->IsInitState();
}

bool FPropertyReplicationState::IsDirtyForPolling() const
{
	return IsDirty() || Private::GetMemberPollMask(StateBuffer, ReplicationStateDescriptor).IsAnyBitSet();
}

bool FPropertyReplicationState::IsDirty() const
{
	if (IsValid())
	{
		const FReplicationStateHeader& Header = Private::GetReplicationStateHeader(StateBuffer, ReplicationStateDescriptor);
		return Private::FReplicationStateHeaderAccessor::GetIsStateDirty(Header) || Private::FReplicationStateHeaderAccessor::GetIsInitStateDirty(Header);
	}

	return false;
}

void FPropertyReplicationState::ConstructStateInternal()
{
	// allocate memory
	check(StateBuffer == nullptr);

	StateBuffer = (uint8*)FMemory::Malloc(ReplicationStateDescriptor->ExternalSize, ReplicationStateDescriptor->ExternalAlignment);
	// There can be properties in here that assume the memory is cleared and do nothing in InitializeValue
	FMemory::Memzero(StateBuffer, ReplicationStateDescriptor->ExternalSize);

	// Construct the state
	Private::ConstructPropertyReplicationState(StateBuffer, ReplicationStateDescriptor);
}

void FPropertyReplicationState::DestructStateInternal()
{
	// Destruct state and free memory if we own it
	if (bOwnState && StateBuffer)
	{
		Private::DestructPropertyReplicationState(StateBuffer, ReplicationStateDescriptor);
		FMemory::Free(StateBuffer);
		StateBuffer = nullptr;
	}
}

void FPropertyReplicationState::InjectState(const FReplicationStateDescriptor* Descriptor, uint8* InStateBuffer)
{
	// allocate memory
	check(StateBuffer == nullptr);
	check(InStateBuffer != nullptr);

	ReplicationStateDescriptor = Descriptor;
	StateBuffer = InStateBuffer;
}

void FPropertyReplicationState::Set(const FPropertyReplicationState& Other)
{
	if (IsValid() && this != &Other)
	{
		const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;

		const uint8* SrcStateBuffer = Other.StateBuffer;
		for (uint32 MemberIt = 0, MemberEndIt = Descriptor->MemberCount; MemberIt < MemberEndIt; ++MemberIt)
		{
			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const uint8* SrcValue = SrcStateBuffer + MemberDescriptor.ExternalMemberOffset;
			SetPropertyValue(MemberIt, SrcValue);
		}
	}
}

void FPropertyReplicationState::PollPropertyValue(uint32 Index, const void* SrcValue)
{
	const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
	void* DstValue = StateBuffer + Descriptor->MemberDescriptors[Index].ExternalMemberOffset;

	// Special handling for top level arrays with changemask bits for elements
	const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[Index];
	if (IsUsingArrayPropertyNetSerializer(MemberSerializerDescriptor))
	{
		const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskInfo = Descriptor->MemberChangeMaskDescriptors[Index];
		if (!IsInitState() && IsCustomConditionEnabled(Index) && ChangeMaskInfo.BitCount > 1U)
		{
			FNetBitArrayView MemberChangeMask = Private::GetMemberChangeMask(StateBuffer, Descriptor);

			const bool bArraysAreEqual = Private::InternalCompareAndCopyArrayWithElementChangeMask(Descriptor, Index, DstValue, SrcValue, MemberChangeMask);
			if (!bArraysAreEqual && !MemberChangeMask.GetBit(ChangeMaskInfo.BitOffset + TArrayPropertyChangeMaskBitIndex))
			{
				MarkArrayDirty(Index);
			}

			return;
		}
	}

	if (!IsDirty(Index) && IsCustomConditionEnabled(Index) && !Private::InternalCompareMember(Descriptor, Index, SrcValue, DstValue))
	{
		MarkDirty(Index);
	}

	Private::InternalCopyPropertyValue(Descriptor, Index, DstValue, SrcValue);
}

void FPropertyReplicationState::SetPropertyValue(uint32 Index, const void* SrcValue)
{
	// We can perform the same operation as normal polling of the state does.
	PollPropertyValue(Index, SrcValue);
}

void FPropertyReplicationState::PushPropertyValue(uint32 Index, void* DstValue) const
{
	void* SrcValue = StateBuffer + ReplicationStateDescriptor->MemberDescriptors[Index].ExternalMemberOffset;
	Private::InternalApplyPropertyValue(ReplicationStateDescriptor, Index, DstValue, SrcValue);
}

void FPropertyReplicationState::GetPropertyValue(uint32 Index, void* DstValue) const
{
	void* SrcValue = StateBuffer + ReplicationStateDescriptor->MemberDescriptors[Index].ExternalMemberOffset;
	Private::InternalCopyPropertyValue(ReplicationStateDescriptor, Index, DstValue, SrcValue);
}

void FPropertyReplicationState::MarkDirty(uint32 Index)
{
	const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
	FReplicationStateHeader& Header = Private::GetReplicationStateHeader(StateBuffer, Descriptor);

	if (IsInitState())
	{
		if (!Private::FReplicationStateHeaderAccessor::GetIsInitStateDirty(Header))
		{
			Private::FReplicationStateHeaderAccessor::MarkInitStateDirty(Header);
			if (Header.IsBound())
			{
				MarkNetObjectStateHeaderDirty(Header);
			}
		}
	}
	else
	{
		FNetBitArrayView MemberChangeMask = Private::GetMemberChangeMask(StateBuffer, Descriptor);
		const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskInfo = Descriptor->MemberChangeMaskDescriptors[Index];
		Private::MarkDirty(Header, MemberChangeMask, ChangeMaskInfo);
	}
}

bool FPropertyReplicationState::IsDirtyForPolling(uint32 Index) const
{
	return IsDirty(Index) || Private::GetMemberPollMask(StateBuffer, ReplicationStateDescriptor).GetBit(Index);
}

bool FPropertyReplicationState::IsDirty(uint32 Index) const
{
	const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
	if (IsInitState())
	{
		const FReplicationStateHeader& Header = Private::GetReplicationStateHeader(StateBuffer, Descriptor);
		return Private::FReplicationStateHeaderAccessor::GetIsInitStateDirty(Header);
	}
	else
	{
		FNetBitArrayView MemberChangeMask = Private::GetMemberChangeMask(StateBuffer, Descriptor);
		const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskInfo = Descriptor->MemberChangeMaskDescriptors[Index];

		return MemberChangeMask.IsAnyBitSet(ChangeMaskInfo.BitOffset, ChangeMaskInfo.BitCount);
	}
}

void FPropertyReplicationState::MarkArrayDirty(uint32 Index)
{
	const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
	FReplicationStateHeader& Header = Private::GetReplicationStateHeader(StateBuffer, Descriptor);

	if (IsInitState())
	{
		if (!Private::FReplicationStateHeaderAccessor::GetIsInitStateDirty(Header))
		{
			Private::FReplicationStateHeaderAccessor::MarkInitStateDirty(Header);
			if (Header.IsBound())
			{
				MarkNetObjectStateHeaderDirty(Header);
			}
		}
	}
	else
	{
		FNetBitArrayView MemberChangeMask = Private::GetMemberChangeMask(StateBuffer, Descriptor);
		FReplicationStateMemberChangeMaskDescriptor ChangeMaskInfo = Descriptor->MemberChangeMaskDescriptors[Index];
		ChangeMaskInfo.BitOffset += TArrayPropertyChangeMaskBitIndex;
		ChangeMaskInfo.BitCount = 1;
		Private::MarkDirty(Header, MemberChangeMask, ChangeMaskInfo);
	}
}

bool FPropertyReplicationState::PollPropertyReplicationState(const void* RESTRICT SrcStateData)
{
	if (IsValid())
	{
		const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
		const uint8* SrcBuffer = reinterpret_cast<const uint8*>(SrcStateData);

#if UE_NET_WITH_VERBOSE_POLL_PROFILING
		IRIS_PROFILER_PROTOCOL_NAME(ReplicationStateDescriptor->DebugName->Name);
#endif

		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FProperty** MemberProperties = Descriptor->MemberProperties;
		const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
		const uint32 MemberCount = Descriptor->MemberCount;

		for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
		{
			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
			const FProperty* Property = MemberProperties[MemberIt];

			//$TODO: make special version to avoid unnecessary overhead.
			PollPropertyValue(MemberIt, SrcBuffer + Property->GetOffset_ForGC() + Property->GetElementSize()*MemberPropertyDescriptor.ArrayIndex);
		}
	}

	return IsDirty();
}

bool FPropertyReplicationState::PollPushBasedPropertyReplicationState(const void* RESTRICT SrcStateData, const FPollParameters& PollParameters)
{
	if (IsValid())
	{
		const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
		const uint8* SrcBuffer = reinterpret_cast<const uint8*>(SrcStateData);

#if UE_NET_WITH_VERBOSE_POLL_PROFILING
		IRIS_PROFILER_SCOPE_TEXT_CONDITIONAL(Descriptor->DebugName->Name, PollParameters.bEnableVerboseProfiling);
#endif

		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FProperty** MemberProperties = Descriptor->MemberProperties;
		const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
		const uint32 MemberCount = Descriptor->MemberCount;
		const FReplicationStateMemberTraitsDescriptor* MemberTraits = Descriptor->MemberTraitsDescriptors;
		for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
		{
			const bool bForcePoll = (PollParameters.bForceRefreshObjectReferences && EnumHasAnyFlags(MemberTraits[MemberIt].Traits, EReplicationStateMemberTraits::HasObjectReference)) || !EnumHasAnyFlags(MemberTraits[MemberIt].Traits, EReplicationStateMemberTraits::HasPushBasedDirtiness);
			if (bForcePoll || IsDirtyForPolling(MemberIt))
			{
#if UE_NET_WITH_VERBOSE_POLL_PROFILING
				IRIS_PROFILER_SCOPE_TEXT_CONDITIONAL(EnumHasAnyFlags(MemberTraits[MemberIt].Traits, EReplicationStateMemberTraits::HasPushBasedDirtiness) ? TEXT("PushBased") : TEXT("AlwaysPolled"), PollParameters.bEnableVerboseProfiling);
				IRIS_PROFILER_SCOPE_TEXT_CONDITIONAL(Descriptor->MemberDebugDescriptors[MemberIt].DebugName->Name, PollParameters.bEnableVerboseProfiling);
#endif
				const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
				const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
				const FProperty* Property = MemberProperties[MemberIt];

				// $TODO: make special version to avoid unnecessary overhead related to looking up same things for every single member.
				PollPropertyValue(MemberIt, SrcBuffer + Property->GetOffset_ForGC() + Property->GetElementSize()*MemberPropertyDescriptor.ArrayIndex);
			}
		}		
	}

	return IsDirty();
}

bool FPropertyReplicationState::StoreCurrentPropertyReplicationStateForRepNotifies(const void* RESTRICT SrcStateData, const FPropertyReplicationState* NewStateToBeApplied)
{
	if (IsValid())
	{
		const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
		const uint8* SrcBuffer = reinterpret_cast<const uint8*>(SrcStateData);

		IRIS_PROFILER_PROTOCOL_NAME(ReplicationStateDescriptor->DebugName->Name);

		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FProperty** MemberProperties = Descriptor->MemberProperties;
		const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
		const uint32 MemberCount = Descriptor->MemberCount;

		// Copy all if this is a state with no changemask or if NewStateToBeApplied is not set
		const bool bCopyAll = IsInitState() || NewStateToBeApplied == nullptr;
		for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
		{
			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];

			if (MemberPropertyDescriptor.RepNotifyFunction && (bCopyAll || NewStateToBeApplied->IsDirty(MemberIt)))
			{
				const FProperty* Property = MemberProperties[MemberIt];

				void* DstValue = StateBuffer + Descriptor->MemberDescriptors[MemberIt].ExternalMemberOffset;
				const void* SrcValue = SrcBuffer + Property->GetOffset_ForGC() + Property->GetElementSize()*MemberPropertyDescriptor.ArrayIndex;

				Private::InternalCopyPropertyValue(Descriptor, MemberIt, DstValue, SrcBuffer + Property->GetOffset_ForGC() + Property->GetElementSize()*MemberPropertyDescriptor.ArrayIndex);
			}
		}
	}

	return IsDirty();
}

void FPropertyReplicationState::PushPropertyReplicationState(const UObject* Owner, void* RESTRICT DstData, bool bInPushAll) const
{
	// $IRIS TODO: Rewrite this to iterate over change mask instead of iterating over all members and querying the mask
	// Note, we need to use a NetBitStreamReader and the changemask descriptor since each member might have different number of bits.
	if (IsValid())
	{
#if WITH_PUSH_MODEL
		using RepIndexType = decltype(FProperty::RepIndex);
		TArray<RepIndexType, TInlineAllocator<128>> DirtyRepIndices;
#endif

		const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
		uint8* DstBuffer = static_cast<uint8*>(DstData);

#if UE_NET_WITH_VERBOSE_POLL_PROFILING
		IRIS_PROFILER_PROTOCOL_NAME(ReplicationStateDescriptor->DebugName->Name);
#endif

		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FProperty** MemberProperties = Descriptor->MemberProperties;
		const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
		const uint32 MemberCount = Descriptor->MemberCount;

		const bool bPushAll = bInPushAll || IsInitState();
		for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
		{
			// Note: Currently not checking whether the condition is enabled or not. This is assuming few states would be affected by an early out here.
			// We need to mask off conditional state regardless at replication time to avoid replicating lost packets which contained state that has since been disabled.
			if (bPushAll || IsDirty(MemberIt))
			{
				const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
				const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
				const FProperty* Property = MemberProperties[MemberIt];

				PushPropertyValue(MemberIt, DstBuffer + Property->GetOffset_ForGC() + Property->GetElementSize()*MemberPropertyDescriptor.ArrayIndex);

#if WITH_PUSH_MODEL
				if (MemberPropertyDescriptor.ArrayIndex == 0)
				{
					DirtyRepIndices.Add(Property->RepIndex);
				}
#endif
			}
		}

#if WITH_PUSH_MODEL
		if (Owner != nullptr)
		{
			for (RepIndexType RepIndex : DirtyRepIndices)
			{
				MARK_PROPERTY_DIRTY_UNSAFE(Owner, RepIndex);
			}
		}
#endif
	}
}

void FPropertyReplicationState::CopyDirtyProperties(const FPropertyReplicationState& Other)
{
	check(this != &Other && IsValid());
	check(ReplicationStateDescriptor.GetReference() == Other.ReplicationStateDescriptor.GetReference());

	if (!Private::IsReplicationStateBound(StateBuffer, ReplicationStateDescriptor.GetReference()))
	{
		Private::CopyDirtyMembers(StateBuffer, Other.StateBuffer, ReplicationStateDescriptor.GetReference());
	}
	else
	{
		Set(Other);
	}
}

bool FPropertyReplicationState::PollObjectReferences(const void* RESTRICT SrcStateData)
{
	if (IsValid())
	{
		const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
		const uint8* SrcBuffer = static_cast<const uint8*>(SrcStateData);

		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FReplicationStateMemberTraitsDescriptor* MemberTraitsDescriptors = Descriptor->MemberTraitsDescriptors;
		const FProperty** MemberProperties = Descriptor->MemberProperties;
		const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
		const uint32 MemberCount = Descriptor->MemberCount;

		for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
		{
			if (EnumHasAnyFlags(MemberTraitsDescriptors[MemberIt].Traits, EReplicationStateMemberTraits::HasObjectReference))
			{
				const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
				const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
				const FProperty* Property = MemberProperties[MemberIt];

				PollPropertyValue(MemberIt, SrcBuffer + Property->GetOffset_ForGC() + Property->GetElementSize()*MemberPropertyDescriptor.ArrayIndex);
			}
		}
	}

	return IsDirty();
}

void FPropertyReplicationState::CallRepNotifies(void* RESTRICT DstData, const FCallRepNotifiesParameters& Params) const
{
	// $IRIS TODO: Rewrite this to iterate over change mask instead of iterating over all members and querying the mask
	// Note, we need to use a NetBitStreamReader and the changemask descriptor since each member might have different number of bits.
	if (IsValid())
	{
		const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FProperty** MemberProperties = Descriptor->MemberProperties;
		const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
		const uint32 MemberCount = Descriptor->MemberCount;

		const FProperty* LastPropertyWithRepNotify = nullptr;
		// Note: IsInitState indicates that the state itself is only ever will be applied at Init
		const bool bIsInitState = IsInitState();
		for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
		{
			const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
			const UFunction* RepNotifyFunction = MemberPropertyDescriptor.RepNotifyFunction;
			if (RepNotifyFunction && (bIsInitState || IsDirty(MemberIt)))
			{
				const FProperty* Property = MemberProperties[MemberIt];
				// If this is the same property we already processed it's yet another element in a C array.
				if (Property == LastPropertyWithRepNotify)
				{
					checkSlow(MemberPropertyDescriptor.ArrayIndex > 0);
					continue;
				}

				// For C arrays the RepNotify is for the entire array, not an individual element.
				const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
				const uint32 ExternalMemberOffset = MemberDescriptors[MemberIt].ExternalMemberOffset;
				
				UObject* Object = reinterpret_cast<UObject*>(DstData);
				const uint8* PrevValuePtr = Params.PreviousState ? Params.PreviousState->StateBuffer + ExternalMemberOffset : nullptr;
				const uint8* ValuePtr = StateBuffer + ExternalMemberOffset;				

				bool bShouldCallRepNotify = false;
				if (Params.bOnlyCallIfDiffersFromLocal)
				{
					// We try to be backwards compatible and respect RepNotify_Always/RepNotify_Changed unless it is the initial state where we only will call the repnotify of the received value differs from the local one.
					bShouldCallRepNotify = Params.bIsInit ? !Private::InternalCompareMember(Descriptor, MemberIt, ValuePtr, PrevValuePtr) : EnumHasAnyFlags(Descriptor->MemberTraitsDescriptors[MemberIt].Traits, EReplicationStateMemberTraits::HasRepNotifyAlways) || !Private::InternalCompareMember(Descriptor, MemberIt, ValuePtr, PrevValuePtr);
				}
				else
				{
					// Trust data from server and call RepNotify without doing additonal compare unless it is the initial state.
					bShouldCallRepNotify = !Params.bIsInit || !Private::InternalCompareMember(Descriptor, MemberIt, ValuePtr, PrevValuePtr);
				}

				if (bShouldCallRepNotify)
				{
#if IRIS_CLIENT_PROFILER_ENABLE
					UE::Net::FClientProfiler::RecordRepNotify(RepNotifyFunction->GetFName());
#endif

					// We only want to call RepNotify once for c-arrays
					LastPropertyWithRepNotify = Property;
					Object->ProcessEvent(const_cast<UFunction*>(RepNotifyFunction), const_cast<uint8*>(PrevValuePtr));
				}

#if !UE_BUILD_SHIPPING
				if (UE_LOG_ACTIVE(LogIrisRepNotify, Verbose) && IrisDebugHelper::FilterDebuggedObject(Object))
				{
					const bool bIsRepNotifyAlways = EnumHasAnyFlags(Descriptor->MemberTraitsDescriptors[MemberIt].Traits, EReplicationStateMemberTraits::HasRepNotifyAlways);
					if (bShouldCallRepNotify)
					{
						UE_LOGF(LogIrisRepNotify, Verbose, "Calling RepNotify. Object: %ls, Function: %ls IsInit: %d IsRepAlways: %d", *Object->GetFullName(), ToCStr(RepNotifyFunction->GetName()), Params.bIsInit, bIsRepNotifyAlways);
					}
					else
					{
						UE_LOGF(LogIrisRepNotify, VeryVerbose, "Skipping RepNotify. Object: %ls, Function: %ls IsInit: %d IsRepAlways: %d", *Object->GetFullName(), ToCStr(RepNotifyFunction->GetName()), Params.bIsInit, bIsRepNotifyAlways);
					}					
				}
#endif
			}
		}
	}
}

const TCHAR* FPropertyReplicationState::ToString(FStringBuilderBase& StringBuilder, bool bInIncludeAll) const
{
	if (IsValid())
	{
		const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;

		const FProperty** MemberProperties = Descriptor->MemberProperties;
		const uint32 MemberCount = Descriptor->MemberCount;

		StringBuilder.Appendf(TEXT("FPropertyReplicationState %s\n"), Descriptor->DebugName->Name);

		const bool bIncludeAll = bInIncludeAll || Descriptor->IsInitState();
		FString TempString;
		for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
		{
			const FProperty* Property = MemberProperties[MemberIt];

			if (bIncludeAll || IsDirty(MemberIt))
			{
				void* PropertyData = StateBuffer + Descriptor->MemberDescriptors[MemberIt].ExternalMemberOffset;
				Property->ExportTextItem_Direct(TempString, PropertyData, PropertyData, nullptr, PPF_SimpleObjectText|PPF_IncludeTransient|PPF_UseDeprecatedProperties);
				StringBuilder.Appendf(TEXT("%u - %s : %s\n"), MemberIt, *Property->GetName(), ToCStr(TempString));
				TempString.Reset();
			}
		}
	}

	return StringBuilder.ToString();
}

FString FPropertyReplicationState::ToString(bool bIncludeAll) const
{
	TStringBuilder<2048> Builder;
	ToString(Builder, bIncludeAll);

	return FString(Builder.ToString());
}

bool FPropertyReplicationState::IsCustomConditionEnabled(uint32 Index) const
{
	const EReplicationStateTraits Traits = ReplicationStateDescriptor->Traits;
	if (!EnumHasAnyFlags(Traits, EReplicationStateTraits::HasLifetimeConditionals))
	{
		return true;
	}

	const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;
	const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskInfo = Descriptor->MemberChangeMaskDescriptors[Index];

	if (ChangeMaskInfo.BitCount > 0)
	{
		FNetBitArrayView MemberConditionalChangeMask = Private::GetMemberConditionalChangeMask(StateBuffer, Descriptor);
		return MemberConditionalChangeMask.GetBit(ChangeMaskInfo.BitOffset);
	}

	// If there's no bitmask the property cannot be disabled.
	return true;
}

}
