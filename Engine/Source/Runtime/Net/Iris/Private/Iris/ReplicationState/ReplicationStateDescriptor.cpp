// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetSerializer.h"
#include "AutoRTFM.h"
#include "HAL/PlatformAtomics.h"
#include "UObject/UnrealType.h"
#include "UObject/CoreNetTypes.h"
#include "Templates/IsSigned.h"

namespace UE::Net
{

//static_assert(TIsSigned<__underlying_type(ELifetimeCondition)>::Value == TIsSigned<decltype(FReplicationStateMemberLifetimeConditionDescriptor::Condition)>::Value, "FReplicationStateMemberLifetimeConditionDescriptor::Condition is not compatible with ELifetimeCondition");
static_assert(static_cast<__underlying_type(ELifetimeCondition)>(ELifetimeCondition::COND_Max - 1) <= TNumericLimits<decltype(FReplicationStateMemberLifetimeConditionDescriptor::Condition)>::Max(), "FReplicationStateMemberLifetimeConditionDescriptor::Condition is not compatible with ELifetimeCondition");

void DescribeReplicationDescriptor(FString& OutString, const FReplicationStateDescriptor* Descriptor)
{
	uint32 MemberCount = Descriptor->MemberCount;

	OutString.Append(FString::Printf(TEXT("FReplicationStateDescriptor : %" UINT64_FMT "\n"), Descriptor->DescriptorIdentifier.Value));
	OutString.Append(FString::Printf(TEXT("MemberCount : %u\n"), MemberCount));
	OutString.Append(FString::Printf(TEXT("ExternalBufferSize : %u\n"), Descriptor->ExternalSize));
	OutString.Append(FString::Printf(TEXT("ExternalAlignment : %u\n"), Descriptor->ExternalAlignment));
	OutString.Append(FString::Printf(TEXT("InternalBufferSize : %u\n"), Descriptor->InternalSize));
	OutString.Append(FString::Printf(TEXT("InternalAlignment : %u\n"), Descriptor->InternalAlignment));

	OutString.Append(FString::Printf(TEXT("ChangeMaskBitCount : %u\n"), Descriptor->ChangeMaskBitCount));

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = Descriptor->MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializer = Descriptor->MemberSerializerDescriptors[MemberIt];

		const uint32 ChangeMaskBitCount = Descriptor->MemberChangeMaskDescriptors ? Descriptor->MemberChangeMaskDescriptors[MemberIt].BitCount : 0u;

		if (Descriptor->MemberProperties)
		{
			const FProperty* Property = Descriptor->MemberProperties[MemberIt];
			OutString.Append(FString::Printf(TEXT("..%s Type: %s: Serializer: %s ExternalOffset %u, InternalOffset %u ChangeMaskBits: %u\n"), *Property->GetName(), *Property->GetCPPType(), MemberSerializer.Serializer->Name, MemberDescriptor.ExternalMemberOffset, MemberDescriptor.InternalMemberOffset, ChangeMaskBitCount));
		}
		else
		{
			OutString.Append(FString::Printf(TEXT("..Member:%u Serializer: %s: ExternalOffset %u, InternalOffset %u ChangeMaskBits: %u\n"), MemberIt, MemberSerializer.Serializer->Name, MemberDescriptor.ExternalMemberOffset, MemberDescriptor.InternalMemberOffset, ChangeMaskBitCount));
		}
	}
}

void FReplicationStateDescriptor::AddRef() const
{
	if (EnumHasAnyFlags(Traits, EReplicationStateTraits::NeedsRefCount))
	{
		UE_AUTORTFM_OPEN
		{
			++RefCount;
		};
		UE_AUTORTFM_ONABORT(this)
		{
			--RefCount;
		};
	}
}

void FReplicationStateDescriptor::Release() const
{
	using namespace UE::Net::Private;

	if (EnumHasAnyFlags(Traits, EReplicationStateTraits::NeedsRefCount))
	{
		UE_AUTORTFM_ONCOMMIT(this)
		{
			if (--RefCount == 0)
			{
				// We must destruct default state if we have one
				if (uint8* StateBufferToFree = const_cast<uint8*>(DefaultStateBuffer))
				{
					if (EnumHasAnyFlags(Traits, EReplicationStateTraits::HasDynamicState))
					{
						FNetSerializationContext FreeContext;
						FInternalNetSerializationContext InternalContext;
						FreeContext.SetInternalContext(&InternalContext);

						FReplicationStateOperationsInternal::FreeDynamicState(FreeContext, StateBufferToFree, this);
					}
					FMemory::Free(StateBufferToFree);
				}

				// We must destruct configs pointing to generated structs
				for (uint32 It = 0; It < MemberCount; ++It)
				{
					const FNetSerializerConfig* Config = MemberSerializerDescriptors[It].SerializerConfig;
					if (EnumHasAnyFlags(Config->ConfigTraits, ENetSerializerConfigTraits::NeedDestruction))
					{
						Config->~FNetSerializerConfig();
					}
				}

				// We must destruct member function descriptors
				for (const FReplicationStateMemberFunctionDescriptor& FunctionDescriptor : MakeArrayView(MemberFunctionDescriptors, FunctionCount))
				{
					FunctionDescriptor.Descriptor->Release();
				}

				// Allocated via FMemory::Malloc in ReplicationStateDescriptorBuilder
				FMemory::Free(const_cast<FReplicationStateDescriptor*>(this));
			}
		};
	}
}

}
