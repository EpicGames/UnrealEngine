// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "CoreTypes.h"
#include "Containers/Set.h"
#include "Iris/Core/BitTwiddling.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/MemoryLayoutUtil.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/Misc/NetCVars.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/InternalPropertyReplicationState.h"
#include "Iris/ReplicationState/IrisFastArraySerializer.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorConfig.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorRegistry.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/NetBlob/PartialNetObjectAttachmentHandler.h"
#include "Iris/ReplicationSystem/RepTag.h"
#include "Iris/Serialization/InternalNetSerializerUtils.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetSerializers.h"
#include "HAL/ConsoleManager.h"
#include "Hash/CityHash.h"
#include "Logging/LogMacros.h"
#include "Misc/App.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/AlignmentTemplates.h"
#include "UObject/UnrealType.h"
#include "UObject/StrongObjectPtr.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"

#ifndef UE_NET_TEST_FAKE_REP_TAGS
#	define UE_NET_TEST_FAKE_REP_TAGS 0
#endif

#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#	define UE_NET_ENABLE_DESCRIPTORBUILDER_LOG 0
#else
#	define UE_NET_ENABLE_DESCRIPTORBUILDER_LOG 1
#endif 

#if UE_NET_ENABLE_DESCRIPTORBUILDER_LOG
#	define UE_LOG_DESCRIPTORBUILDER(Log, Format, ...)  UE_LOG(LogIris, Log, Format, ##__VA_ARGS__)
#else
#	define UE_LOG_DESCRIPTORBUILDER(...)
#endif

#define UE_LOG_DESCRIPTORBUILDER_WARNING(Format, ...)  UE_LOG(LogIris, Warning, Format, ##__VA_ARGS__)

namespace UE::Net::Private
{

static const FName PropertyNetSerializerRegistry_NAME_ReplicationID("ReplicationID");
static const FName PropertyNetSerializerRegistry_NAME_ChangeMaskStorage("ChangeMaskStorage");
static const FName ReplicationStateDescriptorBuilder_NAME_NetCullDistanceSquared("NetCullDistanceSquared");
static const FName ReplicationStateDescriptorBuilder_NAME_RoleGroup("RoleGroup");

static bool bIrisUseNativeFastArray = true;
static FAutoConsoleVariableRef CVarIrisUseNativeFastArray(TEXT("net.Iris.UseNativeFastArray"), bIrisUseNativeFastArray, TEXT("Enable or disable IrisNativeFastArray."));

static bool bIrisUseChangeMaskForTArray = true;
static FAutoConsoleVariableRef CVarIrisUseChangeMaskForTArray(TEXT("net.Iris.UseChangeMaskForTArray"), bIrisUseChangeMaskForTArray, TEXT("Enable or disable the use of a changemask for individual elements in TArrays. When enabled and packet loss occurs the received array may not reflect a state which was ever present on the sending side since the array won't be replicated atomically. Enabled by default."));

static bool bWarnAboutStructsWithCustomSerialization = true;
static FAutoConsoleVariableRef CVarIrisWarnAboutStructsWithCustomSerialization(TEXT("net.Iris.WarnAboutStructsWithCustomSerialization"), bWarnAboutStructsWithCustomSerialization, TEXT("Warn when generating descriptors for structs with custom serialization."));

static bool bUseSupportsStructNetSerializerList = true;
static FAutoConsoleVariableRef CVarIrisUseSupportsStructNetSerializerList(TEXT("net.Iris.UseSupportsStructNetSerializerList"), bUseSupportsStructNetSerializerList, TEXT("If enabled structs in the SupportsStructNetSerializerList will not raise warnings even if the struct has a NetSerialize/NetDeltaSerialize but has no custom NetSerializer."));

static bool bWarnAboutStructPropertiesWithSuspectedNotReplicatedProperties = false;
static FAutoConsoleVariableRef CVarIrisWarnAboutStructPropertiesWithSuspectedNotReplicatedProperties(TEXT("net.Iris.bWarnAboutStructPropertiesWithSuspectedNotReplicatedProperties"), bWarnAboutStructPropertiesWithSuspectedNotReplicatedProperties, TEXT("Try to detect if a struct replicated as a property might contain unannotated members, disabled by default."));

static TStrongObjectPtr<UIrisObjectReferencePackageMap> s_IrisObjectReferencePackageMap = nullptr;

static UIrisObjectReferencePackageMap* GetOrCreateIrisObjectReferencePackageMap()
{
	if (!s_IrisObjectReferencePackageMap.IsValid())
	{
		s_IrisObjectReferencePackageMap = TStrongObjectPtr<UIrisObjectReferencePackageMap>(NewObject<UIrisObjectReferencePackageMap>());
	}
	return s_IrisObjectReferencePackageMap.Get();
}

enum class EMemberPropertyTraits : uint32
{
	None								= 0U,
	InitOnly							= 1U << 0U,
	HasLifetimeConditionals				= InitOnly << 1U,
	RepNotifyOnChanged					= HasLifetimeConditionals << 1U,
	RepNotifyAlways						= RepNotifyOnChanged << 1U,
	NeedPreviousState					= RepNotifyAlways << 1U,
	HasDynamicState						= NeedPreviousState << 1U,
	HasObjectReference					= HasDynamicState << 1U,
	HasCustomObjectReference			= HasObjectReference << 1U,
	IsSourceTriviallyConstructible		= HasCustomObjectReference << 1U,
	IsSourceTriviallyDestructible		= IsSourceTriviallyConstructible << 1U,
	IsTArray							= IsSourceTriviallyDestructible << 1U,
	IsFastArray							= IsTArray << 1U,
	IsNativeFastArray					= IsFastArray << 1U,
	IsFastArrayWithExtraProperties		= IsNativeFastArray << 1U,
	IsFastArrayItem						= IsFastArrayWithExtraProperties << 1U,
	IsInvalidFastArray					= IsFastArrayItem << 1U,
	HasConnectionSpecificSerialization	= IsInvalidFastArray << 1U,
	HasPushBasedDirtiness				= HasConnectionSpecificSerialization << 1U,
	UseSerializerIsEqual				= HasPushBasedDirtiness << 1U,
};
ENUM_CLASS_FLAGS(EMemberPropertyTraits);

enum class EStructNetSerializerType : unsigned
{
	// Struct should use StructNetSerializer
	Struct,
	// Struct has a custom NetSerializer
	Custom,
	// Struct has a parent struct in the hiearchy which has a custom NetSerializer
	DerivedFromCustom,
};

struct FLazyGetPathNameHelper
{
	const FStringBuilderBase* GetPathName(const UObject* Object)
	{
		if (StringBuilder.Len() == 0)
		{
			Object->GetPathName(nullptr, StringBuilder);

			// Ensure we have a valid string
			StringBuilder.ToString();

			// Make sure that the path have consistent casing
			TCString<TCHAR>::Strupr(StringBuilder.GetData(), StringBuilder.Len());
		}
		return &StringBuilder;
	}

	void Reset()
	{
		StringBuilder.Reset();
	}

	TStringBuilder<4096> StringBuilder;
};

class FPropertyReplicationStateDescriptorBuilder
{
public:

	struct FMemberProperty
	{
		const FProperty* Property = nullptr;
		const UFunction* PropertyRepNotifyFunction = nullptr;
		const FPropertyNetSerializerInfo* SerializerInfo = nullptr;
		// Optional create method for fragment of properties that supports separate states, such as FastArraySerializers and custom struct serializers with custom fragment.
		CreateAndRegisterReplicationFragmentFunc CreateAndRegisterReplicationFragmentFunction = nullptr;
		EMemberPropertyTraits Traits = EMemberPropertyTraits::None;
		// Condition for when and where the property should be replicated
		ELifetimeCondition ReplicationCondition = ELifetimeCondition::COND_None;
		// Number of bits needed to track dirtiness for this property. Init state properties ignores this.
		uint16 ChangeMaskBits = 0;
		// We allow members to specify a changemask group to facilitate sharing of the same changemask bit
		FName ChangeMaskGroupName;
		// For members without a corresponding property this needs to be set explicitly
		FMemoryLayoutUtil::FSizeAndAlignment ExternalSizeAndAlignment;
	};

	struct FMemberFunction
	{
		const UFunction* Function;
	};

	// Utility methods
	static void GetSerializerTraits(FMemberProperty& OutMemberProperty, const FProperty* Property, const FPropertyNetSerializerInfo* Info, bool bAllowFastArrayWithExtraProperties = false);
	struct FIsSupportedPropertyParams
	{
		const TArray<FLifetimeProperty>* LifeTimeProperties = nullptr;
		UClass* InObjectClass = nullptr;
		bool bAllowFastArrayWithExtraReplicatedProperties = false;
	};
	static bool IsSupportedProperty(FMemberProperty& OutMemberProperty, const FProperty* Property, const FIsSupportedPropertyParams& Params);
	static bool IsSupportedProperty(FMemberProperty& OutMemberProperty, const FProperty* Property) { FIsSupportedPropertyParams Params; return IsSupportedProperty(OutMemberProperty, Property, Params); }
	static EStructNetSerializerType IsSupportedStructWithCustomSerializer(FMemberProperty& OutMemberProperty, const UStruct* InStruct);
	static bool IsStructWithCustomSerializer(const UStruct* InStruct);
	static const UStruct* FindSuperStructWithCustomSerializer(const UStruct* Struct);

	static bool CanStructUseStructNetSerializer(FName StructName);
	
	static EMemberPropertyTraits GetConnectionFilterTrait(ELifetimeCondition Condition);
	static EMemberPropertyTraits GetInitOnlyTrait(ELifetimeCondition Condition);
	static EMemberPropertyTraits GetFastArrayPropertyTraits(const FNetSerializer* NetSerializer, const FProperty* Property, bool bAllowFastArrayWithExtraProperties);
	static SIZE_T GetFastArrayChangeMaskOffset(const FProperty* Property);
	static EMemberPropertyTraits GetHasObjectReferenceTraits(const FNetSerializer* NetSerializer);
	static void GetIrisPropertyTraits(FMemberProperty& OutMemberProperty, const FProperty* Property, const TArray<FLifetimeProperty>* LifeTimeProperties, UClass* ObjectClass);

	static bool ShouldValidateIsFullyPushModel(const UClass* Class);

public:
	enum class EDescriptorType : unsigned
	{
		Invalid,
		Class,
		Struct,
		Function,
		SingleProperty,
	};

	struct FBuildParameters
	{
		// Optionally pass in the full PathName which is used to build the identifier for the descriptor
		const FStringBuilderBase* PathName = nullptr;
		// Optionally pass in pointer to source data buffer from which we should create the default state
		const uint8* DefaultStateSourceData = nullptr;
		// When building default state we need the ReplicationSystem in order to capture exports.
		UReplicationSystem* ReplicationSystem = nullptr;
		EDescriptorType DescriptorType = EDescriptorType::Invalid;
		bool bIsInitState = false;
		bool bAllMembersAreReplicated = false;
		bool bShouldValidateIsFullyPushModel = false;
	};

	void AddMemberProperty(const FMemberProperty& Info) { Members.Add(Info); }
	void AddMemberFunction(const FMemberFunction& Info) { Functions.Add(Info); }

	void SetStructSizeAndAlignment(SIZE_T Size, SIZE_T Alignment)
	{
		StructInfo.SizeAndAlignment.Size = Size;
		StructInfo.SizeAndAlignment.Alignment = Alignment;
	}

	void SetStruct(const UStruct* Struct)
	{
		StructInfo.Struct = Struct;
	}

	void SetStructSerializerType(EStructNetSerializerType SerializerType)
	{
		StructInfo.SerializerType = SerializerType;
	}

	bool HasDataToBuild() const { return (Members.Num() > 0) || (Functions.Num() > 0U); }
	TRefCountPtr<const FReplicationStateDescriptor> Build(const FString& StateName, FReplicationStateDescriptorRegistry* DescriptorRegistry, const FBuildParameters& BuildParams);

private:

	static const SIZE_T MinAlignment = 1;

	using FSizeAndAlignment = FMemoryLayoutUtil::FSizeAndAlignment;
	using FOffsetAndSize = FMemoryLayoutUtil::FOffsetAndSize;

	// We need to cache some information before we can calculate the required size of our descriptor
	struct FMemberCacheEntry
	{
		FMemberCacheEntry() : bUseSerializerDefaultConfig(0), bIsStruct(0), bIsDynamicArray(0), bHasCustomObjectReference(0) {}

		const FNetSerializer* Serializer;							// Serializer
		TRefCountPtr<const FReplicationStateDescriptor>	Descriptor;	// Descriptor for structs or nested replication states?
		FSizeAndAlignment ExternalSizeAndAlignment;					// Size and alignment of external member data
		FSizeAndAlignment InternalSizeAndAlignment;					// Size and alignment of quantized member data
		FSizeAndAlignment SerializerConfigSizeAndAlignment;			// Size and alignment for SerializerConfig
		uint32 bUseSerializerDefaultConfig : 1;						// Whether the member can use the serializer's default config

		uint32 bIsStruct : 1;
		uint32 bIsDynamicArray : 1;
		uint32 bHasCustomObjectReference : 1;
	};

	typedef TArray<FMemberCacheEntry> FMemberCache;

	struct FMemberTagCacheEntry
	{
		FRepTag Tag;
		uint16 MemberIndex;
		// If set then tags can be found in the member's ReplicationStateDescriptor
		bool bInDescriptor;
	};

	struct FMemberTagCache
	{
		FMemberTagCache() : TagCount(0) {}

		TArray<FMemberTagCacheEntry> CacheEntries;
		uint32 TagCount;
	};
	
	struct FMemberReferenceCacheEntry
	{
		uint32 MemberIndex;
		uint32 InMemberStateDescriptor : 1; // Reference is nested and we need to look it up
	};

	struct FMemberReferenceCache
	{
		FMemberReferenceCache() : ReferenceCount(0) {}

		TArray<FMemberReferenceCacheEntry> CacheEntries;
		uint32 ReferenceCount;
	};

	struct FMemberFunctionCacheEntry
	{
		const UFunction* Function;
		TRefCountPtr<const FReplicationStateDescriptor>	Descriptor;
	};
	typedef TArray<FMemberFunctionCacheEntry> FMemberFunctionCache;

	struct FBuilderContext
	{
		FBuilderContext()
		: External({0, MinAlignment})
		, Internal({0, MinAlignment})
		, CombinedPropertyTraits(EMemberPropertyTraits::None)
		, SharedPropertyTraits(EMemberPropertyTraits::None)
		, BuildParams({})
		, bGeneratePropertyDescriptors(true)
		{
		}

		bool IsInitState() const { return BuildParams.bIsInitState; }

		FMemberCache MemberCache;
		FMemberReferenceCache ReferenceCache;
		FMemberTagCache MemberTagCache;
		FMemberFunctionCache MemberFunctionCache;

		FSizeAndAlignment External;
		FSizeAndAlignment Internal;

		// Or:ed traits
		EMemberPropertyTraits CombinedPropertyTraits;
		// And:ed traits
		EMemberPropertyTraits SharedPropertyTraits;

		FBuildParameters BuildParams;
	
		// Set if the descriptor we are building has should include property descriptor, there are some special cases like structs with custom NetSerializer which does not have any properties.
		bool bGeneratePropertyDescriptors;
	};

	struct FStructInfo
	{
		FSizeAndAlignment SizeAndAlignment = {};
		const UStruct* Struct = nullptr;
		EStructNetSerializerType SerializerType = EStructNetSerializerType::Struct;
	};

	void AddSerializerConfigMemoryBlockToLayout(FMemoryLayoutUtil::FLayout& Layout, FOffsetAndSize& OutOffsetAndSize, const FMemberCache& MemberCache) const;

	void BuildMemberCache(FBuilderContext& Context, FReplicationStateDescriptorRegistry* DescriptorRegistry);
	void BuildMemberReferenceCache(FMemberReferenceCache& MemberReferenceCache, const FMemberCache& MemberCache) const;	
	void BuildMemberTagCache(FMemberTagCache& MemberTagCache, const FMemberCache& MemberCache) const;
	void BuildMemberFunctionCache(FBuilderContext& Context, FReplicationStateDescriptorRegistry* DescriptorRegistry) const;

	void BuildReplicationStateInternal(FReplicationStateDescriptor* Descriptor, FBuilderContext& Context) const;
	void BuildMemberDescriptors(FReplicationStateMemberDescriptor* MemberDescriptors, FReplicationStateDescriptor* Descriptor, FBuilderContext& Context) const;
	void BuildMemberDescriptorsForStruct(FReplicationStateMemberDescriptor* MemberDescriptors, FReplicationStateDescriptor* Descriptor, FBuilderContext& Context) const;
	void BuildMemberChangeMaskDescriptors(FReplicationStateMemberChangeMaskDescriptor* MemberChangeMaskDescriptors, FReplicationStateDescriptor* Descriptor, FBuilderContext& Context) const;
	void BuildMemberSerializerDescriptors(FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors, FReplicationStateDescriptor* Descriptor) const;
	void BuildMemberTraitsDescriptors(FReplicationStateMemberTraitsDescriptor* MemberTraitsDescriptors, FReplicationStateDescriptor* Descriptor, const FBuilderContext& Context) const;
	void BuildMemberTagDescriptors(FReplicationStateMemberTagDescriptor* MemberRepTagDescriptors, FReplicationStateDescriptor* Descriptor, const FBuilderContext& Context) const;
	void BuildMemberReferenceDescriptors(FReplicationStateMemberReferenceDescriptor* MemberReferenceDescriptors, FReplicationStateDescriptor* Descriptor, const FBuilderContext& Context) const;
	void BuildMemberFunctionDescriptors(FReplicationStateMemberFunctionDescriptor* MemberFunctionDescriptors, FReplicationStateDescriptor* Descriptor, const FBuilderContext& Context) const;
	void BuildMemberProperties(const FProperty** MemberProperties, FReplicationStateDescriptor* Descriptor) const;
	void BuildMemberPropertyDescriptors(FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors, FReplicationStateDescriptor* Descriptor) const;
	void BuildMemberLifetimeConditionDescriptors(FReplicationStateMemberLifetimeConditionDescriptor* MemberLifetimeConditionDescriptors, FReplicationStateDescriptor* Descriptor) const;
	void BuildMemberRepIndexToMemberDescriptors(FReplicationStateMemberRepIndexToMemberIndexDescriptor* MemberRepIndexToMemberIndexDescriptors, FReplicationStateDescriptor* Descriptor) const;
	void BuildMemberDebugDescriptors(FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors, FReplicationStateDescriptor* Descriptor, const FBuilderContext& Context) const;
	void BuildMemberSerializerConfigs(uint8* SerializerConfigBuffer, FReplicationStateDescriptor* Descriptor, const FMemberCache& MemberCache) const;
	void FixupNetRoleNetSerializerConfigs(FReplicationStateDescriptor* Descriptor);
	void FixupDescriptorForNativeFastArray(FReplicationStateDescriptor* Descriptor) const;
	void AllocateAndInitializeDefaultInternalStateBuffer(const uint8* RESTRICT SrcBuffer, const FReplicationStateDescriptor* Descriptor, UReplicationSystem* ReplicationSystem, const uint8*& OutDefaultStateBuffer) const;
	bool CalculateDefaultStateChecksum(const FReplicationStateDescriptor* Descriptor, UReplicationSystem* ReplicationSystem, const uint8* OutDefaultStateBuffer, uint64& OutHashValue) const;

	EReplicationStateTraits BuildReplicationStateTraits(const FBuilderContext& Context) const;

	static FRepTag GetRepTagFromProperty(const FMemberCacheEntry& MemberCacheEntry, const FMemberProperty& Property);
	static uint32 GetRepIndexCount(const TConstArrayView<FMemberProperty>& Members);
	void FinalizeDescriptor(FReplicationStateDescriptor* Descriptor, FBuilderContext& Context) const;

	TRefCountPtr<const FReplicationStateDescriptor> GetDescriptorForStructProperty(const FMemberProperty& MemberProperty, FReplicationStateDescriptorRegistry* DescriptorRegistry) const;
	TRefCountPtr<const FReplicationStateDescriptor> GetDescriptorForArrayProperty(const FProperty* Property, FReplicationStateDescriptorRegistry* DescriptorRegistry) const;
	TRefCountPtr<const FReplicationStateDescriptor> GetDescriptorForOptionalProperty(const FProperty* Property, FReplicationStateDescriptorRegistry* DescriptorRegistry) const;
	TRefCountPtr<const FReplicationStateDescriptor> GetOrCreateDescriptorForFunction(const UFunction* Function, FReplicationStateDescriptorRegistry* DescriptorRegistry) const;
	TRefCountPtr<const FReplicationStateDescriptor> CreateDescriptorForProperty(const FString& DescriptorName, const FProperty* Property, const FReplicationStateDescriptorBuilder::FParameters& Parameters) const;

	static bool ValidateStructPropertyDescriptor(const FMemberProperty& MemberProperty, const FReplicationStateDescriptor* Descriptor);
	void ValidateIsFullyPushModelClassDescriptor(const FBuilderContext& Context, const FReplicationStateDescriptor* Descriptor);

	// Helpers
	static void GetPropertyPathName(const FProperty* Property, FString& PathName);

	FString Name;
	TArray<FMemberProperty> Members;
	TArray<FMemberFunction> Functions;
	FStructInfo StructInfo;

	static TSet<FName> UniqueClassNames;
};

TSet<FName> FPropertyReplicationStateDescriptorBuilder::UniqueClassNames;

TRefCountPtr<const FReplicationStateDescriptor> FPropertyReplicationStateDescriptorBuilder::GetDescriptorForStructProperty(const FMemberProperty& MemberProperty, FReplicationStateDescriptorRegistry* DescriptorRegistry) const
{
	const FStructProperty* StructProp = CastFieldChecked<const FStructProperty>(MemberProperty.Property);
	const UScriptStruct* Struct = StructProp->Struct;

	FReplicationStateDescriptorBuilder::FParameters Params;
	Params.DescriptorRegistry = DescriptorRegistry;
	Params.EnableFastArrayHandling = EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::IsFastArray);
	Params.AllowFastArrayWithExtraReplicatedProperties = EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::IsFastArrayWithExtraProperties);

	return FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(Struct, Params);
}

// Tries to validate the layout of a struct property if it is tagged as AllMembersAreReplicated in order to detect members that might be affected 
// if we would assign the struct atomically, since we get too many false positives caused by for example structs with virtual functions or by padding at the end
// this is normally disabled.
bool FPropertyReplicationStateDescriptorBuilder::ValidateStructPropertyDescriptor(const FMemberProperty& MemberProperty, const FReplicationStateDescriptor* Descriptor)
{
	if (!EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated))
	{
		return true;
	}

	const FStructProperty* StructProp = CastFieldChecked<const FStructProperty>(MemberProperty.Property);
	const UScriptStruct* Struct = StructProp->Struct;

	if (FPropertyReplicationStateDescriptorBuilder::IsStructWithCustomSerializer(Struct))
	{
		return true;
	}

	SIZE_T CurrentExpectedOffset =  0;
	SIZE_T PrevExpectedOffset = CurrentExpectedOffset;

	const FProperty* PrevProperty = nullptr;
	for (const FProperty* Property : MakeArrayView(Descriptor->MemberProperties, Descriptor->MemberCount))
	{
		const SIZE_T MemberIndex = &Property - &Descriptor->MemberProperties[0];

		const SIZE_T PropertySize = Property->GetSize();
		const SIZE_T PropertyAlignment  = Property->GetMinAlignment();

		CurrentExpectedOffset = Align(CurrentExpectedOffset, PropertyAlignment);

		if (Property != PrevProperty)
		{
			const SIZE_T PropertyOffset = Property->GetOffset_ForGC();
			if ((PropertyOffset != CurrentExpectedOffset) && (PropertyOffset != PrevExpectedOffset))
			{
				UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("Found replicated struct %s that might contain members not covered by replication."), *(Struct->GetName()));
				return false;
			}
				
			if (PrevExpectedOffset != PropertyOffset)
			{
				PrevExpectedOffset = CurrentExpectedOffset;
			}
			else
			{
				// Special case for bitfields
				CurrentExpectedOffset = PrevExpectedOffset;					
			}

			CurrentExpectedOffset += PropertySize;
			PrevProperty = Property;
		}
	}

	if (Struct->GetPropertiesSize() != CurrentExpectedOffset)
	{
		UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("Found replicated struct %s that might contain members not covered by replication, Struct GetPropertiesSize(),%d accumulated size %" SIZE_T_FMT), *(Struct->GetName()), Struct->GetPropertiesSize(), CurrentExpectedOffset);
		return false;
	}

	return true;
}

bool FPropertyReplicationStateDescriptorBuilder::ShouldValidateIsFullyPushModel(const UClass* Class)
{
#if UE_BUILD_SHIPPING
	return false;
#else
	// If we have processed this class before there's no need to verify it's push basedness again.
	bool bAlreadyInSet = false;
	UniqueClassNames.Emplace(Class->GetFName(), &bAlreadyInSet);
	if (bAlreadyInSet)
	{
		return false;
	}

	const UReplicationStateDescriptorConfig* ReplicationStateDescriptorConfig = GetDefault<UReplicationStateDescriptorConfig>();
	if (ReplicationStateDescriptorConfig->EnsureAllClassesAreFullyPushModel())
	{
		return true;
	}

	if (const FReplicationStateDescriptorClassPushModelConfig* ClassConfig = ReplicationStateDescriptorConfig->GetEnsureFullyPushModelClassNames().FindByPredicate([ClassName = Class->GetFName()](const FReplicationStateDescriptorClassPushModelConfig& ClassConfig) { return ClassConfig.ClassName == ClassName; }))
	{
		return true;
	}

 	return false;
#endif
}

void FPropertyReplicationStateDescriptorBuilder::ValidateIsFullyPushModelClassDescriptor(const FBuilderContext& Context, const FReplicationStateDescriptor* Descriptor)
{
	if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness))
	{
		return;
	}

	// A class and its descriptors is expected to be validated at most once. We want to report all of the failing descriptors because the user has explicitly requested us to.
	ensureAlwaysMsgf(false, TEXT("ReplicationStateDescriptor %s for class %s does not have the HasPushBasedDirtiness trait. This prevents the class from being fully push based. See log for additional details."),
		ToCStr(Descriptor->DebugName), *WriteToString<128>(Context.BuildParams.PathName ? Context.BuildParams.PathName->ToView() : TEXTVIEW("UNKNOWN")));

	// Try to figure out the cause of the lack of push based trait
	if (!Members.IsEmpty())
	{
		TStringBuilder<2048> Message;
		Message.Appendf(TEXT("Members of ReplicationStateDescriptor %s lacking HasPushBasedDirtiness trait:\n"), ToCStr(Descriptor->DebugName));
		for (const FMemberProperty& Member : Members)
		{
			if (!EnumHasAnyFlags(Member.Traits, EMemberPropertyTraits::HasPushBasedDirtiness))
			{
				if (Member.Property)
				{
					Message.Appendf(TEXT("Member property name: %s\n"), ToCStr(Member.Property->GetName()));
				}
				else if (const FReplicationStateMemberDebugDescriptor* DebugDescriptor = Descriptor->MemberDebugDescriptors)
				{
					const SIZE_T MemberIndex = &Member - Members.GetData();
					Message.Appendf(TEXT("Member debug name: %s\n"), ToCStr(DebugDescriptor[MemberIndex].DebugName));
				}
				else
				{
					Message.Appendf(TEXT("Member index: %u\n"), unsigned(&Member - Members.GetData()));
				}
			}
		}
		UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("%s"), Message.ToString());
	}
	else
	{
		UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("ReplicationStateDescriptor %s for class %s does not have the HasPushBasedDirtiness trait. No additional info."),
			ToCStr(Descriptor->DebugName), *WriteToString<128>(Context.BuildParams.PathName ? Context.BuildParams.PathName->ToView() : TEXTVIEW("")));
	}
}


TRefCountPtr<const FReplicationStateDescriptor> FPropertyReplicationStateDescriptorBuilder::GetDescriptorForArrayProperty(const FProperty* Property, FReplicationStateDescriptorRegistry* DescriptorRegistry) const
{
	FReplicationStateDescriptorBuilder::FParameters Params;
	Params.DescriptorRegistry = DescriptorRegistry;

	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Property);
	const FProperty* InnerProperty = ArrayProperty->Inner;

	FString DescriptorName;
	GetPropertyPathName(ArrayProperty, DescriptorName);

	return CreateDescriptorForProperty(DescriptorName, InnerProperty, Params);
}

TRefCountPtr<const FReplicationStateDescriptor> FPropertyReplicationStateDescriptorBuilder::GetDescriptorForOptionalProperty(const FProperty* Property, FReplicationStateDescriptorRegistry* DescriptorRegistry) const
{
	FReplicationStateDescriptorBuilder::FParameters Params;
	Params.DescriptorRegistry = DescriptorRegistry;

	const FOptionalProperty* OptionalProperty = CastFieldChecked<FOptionalProperty>(Property);
	const FProperty* ValueProperty = OptionalProperty->GetValueProperty();

	FString DescriptorName;
	GetPropertyPathName(OptionalProperty, DescriptorName);

	return CreateDescriptorForProperty(DescriptorName, ValueProperty, Params);
}

TRefCountPtr<const FReplicationStateDescriptor> FPropertyReplicationStateDescriptorBuilder::GetOrCreateDescriptorForFunction(const UFunction* Function, FReplicationStateDescriptorRegistry* DescriptorRegistry) const
{
	FReplicationStateDescriptorBuilder::FParameters Params;
	Params.DescriptorRegistry = DescriptorRegistry;

	return FReplicationStateDescriptorBuilder::CreateDescriptorForFunction(Function, Params);
}

TRefCountPtr<const FReplicationStateDescriptor> FPropertyReplicationStateDescriptorBuilder::CreateDescriptorForProperty(const FString& DescriptorName, const FProperty* Property, const FReplicationStateDescriptorBuilder::FParameters& Parameters) const
{
	IRIS_PROFILER_SCOPE(FPropertyReplicationStateDescriptorBuilder_CreateDescriptorForProperty);

	// Check registry if we already have created descriptors for this class
	if (Parameters.DescriptorRegistry)
	{	
		if (const FReplicationStateDescriptorRegistry::FDescriptors* Result = Parameters.DescriptorRegistry->Find(Property))
		{
			// For structs we do only expect a single entry
			ensure(Result->Num() == 1);

			return (*Result)[0];
		}
	}

	FPropertyReplicationStateDescriptorBuilder Builder;
	FPropertyReplicationStateDescriptorBuilder::FMemberProperty MemberProperty;

	const bool bIsSupportedProperty = FPropertyReplicationStateDescriptorBuilder::IsSupportedProperty(MemberProperty, Property);
	if (!ensure(bIsSupportedProperty))
	{
		return nullptr;
	}

	Builder.AddMemberProperty(MemberProperty);

	FPropertyReplicationStateDescriptorBuilder::FBuildParameters BuildParameters = {};
	BuildParameters.DescriptorType = FPropertyReplicationStateDescriptorBuilder::EDescriptorType::SingleProperty;
	BuildParameters.bIsInitState = false;
	BuildParameters.bAllMembersAreReplicated = true;
	TRefCountPtr<const FReplicationStateDescriptor> Descriptor = Builder.Build(DescriptorName, Parameters.DescriptorRegistry, BuildParameters);

	if (Parameters.DescriptorRegistry && Descriptor.IsValid())
	{
		Parameters.DescriptorRegistry->Register(Property, Descriptor);
	}

	return Descriptor;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberCache(FBuilderContext& Context, FReplicationStateDescriptorRegistry* DescriptorRegistry)
{
	EMemberPropertyTraits PropertyTraits = EMemberPropertyTraits::None;
	EMemberPropertyTraits SharedPropertyTraits = (Members.Num() > 0 ? ~EMemberPropertyTraits(EMemberPropertyTraits::None) : EMemberPropertyTraits::None);
	bool bAllMembersAreReplicated = Context.BuildParams.bAllMembersAreReplicated;

	FMemberCache& MemberCache = Context.MemberCache;
	MemberCache.SetNum(Members.Num());
	FMemberCacheEntry* CurrentCacheEntry = MemberCache.GetData();

	bool bSomeMembersAreProperties = false;
	for (FMemberProperty& Member : Members)
	{
		const FNetSerializer* Serializer = Member.SerializerInfo->GetNetSerializer(Member.Property);
		CurrentCacheEntry->Serializer = Serializer;

		// Disqualify any member having a serializer with an Apply from being considered as AllMembersAreReplicated as we need to be sure Apply is called as expected.
		if (EnumHasAnyFlags(Serializer->Traits, ENetSerializerTraits::HasApply))
		{
			bAllMembersAreReplicated = false;
		}

		// This is a property based struct, make sure we have a descriptor for it.
		if (IsStructNetSerializer(Serializer))
		{
			bSomeMembersAreProperties = true;

			// Build descriptor for struct
			CurrentCacheEntry->Descriptor = GetDescriptorForStructProperty(Member, DescriptorRegistry);

			if (bWarnAboutStructPropertiesWithSuspectedNotReplicatedProperties)
			{
				ValidateStructPropertyDescriptor(Member, CurrentCacheEntry->Descriptor);
			}

			const FReplicationStateDescriptor* Descriptor = CurrentCacheEntry->Descriptor.GetReference();

			// Propagate some traits. Doing this earlier would be quite expensive.
			if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasObjectReference))
			{
				Member.Traits |= EMemberPropertyTraits::HasObjectReference;
			}

			if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState))
			{
				Member.Traits |= EMemberPropertyTraits::HasDynamicState;
			}

			if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasConnectionSpecificSerialization))
			{
				Member.Traits |= EMemberPropertyTraits::HasConnectionSpecificSerialization;
			}

			if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::UseSerializerIsEqual))
			{
				Member.Traits |= EMemberPropertyTraits::UseSerializerIsEqual;
			}

			if (!EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated))
			{
				// If we are building a struct containing a struct that isn't fully replicated then this struct isn't fully replicated either.
				bAllMembersAreReplicated = false;
			}

			/**
			  * Use ElementSize instead of GetSize() since GetSize() will multiply by array size and currently we treat
			  * each element individually. 
			  */
			CurrentCacheEntry->ExternalSizeAndAlignment = { (SIZE_T)Member.Property->GetElementSize(), (SIZE_T)Member.Property->GetMinAlignment() };
			CurrentCacheEntry->InternalSizeAndAlignment = { Descriptor->InternalSize, Descriptor->InternalAlignment };

			CurrentCacheEntry->bIsStruct = 1;
		}
		else if (IsArrayPropertyNetSerializer(Serializer))
		{
			bSomeMembersAreProperties = true;

			// Build descriptor for inner property
			CurrentCacheEntry->Descriptor = GetDescriptorForArrayProperty(Member.Property, DescriptorRegistry);

			const FReplicationStateDescriptor* Descriptor = CurrentCacheEntry->Descriptor.GetReference();

			// Propagate some traits. Doing this earlier would be quite expensive.

			if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasObjectReference))
			{
				Member.Traits |= EMemberPropertyTraits::HasObjectReference;
			}

			if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::UseSerializerIsEqual))
			{
				Member.Traits |= EMemberPropertyTraits::UseSerializerIsEqual;
			}

			// We propagate this up so that a struct wrapping an array that is not fully replicated will also have this set
			if (!EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated))
			{
				// If we are building a struct containing a struct that isn't fully replicated then this struct isn't fully replicated either.
				bAllMembersAreReplicated = false;
			}

			CurrentCacheEntry->ExternalSizeAndAlignment = { (SIZE_T)Member.Property->GetElementSize(), (SIZE_T)Member.Property->GetMinAlignment() };
			CurrentCacheEntry->InternalSizeAndAlignment = { Serializer->QuantizedTypeSize, Serializer->QuantizedTypeAlignment };

			CurrentCacheEntry->bIsDynamicArray = 1;
		}
		else if (IsOptionalPropertyNetSerializer(Serializer))
		{
			bSomeMembersAreProperties = true;

			// Build descriptor for inner property
			CurrentCacheEntry->Descriptor = GetDescriptorForOptionalProperty(Member.Property, DescriptorRegistry);

			const FReplicationStateDescriptor* Descriptor = CurrentCacheEntry->Descriptor.GetReference();

			// Propagate some traits. Doing this earlier would be quite expensive.
			if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasObjectReference))
			{
				Member.Traits |= EMemberPropertyTraits::HasObjectReference;
			}

			if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::UseSerializerIsEqual))
			{
				Member.Traits |= EMemberPropertyTraits::UseSerializerIsEqual;
			}

			// We propagate this up so that a optional with a struct that is not fully replicated will also have this set
			if (!EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::AllMembersAreReplicated))
			{
				bAllMembersAreReplicated = false;
			}

			CurrentCacheEntry->ExternalSizeAndAlignment = { (SIZE_T)Member.Property->GetElementSize(), (SIZE_T)Member.Property->GetMinAlignment() };
			CurrentCacheEntry->InternalSizeAndAlignment = { Serializer->QuantizedTypeSize, Serializer->QuantizedTypeAlignment };
		}
		else if (Member.Property)
		{
			bSomeMembersAreProperties = true;

			CurrentCacheEntry->ExternalSizeAndAlignment = { (SIZE_T)Member.Property->GetElementSize(), (SIZE_T)Member.Property->GetMinAlignment() };
			CurrentCacheEntry->InternalSizeAndAlignment = { Serializer->QuantizedTypeSize, Serializer->QuantizedTypeAlignment };
		}
		else
		{
			CurrentCacheEntry->ExternalSizeAndAlignment = Member.ExternalSizeAndAlignment;
			CurrentCacheEntry->InternalSizeAndAlignment = { Serializer->QuantizedTypeSize, Serializer->QuantizedTypeAlignment };
		}

		CurrentCacheEntry->SerializerConfigSizeAndAlignment = { Serializer->ConfigTypeSize, Serializer->ConfigTypeAlignment };
		CurrentCacheEntry->bUseSerializerDefaultConfig = Member.SerializerInfo->CanUseDefaultConfig(Member.Property);

		// Combined Traits. Traits can be propagated from structs so keep this near the end of the loop.
		PropertyTraits |= Member.Traits;
		SharedPropertyTraits &= Member.Traits;

		++CurrentCacheEntry;
	}

	Context.CombinedPropertyTraits = PropertyTraits;
	Context.SharedPropertyTraits = SharedPropertyTraits;

	// If some members are property based we generate the property descriptor.
	Context.bGeneratePropertyDescriptors = bSomeMembersAreProperties;

	// 
	Context.BuildParams.bAllMembersAreReplicated = bAllMembersAreReplicated;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberTagCache(FMemberTagCache& MemberTagCache, const FMemberCache& MemberCache) const
{
	const FMemberProperty* MemberProperties = Members.GetData();

	uint16 EntryIndex = 0;
	for (const FMemberCacheEntry& MemberCacheEntry : MemberCache)
	{
		const FMemberProperty& MemberProperty = MemberProperties[EntryIndex];

		// Find tags set on the member itself.
		{
			FRepTag Tag = GetRepTagFromProperty(MemberCacheEntry, MemberProperty);
			if (Tag != GetInvalidRepTag())
			{
				FMemberTagCacheEntry TagEntry;
				TagEntry.Tag = Tag;
				TagEntry.MemberIndex = EntryIndex;
				TagEntry.bInDescriptor = false;

				++MemberTagCache.TagCount;
				MemberTagCache.CacheEntries.Add(TagEntry);
			}
		}

		// If the member is a struct it can have tags on its members. We want to expose those as well.
		if (const FReplicationStateDescriptor* Descriptor = MemberCacheEntry.Descriptor)
		{
			// Pull out tags to our descriptor. We expect few tags and we want to find them as quickly as possible when needed.
			// We do NOT support a tag on something that require a separate descriptor. That would require us to store 
			// additional information per tag. Easy to implement but not a priority at the moment.
			if (!MemberCacheEntry.bIsDynamicArray && Descriptor->TagCount)
			{
				FMemberTagCacheEntry TagEntry;
				TagEntry.Tag = 0;
				TagEntry.MemberIndex = EntryIndex;
				TagEntry.bInDescriptor = true;

				MemberTagCache.TagCount += Descriptor->TagCount;
				MemberTagCache.CacheEntries.Add(TagEntry);
			}
		}

		++EntryIndex;
	}
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberReferenceCache(FMemberReferenceCache& MemberReferenceCache, const FMemberCache& MemberCache) const
{
	const FMemberProperty* MemberProperties = Members.GetData();

	uint32 EntryIndex = 0;
	for (const FMemberCacheEntry& MemberCacheEntry : MemberCache)
	{
		const FMemberProperty& MemberProperty = MemberProperties[EntryIndex];

		// If the member does not have any references we skip it
		if (!EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::HasObjectReference))
		{
			++EntryIndex;
			continue;
		}

		// If we are a struct we unroll references in the outer
		if (MemberCacheEntry.bIsStruct)
		{
			const FReplicationStateDescriptor* Descriptor = MemberCacheEntry.Descriptor;

			FMemberReferenceCacheEntry Entry;
		
			Entry.InMemberStateDescriptor = 1U;
			Entry.MemberIndex = EntryIndex;

			MemberReferenceCache.ReferenceCount += Descriptor->ObjectReferenceCount;
			MemberReferenceCache.CacheEntries.Add(Entry);
		}
		// For dynamic arrays and serializers with custom references we must always use the descriptor to get nested references
		else if (MemberCacheEntry.bIsDynamicArray || EnumHasAnyFlags(MemberCacheEntry.Serializer->Traits, ENetSerializerTraits::HasCustomNetReference))
		{
			FMemberReferenceCacheEntry Entry;
		
			Entry.InMemberStateDescriptor = 1U;
			Entry.MemberIndex = EntryIndex;

			++MemberReferenceCache.ReferenceCount;
			MemberReferenceCache.CacheEntries.Add(Entry);
		}
		else
		{
			FMemberReferenceCacheEntry Entry;
		
			Entry.InMemberStateDescriptor = 0U;
			Entry.MemberIndex = EntryIndex;

			++MemberReferenceCache.ReferenceCount;
			MemberReferenceCache.CacheEntries.Add(Entry);
		}
		++EntryIndex;
	}
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberFunctionCache(FBuilderContext& Context, FReplicationStateDescriptorRegistry* DescriptorRegistry) const
{
	FMemberFunctionCache& MemberFunctionCache = Context.MemberFunctionCache;
	MemberFunctionCache.SetNum(Functions.Num());
	
	FMemberFunctionCacheEntry* CurrentCacheEntry = MemberFunctionCache.GetData();
	for (const FMemberFunction& MemberFunction : Functions)
	{
		CurrentCacheEntry->Function = MemberFunction.Function;
		CurrentCacheEntry->Descriptor = GetOrCreateDescriptorForFunction(MemberFunction.Function, DescriptorRegistry);
		++CurrentCacheEntry;
	}
}

void FPropertyReplicationStateDescriptorBuilder::AllocateAndInitializeDefaultInternalStateBuffer(const uint8* RESTRICT SrcBuffer, const FReplicationStateDescriptor* Descriptor, UReplicationSystem* ReplicationSystem, const uint8*& OutDefaultStateBuffer) const
{
	// Allocate storage for incoming data, it will be freed when we destroy the descriptor
	uint8* DstStateBuffer = static_cast<uint8*>(FMemory::MallocZeroed(Descriptor->InternalSize, Descriptor->InternalAlignment));

	// We setup a default context, we can however not capture object references as they need to resolve to the same value
	// on both server and client
	FNetSerializationContext Context;
	FInternalNetSerializationContext InternalContext;
	if (ReplicationSystem)
	{
		InternalContext = FInternalNetSerializationContext(ReplicationSystem);
	}
	else
	{
		InternalContext.PackageMap = GetOrCreateIrisObjectReferencePackageMap();
	}
	Context.SetInternalContext(&InternalContext);
	Context.SetIsInitializingDefaultState(true);

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FProperty** MemberProperties = Descriptor->MemberProperties;
	const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = Descriptor->MemberPropertyDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
		const FProperty* Property = MemberProperties[MemberIt];

		FNetQuantizeArgs Args;
		Args.Version = 0;
		Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcBuffer + Property->GetOffset_ForGC() + Property->GetElementSize() * MemberPropertyDescriptor.ArrayIndex);
		Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstStateBuffer + MemberDescriptor.InternalMemberOffset);

		MemberSerializerDescriptor.Serializer->Quantize(Context, Args);
	}

	OutDefaultStateBuffer = DstStateBuffer;
}

bool FPropertyReplicationStateDescriptorBuilder::CalculateDefaultStateChecksum(const FReplicationStateDescriptor* Descriptor, UReplicationSystem* ReplicationSystem, const uint8* OutDefaultStateBuffer, uint64& OutHashValue) const
{
	const UPartialNetObjectAttachmentHandlerConfig* PartialNetObjectAttachmentHandlerConfig = GetDefault<UPartialNetObjectAttachmentHandlerConfig>();

	constexpr uint32 SmallBufferSize = 1024;
	const uint64 MaxBufferSize = PartialNetObjectAttachmentHandlerConfig->GetTotalMaxPayloadBitCount()/8U;
	check(MaxBufferSize/4 < std::numeric_limits<int32>::max());

	// Setup small buffer first as it might be written to when the writer's destructor is called.
	TArray<uint32, TInlineAllocator<SmallBufferSize/sizeof(uint32)>> TempBuffer;
	TempBuffer.SetNum(SmallBufferSize/sizeof(uint32));

	// We setup a default context
	FNetBitStreamWriter Writer;
	FNetSerializationContext Context(&Writer);
	FInternalNetSerializationContext InternalContext;
	if (ReplicationSystem)
	{
		InternalContext = FInternalNetSerializationContext(ReplicationSystem);
	}
	else
	{
		InternalContext.PackageMap = GetOrCreateIrisObjectReferencePackageMap();
	}
	Context.SetInternalContext(&InternalContext);

	// Tell serializers we are serializing default state. It allows serializers to opt out of being part of the checksum by simply not serializing any data.
	Context.SetIsInitializingDefaultState(true);

	// First try to serialize to small inline buffer
	Writer.InitBytes((uint8*)TempBuffer.GetData(), SmallBufferSize);
	FReplicationStateOperations::Serialize(Context, Descriptor->DefaultStateBuffer, Descriptor);

	// Try again with largest supported buffer
	if (Writer.IsOverflown())
	{
		TempBuffer.SetNum(static_cast<int32>(static_cast<int64>(MaxBufferSize/sizeof(uint32))));
		Writer.InitBytes((uint8*)TempBuffer.GetData(), static_cast<uint32>(MaxBufferSize));
		FReplicationStateOperations::Serialize(Context, Descriptor->DefaultStateBuffer, Descriptor);
	}

	// Make sure last byte has well defined data as we include it in hash
	if (const uint32 BitsToFill = (8U - (Writer.GetPosBits() & 7U)) & 7U)
	{
		Writer.WriteBits(0U, BitsToFill);
	}

	// State is too large, we cannot calculate a default hash for it
	if (Context.HasErrorOrOverflow())
	{
		return false;
	}

	Writer.CommitWrites();

	// Hash the serialized buffer
	OutHashValue = CityHash64((const char*)TempBuffer.GetData(), Writer.GetPosBytes());

	return true;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberDescriptors(FReplicationStateMemberDescriptor* MemberDescriptors, FReplicationStateDescriptor* Descriptor, FBuilderContext& Context) const
{
	const FMemberCache& MemberCache = Context.MemberCache;

	SIZE_T ExternalOffset = Context.External.Size;
	SIZE_T ExternalBufferAlignment = Context.External.Alignment;

	SIZE_T InternalOffset = Context.Internal.Size;
	SIZE_T InternalBufferAlignment = Context.Internal.Alignment;

	FReplicationStateMemberDescriptor* CurrentMemberDescriptor = MemberDescriptors;

	const FMemberCacheEntry* MemberCacheEntry = MemberCache.GetData();
	for (const FMemberProperty& Member : Members)
	{
		// External
		const SIZE_T ExternalMemberAlignment = MemberCacheEntry->ExternalSizeAndAlignment.Alignment;
		const SIZE_T ExternalMemberSize = MemberCacheEntry->ExternalSizeAndAlignment.Size;

		ExternalBufferAlignment = FMath::Max(ExternalBufferAlignment, ExternalMemberAlignment);
		ExternalOffset = Align(ExternalOffset, ExternalMemberAlignment);
		CurrentMemberDescriptor->ExternalMemberOffset = static_cast<uint32>(ExternalOffset);
		ExternalOffset += ExternalMemberSize;

		// Internal
		const FNetSerializer* MemberSerializer = MemberCacheEntry->Serializer;
		const SIZE_T InternalMemberAlignment = MemberCacheEntry->InternalSizeAndAlignment.Alignment;
		const SIZE_T InternalMemberSize = MemberCacheEntry->InternalSizeAndAlignment.Size;

		InternalBufferAlignment = FMath::Max(InternalBufferAlignment, InternalMemberAlignment);
		InternalOffset = Align(InternalOffset, InternalMemberAlignment);
		CurrentMemberDescriptor->InternalMemberOffset = static_cast<uint32>(InternalOffset);
		InternalOffset += InternalMemberSize;

		++CurrentMemberDescriptor;
		++MemberCacheEntry;
	}

	// Fill in descriptor
	Descriptor->MemberDescriptors = MemberDescriptors;
	Descriptor->MemberCount = static_cast<uint16>(Members.Num());

	// Update Context
	Context.External.Alignment = ExternalBufferAlignment;
	Context.External.Size = ExternalOffset;

	Context.Internal.Alignment = InternalBufferAlignment;
	Context.Internal.Size = InternalOffset;
}

/**
  * A struct's external state will be the same size as the struct. This in order to be able to handle rep notifies with the previous value as parameter.
  * That means we cannot compact the representation. We must use the offsets stored in the properties.
  */
void FPropertyReplicationStateDescriptorBuilder::BuildMemberDescriptorsForStruct(FReplicationStateMemberDescriptor* MemberDescriptors, FReplicationStateDescriptor* Descriptor, FBuilderContext& Context) const
{
	const FMemberCache& MemberCache = Context.MemberCache;

	SIZE_T ExternalOffset = Context.External.Size;
	// When the struct is a function we get incorrect information regarding alignment requirements. This means we have to calculate it ourselves.
	SIZE_T ExternalBufferAlignment = FMath::Max(Context.External.Alignment, StructInfo.SizeAndAlignment.Alignment);

	SIZE_T InternalOffset = Context.Internal.Size;
	SIZE_T InternalBufferAlignment = Context.Internal.Alignment;

	FReplicationStateMemberDescriptor* CurrentMemberDescriptor = MemberDescriptors;

	const FProperty* LastProperty = nullptr;
	uint16 ArrayIndex = 0;

	const FMemberCacheEntry* MemberCacheEntry = MemberCache.GetData();
	for (const FMemberProperty& Member : Members)
	{
		// External
		const FProperty* Property = Member.Property;

		ArrayIndex = (Property == LastProperty ? uint16(ArrayIndex + 1) : uint16(0));
		LastProperty = Property;

		ExternalBufferAlignment = FMath::Max(ExternalBufferAlignment, MemberCacheEntry->ExternalSizeAndAlignment.Alignment);
		CurrentMemberDescriptor->ExternalMemberOffset = Property ? (Property->GetOffset_ForGC() + (ArrayIndex * (uint32)Property->GetElementSize())) : 0U;

		// Internal
		const FNetSerializer* MemberSerializer = MemberCacheEntry->Serializer;
		const SIZE_T InternalMemberAlignment = MemberCacheEntry->InternalSizeAndAlignment.Alignment;
		const SIZE_T InternalMemberSize = MemberCacheEntry->InternalSizeAndAlignment.Size;

		InternalBufferAlignment = FMath::Max(InternalBufferAlignment, InternalMemberAlignment);
		InternalOffset = Align(InternalOffset, InternalMemberAlignment);
		CurrentMemberDescriptor->InternalMemberOffset = static_cast<uint32>(InternalOffset);
		InternalOffset += InternalMemberSize;

		++CurrentMemberDescriptor;
		++MemberCacheEntry;
	}

	// Fill in descriptor
	Descriptor->MemberDescriptors = MemberDescriptors;
	Descriptor->MemberCount = static_cast<uint16>(Members.Num());

	// Update Context
	Context.External.Alignment = FMath::Max(Context.External.Alignment, ExternalBufferAlignment);
	Context.External.Size += StructInfo.SizeAndAlignment.Size;

	Context.Internal.Alignment = InternalBufferAlignment;
	Context.Internal.Size = InternalOffset;
}

void FPropertyReplicationStateDescriptorBuilder::BuildReplicationStateInternal(FReplicationStateDescriptor* Descriptor, FBuilderContext& Context) const
{
	// Reserve space for internals
	const SIZE_T InternalStateAlignment = alignof(FReplicationStateHeader);
	const SIZE_T InternalStateOffset = Align(Context.External.Size, InternalStateAlignment);

	// This should be the first member of a replication state
	check(InternalStateOffset == 0);

	Context.External.Size = InternalStateOffset + sizeof(FReplicationStateHeader);
	Context.External.Alignment =  FMath::Max(Context.External.Alignment, InternalStateAlignment);
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberChangeMaskDescriptors(FReplicationStateMemberChangeMaskDescriptor* MemberChangeMaskDescriptors, FReplicationStateDescriptor* Descriptor, FBuilderContext& Context) const
{	
	check(Descriptor->MemberDescriptors == nullptr);

	uint32 CurrentBitOffset = 0;

	FReplicationStateMemberChangeMaskDescriptor* CurrentMemberChangeMaskDescriptor = MemberChangeMaskDescriptors;
	const bool bIsInitState = Context.IsInitState();

	struct FChangeMaskGroupInfo
	{
		FName ChangeMaskGroupName;
		FReplicationStateMemberChangeMaskDescriptor ChangeMaskDescriptor;
	};
	TArray<FChangeMaskGroupInfo, TInlineAllocator<16>> ChangemaskGroups;

	for (const FMemberProperty& Info : Members)
	{
		const uint16 BitCount = (bIsInitState ? 0U : Info.ChangeMaskBits);
		const FName ChangeMaskGroupName = Info.ChangeMaskGroupName;

		if (ChangeMaskGroupName.IsNone())
		{
			CurrentMemberChangeMaskDescriptor->BitCount = BitCount;
			CurrentMemberChangeMaskDescriptor->BitOffset = static_cast<uint16>(bIsInitState ? 0U : CurrentBitOffset);
			CurrentBitOffset += BitCount;
		}
		else if (FChangeMaskGroupInfo* SharedChangeMaskGroupInfo = ChangemaskGroups.FindByPredicate([ChangeMaskGroupName](const FChangeMaskGroupInfo& GroupInfo) { return GroupInfo.ChangeMaskGroupName == ChangeMaskGroupName; }))
		{
			*CurrentMemberChangeMaskDescriptor = SharedChangeMaskGroupInfo->ChangeMaskDescriptor;
		}
		else
		{
			CurrentMemberChangeMaskDescriptor->BitCount = BitCount;
			CurrentMemberChangeMaskDescriptor->BitOffset = static_cast<uint16>(bIsInitState ? 0U : CurrentBitOffset);
			CurrentBitOffset += BitCount;

			ChangemaskGroups.Add({ ChangeMaskGroupName, *CurrentMemberChangeMaskDescriptor });
		}
		++CurrentMemberChangeMaskDescriptor;
	}

	Descriptor->MemberChangeMaskDescriptors = MemberChangeMaskDescriptors;

	// Info about the state mask for this state
	const SIZE_T ChangeMaskMemberAlignment = alignof(FNetBitArrayView::StorageWordType);
	const SIZE_T ChangeMaskMemberSize = FNetBitArrayView::CalculateRequiredWordCount(CurrentBitOffset) * sizeof(FNetBitArrayView::StorageWordType);
	// If state has custom conditionals we need an additional changemask.
	const SIZE_T ConditionalChangeMaskMemberSize = (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals) ? ChangeMaskMemberSize : 0U);

	checkSlow(CurrentBitOffset <= std::numeric_limits<uint16>::max());

	Descriptor->ChangeMaskBitCount = static_cast<uint16>(CurrentBitOffset);
	Descriptor->ChangeMasksExternalOffset = static_cast<uint32>(Align(Context.External.Size, ChangeMaskMemberAlignment));

	// We also reserve room for the MemberPollMask
	const SIZE_T MemberPollMaskMemberSize = FNetBitArrayView::CalculateRequiredWordCount(Members.Num()) * sizeof(FNetBitArrayView::StorageWordType);

	// Update context
	Context.External.Alignment = FMath::Max(Context.External.Alignment, ChangeMaskMemberAlignment);
	Context.External.Size = Descriptor->ChangeMasksExternalOffset + ChangeMaskMemberSize + ConditionalChangeMaskMemberSize + MemberPollMaskMemberSize;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberSerializerDescriptors(FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors, FReplicationStateDescriptor* Descriptor) const
{
	check(Descriptor->MemberDescriptors);

	uint32 CurrentBitOffset = 0;

	FReplicationStateMemberSerializerDescriptor* CurrentMemberSerializerDescriptor = MemberSerializerDescriptors;	
	for (const FMemberProperty& Info : Members)
	{
		CurrentMemberSerializerDescriptor->Serializer = Info.SerializerInfo->GetNetSerializer(Info.Property);
		// The config is set in BuildMemberSerializerConfigs
		CurrentMemberSerializerDescriptor->SerializerConfig = nullptr;
		++CurrentMemberSerializerDescriptor;
	}

	// Fill in descriptor
	Descriptor->MemberSerializerDescriptors = MemberSerializerDescriptors;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberTraitsDescriptors(FReplicationStateMemberTraitsDescriptor* MemberTraitsDescriptors, FReplicationStateDescriptor* Descriptor, const FBuilderContext& Context) const
{
	FReplicationStateMemberTraitsDescriptor* CurrentMemberTraitsDescriptor = MemberTraitsDescriptors;	
	for (const FMemberProperty& Info : Members)
	{
		CurrentMemberTraitsDescriptor->Traits |= (EnumHasAnyFlags(Info.Traits, EMemberPropertyTraits::HasDynamicState) ? EReplicationStateMemberTraits::HasDynamicState : EReplicationStateMemberTraits::None);
		CurrentMemberTraitsDescriptor->Traits |= (EnumHasAnyFlags(Info.Traits, EMemberPropertyTraits::HasObjectReference) ? EReplicationStateMemberTraits::HasObjectReference : EReplicationStateMemberTraits::None);
		CurrentMemberTraitsDescriptor->Traits |= (EnumHasAnyFlags(Info.Traits, EMemberPropertyTraits::RepNotifyAlways) ? EReplicationStateMemberTraits::HasRepNotifyAlways : EReplicationStateMemberTraits::None);
		CurrentMemberTraitsDescriptor->Traits |= (EnumHasAnyFlags(Info.Traits, EMemberPropertyTraits::UseSerializerIsEqual) ? EReplicationStateMemberTraits::UseSerializerIsEqual : EReplicationStateMemberTraits::None);
		CurrentMemberTraitsDescriptor->Traits |= (EnumHasAnyFlags(Info.Traits, EMemberPropertyTraits::HasPushBasedDirtiness) ? EReplicationStateMemberTraits::HasPushBasedDirtiness : EReplicationStateMemberTraits::None);
		++CurrentMemberTraitsDescriptor;
	}

	// Fill in descriptor
	Descriptor->MemberTraitsDescriptors = MemberTraitsDescriptors;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberTagDescriptors(FReplicationStateMemberTagDescriptor* MemberTagDescriptors, FReplicationStateDescriptor* Descriptor, const FBuilderContext& Context) const
{
	check(Descriptor->MemberDescriptors);

	const FMemberCacheEntry* MemberCacheEntries = Context.MemberCache.GetData();
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;

	FReplicationStateMemberTagDescriptor* CurrentMemberTagDescriptor = MemberTagDescriptors;	
	for (const FMemberTagCacheEntry& TagEntry : Context.MemberTagCache.CacheEntries)
	{
		const FMemberCacheEntry& MemberEntry = MemberCacheEntries[TagEntry.MemberIndex];
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[TagEntry.MemberIndex];
		if (TagEntry.bInDescriptor)
		{
			// Copy tags from descriptor
			const FReplicationStateDescriptor* MemberStateDescriptor = MemberEntry.Descriptor; 
			for (uint16 TagIt = 0, TagEndIt = MemberStateDescriptor->TagCount; TagIt != TagEndIt; ++TagIt)
			{
				CurrentMemberTagDescriptor->Tag = MemberStateDescriptor->MemberTagDescriptors[TagIt].Tag;
				CurrentMemberTagDescriptor->MemberIndex = TagEntry.MemberIndex;
				CurrentMemberTagDescriptor->InnerTagIndex = TagIt;
				++CurrentMemberTagDescriptor;
			}
		}
		else
		{
			CurrentMemberTagDescriptor->Tag = TagEntry.Tag;
			CurrentMemberTagDescriptor->MemberIndex = TagEntry.MemberIndex;
			CurrentMemberTagDescriptor->InnerTagIndex = MAX_uint16;
			++CurrentMemberTagDescriptor;
		}
	}

	// Fill in descriptor
	ensureMsgf(Context.MemberTagCache.TagCount < MAX_uint16 - 1U, TEXT("Tag count cannot exceed %u. Building descriptor %s."), MAX_uint16 - 1U, ToCStr(Descriptor->DebugName));
	Descriptor->MemberTagDescriptors = MemberTagDescriptors;
	Descriptor->TagCount = static_cast<uint16>(Context.MemberTagCache.TagCount);
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberReferenceDescriptors(FReplicationStateMemberReferenceDescriptor* MemberReferenceDescriptors, FReplicationStateDescriptor* Descriptor, const FBuilderContext& Context) const
{
	check(Descriptor->MemberDescriptors);

	const FMemberCacheEntry* MemberCacheEntries = Context.MemberCache.GetData();
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;

	FReplicationStateMemberReferenceDescriptor* CurrentMemberReferenceDescriptor = MemberReferenceDescriptors;	
	for (const FMemberReferenceCacheEntry& Entry : Context.ReferenceCache.CacheEntries)
	{
		const FMemberCacheEntry& MemberEntry = MemberCacheEntries[Entry.MemberIndex];
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[Entry.MemberIndex];
		if (Entry.InMemberStateDescriptor)
		{
			if (MemberEntry.bIsStruct)
			{
				// Flatten out references from nested members with static storage
				const FReplicationStateDescriptor* MemberStateDescriptor = MemberEntry.Descriptor; 
				for (uint16 InnerIt = 0, InnerEndIt = MemberStateDescriptor->ObjectReferenceCount; InnerIt != InnerEndIt; ++InnerIt)
				{
					// Copy inner info
					CurrentMemberReferenceDescriptor->Info = MemberStateDescriptor->MemberReferenceDescriptors[InnerIt].Info;

					// Update offset to point to inner offset
					CurrentMemberReferenceDescriptor->Offset = MemberDescriptors[Entry.MemberIndex].InternalMemberOffset + MemberStateDescriptor->MemberReferenceDescriptors[InnerIt].Offset;

					CurrentMemberReferenceDescriptor->MemberIndex = static_cast<uint16>(Entry.MemberIndex);
					CurrentMemberReferenceDescriptor->InnerReferenceIndex = InnerIt;

					++CurrentMemberReferenceDescriptor;
				}
			}
			else if (MemberEntry.bIsDynamicArray || EnumHasAnyFlags(MemberEntry.Serializer->Traits, ENetSerializerTraits::HasCustomNetReference))
			{
				// store entry indicating that this is a dynamic array that have references
				// We use an invalid resolve-type to indicate that we need to look this up from dynamic data
				CurrentMemberReferenceDescriptor->Info.ResolveType = FNetReferenceInfo::EResolveType::Invalid;

				// Offset to member containing data about the array
				CurrentMemberReferenceDescriptor->Offset = MemberDescriptors[Entry.MemberIndex].InternalMemberOffset;
	
				CurrentMemberReferenceDescriptor->MemberIndex = static_cast<uint16>(Entry.MemberIndex);
				CurrentMemberReferenceDescriptor->InnerReferenceIndex = MAX_uint16;

				++CurrentMemberReferenceDescriptor;
			}
		}
		else
		{
			CurrentMemberReferenceDescriptor->Info.ResolveType = FNetReferenceInfo::EResolveType::ResolveOnClient;

			// Offset to reference
			CurrentMemberReferenceDescriptor->Offset = MemberDescriptors[Entry.MemberIndex].InternalMemberOffset;

			CurrentMemberReferenceDescriptor->MemberIndex = static_cast<uint16>(Entry.MemberIndex);
			CurrentMemberReferenceDescriptor->InnerReferenceIndex = MAX_uint16;
			++CurrentMemberReferenceDescriptor;
		}
	}

	// Fill in descriptor
	ensureMsgf(Context.ReferenceCache.ReferenceCount < std::numeric_limits<uint16>::max(), TEXT("Object reference count cannot exceed %u. Building descriptor %s."), std::numeric_limits<uint16>::max(), ToCStr(Descriptor->DebugName));
	Descriptor->MemberReferenceDescriptors = MemberReferenceDescriptors;
	Descriptor->ObjectReferenceCount = static_cast<uint16>(Context.ReferenceCache.ReferenceCount);
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberFunctionDescriptors(FReplicationStateMemberFunctionDescriptor* MemberFunctionDescriptors, FReplicationStateDescriptor* Descriptor, const FBuilderContext& Context) const
{
	FReplicationStateMemberFunctionDescriptor* CurrentMemberFunctionDescriptor = MemberFunctionDescriptors;
	for (const FMemberFunctionCacheEntry& MemberFunction : Context.MemberFunctionCache)
	{
		CurrentMemberFunctionDescriptor->Function = MemberFunction.Function;
		CurrentMemberFunctionDescriptor->Descriptor = MemberFunction.Descriptor;
		// We store raw pointers in the descriptor that gets Released() when the descriptor is released.
		CurrentMemberFunctionDescriptor->Descriptor->AddRef();
		++CurrentMemberFunctionDescriptor;

		UE_LOG_DESCRIPTORBUILDER(Verbose, TEXT("FPropertyReplicationStateDescriptorBuilder::BuildMemberFunctionDescriptors AddingFunction %s"), ToCStr(MemberFunction.Function->GetName()));		
	}

	ensureMsgf(Context.MemberFunctionCache.Num() <= std::numeric_limits<uint16>::max(), TEXT("Function count cannot exceed %u. Building descriptor %s."), std::numeric_limits<uint16>::max(), ToCStr(Descriptor->DebugName));
	Descriptor->FunctionCount = static_cast<uint16>(Context.MemberFunctionCache.Num());
	Descriptor->MemberFunctionDescriptors = MemberFunctionDescriptors;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberProperties(const FProperty** MemberProperties, FReplicationStateDescriptor* Descriptor) const
{
	const FProperty** CurrentMemberProperty = MemberProperties;
	for (const FMemberProperty& Info : Members)
	{		
		*CurrentMemberProperty = Info.Property;

		++CurrentMemberProperty;
	}

	// Fill in descriptor
	Descriptor->MemberProperties = MemberProperties;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberPropertyDescriptors(FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors, FReplicationStateDescriptor* Descriptor) const
{
	FReplicationStateMemberPropertyDescriptor* CurrentMemberPropertyDescriptor = MemberPropertyDescriptors;

	const FProperty* LastProperty = nullptr;
	uint16 ArrayIndex = 0;
	for (const FMemberProperty& Info : Members)
	{
		ArrayIndex = (LastProperty && Info.Property == LastProperty ? uint16(ArrayIndex + 1) : uint16(0));
		LastProperty = Info.Property;

		CurrentMemberPropertyDescriptor->RepNotifyFunction = Info.PropertyRepNotifyFunction;
		CurrentMemberPropertyDescriptor->ArrayIndex = ArrayIndex;
		++CurrentMemberPropertyDescriptor;
	}

	// Fill in descriptor
	Descriptor->MemberPropertyDescriptors = MemberPropertyDescriptors;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberLifetimeConditionDescriptors(FReplicationStateMemberLifetimeConditionDescriptor* MemberLifetimeConditionDescriptors, FReplicationStateDescriptor* Descriptor) const
{
	FReplicationStateMemberLifetimeConditionDescriptor* CurrentMemberLifetimeConditionDescriptors = MemberLifetimeConditionDescriptors;
	for (const FMemberProperty& Info : Members)
	{
		CurrentMemberLifetimeConditionDescriptors->Condition = static_cast<int8>(Info.ReplicationCondition);
		++CurrentMemberLifetimeConditionDescriptors;
	}

	Descriptor->MemberLifetimeConditionDescriptors = MemberLifetimeConditionDescriptors;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberRepIndexToMemberDescriptors(FReplicationStateMemberRepIndexToMemberIndexDescriptor* MemberRepIndexToMemberIndexDescriptors, FReplicationStateDescriptor* Descriptor) const
{
	const uint32 RepIndexCount = GetRepIndexCount(MakeConstArrayView(Members));

	// Faster way of setting all the MemberIndex values to InvalidEntry
	{
		static_assert(FReplicationStateMemberRepIndexToMemberIndexDescriptor::InvalidEntry == 65535U, "");
		FPlatformMemory::Memset(MemberRepIndexToMemberIndexDescriptors, 0xFF, RepIndexCount*sizeof(FReplicationStateMemberRepIndexToMemberIndexDescriptor));
	}

	const FProperty* LastProperty = nullptr;
	uint16 ArrayIndex = 0;
	for (const FMemberProperty& Info : Members)
	{
		ArrayIndex = (LastProperty && Info.Property == LastProperty ? uint16(ArrayIndex + 1) : uint16(0));
		LastProperty = Info.Property;
		CA_ASSUME(Info.Property != nullptr);

		const SIZE_T MemberIndex = &Info - Members.GetData();
		const uint16 RepIndex = Info.Property->RepIndex + ArrayIndex;
		checkSlow(RepIndex < RepIndexCount);
		MemberRepIndexToMemberIndexDescriptors[RepIndex].MemberIndex = static_cast<uint16>(MemberIndex);
	}

	Descriptor->MemberRepIndexToMemberIndexDescriptors = MemberRepIndexToMemberIndexDescriptors;
	Descriptor->RepIndexCount = static_cast<uint16>(RepIndexCount);
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberDebugDescriptors(FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors, FReplicationStateDescriptor* Descriptor, const FBuilderContext& Context) const
{
	FReplicationStateMemberDebugDescriptor* CurrentMemberDebugDescriptor = MemberDebugDescriptors;

	TCHAR StringBuffer[FName::StringBufferSize];
	// Structs derived from a struct with a custom NetSerializer can contain some replicated properties.
	for (const FMemberProperty& Info : Members)
	{
		if (const FProperty* Property = Info.Property)
		{
			Property->GetFName().ToString(StringBuffer);
			CurrentMemberDebugDescriptor->DebugName = CreatePersistentNetDebugName(StringBuffer);
		}
		else
		{
			// If we dont have a property tied to the member we use the debugname of the descriptor
			CurrentMemberDebugDescriptor->DebugName = Descriptor->DebugName;

		}

		++CurrentMemberDebugDescriptor;
	}

	// Fill in descriptor
	Descriptor->MemberDebugDescriptors = MemberDebugDescriptors;
}

void FPropertyReplicationStateDescriptorBuilder::AddSerializerConfigMemoryBlockToLayout(FMemoryLayoutUtil::FLayout& Layout, FOffsetAndSize& OutOffsetAndSize, const FMemberCache& MemberCache) const
{
	SIZE_T OffsetOfFirstSerializerWithCustomConfig = ~SIZE_T(0);

	FOffsetAndSize DummyConfigOffsetAndSize;
	for (const FMemberCacheEntry& MemberCacheEntry : MemberCache)
	{
		if (MemberCacheEntry.bUseSerializerDefaultConfig)
		{
			continue;
		}

		// Store offset to config of first member in need of a custom config
		if (OffsetOfFirstSerializerWithCustomConfig == ~SIZE_T(0))
		{
			OffsetOfFirstSerializerWithCustomConfig = Align(Layout.CurrentOffset, MemberCacheEntry.SerializerConfigSizeAndAlignment.Alignment);
		}
		
		FMemoryLayoutUtil::AddToLayout(Layout, DummyConfigOffsetAndSize, MemberCacheEntry.SerializerConfigSizeAndAlignment.Size, MemberCacheEntry.SerializerConfigSizeAndAlignment.Alignment, 1);
	}

	// Calculate size of the section
	OutOffsetAndSize.Offset = (OffsetOfFirstSerializerWithCustomConfig == ~SIZE_T(0) ? Layout.CurrentOffset : OffsetOfFirstSerializerWithCustomConfig);
	OutOffsetAndSize.Size = Layout.CurrentOffset - OutOffsetAndSize.Offset;
}

void FPropertyReplicationStateDescriptorBuilder::BuildMemberSerializerConfigs(uint8* SerializerConfigBuffer, FReplicationStateDescriptor* Descriptor, const FMemberCache& MemberCache) const
{
	const FMemberCacheEntry* MemberCacheEntry = MemberCache.GetData();
	FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptor = const_cast<FReplicationStateMemberSerializerDescriptor*>(Descriptor->MemberSerializerDescriptors);

	for (const FMemberProperty& Member : Members)
	{
		// Construct config in provided memory and update descriptor
		if (MemberCacheEntry->bUseSerializerDefaultConfig)
		{
			checkf(MemberCacheEntry->Serializer->DefaultConfig != nullptr, TEXT("Expected NetSerializer %s to have a default config for property %s::%s"), MemberCacheEntry->Serializer->Name, *Member.Property->GetOwner<UObject>()->GetFName().ToString(), *Member.Property->GetFName().ToString());
			MemberSerializerDescriptor->SerializerConfig = MemberCacheEntry->Serializer->DefaultConfig;
		}
		else
		{
			SerializerConfigBuffer = Align(SerializerConfigBuffer, MemberCacheEntry->SerializerConfigSizeAndAlignment.Alignment);

			// Special case for structs
			if (IsStructNetSerializer(MemberCacheEntry->Serializer))
			{
				FStructNetSerializerConfig* SerializerConfig = new (SerializerConfigBuffer) FStructNetSerializerConfig();
				SerializerConfig->StateDescriptor = MemberCacheEntry->Descriptor;
				MemberSerializerDescriptor->SerializerConfig = SerializerConfig;
			}
			else if (IsArrayPropertyNetSerializer(MemberCacheEntry->Serializer))
			{
				FArrayPropertyNetSerializerConfig* SerializerConfig = new (SerializerConfigBuffer) FArrayPropertyNetSerializerConfig();
				SerializerConfig->MaxElementCount = 65535U;
				SerializerConfig->ElementCountBitCount = static_cast<uint16>(GetBitsNeeded(SerializerConfig->MaxElementCount));
				SerializerConfig->Property = TFieldPath<FArrayProperty>(const_cast<FArrayProperty*>(CastField<FArrayProperty>(Member.Property)));
				SerializerConfig->StateDescriptor = MemberCacheEntry->Descriptor;
				MemberSerializerDescriptor->SerializerConfig = SerializerConfig;
			}
			else if (IsOptionalPropertyNetSerializer(MemberCacheEntry->Serializer))
			{
				FOptionalPropertyNetSerializerConfig* SerializerConfig = new (SerializerConfigBuffer) FOptionalPropertyNetSerializerConfig();
				SerializerConfig->StateDescriptor = MemberCacheEntry->Descriptor;
				SerializerConfig->Property = TFieldPath<FOptionalProperty>(const_cast<FOptionalProperty*>(CastField<FOptionalProperty>(Member.Property)));
				MemberSerializerDescriptor->SerializerConfig = SerializerConfig;				
			}
			else
			{
				const FNetSerializerConfig* SerializerConfig = Member.SerializerInfo->BuildNetSerializerConfig(SerializerConfigBuffer, Member.Property);
				MemberSerializerDescriptor->SerializerConfig = SerializerConfig;
			}

			SerializerConfigBuffer += MemberCacheEntry->SerializerConfigSizeAndAlignment.Size;
		}

		++MemberCacheEntry;
		++MemberSerializerDescriptor;
	}	
}

// Need to do this to support role swapping.
void FPropertyReplicationStateDescriptorBuilder::FixupNetRoleNetSerializerConfigs(FReplicationStateDescriptor* Descriptor)
{
	FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = const_cast<FReplicationStateMemberSerializerDescriptor*>(Descriptor->MemberSerializerDescriptors);

	uint32 FirstNetRoleMemberIndex = ~0U;
	uint32 SecondNetRoleMemberIndex = ~0U;

	for (const FReplicationStateMemberSerializerDescriptor& SerializerDescriptor : MakeArrayView(MemberSerializerDescriptors, Descriptor->MemberCount))
	{
		if (SerializerDescriptor.Serializer == &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer))
		{
			const uint32 MemberIndex = static_cast<uint32>(&SerializerDescriptor - MemberSerializerDescriptors);
			if (FirstNetRoleMemberIndex == ~0U)
			{
				FirstNetRoleMemberIndex = MemberIndex;
			}
			else
			{
				SecondNetRoleMemberIndex = MemberIndex;
				break;
			}
		}
	}

	if (SecondNetRoleMemberIndex != ~0U)
	{
		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const int32 InternalOffsetFromFirstToSecondNetRole = MemberDescriptors[SecondNetRoleMemberIndex].InternalMemberOffset - MemberDescriptors[FirstNetRoleMemberIndex].InternalMemberOffset;
		const int32 ExternalOffsetFromFirstToSecondNetRole = MemberDescriptors[SecondNetRoleMemberIndex].ExternalMemberOffset - MemberDescriptors[FirstNetRoleMemberIndex].ExternalMemberOffset;

		FReplicationStateMemberSerializerDescriptor& FirstSerializerDescriptor = MemberSerializerDescriptors[FirstNetRoleMemberIndex];
		FReplicationStateMemberSerializerDescriptor& SecondSerializerDescriptor = MemberSerializerDescriptors[SecondNetRoleMemberIndex];

		FNetRoleNetSerializerConfig* FirstConfig = static_cast<FNetRoleNetSerializerConfig*>(const_cast<FNetSerializerConfig*>(FirstSerializerDescriptor.SerializerConfig));
		FNetRoleNetSerializerConfig* SecondConfig = static_cast<FNetRoleNetSerializerConfig*>(const_cast<FNetSerializerConfig*>(SecondSerializerDescriptor.SerializerConfig));

		FirstConfig->RelativeInternalOffsetToOtherRole = InternalOffsetFromFirstToSecondNetRole;
		FirstConfig->RelativeExternalOffsetToOtherRole = ExternalOffsetFromFirstToSecondNetRole;

		SecondConfig->RelativeInternalOffsetToOtherRole = -InternalOffsetFromFirstToSecondNetRole;
		SecondConfig->RelativeExternalOffsetToOtherRole = -ExternalOffsetFromFirstToSecondNetRole;
	}
}

void FPropertyReplicationStateDescriptorBuilder::FixupDescriptorForNativeFastArray(FReplicationStateDescriptor* Descriptor) const
{
	// Native fast arrays quantized directly from the src state data which is the fastarray itself so the offset must be set to zero.
	const_cast<FReplicationStateMemberDescriptor*>(&Descriptor->MemberDescriptors[0])->ExternalMemberOffset = 0U;
	Descriptor->ChangeMasksExternalOffset = static_cast<uint32>(GetFastArrayChangeMaskOffset(Descriptor->MemberProperties[0]));
}

EReplicationStateTraits FPropertyReplicationStateDescriptorBuilder::BuildReplicationStateTraits(const FBuilderContext& Context) const
{
	const EMemberPropertyTraits PropertyTraits = Context.CombinedPropertyTraits;
	const EMemberPropertyTraits SharedPropertyTraits = Context.SharedPropertyTraits;
	const bool bIsInitState = Context.IsInitState();

	EReplicationStateTraits Traits = EReplicationStateTraits::None;

	// Init state does not support conditionals as it has no changemask.
	if (EnumHasAnyFlags(PropertyTraits, EMemberPropertyTraits::HasLifetimeConditionals) && !bIsInitState)
	{
		Traits |= EReplicationStateTraits::HasLifetimeConditionals;
	}

	if (bIsInitState)
	{
		Traits |= EReplicationStateTraits::InitOnly;
	}

	// Has Rep notifies
	if (EnumHasAnyFlags(PropertyTraits, EMemberPropertyTraits::RepNotifyAlways | EMemberPropertyTraits::RepNotifyOnChanged))
	{
		Traits |= EReplicationStateTraits::HasRepNotifies;

		// Do we need to keep a copy of the previous state for rep notifies?
		if (EnumHasAnyFlags(PropertyTraits, EMemberPropertyTraits::NeedPreviousState))
		{
			Traits |= EReplicationStateTraits::KeepPreviousState;
		}
	}

	if (EnumHasAnyFlags(PropertyTraits, EMemberPropertyTraits::HasDynamicState))
	{
		Traits |= EReplicationStateTraits::HasDynamicState;
	}

	if (EnumHasAnyFlags(PropertyTraits, EMemberPropertyTraits::HasObjectReference))
	{
		Traits |= EReplicationStateTraits::HasObjectReference;
	}

	if (EnumHasAnyFlags(PropertyTraits, EMemberPropertyTraits::HasConnectionSpecificSerialization))
	{
		Traits |= EReplicationStateTraits::HasConnectionSpecificSerialization;
	}

	if (EnumHasAnyFlags(PropertyTraits, EMemberPropertyTraits::UseSerializerIsEqual))
	{
		Traits |= EReplicationStateTraits::UseSerializerIsEqual;
	}
	
	if (EnumHasAnyFlags(PropertyTraits, EMemberPropertyTraits::HasPushBasedDirtiness))
	{
		Traits |= EReplicationStateTraits::HasPushBasedDirtiness;

		// Only set the HasFullPushBasedDirtiness trait if all properties are push based. An alternative could be to split into multiple states.
		if (EnumHasAnyFlags(SharedPropertyTraits, EMemberPropertyTraits::HasPushBasedDirtiness))
		{
			Traits |= EReplicationStateTraits::HasFullPushBasedDirtiness;
		}
	}

	// Struct traits
	if (Context.BuildParams.DescriptorType == EDescriptorType::Struct)
	{
		if (StructInfo.SerializerType == EStructNetSerializerType::DerivedFromCustom)
		{
			Traits |= EReplicationStateTraits::IsDerivedFromStructWithCustomSerializer;
		}
		else if (StructInfo.SerializerType == EStructNetSerializerType::Custom)
		{
			Traits |= EReplicationStateTraits::IsStructWithCustomSerializer;
		}
	}

	// Special traits when all members are replicated
	if (Context.BuildParams.bAllMembersAreReplicated)
	{
		const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(StructInfo.Struct);
		
		// We cannot determine whether all properties are trivially constructible or destructible unless they're all replicated.
		if (EnumHasAnyFlags(SharedPropertyTraits, EMemberPropertyTraits::IsSourceTriviallyConstructible))
		{
			const bool bPropagateSourceIsTrivallyConstructible = !ScriptStruct || ((ScriptStruct->StructFlags & (STRUCT_IsPlainOldData | STRUCT_ZeroConstructor)) != STRUCT_NoFlags);
			if (bPropagateSourceIsTrivallyConstructible)
			{
				Traits |= EReplicationStateTraits::IsSourceTriviallyConstructible;
			}
		}

		if (EnumHasAnyFlags(SharedPropertyTraits, EMemberPropertyTraits::IsSourceTriviallyDestructible))
		{
			const bool bPropagateSourceIsSourceTriviallyDestructible = !ScriptStruct || ((ScriptStruct->StructFlags & (STRUCT_IsPlainOldData | STRUCT_NoDestructor)) != STRUCT_NoFlags);
			if (bPropagateSourceIsSourceTriviallyDestructible)
			{
				Traits |= EReplicationStateTraits::IsSourceTriviallyDestructible;
			}
		}

		Traits |= EReplicationStateTraits::AllMembersAreReplicated;
	}

	if (EnumHasAnyFlags(SharedPropertyTraits, EMemberPropertyTraits::IsFastArray) && Members.Num() == 1)
	{
		Traits |= EReplicationStateTraits::IsFastArrayReplicationState;
		Traits |= EnumHasAnyFlags(PropertyTraits, EMemberPropertyTraits::IsNativeFastArray) ? EReplicationStateTraits::IsNativeFastArrayReplicationState : EReplicationStateTraits::None;
	}

	// There might be cases where there are members but none of them consumes bandwidth.
	// It's ok to mark such states as supporting delta compression even though it makes little sense.
	// At this stage it's not possible to know it.
	if (!bIsInitState && Context.BuildParams.DescriptorType != EDescriptorType::Function && Members.Num() > 0)
	{
		Traits |= EReplicationStateTraits::SupportsDeltaCompression;
	}

	// For the main replication state for functions we do EReplicationStateTraits::HasPushBasedDirtiness even though it does not have any members.
	if (Context.MemberFunctionCache.Num() && Members.Num() == 0)
	{
		Traits |= EReplicationStateTraits::HasPushBasedDirtiness;
		Traits |= EReplicationStateTraits::HasFullPushBasedDirtiness;
	}

	Traits |= EReplicationStateTraits::NeedsRefCount;

	return Traits;
}

void FPropertyReplicationStateDescriptorBuilder::FinalizeDescriptor(FReplicationStateDescriptor* Descriptor, FBuilderContext& Context) const
{
	const SIZE_T AlignedExternalSize = Align(Context.External.Size, Context.External.Alignment);
	const SIZE_T AlignedInternalSize = Align(Context.Internal.Size, Context.Internal.Alignment);

	checkSlow(Context.External.Alignment < std::numeric_limits<uint16>::max());
	checkSlow(Context.Internal.Alignment < std::numeric_limits<uint16>::max());
	checkSlow(AlignedExternalSize < std::numeric_limits<uint32>::max());
	checkSlow(AlignedInternalSize < std::numeric_limits<uint32>::max());

	Descriptor->ExternalAlignment = static_cast<uint16>(Context.External.Alignment);
	Descriptor->ExternalSize = static_cast<uint32>(AlignedExternalSize);

	Descriptor->InternalAlignment = static_cast<uint16>(Context.Internal.Alignment);
	Descriptor->InternalSize = static_cast<uint32>(AlignedInternalSize);
}

TRefCountPtr<const FReplicationStateDescriptor>
FPropertyReplicationStateDescriptorBuilder::Build(const FString& StateName, FReplicationStateDescriptorRegistry* DescriptorRegistry, const FBuildParameters& BuildParams)
{
	// Cache information required to calculate layout
	const SIZE_T MemberCount = Members.Num();
	const SIZE_T FunctionCount = Functions.Num();

	checkSlow(MemberCount < std::numeric_limits<uint16>::max());
	checkSlow(FunctionCount < std::numeric_limits<uint16>::max());

	FBuilderContext Context;
	Context.BuildParams = BuildParams;
	BuildMemberCache(Context, DescriptorRegistry);
	BuildMemberReferenceCache(Context.ReferenceCache, Context.MemberCache);
	BuildMemberTagCache(Context.MemberTagCache, Context.MemberCache);
	BuildMemberFunctionCache(Context, DescriptorRegistry);

	// We need to setup the traits early on as we use them to decide what sections we need to include in the descriptor
	const EReplicationStateTraits Traits = BuildReplicationStateTraits(Context);

	// Layout for our memory allocation
	struct FReplicationDescriptorLayout
	{
		FOffsetAndSize ReplicationStateDescriptorSizeAndOffset;
		FOffsetAndSize MemberDescriptorSizeAndOffset;
		FOffsetAndSize MemberChangeMaskDescriptorSizeAndOffset;	
		FOffsetAndSize MemberSerializerDescriptorSizeAndOffset;
		FOffsetAndSize MemberTraitsDescriptorSizeAndOffset;
		FOffsetAndSize MemberTagDescriptorSizeAndOffset;
		FOffsetAndSize MemberReferenceDescriptorSizeAndOffset;
		FOffsetAndSize MemberFunctionDescriptorSizeAndOffset;
		FOffsetAndSize MemberPropertiesSizeAndOffset;
		FOffsetAndSize MemberPropertyDescriptorsSizeAndOffset;
		FOffsetAndSize MemberLifetimeConditionDescriptorsSizeAndOffset;
		FOffsetAndSize MemberRepIndexToMemberIndexDescriptorsSizeAndOffset;
		FOffsetAndSize MemberDebugDescriptorsSizeAndOffset;
		FOffsetAndSize MemberSerializerConfigSizeAndOffset;
	};
	FReplicationDescriptorLayout LayoutData = {};
	
	// Memory layout 
	FMemoryLayoutUtil::FLayout Layout;

	// The descriptor itself
	FMemoryLayoutUtil::AddToLayout<FReplicationStateDescriptor>(Layout, LayoutData.ReplicationStateDescriptorSizeAndOffset, 1);

	// MemberDescriptors
	FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberDescriptor>(Layout, LayoutData.MemberDescriptorSizeAndOffset, MemberCount);

	// MemberChangeMaskDescriptors
	if (Context.BuildParams.DescriptorType == FPropertyReplicationStateDescriptorBuilder::EDescriptorType::Class)
	{
		FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberChangeMaskDescriptor>(Layout, LayoutData.MemberChangeMaskDescriptorSizeAndOffset, MemberCount);
	}

	// MemberSerializerDescriptor
	FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberSerializerDescriptor>(Layout, LayoutData.MemberSerializerDescriptorSizeAndOffset, MemberCount);

	// MemberTraitsDescriptor
	FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberTraitsDescriptor>(Layout, LayoutData.MemberTraitsDescriptorSizeAndOffset, MemberCount);

	// MemberTagDescriptor
	FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberTagDescriptor>(Layout, LayoutData.MemberTagDescriptorSizeAndOffset, Context.MemberTagCache.TagCount);

	// MemberReferenceDescriptor
	FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberReferenceDescriptor>(Layout, LayoutData.MemberReferenceDescriptorSizeAndOffset, Context.ReferenceCache.ReferenceCount);

	// MemberFunctionDescriptors
	FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberFunctionDescriptor>(Layout, LayoutData.MemberFunctionDescriptorSizeAndOffset, FunctionCount);

	// MemberProperties
	if (Context.bGeneratePropertyDescriptors)
	{
		FMemoryLayoutUtil::AddToLayout<FProperty*>(Layout, LayoutData.MemberPropertiesSizeAndOffset, MemberCount);

		// MemberPropertyDescriptors
		FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberPropertyDescriptor>(Layout, LayoutData.MemberPropertyDescriptorsSizeAndOffset, MemberCount);
	}

	// MemberLifetimeConditionDescriptors
	if (EnumHasAnyFlags(Traits, EReplicationStateTraits::HasLifetimeConditionals))
	{
		FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberLifetimeConditionDescriptor>(Layout, LayoutData.MemberLifetimeConditionDescriptorsSizeAndOffset, MemberCount);
	}

	// MemberRepIndexToMemberIndexDescriptors
	if (Context.BuildParams.DescriptorType == EDescriptorType::Class || Context.BuildParams.DescriptorType == EDescriptorType::SingleProperty)
	{
		if (Members.Num() > 0)
		{
			const uint32 RepIndexCount = GetRepIndexCount(MakeConstArrayView(Members));
			FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberRepIndexToMemberIndexDescriptor>(Layout, LayoutData.MemberRepIndexToMemberIndexDescriptorsSizeAndOffset, RepIndexCount);
		}
	}

	// $TODO: generate this based on config/define 
	if (1)
	{
		FMemoryLayoutUtil::AddToLayout<FReplicationStateMemberDebugDescriptor>(Layout, LayoutData.MemberDebugDescriptorsSizeAndOffset, MemberCount);
	}

	// Reserve space all NetSerializerConfig instances we need to allocate
	AddSerializerConfigMemoryBlockToLayout(Layout, LayoutData.MemberSerializerConfigSizeAndOffset, Context.MemberCache);

	// Allocate required memory for descriptor and space for all sections
	uint8* Buffer = (uint8*)FMemory::Malloc(Layout.CurrentOffset, static_cast<uint32>(Layout.MaxAlignment));

	if (Buffer)
	{
		FMemory::Memzero(Buffer, Layout.CurrentOffset);
	
		FReplicationStateDescriptor* Descriptor = new (Buffer + LayoutData.ReplicationStateDescriptorSizeAndOffset.Offset) FReplicationStateDescriptor;

		check((void*)Descriptor == (void*)Buffer);
		
		// DebugName is handy for logging.
		Descriptor->DebugName = CreatePersistentNetDebugName(StateName.GetCharArray().GetData());

		// Setup Traits early on
		Descriptor->Traits = Traits;

		// Setup descriptor
		if (Context.BuildParams.DescriptorType == FPropertyReplicationStateDescriptorBuilder::EDescriptorType::Class)
		{
			// Reserve space for internal state data
			BuildReplicationStateInternal(Descriptor, Context);

			// Build member statemask descriptors
			BuildMemberChangeMaskDescriptors(reinterpret_cast<FReplicationStateMemberChangeMaskDescriptor*>(Buffer + LayoutData.MemberChangeMaskDescriptorSizeAndOffset.Offset), Descriptor, Context);
		}
			
		// Build member descriptors
		{
			// There's some logic in here that need to be executed even if there are zero replicated members
			FReplicationStateMemberDescriptor* MemberDescriptor = nullptr;
			if (LayoutData.MemberDescriptorSizeAndOffset.Size > 0)
			{
				MemberDescriptor = reinterpret_cast<FReplicationStateMemberDescriptor*>(Buffer + LayoutData.MemberDescriptorSizeAndOffset.Offset);
			}

			if ((Context.BuildParams.DescriptorType == EDescriptorType::Struct) || (Context.BuildParams.DescriptorType == EDescriptorType::Function))
			{
				BuildMemberDescriptorsForStruct(MemberDescriptor, Descriptor, Context);
			}
			else
			{
				BuildMemberDescriptors(MemberDescriptor, Descriptor, Context);
			}
		}

		// Build member serializer descriptors
		if (LayoutData.MemberSerializerDescriptorSizeAndOffset.Size > 0)
		{
			BuildMemberSerializerDescriptors(reinterpret_cast<FReplicationStateMemberSerializerDescriptor*>(Buffer + LayoutData.MemberSerializerDescriptorSizeAndOffset.Offset), Descriptor);
		}
	
		// Build member traits descriptors
		if (LayoutData.MemberTraitsDescriptorSizeAndOffset.Size > 0)
		{
			BuildMemberTraitsDescriptors(reinterpret_cast<FReplicationStateMemberTraitsDescriptor*>(Buffer + LayoutData.MemberTraitsDescriptorSizeAndOffset.Offset), Descriptor, Context);
		}

		// Build member RepTag descriptors
		if (LayoutData.MemberTagDescriptorSizeAndOffset.Size > 0)
		{
			BuildMemberTagDescriptors(reinterpret_cast<FReplicationStateMemberTagDescriptor*>(Buffer + LayoutData.MemberTagDescriptorSizeAndOffset.Offset), Descriptor, Context);
		}

		// Build member Reference descriptors
		if (LayoutData.MemberReferenceDescriptorSizeAndOffset.Size > 0)
		{
			BuildMemberReferenceDescriptors(reinterpret_cast<FReplicationStateMemberReferenceDescriptor*>(Buffer + LayoutData.MemberReferenceDescriptorSizeAndOffset.Offset), Descriptor, Context);
		}

		if (LayoutData.MemberFunctionDescriptorSizeAndOffset.Size > 0)
		{
			BuildMemberFunctionDescriptors(reinterpret_cast<FReplicationStateMemberFunctionDescriptor*>(Buffer + LayoutData.MemberFunctionDescriptorSizeAndOffset.Offset), Descriptor, Context);
		}

		// Init property pointers
		if (LayoutData.MemberPropertiesSizeAndOffset.Size > 0)
		{
			BuildMemberProperties(reinterpret_cast<const FProperty**>(Buffer + LayoutData.MemberPropertiesSizeAndOffset.Offset), Descriptor);
		}

		// Build MemberPropertyDescriptors
		if (LayoutData.MemberPropertyDescriptorsSizeAndOffset.Size > 0)
		{
			BuildMemberPropertyDescriptors(reinterpret_cast<FReplicationStateMemberPropertyDescriptor*>(Buffer + LayoutData.MemberPropertyDescriptorsSizeAndOffset.Offset), Descriptor);
		}

		// Build MemberLifetimeConditionDescriptors
		if (LayoutData.MemberLifetimeConditionDescriptorsSizeAndOffset.Size > 0)
		{
			BuildMemberLifetimeConditionDescriptors(reinterpret_cast<FReplicationStateMemberLifetimeConditionDescriptor*>(Buffer + LayoutData.MemberLifetimeConditionDescriptorsSizeAndOffset.Offset), Descriptor);
		}

		// Build MemberRepIndexToMemberDescriptors
		if (LayoutData.MemberRepIndexToMemberIndexDescriptorsSizeAndOffset.Size > 0)
		{
			BuildMemberRepIndexToMemberDescriptors(reinterpret_cast<FReplicationStateMemberRepIndexToMemberIndexDescriptor*>(Buffer + LayoutData.MemberRepIndexToMemberIndexDescriptorsSizeAndOffset.Offset), Descriptor);
		}

		// BaseStruct
		if (BuildParams.DescriptorType == EDescriptorType::Struct)
		{
			if (StructInfo.SerializerType == EStructNetSerializerType::DerivedFromCustom)
			{
				const UScriptStruct* BaseStruct = Cast<const UScriptStruct>(FPropertyReplicationStateDescriptorBuilder::FindSuperStructWithCustomSerializer(StructInfo.Struct));
				ensure(BaseStruct != nullptr);
				Descriptor->BaseStruct = BaseStruct;
			}
			else
			{
				Descriptor->BaseStruct = Cast<const UScriptStruct>(StructInfo.Struct);
				ensure(Descriptor->BaseStruct);
			}
		}

		if (LayoutData.MemberDebugDescriptorsSizeAndOffset.Size > 0)
		{
			BuildMemberDebugDescriptors(reinterpret_cast<FReplicationStateMemberDebugDescriptor*>(Buffer + LayoutData.MemberDebugDescriptorsSizeAndOffset.Offset), Descriptor, Context);
		}

		// Construct all MemberSerializerConfigs and update pointers in MemberSerializerDescriptors.
		// Should be called even if LayoutData.MemberSerializerConfigSizeAndOffset.Size is zero.
		BuildMemberSerializerConfigs(Buffer + LayoutData.MemberSerializerConfigSizeAndOffset.Offset, Descriptor, Context.MemberCache);

		if (Context.BuildParams.DescriptorType == EDescriptorType::Class)
		{
			FixupNetRoleNetSerializerConfigs(Descriptor);
		}

		// Need a way to detect that we should fixup descriptor for native use
		if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::IsNativeFastArrayReplicationState))
		{
			FixupDescriptorForNativeFastArray(Descriptor);
		}

		// Finalize the descriptor and fill in remaining information
		FinalizeDescriptor(Descriptor, Context);

		// Setup function pointers to required methods
		Descriptor->ConstructReplicationState = ConstructPropertyReplicationState;
		Descriptor->DestructReplicationState = DestructPropertyReplicationState;

		if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::IsFastArrayReplicationState) || (Descriptor->MemberCount == 1 && Members[0].CreateAndRegisterReplicationFragmentFunction))
		{
			Descriptor->CreateAndRegisterReplicationFragmentFunction = Members[0].CreateAndRegisterReplicationFragmentFunction;
		}

		// Use the full pathname as part of the identifier to ensure that it is unique
		uint64 Seed = 0;
		const uint32 PathLen(BuildParams.PathName ? BuildParams.PathName->Len() : 0);
		if (PathLen)
		{
			Seed = CityHash64((const char*)BuildParams.PathName->GetData(), PathLen * sizeof(FStringBuilderBase::ElementType));
		}
		FString UpperCaseNameForHash(StateName);
		// Append MemberCount, FunctionCount and ChangeMaskBitcount to get some validation of the descriptor.
		UpperCaseNameForHash.Appendf(TEXT("%u%u%u"), Descriptor->MemberCount, Descriptor->FunctionCount, Descriptor->ChangeMaskBitCount);
		UpperCaseNameForHash.ToUpperInline();
		Descriptor->DescriptorIdentifier.Value = CityHash64WithSeed((const char*)UpperCaseNameForHash.GetCharArray().GetData(), sizeof(TCHAR) * UpperCaseNameForHash.GetCharArray().Num(), Seed);

		UE_LOG_DESCRIPTORBUILDER(Verbose, TEXT("Generated identifier(0x%" UINT64_x_FMT ") From %s:%s"), Descriptor->DescriptorIdentifier.Value, *WriteToString<128>(PathLen ? BuildParams.PathName->ToView() : TEXTVIEW("")), *StateName);

		if (Descriptor->InternalSize && BuildParams.DefaultStateSourceData)
		{
			// Initialize default state, we need to do this after finalizing the descriptor as we use the descriptor to iterate over the members
			AllocateAndInitializeDefaultInternalStateBuffer(BuildParams.DefaultStateSourceData, Descriptor, BuildParams.ReplicationSystem, Descriptor->DefaultStateBuffer);

			uint64 DefaultStateHash = uint64(0);
			if (CalculateDefaultStateChecksum(Descriptor, BuildParams.ReplicationSystem, Descriptor->DefaultStateBuffer, DefaultStateHash))
			{
				// Store the default state checksum for later use when verifying protocol
				Descriptor->DescriptorIdentifier.DefaultStateHash = DefaultStateHash;
				UE_LOG_DESCRIPTORBUILDER(Verbose, TEXT("DefaultStateHash for identifier (0x%" UINT64_x_FMT ") is 0x%" UINT64_x_FMT), Descriptor->DescriptorIdentifier.Value, Descriptor->DescriptorIdentifier.DefaultStateHash);
			}
		}

		if (Context.BuildParams.bShouldValidateIsFullyPushModel)
		{
			ValidateIsFullyPushModelClassDescriptor(Context, Descriptor);
		}

		return TRefCountPtr<const FReplicationStateDescriptor>(Descriptor);
	}
	else
	{
		return TRefCountPtr<const FReplicationStateDescriptor>(nullptr);
	}
}

// Helper for GetConnectionFilterTrait. Needs to live outside of function until we can use constepr lambdas (C++17).
template<typename ConditionPair, SIZE_T Size>
constexpr static bool TConditionPairIsSorted(const ConditionPair (&Array)[Size])
{
	if (Size == 0)
	{
		return true;
	}

	for (SIZE_T It = 0, EndIt = Size - 1; It < EndIt; ++It)
	{
		if (Array[It].Condition >= Array[It + 1].Condition)
		{
			return false;
		}
	}

	return true;
};

// Util methods
EMemberPropertyTraits FPropertyReplicationStateDescriptorBuilder::GetConnectionFilterTrait(ELifetimeCondition Condition)
{
	EMemberPropertyTraits Traits = EMemberPropertyTraits::None;

	struct FConditionPair
	{
		ELifetimeCondition Condition;
		EMemberPropertyTraits PropertyTrait;
	};

	constexpr FConditionPair ConditionPairs[] =
	{
		{COND_OwnerOnly, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_SkipOwner, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_SimulatedOnly, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_AutonomousOnly, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_SimulatedOrPhysics, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_InitialOrOwner, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_Custom, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_ReplayOrOwner, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_ReplayOnly, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_SimulatedOnlyNoReplay, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_SimulatedOrPhysicsNoReplay, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_SkipReplay, EMemberPropertyTraits::HasLifetimeConditionals},
		{COND_Dynamic, EMemberPropertyTraits::HasLifetimeConditionals},
	};

	static_assert(TConditionPairIsSorted<>(ConditionPairs), "ConditionPairs order needs to be updated.");
	
	const SIZE_T ConditionPairIndex = Algo::BinarySearchBy(ConditionPairs, Condition, [](const FConditionPair& Pair) { return Pair.Condition; });
	// BinarySearchBy can return INDEX_NONE (-1) but compiler believes the auto return value should be SIZE_T.
	if (ConditionPairIndex < UE_ARRAY_COUNT(ConditionPairs))
	{
		Traits |= ConditionPairs[ConditionPairIndex].PropertyTrait;
	}
	
	return Traits;
}

EMemberPropertyTraits FPropertyReplicationStateDescriptorBuilder::GetInitOnlyTrait(ELifetimeCondition Condition)
{ 
	bool bResult = Condition == COND_InitialOnly;

	return bResult ? EMemberPropertyTraits::InitOnly : EMemberPropertyTraits::None;
}

EMemberPropertyTraits FPropertyReplicationStateDescriptorBuilder::GetHasObjectReferenceTraits(const FNetSerializer* NetSerializer)
{
	EMemberPropertyTraits Traits = EMemberPropertyTraits::None;
	Traits |= EnumHasAnyFlags(NetSerializer->Traits, ENetSerializerTraits::HasCustomNetReference)  ? (EMemberPropertyTraits::HasObjectReference | EMemberPropertyTraits::HasCustomObjectReference) : EMemberPropertyTraits::None;
	Traits |= IsObjectReferenceNetSerializer(NetSerializer) ? EMemberPropertyTraits::HasObjectReference : EMemberPropertyTraits::None;

	return Traits;
}

void FPropertyReplicationStateDescriptorBuilder::GetIrisPropertyTraits(FMemberProperty& OutMemberProperty, const FProperty* Property, const TArray<FLifetimeProperty>* LifeTimeProperties, UClass* ObjectClass)
{
	EMemberPropertyTraits Traits = EMemberPropertyTraits::None;

	const uint16 RepIndex = Property->RepIndex;

	// We assume we can use the LifetimeProperty data even if the RepIndex would not match.
	const FLifetimeProperty* Data = LifeTimeProperties ? &(*LifeTimeProperties)[RepIndex] : nullptr;

	const UFunction* PropertyRepNotifyFunction = nullptr;
	CreateAndRegisterReplicationFragmentFunc CreateAndRegisterReplicationFragmentFunction = nullptr;

	Traits |= ((Property->PropertyFlags & (CPF_NoDestructor | CPF_IsPlainOldData)) != 0 ? EMemberPropertyTraits::IsSourceTriviallyDestructible : EMemberPropertyTraits::None);
	Traits |= ((Property->PropertyFlags & (CPF_ZeroConstructor | CPF_IsPlainOldData)) != 0 ? EMemberPropertyTraits::IsSourceTriviallyConstructible : EMemberPropertyTraits::None);
	Traits |= (Property->IsA(FArrayProperty::StaticClass()) ? EMemberPropertyTraits::IsTArray : EMemberPropertyTraits::None);

	if (Data)
	{
		Traits |= GetConnectionFilterTrait(Data->Condition);
		Traits |= GetInitOnlyTrait(Data->Condition);
		Traits |= (Data->bIsPushBased ? EMemberPropertyTraits::HasPushBasedDirtiness : EMemberPropertyTraits::None);
		
		if (Property->RepNotifyFunc.IsNone() == false)
		{
			PropertyRepNotifyFunction = ObjectClass->FindFunctionByName(Property->RepNotifyFunc);

			Traits |= Data->RepNotifyCondition == ELifetimeRepNotifyCondition::REPNOTIFY_OnChanged ? EMemberPropertyTraits::RepNotifyOnChanged : EMemberPropertyTraits::None;
			Traits |= Data->RepNotifyCondition == ELifetimeRepNotifyCondition::REPNOTIFY_Always ? EMemberPropertyTraits::RepNotifyAlways : EMemberPropertyTraits::None;
			Traits |= PropertyRepNotifyFunction->NumParms > 0 ? EMemberPropertyTraits::NeedPreviousState : EMemberPropertyTraits::None;

			// We currently DO NOT support the random case where NumParms is more than one (MetaData?)
			check(PropertyRepNotifyFunction->NumParms < 2)
		}

		CreateAndRegisterReplicationFragmentFunction = Data->CreateAndRegisterReplicationFragmentFunction;
	}

	OutMemberProperty.PropertyRepNotifyFunction = PropertyRepNotifyFunction;
	OutMemberProperty.Traits = Traits;
	OutMemberProperty.ReplicationCondition = (Data ? Data->Condition : COND_None);
	OutMemberProperty.CreateAndRegisterReplicationFragmentFunction = CreateAndRegisterReplicationFragmentFunction;
}

void FPropertyReplicationStateDescriptorBuilder::GetSerializerTraits(FMemberProperty& OutMemberProperty, const FProperty* Property, const FPropertyNetSerializerInfo* NetSerializerInfo, bool bAllowFastArrayWithExtraProperties)
{
	const FNetSerializer* NetSerializer = NetSerializerInfo->GetNetSerializer(Property);

	OutMemberProperty.Traits |= GetHasObjectReferenceTraits(NetSerializer);
	OutMemberProperty.Traits |= EnumHasAnyFlags(NetSerializer->Traits, ENetSerializerTraits::HasDynamicState) ? EMemberPropertyTraits::HasDynamicState : EMemberPropertyTraits::None;
	OutMemberProperty.Traits |= EnumHasAnyFlags(NetSerializer->Traits, ENetSerializerTraits::HasConnectionSpecificSerialization) ? EMemberPropertyTraits::HasConnectionSpecificSerialization : EMemberPropertyTraits::None;
	OutMemberProperty.Traits |= EnumHasAnyFlags(NetSerializer->Traits, ENetSerializerTraits::UseSerializerIsEqual) ? EMemberPropertyTraits::UseSerializerIsEqual : EMemberPropertyTraits::None;
	OutMemberProperty.Traits |= GetFastArrayPropertyTraits(NetSerializer, Property, bAllowFastArrayWithExtraProperties);

	// Tag Role/RemoteRole to share the same changemask as the serializer operates on multiple properties
	if (NetSerializer == &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer))
	{
		OutMemberProperty.ChangeMaskGroupName = ReplicationStateDescriptorBuilder_NAME_RoleGroup;
	}
}

bool FPropertyReplicationStateDescriptorBuilder::IsSupportedProperty(FMemberProperty& OutMemberProperty, const FProperty* Property, const FIsSupportedPropertyParams& Parameters)
{
	// A property is supported if it has a registered SerializerInfo.
	const FPropertyNetSerializerInfo* NetSerializerInfo = FPropertyNetSerializerInfoRegistry::FindSerializerInfo(Property);
	if (!NetSerializerInfo)
	{
		return false;
	}

	// Init info
	OutMemberProperty.Property = Property;
	OutMemberProperty.SerializerInfo = NetSerializerInfo;
	OutMemberProperty.ChangeMaskBits = 1u;
	OutMemberProperty.ExternalSizeAndAlignment = { 0, 0 };

	// Init traits
	GetIrisPropertyTraits(OutMemberProperty, Property, Parameters.LifeTimeProperties, Parameters.InObjectClass);
	GetSerializerTraits(OutMemberProperty, Property, NetSerializerInfo, Parameters.bAllowFastArrayWithExtraReplicatedProperties);

	// $IRIS: $TODO: Add property attribute to control how many changemask bits we want to use per property
	if (EnumHasAnyFlags(OutMemberProperty.Traits, EMemberPropertyTraits::IsFastArray))
	{
		OutMemberProperty.ChangeMaskBits = 1U + FIrisFastArraySerializer::IrisFastArrayChangeMaskBits;
	}
	else if (EnumHasAnyFlags(OutMemberProperty.Traits, EMemberPropertyTraits::IsTArray))
	{
		if (bIrisUseChangeMaskForTArray)
		{
			OutMemberProperty.ChangeMaskBits = 1U + FPropertyReplicationState::TArrayElementChangeMaskBits;
		}
	}

	// Structs with custom serializers may have a default custom replication fragment.
	if (!OutMemberProperty.CreateAndRegisterReplicationFragmentFunction)
	{
		OutMemberProperty.CreateAndRegisterReplicationFragmentFunction = NetSerializerInfo->GetCreateAndRegisterReplicationFragmentFunction();
	}

	return true;
}

const UStruct* FPropertyReplicationStateDescriptorBuilder::FindSuperStructWithCustomSerializer(const UStruct* Struct)
{
	for (const UStruct* SuperStruct = Struct->GetSuperStruct(); SuperStruct != nullptr; SuperStruct = SuperStruct->GetSuperStruct())
	{
		if (const FPropertyNetSerializerInfo* NetSerializerInfo = FPropertyNetSerializerInfoRegistry::FindStructSerializerInfo(SuperStruct->GetFName()))
		{
			return SuperStruct;
		}
	}

	return nullptr;
}

EStructNetSerializerType FPropertyReplicationStateDescriptorBuilder::IsSupportedStructWithCustomSerializer(FMemberProperty& OutMemberProperty, const UStruct* InStruct)
{
	// See if we got a matching custom serializer for the entire struct
	EStructNetSerializerType SerializerType = EStructNetSerializerType::Struct;
	const FPropertyNetSerializerInfo* NetSerializerInfo = FPropertyNetSerializerInfoRegistry::FindStructSerializerInfo(InStruct->GetFName());
	if (NetSerializerInfo)
	{
		SerializerType = EStructNetSerializerType::Custom;
	}
	else
	{
		// Find closest parent with custom serializer.
		for (const UStruct* SuperStruct = InStruct->GetSuperStruct(); SuperStruct != nullptr; SuperStruct = SuperStruct->GetSuperStruct())
		{
			NetSerializerInfo = FPropertyNetSerializerInfoRegistry::FindStructSerializerInfo(SuperStruct->GetFName());
			if (NetSerializerInfo != nullptr)
			{
				SerializerType = EStructNetSerializerType::DerivedFromCustom;
				break;
			}
		}
	}

	if (!NetSerializerInfo)
	{
		return EStructNetSerializerType::Struct;
	}

	OutMemberProperty.Property = nullptr;
	OutMemberProperty.SerializerInfo = NetSerializerInfo;
	OutMemberProperty.ChangeMaskBits = 1U;
	OutMemberProperty.ExternalSizeAndAlignment = { (SIZE_T)InStruct->GetStructureSize(), (SIZE_T)InStruct->GetMinAlignment() };
	OutMemberProperty.ReplicationCondition = ELifetimeCondition::COND_None;
	OutMemberProperty.PropertyRepNotifyFunction = nullptr;
	OutMemberProperty.CreateAndRegisterReplicationFragmentFunction = NetSerializerInfo->GetCreateAndRegisterReplicationFragmentFunction();
	
	// Init traits
	OutMemberProperty.Traits = EMemberPropertyTraits::None;
	GetSerializerTraits(OutMemberProperty, nullptr, NetSerializerInfo);

	return SerializerType;
}

bool FPropertyReplicationStateDescriptorBuilder::IsStructWithCustomSerializer(const UStruct* InStruct)
{
	// See if we got a matching custom serializer for the entire struct
	if (FPropertyNetSerializerInfoRegistry::FindStructSerializerInfo(InStruct->GetFName()))
	{
		return true;
	}

	// Check if super has a defined custom serializer
	const UStruct* Super = InStruct->GetSuperStruct();
	return Super ? IsStructWithCustomSerializer(Super) : false;
}

// Returns true if the named struct is marked as working using the default StructNetSerializer
bool FPropertyReplicationStateDescriptorBuilder::CanStructUseStructNetSerializer(FName StructName)
{
	if (bUseSupportsStructNetSerializerList)
	{
		const UReplicationStateDescriptorConfig* ReplicationStateDescriptorConfig = GetDefault<UReplicationStateDescriptorConfig>();
		if (const FSupportsStructNetSerializerConfig* StructNetSerializerConfig = ReplicationStateDescriptorConfig->GetSupportsStructNetSerializerList().FindByPredicate([StructName](const FSupportsStructNetSerializerConfig& Item) { return Item.StructName == StructName; } ))
		{
			return StructNetSerializerConfig->bCanUseStructNetSerializer;
		}
	}
	return false;
}

// RepTags are not intended to be derived from arbitrary properties, but rather be explicitly declared on a member. This is a hack.
FRepTag FPropertyReplicationStateDescriptorBuilder::GetRepTagFromProperty(const FMemberCacheEntry& MemberCacheEntry, const FMemberProperty& MemberProperty)
{
	const FProperty* Property = MemberProperty.Property;
	const FName PropertyName = Property ? Property->GetFName() : FName();
	if (PropertyName == NAME_Role || PropertyName == NAME_RemoteRole)
	{
		if (MemberCacheEntry.Serializer == &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer))
		{
			return PropertyName == NAME_Role ? RepTag_NetRole : RepTag_NetRemoteRole;
		}
	}
	else if (Property && PropertyName == ReplicationStateDescriptorBuilder_NAME_NetCullDistanceSquared)
	{
		const FFieldClass* Class = Property->GetClass();
		if (Class->GetFName() == NAME_FloatProperty)
		{
			return RepTag_CullDistanceSqr;
		}
		else
		{
			UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("Found NetCullDistanceSquared property that is of type %s instead of a float. This prevents the property from being accessed by networking systems."), ToCStr(Class->GetFName().ToString()));
		}
	}
	//$IRIS TODO: Temp until proper RepTag support is added. 
	//For unit testing purposes. Keep this code last.
#if UE_NET_TEST_FAKE_REP_TAGS
	else
	{
		static const FName ReplicationStateDescriptorBuilder_NAME_WorldLocation("WorldLocation");
		if (PropertyName == ReplicationStateDescriptorBuilder_NAME_WorldLocation)
		{
			return RepTag_WorldLocation;
		}

		ANSICHAR PropertyNameString[NAME_SIZE];
		PropertyName.GetPlainANSIString(PropertyNameString);
		if (0 == FCStringAnsi::Strncmp(PropertyNameString, "NetTest_", 8))
		{
			return MakeRepTag(PropertyNameString);
		}
	}
#endif

	return GetInvalidRepTag();
}

uint32 FPropertyReplicationStateDescriptorBuilder::GetRepIndexCount(const TConstArrayView<FMemberProperty>& Members)
{
	uint16 MaxRepIndex = 0;
	{
		const FProperty* LastProperty = nullptr;
		uint16 ArrayIndex = 0;
		for (const FMemberProperty& Member : Members)
		{
			ArrayIndex = (LastProperty && Member.Property == LastProperty ? uint16(ArrayIndex + 1) : uint16(0));
			LastProperty = Member.Property;
			CA_ASSUME(Member.Property != nullptr);
			MaxRepIndex = FPlatformMath::Max(MaxRepIndex, static_cast<uint16>(Member.Property->RepIndex + ArrayIndex));
		}
	}
	
	const uint32 RepIndexCount = static_cast<uint32>(MaxRepIndex) + 1U;
	return RepIndexCount;
}

void FPropertyReplicationStateDescriptorBuilder::GetPropertyPathName(const FProperty* Property, FString& PathName)
{
	constexpr uint32 MaxHierarchyDepth = 64;
	FFieldVariant PropertyChain[MaxHierarchyDepth];

	const UClass* Class = UClass::StaticClass();
	const UClass* Struct = UStruct::StaticClass();

	uint32 PropertyChainIndex = MaxHierarchyDepth;
	for (FFieldVariant Object = Property; Object.IsValid(); Object = Object.GetOwnerVariant())
	{
		if (!ensureMsgf(PropertyChainIndex > 0, TEXT("Property hieararchy depth exceeds %u"), MaxHierarchyDepth))
		{
			break;
		}
		PropertyChain[--PropertyChainIndex] = Object;
		if (Object.IsA(Class) || Object.IsA(Struct))
		{
			break;
		}
	}

	PathName.Empty(512);
	for (; PropertyChainIndex < MaxHierarchyDepth; ++PropertyChainIndex)
	{
		const FFieldVariant& FieldVariant = PropertyChain[PropertyChainIndex];
		if (const UObject* Object = FieldVariant.ToUObject())
		{
			PathName.Append(Object->GetName()).AppendChar(TEXT('.'));
		}
		else if (const FField* Field = FieldVariant.ToField())
		{
			PathName.Append(Field->GetName()).Append((PropertyChainIndex == MaxHierarchyDepth - 1U) ? TEXT("") : TEXT("."));
		}
	}
}

SIZE_T FPropertyReplicationStateDescriptorBuilder::GetFastArrayChangeMaskOffset(const FProperty* Property)
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		const UScriptStruct* Struct = StructProperty->Struct;

		check(Struct->IsChildOf(FIrisFastArraySerializer::StaticStruct()));

		// Need to figure out if we are native, currently we do this by checking if we have a FastArrayChangeMaskMember
		const FProperty* ChangeMaskProperty = Struct->FindPropertyByName(PropertyNetSerializerRegistry_NAME_ChangeMaskStorage);
		if (ChangeMaskProperty)
		{
			// ChangeMask member must be repskip
			check(EnumHasAnyFlags(ChangeMaskProperty->PropertyFlags, EPropertyFlags::CPF_RepSkip));

			return ChangeMaskProperty->GetOffset_ForGC();
		}
	}

	return 0U;
}

EMemberPropertyTraits FPropertyReplicationStateDescriptorBuilder::GetFastArrayPropertyTraits(const FNetSerializer* NetSerializer, const FProperty* InProperty, bool bAllowFastArrayWithExtraProperties)
{
	if (IsStructNetSerializer(NetSerializer))
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			// Find traits for the FastArraySerializer
			const UScriptStruct* Struct = StructProperty->Struct;
			if (Struct->IsChildOf(FFastArraySerializer::StaticStruct()))
			{
				FMemberProperty MemberProperty;
				if (IsStructWithCustomSerializer(Struct))
				{
					return EMemberPropertyTraits::IsInvalidFastArray;
				}

				// Try to figure out if this is a FastArraySerializer we can support. It is assumed the struct has already been determined to inherit from FastArraySerializer.
				// A proper FastArraySerializer should contain a single TArray property with elements that are a derived from FastArraySerializerItem.
				uint32 ReplicatedPropertyCount = 0;
				bool bHasFastArrayItem = false;
	
				for (TFieldIterator<FProperty> It(Struct, EFieldIteratorFlags::IncludeSuper); It; ++It)
				{
					const FProperty* CurrentProperty = *It;
					if (EnumHasAnyFlags(CurrentProperty->PropertyFlags, EPropertyFlags::CPF_RepSkip))
					{
						continue;
					}

					++ReplicatedPropertyCount;

					if (!FPropertyReplicationStateDescriptorBuilder::IsSupportedProperty(MemberProperty, CurrentProperty))
					{
						return EMemberPropertyTraits::IsInvalidFastArray;
					}

					if (EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::IsFastArrayItem))
					{
						bHasFastArrayItem = true;
					}

					// If we do not allow additional properties, report this as an invalid FastArray
					if (!bAllowFastArrayWithExtraProperties)
					{
						if ((ReplicatedPropertyCount > 1) || (CastField<FArrayProperty>(MemberProperty.Property) == nullptr))
						{
							return EMemberPropertyTraits::IsInvalidFastArray;
						}
					}
				}

				if (!bHasFastArrayItem)
				{
					return EMemberPropertyTraits::IsInvalidFastArray;
				}

				// We have a valid FastArray, setup traits
				EMemberPropertyTraits Traits = EMemberPropertyTraits::IsFastArray;

				// If the struct is derived from FIrisFastArraySerializer and has a single property we can treat it as a native FastArray
				const bool bUseNativeFastArray = bIrisUseNativeFastArray && ReplicatedPropertyCount == 1U;
				if (bUseNativeFastArray && Struct->IsChildOf(FIrisFastArraySerializer::StaticStruct()))
				{
					Traits |= EMemberPropertyTraits::HasPushBasedDirtiness | EMemberPropertyTraits::IsNativeFastArray;
				}
				else if (ReplicatedPropertyCount > 1U)
				{
					Traits |= EMemberPropertyTraits::IsFastArrayWithExtraProperties;
				}

				return Traits;
			}
			else if (Struct->IsChildOf(FFastArraySerializerItem::StaticStruct()))
			{
				// Invalidate FastArrayItem if it has a custom NetSerializer which we currently do not support, the workaround is to wrap the struct with the custom netserializer in a struct
				if (IsStructWithCustomSerializer(Struct))
				{
					UE_LOGF(LogIris, Error, "FPropertyReplicationStateDescriptorBuilder found unsupported custom NetSerializers for FastArrayItems %ls", *Struct->GetName());
					ensureMsgf(false, TEXT("FPropertyReplicationStateDescriptorBuilder Iris does not support custom NetSerializers for FastArrayItems %s, if required use a property in the struct wrapping the custom NetSerializer"), *Struct->GetName());

					return EMemberPropertyTraits::None;
				}

				return EMemberPropertyTraits::IsFastArrayItem;
			}
		}
	}
	else if (IsArrayPropertyNetSerializer(NetSerializer))
	{
		// If the property is an array, unwrap it and check if it is a FFastArraySerializerItem
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
			{
				const UScriptStruct* Struct = StructProperty->Struct;
				if (Struct->IsChildOf(FFastArraySerializerItem::StaticStruct()))
				{
					return EMemberPropertyTraits::IsFastArrayItem;
				}
			}
		}
	}

	return EMemberPropertyTraits::None;
}

}

namespace UE::Net
{

FReplicationStateDescriptorBuilder::FParameters::FParameters()
	: DescriptorRegistry(nullptr)
{
}

/**
 * FReplicationStateDescriptorBuilder Implementation
 */

// For a struct we only create a single state.
TRefCountPtr<const FReplicationStateDescriptor> FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(const UStruct* InStruct, const FParameters& Parameters)
{
	using namespace UE::Net::Private;

	IRIS_PROFILER_SCOPE(FReplicationStateDescriptorBuilder_CreateDescriptorForStruct);

	// Check registry if we already have created descriptors for this class
	if (Parameters.DescriptorRegistry)
	{	
		if (const FReplicationStateDescriptorRegistry::FDescriptors* Result = Parameters.DescriptorRegistry->Find(InStruct))
		{
			// For structs we do only expect a single entry
			ensure(Result->Num() == 1);

			return (*Result)[0];
		}
	}

	bool bIsFastArraySerializerItem = false;
	bool bIsFastArraySerializer = false;
	bool bIsStructWithCustomSerialization = false;
	if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InStruct))
	{
		bIsFastArraySerializerItem = ScriptStruct->IsChildOf(FFastArraySerializerItem::StaticStruct());
		bIsFastArraySerializer = ScriptStruct->IsChildOf(FFastArraySerializer::StaticStruct());
		if (bIsFastArraySerializer && !Parameters.EnableFastArrayHandling)
		{
			bIsFastArraySerializer = false;
			UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("FReplicationStateDescriptorBuilder::CreateDescriptorForStruct Generating descriptor for FastArraySerializer as generic struct due to incompatible members. Struct is %s."), ToCStr(InStruct->GetName()));
		}

		bIsStructWithCustomSerialization = !bIsFastArraySerializer && EnumHasAnyFlags(ScriptStruct->StructFlags, EStructFlags(STRUCT_NetSerializeNative | STRUCT_NetDeltaSerializeNative));
	}

	FPropertyReplicationStateDescriptorBuilder Builder;
	FPropertyReplicationStateDescriptorBuilder::FMemberProperty MemberProperty;

	// Struct members can opt out of replication.
	bool bAllMembersAreReplicated = true;

	// We have a special case for structs with custom serializers
	EStructNetSerializerType SerializerType = EStructNetSerializerType::Struct;
	if (!Parameters.SkipCheckForCustomNetSerializerForStruct)
	{
		SerializerType = FPropertyReplicationStateDescriptorBuilder::IsSupportedStructWithCustomSerializer(MemberProperty, InStruct);
	}

	if (SerializerType != EStructNetSerializerType::Struct)
	{
		Builder.AddMemberProperty(MemberProperty);
		if (SerializerType == EStructNetSerializerType::DerivedFromCustom)
		{
			// Add properties introduced after parent struct with custom serializer
			const UStruct* StructWithCustomSerializer = FPropertyReplicationStateDescriptorBuilder::FindSuperStructWithCustomSerializer(InStruct);
			TArray<const UStruct*, TInlineAllocator<32>> Structs;
			for (const UStruct* Struct = InStruct; Struct != StructWithCustomSerializer; Struct = Struct->GetSuperStruct())
			{
				// Insert struct first to traverse them in natural order when looking for replicated properties.
				constexpr int32 InsertAtIndex = 0;
				Structs.Insert(Struct, InsertAtIndex);
			}

			for (const UStruct* Struct : Structs)
			{
				for (TFieldIterator<FProperty> It(Struct, EFieldIteratorFlags::ExcludeSuper); It; ++It)
				{
					const FProperty* Property = *It;
					if (EnumHasAnyFlags(Property->PropertyFlags, EPropertyFlags::CPF_RepSkip))
					{
						bAllMembersAreReplicated = false;
						continue;
					}

					if (FPropertyReplicationStateDescriptorBuilder::IsSupportedProperty(MemberProperty, Property))
					{
						for (uint32 ArrayIt = 0, ArrayEndIt = Property->ArrayDim; ArrayIt < ArrayEndIt; ++ArrayIt)
						{
							Builder.AddMemberProperty(MemberProperty);
						}
					}
				}
			}
		}
	}
	else
	{
		const bool bIsAllowedToWarn = (ShouldUseIrisReplication() || FApp::IsGame() || IsRunningDedicatedServer() || IsRunningClientOnly()) && !IsRunningCommandlet();

		if (bIsStructWithCustomSerialization)
		{
			// Warn if we did not find a custom NetSerializer for a struct that has overridden NetSerialize or NetDeltaSerialize, unless it is marked as supporting the default StructNetSerializer
			if (!Parameters.SkipCheckForCustomNetSerializerForStruct && bIsAllowedToWarn && bWarnAboutStructsWithCustomSerialization && !FPropertyReplicationStateDescriptorBuilder::CanStructUseStructNetSerializer(InStruct->GetFName()))
			{
				UE_LOGF(LogIris, Warning, "Generating descriptor for struct %ls that has custom serialization.", ToCStr(InStruct->GetName()));
			}
		}

		uint32 PropertyCount = 0U;
		for (TFieldIterator<FProperty> It(InStruct, EFieldIteratorFlags::IncludeSuper); It; ++It)
		{
			const FProperty* Property = *It;
			++PropertyCount;
			if (EnumHasAnyFlags(Property->PropertyFlags, EPropertyFlags::CPF_RepSkip))
			{
				// For FastArraySerializerItems we currently include the ReplicationId even though it is not replicated
				if (!(bIsFastArraySerializerItem && (Property->GetFName() == PropertyNetSerializerRegistry_NAME_ReplicationID)))
				{
					bAllMembersAreReplicated = false;
					continue;
				}
			}

			if (FPropertyReplicationStateDescriptorBuilder::IsSupportedProperty(MemberProperty, Property))
			{
				// We do not support nested fast arrays
				if (EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::IsFastArray))
				{
					UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("FReplicationStateDescriptorBuilder ::CreateDescriptorForStruct Skipping FastArraySerializer %s.%s of type %s in struct which is not supported."), ToCStr(InStruct->GetName()), ToCStr(Property->GetName()), ToCStr(Property->GetCPPType()));
					continue;
				}

				if (bIsFastArraySerializer && !EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::IsFastArrayItem))
				{
					// For FastArraySerializer we do not expect other properties than FastArrayItems unless explicitly allowed.
					if (!Parameters.AllowFastArrayWithExtraReplicatedProperties)
					{
						UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("FReplicationStateDescriptorBuilder ::CreateDescriptorForStruct FastArraySerializerStructs can only hold a single replicated property derived from FastArraySerializerItem. Skipping property: %s.%s of type %s."), ToCStr(InStruct->GetName()), ToCStr(Property->GetName()), ToCStr(Property->GetCPPType()));
						continue;
					}
				}

				for (uint32 ArrayIt = 0, ArrayEndIt = Property->ArrayDim; ArrayIt < ArrayEndIt; ++ArrayIt)
				{
					Builder.AddMemberProperty(MemberProperty);
				}
			}
			else
			{
				UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("FReplicationStateDescriptorBuilder : Skipping unsupported struct member %s.%s of type %s."), ToCStr(InStruct->GetName()), ToCStr(Property->GetName()), ToCStr(Property->GetCPPType()));
			}
		}

		if (bAllMembersAreReplicated && PropertyCount == 0U)
		{
			// Warn if someone tries to replicate a struct with no properties for which we do not have a custom NetSerialzier.
			UE_CLOGF(bWarnAboutStructPropertiesWithSuspectedNotReplicatedProperties, LogIris, Warning, "FReplicationStateDescriptorBuilder ::CreateDescriptorForStruct a replicated struct WITH no replicated properties must have a CustomNetSerializer. Struct: %ls.", ToCStr(InStruct->GetName()));
			bAllMembersAreReplicated = false;
		}
	}

	Builder.SetStructSizeAndAlignment(InStruct->GetStructureSize(), InStruct->GetMinAlignment());
	Builder.SetStruct(InStruct);
	Builder.SetStructSerializerType(SerializerType);

	FLazyGetPathNameHelper LazyPathHelper;
	FPropertyReplicationStateDescriptorBuilder::FBuildParameters BuildParameters = {};
	BuildParameters.PathName = LazyPathHelper.GetPathName(InStruct);
	BuildParameters.DescriptorType = FPropertyReplicationStateDescriptorBuilder::EDescriptorType::Struct;
	BuildParameters.bIsInitState = false;
	BuildParameters.bAllMembersAreReplicated = bAllMembersAreReplicated;

	TRefCountPtr<const FReplicationStateDescriptor> Descriptor = Builder.Build(InStruct->GetName(), Parameters.DescriptorRegistry, BuildParameters);

	if (Parameters.DescriptorRegistry && Descriptor.IsValid())
	{
		Parameters.DescriptorRegistry->Register(InStruct, Descriptor);
	}

	return Descriptor;
};

TRefCountPtr<const FReplicationStateDescriptor> FReplicationStateDescriptorBuilder::CreateDescriptorForFunction(const UFunction* Function, const FParameters& Parameters)
{
	using namespace UE::Net::Private;

	IRIS_PROFILER_SCOPE(FReplicationStateDescriptorBuilder_CreateDescriptorForFunction);

	// Check registry if we already have created descriptors for this function
	if (Parameters.DescriptorRegistry)
	{	
		if (const FReplicationStateDescriptorRegistry::FDescriptors* Result = Parameters.DescriptorRegistry->Find(Function))
		{
			// For function we expect a single entry
			ensure(Result->Num() == 1);

			return (*Result)[0];
		}
	}

	FPropertyReplicationStateDescriptorBuilder Builder;
	FPropertyReplicationStateDescriptorBuilder::FMemberProperty MemberProperty;

	for (TFieldIterator<FProperty> It(Function); It && (It->PropertyFlags & (CPF_Parm | CPF_ReturnParm)) == CPF_Parm; ++It)
	{
		const FProperty* Property = *It;
		if (Property->PropertyFlags & CPF_RepSkip)
		{
			continue;
		}
		
		checkfSlow(Property->ArrayDim == 1, TEXT("Unexpected parameter with array size. Function %s parameter %s."), ToCStr(Function->GetName()), ToCStr(Property->GetName()));

		if (FPropertyReplicationStateDescriptorBuilder::IsSupportedProperty(MemberProperty, Property))
		{
			Builder.AddMemberProperty(MemberProperty);
		}
		else
		{
			UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("FPropertyReplicationStateDescriptorBuilder: Skipping unsupported function parameter %s.%s of type %s."), ToCStr(Function->GetName()), ToCStr(Property->GetName()), ToCStr(Property->GetCPPType()));
		}
	}

	Builder.SetStructSizeAndAlignment(Function->ParmsSize, Function->GetMinAlignment());

	FPropertyReplicationStateDescriptorBuilder::FBuildParameters BuildParameters = {};
	BuildParameters.DescriptorType = FPropertyReplicationStateDescriptorBuilder::EDescriptorType::Function;
	BuildParameters.bIsInitState = false;
	// For this use case we don't care about function parameters that aren't replicated.
	BuildParameters.bAllMembersAreReplicated = true;
	TRefCountPtr<const FReplicationStateDescriptor> Descriptor = Builder.Build(Function->GetName(), Parameters.DescriptorRegistry, BuildParameters);

	if (Parameters.DescriptorRegistry && Descriptor.IsValid())
	{
		Parameters.DescriptorRegistry->Register(Function, Descriptor);
	}

	return Descriptor;
}

struct FPropertyReplicationStateType
{
	const TCHAR* NamePostFix;
};

static constexpr FPropertyReplicationStateType InternalPropertyReplicationStateTypes[] = 
{
	{ TEXT("_Functions"), },
	{ TEXT("_InitOnly"), },
	{ TEXT("_LifetimeConditionals"), },
	{ TEXT("_State"), },
};

static constexpr SIZE_T FunctionsPropertyReplicationStateBuilderIndex = 0;
static constexpr SIZE_T InitPropertyReplicationStateBuilderIndex = 1;
static constexpr SIZE_T LifetimeConditionalsReplicationStateBuilderIndex = 2;
static constexpr SIZE_T RegularPropertyReplicationStateBuilderIndex = 3;

SIZE_T FReplicationStateDescriptorBuilder::CreateDescriptorsForClass(FResult& CreatedDescriptors, UClass* InObjectClass, const FParameters& Parameters)
{
	using namespace UE::Net::Private;

	IRIS_PROFILER_SCOPE(FReplicationStateDescriptorBuilder_CreateDescriptorForClass);

	if (!IsValid(InObjectClass))
	{
		UE_LOGF(LogIris, Error, "Trying to create descriptor for invalid class %ls", *GetNameSafe(InObjectClass));
		return 0U;
	}

	// Just to make sure we don't mess up the parameters when building descriptor for single property
	//check(Parameters.SinglePropertyIndex == -1 || Parameters.DescriptorRegistry == nullptr)

	// Use the template as the default key or the class CDO if there is no template.
	const UObject* DefaultStateSource = Parameters.DefaultStateSource ? Parameters.DefaultStateSource : InObjectClass->GetDefaultObject();

	// Check registry first to see if we already have created descriptors for this class
	if (Parameters.DescriptorRegistry)
	{
		if (const FReplicationStateDescriptorRegistry::FDescriptors* Result = Parameters.DescriptorRegistry->Find(DefaultStateSource, InObjectClass))
		{
			UE_LOG_DESCRIPTORBUILDER(VeryVerbose, TEXT("Found existing descriptors for Class %s : DescriptorKey: %s ptr: 0x%p"), *InObjectClass->GetName(), *DefaultStateSource->GetPathName(), DefaultStateSource);
			CreatedDescriptors.Insert(*Result, 0);
			return CreatedDescriptors.Num();
		}
	}

	const bool bIncludeSuper = !!Parameters.IncludeSuper;
	const bool bGetLifeTimeProperties = !!Parameters.GetLifeTimeProperties;
	const bool bSingleProperty =  Parameters.SinglePropertyIndex != -1;
	const bool bRegisterCreatedDescriptors = Parameters.DescriptorRegistry && !bSingleProperty;

	{
		IRIS_PROFILER_SCOPE(FReplicationStateDescriptorBuilder_SetUpRuntimeReplicationData);
		// This is a bit unfortunate, but for now we need to do this since we need the replication indices and the ClassReps
		InObjectClass->SetUpRuntimeReplicationData();
	}

	// We need the lifetime properties array to get conditionals
	TArray<FLifetimeProperty> LifeTimePropertiesOriginalOrder;
	TArray<FLifetimeProperty> LifeTimeProperties;
	if (bGetLifeTimeProperties)
	{
		IRIS_PROFILER_SCOPE(FReplicationStateDescriptorBuilder_GetLifeTimeProperties);

		// Keep track of non user registered properties
		FNetBitArray RegisteredProperty(InObjectClass->ClassReps.Num());

		LifeTimePropertiesOriginalOrder.Reserve(InObjectClass->ClassReps.Num());
		UObject* Object = InObjectClass->GetDefaultObject();
		Object->GetLifetimeReplicatedProps(LifeTimePropertiesOriginalOrder);

		// Lifetime properties aren't sorted and searching for a property with a specific RepIndex in the array is linear.
		// Let's put them where we want them.
		LifeTimeProperties.SetNum(InObjectClass->ClassReps.Num());
		for (const FLifetimeProperty& LifetimeProperty : LifeTimePropertiesOriginalOrder)
		{
			LifeTimeProperties[LifetimeProperty.RepIndex] = LifetimeProperty;
			RegisteredProperty.SetBit(LifetimeProperty.RepIndex);
		}

		RegisteredProperty.ForAllUnsetBits([&LifeTimeProperties, InObjectClass](uint32 Index)
		{
			LifeTimeProperties[Index].RepIndex = static_cast<uint16>(Index);

			if (UE::Net::bAutoRegisterReplicatedProperties == false)
			{
				// Never replicate this property when auto-registration is disabled
				LifeTimeProperties[Index].Condition = COND_Never;

				if (UE::Net::bEnsureForMissingProperties)
				{
					ensureMsgf(false, TEXT("Property %s::%s (SourceClass: %s) was not registered in GetLifetimeReplicatedProps. This property will not be replicated. Use DISABLE_REPLICATED_PROPERTY if not replicating was intentional."), 
							   *GetNameSafe(InObjectClass), *GetNameSafe(InObjectClass->ClassReps[Index].Property), *GetNameSafe(InObjectClass->ClassReps[Index].Property->GetOwnerClass()));
				}
			}
		});
	}

	// For a UClass we split up our state into separate replication states based on common traits (Conditionals)
	constexpr uint32 BuilderTypeCount = UE_ARRAY_COUNT(InternalPropertyReplicationStateTypes);
	FPropertyReplicationStateDescriptorBuilder Builders[BuilderTypeCount];

	// We add separate descriptors properties with custom replication fragments
	TArray<FPropertyReplicationStateDescriptorBuilder::FMemberProperty, TInlineAllocator<8>> CustomProperties;

	FPropertyReplicationStateDescriptorBuilder::FIsSupportedPropertyParams IsSupportedPropertyParams;
	IsSupportedPropertyParams.LifeTimeProperties = &LifeTimeProperties;
	IsSupportedPropertyParams.InObjectClass = InObjectClass;
	IsSupportedPropertyParams.bAllowFastArrayWithExtraReplicatedProperties = Parameters.AllowFastArrayWithExtraReplicatedProperties;

	// Add properties
	for (const FRepRecord& RepRecord : InObjectClass->ClassReps)
	{
		FPropertyReplicationStateDescriptorBuilder::FMemberProperty MemberProperty;
		const FProperty* Property = RepRecord.Property;

		const int32 RepIndex = static_cast<int32>(Property->RepIndex);
	
		// If we are building a class descriptor for a single property we only include the property matching the index
		if (bSingleProperty && (RepIndex != Parameters.SinglePropertyIndex))
		{
			continue;
		}

		// Skip super class properties depending on parameters. C array elements will be treated individually.
		if ((bIncludeSuper || Property->GetOwnerVariant() == InObjectClass))
		{
			if (FPropertyReplicationStateDescriptorBuilder::IsSupportedProperty(MemberProperty, Property, IsSupportedPropertyParams))
			{
				// Ignore disabled properties
				if (MemberProperty.ReplicationCondition == COND_Never)
				{
					continue;
				}

				if (EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::IsInvalidFastArray))
				{
					// Invalid fastarrays are treated as a normal struct property
					MemberProperty.CreateAndRegisterReplicationFragmentFunction = nullptr;
				}
				else if (MemberProperty.CreateAndRegisterReplicationFragmentFunction)
				{
					// We build separate descriptors for all properties with CustomReplicationFragments					
					CustomProperties.Add(MemberProperty);
					continue;
				}
				else if (EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::IsFastArray))
				{
					// FastArrayProperties should use a custom replication fragment
					UE_LOGF(LogIris, Error, "FReplicationStateDescriptorBuilder::CreateDescriptorsForClass FFastArray property %ls not registered and won't be replicated.", *Property->GetFullName());
					ensureMsgf(false, TEXT("FReplicationStateDescriptorBuilder::CreateDescriptorsForClass FFastArray property %s not registered! Usually this means a call to SetupIrisSupport is needed in the module's Build.cs file."), *Property->GetFullName());
					continue;
				}

				// A property can end up in multiple states, such as when a condition is InitialOrOwner.
				// $IRIS TODO Need NoInit on properties to avoid double send on init.
				if (EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::InitOnly))
				{
					Builders[InitPropertyReplicationStateBuilderIndex].AddMemberProperty(MemberProperty);
				}

				constexpr EMemberPropertyTraits ConditionTraits = EMemberPropertyTraits::HasLifetimeConditionals;
				if (EnumHasAnyFlags(MemberProperty.Traits, ConditionTraits))
				{
					if (EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::HasLifetimeConditionals))
					{
						Builders[LifetimeConditionalsReplicationStateBuilderIndex].AddMemberProperty(MemberProperty);
					}
				}

				if (!EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::InitOnly | ConditionTraits))
				{
					Builders[RegularPropertyReplicationStateBuilderIndex].AddMemberProperty(MemberProperty);
				}
			}
			else
			{
				UE_LOG_DESCRIPTORBUILDER_WARNING(TEXT("FPropertyReplicationStateDescriptorBuilder: Skipping unsupported property %s.%s of type %s."), ToCStr(InObjectClass->GetName()), ToCStr(Property->GetName()), ToCStr(Property->GetCPPType()));
			}
		}
	}

	// Add special Actor properties
	{
		//$IRIS TODO: Probably should use a cached pointer of the Actor class here.
		UClass* ActorClass = CastChecked<UClass>(StaticFindObject(UClass::StaticClass(), nullptr, TEXT("/Script/Engine.Actor"), EFindObjectFlags::ExactClass));
		if (InObjectClass->IsChildOf(ActorClass))
		{
			if (const FProperty* NetCullDistanceSquaredProperty = InObjectClass->FindPropertyByName(ReplicationStateDescriptorBuilder_NAME_NetCullDistanceSquared))
			{
				FPropertyReplicationStateDescriptorBuilder::FMemberProperty NetCullDistanceSqrMemberProperty;
				NetCullDistanceSqrMemberProperty.ChangeMaskBits = 1U;
				NetCullDistanceSqrMemberProperty.Property = NetCullDistanceSquaredProperty;
				NetCullDistanceSqrMemberProperty.ExternalSizeAndAlignment = {};
				NetCullDistanceSqrMemberProperty.PropertyRepNotifyFunction = nullptr;
				NetCullDistanceSqrMemberProperty.ReplicationCondition = COND_None;
				NetCullDistanceSqrMemberProperty.SerializerInfo = FPropertyNetSerializerInfoRegistry::GetNopNetSerializerInfo();
				NetCullDistanceSqrMemberProperty.Traits = EMemberPropertyTraits::None;

				FPropertyReplicationStateDescriptorBuilder::GetIrisPropertyTraits(NetCullDistanceSqrMemberProperty, NetCullDistanceSquaredProperty, nullptr /* lifetime conditions */, InObjectClass);
				FPropertyReplicationStateDescriptorBuilder::GetSerializerTraits(NetCullDistanceSqrMemberProperty, NetCullDistanceSquaredProperty, NetCullDistanceSqrMemberProperty.SerializerInfo);

				// These needs to be set/fixed after the Traits calls.
				NetCullDistanceSqrMemberProperty.Traits |= EMemberPropertyTraits::HasLifetimeConditionals | EMemberPropertyTraits::HasPushBasedDirtiness;
				NetCullDistanceSqrMemberProperty.ReplicationCondition = COND_Never;

				Builders[LifetimeConditionalsReplicationStateBuilderIndex].AddMemberProperty(NetCullDistanceSqrMemberProperty);
			}
			else
			{
				UE_LOGF(LogIris, Error, "FPropertyReplicationStateDescriptorBuilder: Failed to find expected property %ls.%ls", ToCStr(InObjectClass->GetName()), *FName(ReplicationStateDescriptorBuilder_NAME_NetCullDistanceSquared).ToString());
				ensure(false);
			}
		}
	}

	// Add RPCs
	if (!bSingleProperty)
	{
		FPropertyReplicationStateDescriptorBuilder& Builder = Builders[FunctionsPropertyReplicationStateBuilderIndex];
		FPropertyReplicationStateDescriptorBuilder::FMemberFunction MemberFunction;
		const SIZE_T MaxClassHierarchyDepth = 64;
		const UClass* Classes[MaxClassHierarchyDepth];
		SIZE_T ClassIndex = MaxClassHierarchyDepth;
		
		// NetFields only contain the class specific networked fields, excluding super class fields.
		for (const UClass* CurrentClass = InObjectClass; CurrentClass != nullptr; CurrentClass = CurrentClass->GetSuperClass())
		{
			if (!ensureMsgf(ClassIndex > 0, TEXT("Class hieararchy depth exceeds %" SIZE_T_FMT), MaxClassHierarchyDepth))
			{
				break;
			}
			Classes[--ClassIndex] = CurrentClass;
		}

		for (const UClass* CurrentClass : MakeArrayView(Classes + ClassIndex, static_cast<int32>(MaxClassHierarchyDepth - ClassIndex)))
		{
			for (const UField* NetField : MakeArrayView(CurrentClass->NetFields))
			{
				if (const UFunction* Function = Cast<UFunction>(NetField))
				{
					MemberFunction.Function = Function;
					Builder.AddMemberFunction(MemberFunction);
				}
			}
		}
	}

	// For-loop with CurrentClass initialized to InObjectClass causes static analysis warnings.
	CA_ASSUME(InObjectClass != nullptr);

	const bool bShouldValidateIsFullyPushModel = FPropertyReplicationStateDescriptorBuilder::ShouldValidateIsFullyPushModel(InObjectClass);

	// Build state descriptors
	FLazyGetPathNameHelper LazyPathName;

	const uint8* DefaultStateSourceData = reinterpret_cast<const uint8*>(DefaultStateSource);
	for (uint32 BuilderTypeIndex = 0; BuilderTypeIndex < BuilderTypeCount; ++BuilderTypeIndex)
	{
		if (Builders[BuilderTypeIndex].HasDataToBuild())
		{
			FPropertyReplicationStateDescriptorBuilder::FBuildParameters BuildParameters = {};
			BuildParameters.PathName = LazyPathName.GetPathName(InObjectClass);
			BuildParameters.DefaultStateSourceData = DefaultStateSourceData;
			BuildParameters.ReplicationSystem = Parameters.ReplicationSystem;
			BuildParameters.DescriptorType = FPropertyReplicationStateDescriptorBuilder::EDescriptorType::Class;
			BuildParameters.bIsInitState = (BuilderTypeIndex == InitPropertyReplicationStateBuilderIndex);
			// We set this to false for the time being for all replication states
			BuildParameters.bAllMembersAreReplicated = false;
			BuildParameters.bShouldValidateIsFullyPushModel = bShouldValidateIsFullyPushModel;

			TRefCountPtr<const FReplicationStateDescriptor> Descriptor = Builders[BuilderTypeIndex].Build(InObjectClass->GetName() + InternalPropertyReplicationStateTypes[BuilderTypeIndex].NamePostFix, Parameters.DescriptorRegistry, BuildParameters);
			CreatedDescriptors.Add(Descriptor);
		}
	}

	// If we have properties with a CreateAndRegisterReplicationFragmentFunction, we build separate descriptors for them
	// We may have to re-order custom properties if there are more than one.
	if (CustomProperties.Num() > 1)
	{
		// If bReplicateCustomDeltaPropertiesInRepIndexOrder is false then we use the GetLifetimeReplicatedPropsOrder instead, which is the RepLayout legacy behavior.
		if (!UE::Net::bReplicateCustomDeltaPropertiesInRepIndexOrder)
		{
			TArray<FPropertyReplicationStateDescriptorBuilder::FMemberProperty, TInlineAllocator<8>> NewCustomProperties;
			NewCustomProperties.Reserve(CustomProperties.Num());
			for (const FLifetimeProperty& LifetimeProperty : LifeTimePropertiesOriginalOrder)
			{
				if (const FPropertyReplicationStateDescriptorBuilder::FMemberProperty* MemberProperty = CustomProperties.FindByPredicate([RepIndex = LifetimeProperty.RepIndex](const FPropertyReplicationStateDescriptorBuilder::FMemberProperty& MemberProperty) { return RepIndex == MemberProperty.Property->RepIndex; }))
				{
					NewCustomProperties.Add(*MemberProperty);
				}
			}

			CustomProperties = NewCustomProperties;
		}
	}

	for (uint32 CustomPropertyIt = 0, CustomArrayPropertyEndIt = CustomProperties.Num(); CustomPropertyIt != CustomArrayPropertyEndIt; ++CustomPropertyIt)
	{
		const FPropertyReplicationStateDescriptorBuilder::FMemberProperty& MemberProperty = CustomProperties[CustomPropertyIt];

		// Verify some assumptions, for now we do not support putting properties with custom replication fragments in multiple states
		check(!EnumHasAllFlags(MemberProperty.Traits, EMemberPropertyTraits::InitOnly | EMemberPropertyTraits::HasLifetimeConditionals));

		FPropertyReplicationStateDescriptorBuilder Builder;		
		FPropertyReplicationStateDescriptorBuilder::FBuildParameters BuildParameters = {};
		BuildParameters.PathName = LazyPathName.GetPathName(InObjectClass);
		BuildParameters.DefaultStateSourceData = DefaultStateSourceData;
		BuildParameters.ReplicationSystem = Parameters.ReplicationSystem;
		BuildParameters.DescriptorType = FPropertyReplicationStateDescriptorBuilder::EDescriptorType::Class;
		BuildParameters.bIsInitState = EnumHasAnyFlags(MemberProperty.Traits, EMemberPropertyTraits::InitOnly);
		
		// We set this to false for the time being for all replication states
		BuildParameters.bAllMembersAreReplicated = false;

		// Add the property
		Builder.AddMemberProperty(MemberProperty);
		
		FString PropertyName;
		MemberProperty.Property->GetName(PropertyName);

		TRefCountPtr<const FReplicationStateDescriptor> Descriptor = Builder.Build(InObjectClass->GetName() + ToCStr(PropertyName), Parameters.DescriptorRegistry, BuildParameters);
		CreatedDescriptors.Add(Descriptor);
	}

	// If we have a registry, register the created descriptors
	if (bRegisterCreatedDescriptors && CreatedDescriptors.Num())
	{
		UE_LOG_DESCRIPTORBUILDER(VeryVerbose, TEXT("Registering descriptors for Class %s : DescriptorKey: %s ptr: 0x%p"), *InObjectClass->GetName(), *DefaultStateSource->GetPathName(), DefaultStateSource);
		Parameters.DescriptorRegistry->Register(DefaultStateSource, InObjectClass, CreatedDescriptors);
	}
	
	return CreatedDescriptors.Num();
}

}
