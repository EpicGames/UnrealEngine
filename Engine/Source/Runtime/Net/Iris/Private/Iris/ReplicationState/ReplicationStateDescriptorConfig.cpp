// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/ReplicationStateDescriptorConfig.h"
#include "Containers/ArrayView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplicationStateDescriptorConfig)

UReplicationStateDescriptorConfig::UReplicationStateDescriptorConfig()
: Super()
{
}

TConstArrayView<FSupportsStructNetSerializerConfig> UReplicationStateDescriptorConfig::GetSupportsStructNetSerializerList() const
{
	return MakeArrayView(SupportsStructNetSerializerList);
}

TConstArrayView<FReplicationStateDescriptorClassPushModelConfig> UReplicationStateDescriptorConfig::GetEnsureFullyPushModelClassNames() const
{
	return MakeArrayView(EnsureFullyPushModelClassNames);
}
