// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/ReplicationStateDescriptorRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/ReplicationSystem/ReplicationProtocolManager.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisLog.h"
#include "HAL/ConsoleManager.h"

namespace UE::Net::Private
{

static bool bPruneReplicationStateDescriptorsWithArchetype = true;
static FAutoConsoleVariableRef CVarbPruneReplicationStateDescriptorsWithArchetype(TEXT("net.Iris.PruneReplicationStateDescriptorsWithArchetype"), bPruneReplicationStateDescriptorsWithArchetype, 
	TEXT("If true, we will invalidate registered descriptors if archetype is no longer resolvable, otherwise we will keep them around until CDO is no longer valid."));

FReplicationStateDescriptorRegistry::FReplicationStateDescriptorRegistry()
: ProtocolManager(nullptr)
{
}

void FReplicationStateDescriptorRegistry::Init(const FReplicationStateDescriptorRegistryInitParams& Params)
{
	ProtocolManager = Params.ProtocolManager;
}

void FReplicationStateDescriptorRegistry::Register(const FFieldVariant& DescriptorKey, const UObject* ObjectForPruning, const FDescriptors& Descriptors)
{
	check(ObjectForPruning != nullptr);

	// Make sure the object isn't already registered
	if (const FRegisteredDescriptors* Entry = RegisteredDescriptorsMap.Find(DescriptorKey))
	{
		// We do not want to overwrite descriptors for valid objects.
		if (Entry->WeakPtrForPruning.Get() == ObjectForPruning)
		{
			if (Entry->OwnerKey == FObjectKey(GetObjectForPruning(DescriptorKey)))
			{
				checkf(false, TEXT("FReplicationStateDescriptorRegistry::Trying to register descriptors for the same UObject %s"), ToCStr(ObjectForPruning->GetName()));
				return;
			}
		}

		// We found an invalid entry, invalidate it before registering new descriptors.
		UE_LOGF(LogIris, VeryVerbose, "FReplicationStateDescriptorRegistry::Register invalidate descriptors for ptr: 0x%p", DescriptorKey.GetRawPointer());

		// Notify protocol manager about pruned descriptors
		InvalidateDescriptors(Entry->Descriptors);
		RegisteredDescriptorsMap.Remove(DescriptorKey);
	}

	FRegisteredDescriptors NewEntry;
	NewEntry.OwnerKey = FObjectKey(GetObjectForPruning(DescriptorKey));
	NewEntry.WeakPtrForPruning = TWeakObjectPtr<const UObject>(ObjectForPruning);
	NewEntry.Descriptors = Descriptors;
	RegisteredDescriptorsMap.Add(DescriptorKey, NewEntry);
}

void FReplicationStateDescriptorRegistry::Register(const FFieldVariant& DescriptorKey, const UObject* ObjectForPruning, const TRefCountPtr<const FReplicationStateDescriptor>& Descriptor)
{
	check(ObjectForPruning != nullptr);

	// Make sure the descriptor isn't already registered
	if (const FRegisteredDescriptors* Entry = RegisteredDescriptorsMap.Find(DescriptorKey))
	{
		// We do not want to overwrite descriptors for valid objects.
		if (Entry->WeakPtrForPruning.Get() == ObjectForPruning)
		{
			if (Entry->OwnerKey == FObjectKey(GetObjectForPruning(DescriptorKey)))
			{
				checkf(false, TEXT("FReplicationStateDescriptorRegistry::Trying to register descriptor for the same UObject %s"), ToCStr(ObjectForPruning->GetName()));
				return;
			}
		}

		// We found an invalid entry, invalidate it before registering new descriptors.
		UE_LOGF(LogIris, VeryVerbose, "FReplicationStateDescriptorRegistry::Register invalidate descriptor for ptr: 0x%p", DescriptorKey.GetRawPointer());
		// Notify protocol manager about pruned descriptors
		InvalidateDescriptors(Entry->Descriptors);
		RegisteredDescriptorsMap.Remove(DescriptorKey);
	}

	FRegisteredDescriptors& NewEntry = RegisteredDescriptorsMap.Emplace(DescriptorKey);
	NewEntry.OwnerKey = FObjectKey(GetObjectForPruning(DescriptorKey));
	NewEntry.WeakPtrForPruning = TWeakObjectPtr<const UObject>(ObjectForPruning);
	NewEntry.Descriptors.Add(Descriptor);
}

const FReplicationStateDescriptorRegistry::FDescriptors* FReplicationStateDescriptorRegistry::Find(const FFieldVariant& Object, const UObject* ObjectForPruning) const
{
	const FRegisteredDescriptors* Entry = RegisteredDescriptorsMap.Find(Object);

	check(ObjectForPruning != nullptr);

	if (Entry && (Entry->WeakPtrForPruning.Get() == ObjectForPruning))
	{
		// Archetype might have been reused, we will clear this up when registering
		if (Object.IsUObject() && !Entry->OwnerKey.ResolveObjectPtr())
		{
			UE_LOGF(LogIris, VeryVerbose, "FReplicationStateDescriptorRegistry Found invalidated entry ptr: 0x%p", Object.GetRawPointer());
			return nullptr;
		}

		return &Entry->Descriptors;
	}
	else
	{
		return nullptr;
	}
}

void FReplicationStateDescriptorRegistry::PruneStaleDescriptors()
{
	IRIS_PROFILER_SCOPE(FReplicationStateDescriptorRegistry_PruneStaleDescriptors);

	// Iterate over all registered descriptors and see if they have been destroyed
	for (auto It = RegisteredDescriptorsMap.CreateIterator(); It; ++It)
	{
		const FRegisteredDescriptors& RegisteredDescriptors = It.Value();
		const bool bPruneDueToWeakPtrForPruningBeingStale = !RegisteredDescriptors.WeakPtrForPruning.IsValid();
		if (bPruneDueToWeakPtrForPruningBeingStale || (bPruneReplicationStateDescriptorsWithArchetype && (RegisteredDescriptors.OwnerKey.ResolveObjectPtr() == nullptr)))
		{
			UE_LOGF(LogIris, VeryVerbose, "FReplicationStateDescriptorRegistry Pruning descriptors for ptr: 0x%p due to %ls", It.Key().GetRawPointer(), (bPruneDueToWeakPtrForPruningBeingStale ? TEXT("invalidated CDO") : TEXT("invalidated Key/Archetype")));

			// Notify protocol manager about pruned descriptors
			InvalidateDescriptors(RegisteredDescriptors.Descriptors);
			It.RemoveCurrent();
		}
	}
}

const UObject* FReplicationStateDescriptorRegistry::GetObjectForPruning(const FFieldVariant& FieldVariant)
{
	if (FieldVariant.IsUObject())
	{
		return FieldVariant.ToUObject();
	}
	else
	{
		const FField* Field = FieldVariant.ToField();
		const UObject* Object = Field->GetOwnerUObject();
		return Object;
	}
}

void FReplicationStateDescriptorRegistry::InvalidateDescriptors(const FDescriptors& Descriptors) const
{
	if (ProtocolManager)
	{
		for (const TRefCountPtr<const FReplicationStateDescriptor>& Descriptor : MakeArrayView(Descriptors.GetData(), Descriptors.Num()))
		{
			ProtocolManager->InvalidateDescriptor(Descriptor.GetReference());
		}
	}
}

}
