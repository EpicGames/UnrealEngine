// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Templates/RefCounting.h"
#include "UObject/Field.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/ObjectKey.h"

namespace UE::Net
{
	struct FReplicationStateDescriptor;
	namespace Private
	{
		class FReplicationProtocolManager;
	}
}

namespace UE::Net::Private
{

struct FReplicationStateDescriptorRegistryInitParams
{
	FReplicationProtocolManager* ProtocolManager;
};

class FReplicationStateDescriptorRegistry
{
public:
	FReplicationStateDescriptorRegistry();

	void Init(const FReplicationStateDescriptorRegistryInitParams& Params);

	/** We allow for multiple descriptors to be created by a single class */
	typedef TArray<TRefCountPtr<const FReplicationStateDescriptor>> FDescriptors;

	/** Find registered Descriptors for Object */
	const FDescriptors* Find(const FFieldVariant& Object, const UObject* ObjectForPruning) const;

	/** Find registered Descriptors for Object */
	const FDescriptors* Find(const FFieldVariant& Object) const { return Find(Object, GetObjectForPruning(Object)); }

	/** Register created Descriptors for ClassObject */
	void Register(const FFieldVariant& DescriptorKey, const UObject* ObjectForPruning, const FDescriptors& Descriptors);

	/** Register created Descriptors for ClassObject */
	void Register(const FFieldVariant& DescriptorKey, const FDescriptors& Descriptors) { Register(DescriptorKey, GetObjectForPruning(DescriptorKey), Descriptors); }

	/** Register a single Descriptor for ClassObject */
	void Register(const FFieldVariant& DescriptorKey, const UObject* ObjectForPruning, const TRefCountPtr<const FReplicationStateDescriptor>& Descriptor);

	/** Register a single Descriptor for ClassObject */
	void Register(const FFieldVariant& DescriptorKey, const TRefCountPtr<const FReplicationStateDescriptor>& Descriptor) { Register(DescriptorKey, GetObjectForPruning(DescriptorKey), Descriptor); }

	/** Invoked on PostGarbageCollect to prune descriptors for stale weak ptrs */
	void PruneStaleDescriptors();

private:
	static const UObject* GetObjectForPruning(const FFieldVariant& Object);

	void InvalidateDescriptors(const FDescriptors& Descriptors) const;

	struct FRegisteredDescriptors
	{
		TWeakObjectPtr<const UObject> WeakPtrForPruning;
		FDescriptors Descriptors;
		FObjectKey OwnerKey;
	};

	typedef TMap<FFieldVariant, FRegisteredDescriptors> FClassToDescriptorMap;
	FClassToDescriptorMap RegisteredDescriptorsMap;
	FReplicationProtocolManager* ProtocolManager;
};

}
