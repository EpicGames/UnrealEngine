// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/ReplicationStateStorage.h"
#include "Iris/Core/IrisMemoryTracker.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include <limits>

namespace UE::Net
{

FReplicationStateStorage::FReplicationStateStorage()
{
	static_assert(InvalidObjectInfoIndex == 0, "This class assumes InvalidObjectInfoIndex == 0");
}

FReplicationStateStorage::~FReplicationStateStorage()
{
}

void FReplicationStateStorage::Init(FReplicationStateStorageInitParams& InitParams)
{
	const uint32 MaxDeltaCompressedObjectInfoCount = 1U + FPlatformMath::Min(uint32(std::numeric_limits<ObjectInfoIndexType>::max()), FPlatformMath::Min(InitParams.MaxObjectCount, InitParams.MaxDeltaCompressedObjectCount));

	// Make sure the MaxObjectInfoCount calculation can't overflow.
	static_assert(std::numeric_limits<decltype(MaxDeltaCompressedObjectInfoCount)>::max() > std::numeric_limits<ObjectInfoIndexType>::max(), "");

	NetRefHandleManager = InitParams.NetRefHandleManager;

	UsedPerObjectInfos.Init(MaxDeltaCompressedObjectInfoCount);
	UsedPerObjectInfos.SetBit(InvalidObjectInfoIndex);
	
	ObjectIndexToObjectInfoIndex.SetNumZeroed(InitParams.MaxInternalNetRefIndex);

	InternalSerializationContext = Private::FInternalNetSerializationContext(InitParams.ReplicationSystem);
	SerializationContext.SetInternalContext(&InternalSerializationContext);
}

void FReplicationStateStorage::Deinit()
{
#if UE_NET_VALIDATE_DC_BASELINES
	check(BaselineStorageValidation.Num() == 0);
#endif
}

void FReplicationStateStorage::OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex)
{
	ObjectIndexToObjectInfoIndex.SetNumZeroed(NewMaxInternalIndex);
}

const uint8* FReplicationStateStorage::GetState(uint32 ObjectIndex, EReplicationStateType StateType) const
{
	if (StateType != EReplicationStateType::CurrentSendState && StateType != EReplicationStateType::CurrentRecvState)
	{
		return nullptr;
	}

	if (const FPerObjectInfo* ObjectInfo = GetPerObjectInfoForObject(ObjectIndex))
	{
		switch (StateType)
		{
			case EReplicationStateType::CurrentSendState:
			{
				return ObjectInfo->StateBuffers[(unsigned)EStateBufferType::SendState];
			}
			case EReplicationStateType::CurrentRecvState:
			{
				return ObjectInfo->StateBuffers[(unsigned)EStateBufferType::RecvState];
			}
			default:
			{
				checkf(false, TEXT("Unknown EReplicationStateType"));
				return nullptr;
			}
		}
	}

	// Slow path but we don't want to keep PerObjectInfo around unless we have to.
	switch (StateType)
	{
		case EReplicationStateType::CurrentSendState:
		{
			return NetRefHandleManager->GetReplicatedObjectStateBufferNoCheck(ObjectIndex);
		}
		case EReplicationStateType::CurrentRecvState:
		{
			const Private::FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = NetRefHandleManager->GetReplicatedObjectData(ObjectIndex);
			return ReplicatedObjectData.ReceiveStateBuffer;
		}
		default:
		{
			checkf(false, TEXT("Unknown EReplicationStateType"));
			return nullptr;
		}
	}
}

uint8* FReplicationStateStorage::AllocBaseline(uint32 ObjectIndex, EReplicationStateType Base)
{
	LLM_SCOPE_BYTAG(IrisState);

	FPerObjectInfo* ObjectInfo = GetOrCreatePerObjectInfoForObject(ObjectIndex);
	if (!ensureMsgf(ObjectInfo != nullptr, TEXT("Out of object infos. Max number of object infos is %u."), UsedPerObjectInfos.GetNumBits()))
	{
		return nullptr;
	}

	const FReplicationProtocol* Protocol = ObjectInfo->Protocol;

	uint8* Storage = nullptr;
	switch (Base)
	{
		case EReplicationStateType::UninitializedState:
		{
			Storage = static_cast<uint8*>(FMemory::Malloc(Protocol->InternalTotalSize, Protocol->InternalTotalAlignment));
			break;
		};

		case EReplicationStateType::ZeroedState:
		{
			Storage = static_cast<uint8*>(FMemory::MallocZeroed(Protocol->InternalTotalSize, Protocol->InternalTotalAlignment));
			break;
		};

		case EReplicationStateType::DefaultState:
		{
			Storage = static_cast<uint8*>(FMemory::Malloc(Protocol->InternalTotalSize, Protocol->InternalTotalAlignment));
			CloneDefaultState(ObjectInfo->Protocol, Storage);
			break;
		};

		case EReplicationStateType::CurrentSendState:
		{
			Storage = static_cast<uint8*>(FMemory::Malloc(Protocol->InternalTotalSize, Protocol->InternalTotalAlignment));
			CloneState(Protocol, Storage, ObjectInfo->StateBuffers[(unsigned)EStateBufferType::SendState]);
			break;
		};

		case EReplicationStateType::CurrentRecvState:
		{
			Storage = static_cast<uint8*>(FMemory::Malloc(Protocol->InternalTotalSize, Protocol->InternalTotalAlignment));
			CloneState(Protocol, Storage, ObjectInfo->StateBuffers[(unsigned)EStateBufferType::RecvState]);
			break;
		}

		default:
		{
			check(false);
			return nullptr;
		}
	};

	++ObjectInfo->AllocationCount;

#if UE_NET_VALIDATE_DC_BASELINES
	{
		check(Storage != nullptr);
		BaselineStorageValidation.Add(ObjectIndex, Storage);
	}
#endif

	return static_cast<uint8*>(Storage);
}

void FReplicationStateStorage::FreeBaseline(uint32 ObjectIndex, uint8* Storage)
{
	FPerObjectInfo* ObjectInfo = GetPerObjectInfoForObject(ObjectIndex);
	if (!ensureMsgf(ObjectInfo != nullptr, TEXT("Trying to free baseline storage pointer %p for object ( InternalIndex: %u ) with no PerObjectInfo."), Storage, ObjectIndex))
	{
		return;
	}

#if UE_NET_VALIDATE_DC_BASELINES
	{
		const int32 RemovedCount = BaselineStorageValidation.RemoveSingle(ObjectIndex, Storage);
		check(RemovedCount == 1);
	}
#endif

	FreeState(ObjectInfo->Protocol, Storage);
	FMemory::Free(static_cast<void*>(Storage));

	if (0 == --ObjectInfo->AllocationCount)
	{
		FreePerObjectInfoForObject(ObjectIndex);
	}
}

FReplicationStateStorage::FBaselineReservation FReplicationStateStorage::ReserveBaseline(uint32 ObjectIndex, EReplicationStateType Base)
{
	LLM_SCOPE_BYTAG(IrisState);

	FBaselineReservation Reservation;

	FPerObjectInfo* ObjectInfo = GetOrCreatePerObjectInfoForObject(ObjectIndex);
	if (!ensureMsgf(ObjectInfo != nullptr, TEXT("Out of object infos. Max number of object infos is %u."), UsedPerObjectInfos.GetNumBits()))
	{
		return Reservation;
	}

	if (Base == EReplicationStateType::CurrentSendState)
	{
		Reservation.BaselineBaseStorage = ObjectInfo->StateBuffers[unsigned(EStateBufferType::SendState)];
#if UE_NET_VALIDATE_DC_BASELINES
		if (!ensureAlways(Reservation.BaselineBaseStorage != nullptr))
		{
			return Reservation;
		}
#endif
	}
	else if (Base == EReplicationStateType::CurrentRecvState)
	{
		Reservation.BaselineBaseStorage = ObjectInfo->StateBuffers[unsigned(EStateBufferType::RecvState)];
#if UE_NET_VALIDATE_DC_BASELINES
		if (!ensureAlways(Reservation.BaselineBaseStorage != nullptr))
		{
			return Reservation;
		}
#endif
	}

	const FReplicationProtocol* Protocol = ObjectInfo->Protocol;
	Reservation.ReservedStorage = static_cast<uint8*>(FMemory::Malloc(Protocol->InternalTotalSize, Protocol->InternalTotalAlignment));
	if (Reservation.ReservedStorage == nullptr)
	{
		if (ObjectInfo->AllocationCount == 0)
		{
			FreePerObjectInfoForObject(ObjectIndex);
		}
		return FBaselineReservation();
	}

	++ObjectInfo->AllocationCount;

#if UE_NET_VALIDATE_DC_BASELINES
	{
		BaselineStorageValidation.Add(ObjectIndex, Reservation.ReservedStorage);
	}
#endif

	return Reservation;
}

void FReplicationStateStorage::CancelBaselineReservation(uint32 ObjectIndex, uint8* Storage)
{
	FPerObjectInfo* ObjectInfo = GetPerObjectInfoForObject(ObjectIndex);
	if (!ensureMsgf(ObjectInfo != nullptr, TEXT("Trying to cancel a baseline reservation with pointer %p for object ( InternalIndex: %u ) with no PerObjectInfo."), Storage, ObjectIndex))
	{
		return;
	}

#if UE_NET_VALIDATE_DC_BASELINES
	{
		const int32 RemovedCount = BaselineStorageValidation.RemoveSingle(ObjectIndex, Storage);
		check(RemovedCount == 1);
	}
#endif

	FMemory::Free(static_cast<void*>(Storage));

	if (0 == --ObjectInfo->AllocationCount)
	{
		FreePerObjectInfoForObject(ObjectIndex);
	}
}

void FReplicationStateStorage::CommitBaselineReservation(uint32 ObjectIndex, uint8* Storage, EReplicationStateType Base)
{
#if UE_NET_VALIDATE_DC_BASELINES
	{
		check(BaselineStorageValidation.FindPair(ObjectIndex, Storage) != nullptr);
	}
#endif

	FPerObjectInfo* ObjectInfo = GetPerObjectInfoForObject(ObjectIndex);
	if (!ensureMsgf(ObjectInfo != nullptr, TEXT("Trying to commit baseline storage pointer %p for object ( InternalIndex: %u ) with no PerObjectInfo."), Storage, ObjectIndex))
	{
		return;
	}

	const FReplicationProtocol* Protocol = ObjectInfo->Protocol;

	switch (Base)
	{
		case EReplicationStateType::UninitializedState:
		{
			break;
		};

		case EReplicationStateType::ZeroedState:
		{
			FMemory::Memzero(Storage, Protocol->InternalTotalSize);
			break;
		};

		case EReplicationStateType::DefaultState:
		{
			CloneDefaultState(ObjectInfo->Protocol, Storage);
			break;
		};

		case EReplicationStateType::CurrentSendState:
		{
			CloneState(Protocol, Storage, ObjectInfo->StateBuffers[(unsigned)EStateBufferType::SendState]);
			break;
		};

		case EReplicationStateType::CurrentRecvState:
		{
			CloneState(Protocol, Storage, ObjectInfo->StateBuffers[(unsigned)EStateBufferType::RecvState]);
			break;
		}

		default:
		{
			break;
		}
	};
}

FReplicationStateStorage::FPerObjectInfo* FReplicationStateStorage::GetOrCreatePerObjectInfoForObject(uint32 ObjectIndex)
{
	ObjectInfoIndexType& InfoIndex = ObjectIndexToObjectInfoIndex[ObjectIndex];
	if (InfoIndex == InvalidObjectInfoIndex)
	{
		const ObjectInfoIndexType NewInfoIndex = AllocPerObjectInfo();
		if (NewInfoIndex == InvalidObjectInfoIndex)
		{
			return nullptr;
		}
		InfoIndex = NewInfoIndex;

		FPerObjectInfo& ObjectInfo = ObjectInfos[NewInfoIndex];
		
		// Cache some info
		const Private::FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = NetRefHandleManager->GetReplicatedObjectData(ObjectIndex);
		const FReplicationProtocol* Protocol = ReplicatedObjectData.Protocol;
		ObjectInfo.Protocol = Protocol;
		ObjectInfo.StateBuffers[(unsigned)EStateBufferType::SendState] = NetRefHandleManager->GetReplicatedObjectStateBufferNoCheck(ObjectIndex);
		ObjectInfo.StateBuffers[(unsigned)EStateBufferType::RecvState] = ReplicatedObjectData.ReceiveStateBuffer;
		return &ObjectInfo;
	}

	return &ObjectInfos[InfoIndex];
}

FReplicationStateStorage::FPerObjectInfo* FReplicationStateStorage::GetPerObjectInfoForObject(uint32 ObjectIndex)
{
	const ObjectInfoIndexType InfoIndex = ObjectIndexToObjectInfoIndex[ObjectIndex];
	if (InfoIndex == InvalidObjectInfoIndex)
	{
		return nullptr;
	}

	return &ObjectInfos[InfoIndex];
}

const FReplicationStateStorage::FPerObjectInfo* FReplicationStateStorage::GetPerObjectInfoForObject(uint32 ObjectIndex) const
{
	const ObjectInfoIndexType InfoIndex = ObjectIndexToObjectInfoIndex[ObjectIndex];
	if (InfoIndex == InvalidObjectInfoIndex)
	{
		return nullptr;
	}

	return &ObjectInfos[InfoIndex];
}

void FReplicationStateStorage::FreePerObjectInfoForObject(uint32 ObjectIndex)
{
	ObjectInfoIndexType& InfoIndex = ObjectIndexToObjectInfoIndex[ObjectIndex];
	if (InfoIndex != InvalidObjectInfoIndex)
	{
		FreePerObjectInfo(InfoIndex);
		InfoIndex = InvalidObjectInfoIndex;
	}
}

FReplicationStateStorage::ObjectInfoIndexType FReplicationStateStorage::AllocPerObjectInfo()
{
	const uint32 InfoIndex = UsedPerObjectInfos.FindFirstZero();
	if (InfoIndex != FNetBitArray::InvalidIndex)
	{
		UsedPerObjectInfos.SetBit(InfoIndex);

		if (InfoIndex >= uint32(ObjectInfos.Num()))
		{
			ObjectInfos.Add(ObjectInfoGrowCount);
		}

		FPerObjectInfo& ObjectInfo = ObjectInfos[InfoIndex];
		ObjectInfo = FPerObjectInfo();

		return static_cast<ObjectInfoIndexType>(InfoIndex);
	}

	return InvalidObjectInfoIndex;
}

void FReplicationStateStorage::FreePerObjectInfo(ObjectInfoIndexType Index)
{
	UsedPerObjectInfos.ClearBit(Index);
}

void FReplicationStateStorage::CloneState(const FReplicationProtocol* Protocol, uint8* TargetState, const uint8* SourceState)
{
	Private::FReplicationProtocolOperationsInternal::CloneQuantizedState(SerializationContext, TargetState, SourceState, Protocol);
}

void FReplicationStateStorage::CloneDefaultState(const FReplicationProtocol* Protocol, uint8* TargetState)
{
	// N.B. InitializeFromDefaultState does the required memcpying internally.
	FReplicationProtocolOperations::InitializeFromDefaultState(SerializationContext, TargetState, Protocol);
	check(!SerializationContext.HasError());
}

void FReplicationStateStorage::FreeState(const FReplicationProtocol* Protocol, uint8* State)
{
	if (EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasDynamicState))
	{
		Private::FReplicationProtocolOperationsInternal::FreeDynamicState(SerializationContext, State, Protocol);
		check(!SerializationContext.HasError());
	}
}

}
