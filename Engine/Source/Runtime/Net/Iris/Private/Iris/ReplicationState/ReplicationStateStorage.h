// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Iris/IrisConfigInternal.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Containers/ChunkedArray.h"
#if UE_NET_VALIDATE_DC_BASELINES
#include "Containers/Map.h"
#endif

namespace UE::Net
{
	struct FReplicationProtocol;
	
	typedef uint32 FInternalNetRefIndex;

	namespace Private
	{
		class FNetRefHandleManager;
	};
}

namespace UE::Net
{

struct FReplicationStateStorageInitParams
{
	UReplicationSystem* ReplicationSystem = nullptr;
	const Private::FNetRefHandleManager* NetRefHandleManager = nullptr;
	uint32 MaxObjectCount = 0;
	uint32 MaxInternalNetRefIndex = 0;
	uint32 MaxConnectionCount = 0;
	uint32 MaxDeltaCompressedObjectCount = 0;
};

enum class EReplicationStateType : unsigned
{
	UninitializedState,
	ZeroedState,
	DefaultState,
	CurrentSendState,
	CurrentRecvState,
};

class FReplicationStateStorage
{
public:
	class FBaselineReservation
	{
	public:
		bool IsValid() const { return ReservedStorage != nullptr; }

		// The uninitialized memory that was reserved.
		uint8* ReservedStorage = nullptr;
		/*
		 * The state presumed to be cloned if the reservation is committed. N.B.
		 * Can only be valid if CurrentSendState or CurrentRecvState was the base.
		 */
		const uint8* BaselineBaseStorage = nullptr;
	};

public:
	FReplicationStateStorage();
	~FReplicationStateStorage();

	void Init(FReplicationStateStorageInitParams& InitParams);
	void Deinit();

	/*
	 * Returns a buffer to a state of the requested type. Only supports CurrentSendState and CurrentRecvState. May return null
	 * for example if requesting CurrentSendState on the receiving end. Returned pointer may not be written to.
	 */
	const uint8* GetState(uint32 ObjectIndex, EReplicationStateType Base) const;

	/*
	 * Allocates an internal state buffer large enough to accommodate the entire internal state, including init state,
	 * for an object and initializes it to the desired state, using cloning if needed.
	 * Returns a non-null pointer if there's memory to fulfill the request.
	 * If the requested state type is "Uninitialized" the memory must be properly initialized, zeroed or copied from other state,
	 * before calling FreeBaseline!
	 */
	uint8* AllocBaseline(uint32 ObjectIndex, EReplicationStateType Base);
	/* Frees a baseline allocated with AllocBaseline(). The Storage must have been properly initialized as any dynamic state will be freed as well. */
	void FreeBaseline(uint32 ObjectIndex, uint8* Storage);

	/*
	 * Allocates memory for a baseline but does not initialize it.
	 * The memory should either be committed or canceled. A committed baseline
	 * must later be freed.
	 */
	FBaselineReservation ReserveBaseline(uint32 ObjectIndex, EReplicationStateType Base);
	/* Cancels a baseline reservation. The pointer is invalid to use after this call. */
	void CancelBaselineReservation(uint32 ObjectIndex, uint8* Storage);
	/*
	 * Initializes a buffer previously returned by ReserveBaseline. The Base must identical to that in the ReserveBaseline call.
	 * The baseline should later be freed by FreeBaseline.
	 */
	void CommitBaselineReservation(uint32 ObjectIndex, uint8* Storage, EReplicationStateType Base);

	/** Called when the maximum InternalNetRefIndex increased and we need to realloc our lists */
	void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex);

private:
	using ObjectInfoIndexType = uint16;

	enum : unsigned
	{
		InvalidObjectInfoIndex = 0,

		ObjectInfoGrowCount = 256,
	};

	enum class EStateBufferType : unsigned
	{
		SendState,
		RecvState,

		Count
	};

	struct FPerObjectInfo
	{
		const FReplicationProtocol* Protocol = nullptr;
		const uint8* StateBuffers[(unsigned)EStateBufferType::Count] = {};
		uint16 AllocationCount = 0;
	};

	FPerObjectInfo* GetOrCreatePerObjectInfoForObject(uint32 ObjectIndex);
	FPerObjectInfo* GetPerObjectInfoForObject(uint32 ObjectIndex);
	const FPerObjectInfo* GetPerObjectInfoForObject(uint32 ObjectIndex) const;
	void FreePerObjectInfoForObject(uint32 ObjectIndex);

	ObjectInfoIndexType AllocPerObjectInfo();
	void FreePerObjectInfo(ObjectInfoIndexType Index);

	void CloneState(const FReplicationProtocol* Protocol, uint8* TargetState, const uint8* SourceState);
	void CloneDefaultState(const FReplicationProtocol* Protocol, uint8* TargetState);
	void FreeState(const FReplicationProtocol* Protocol, uint8* State);

private:
#if UE_NET_VALIDATE_DC_BASELINES
	TMultiMap<uint32, const void*> BaselineStorageValidation;
#endif

	const Private::FNetRefHandleManager* NetRefHandleManager = nullptr;

	FNetSerializationContext SerializationContext;
	Private::FInternalNetSerializationContext InternalSerializationContext;

	FNetBitArray UsedPerObjectInfos;
	TArray<ObjectInfoIndexType> ObjectIndexToObjectInfoIndex;
	TChunkedArray<FPerObjectInfo, ObjectInfoGrowCount*sizeof(FPerObjectInfo)> ObjectInfos;
};

}
