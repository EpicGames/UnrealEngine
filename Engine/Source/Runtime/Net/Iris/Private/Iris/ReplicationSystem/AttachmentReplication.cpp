// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/AttachmentReplication.h"
#include "HAL/IConsoleManager.h"
#include "Iris/Core/IrisLog.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobAssembler.h"
#include "Iris/ReplicationSystem/NetBlob/PartialNetObjectAttachmentHandler.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamWriter.h"

namespace UE::Net::Private
{

namespace AttachmentReplicationCVars
{
	static int32 UnreliableRPCQueueSize = 10;
	FAutoConsoleVariableRef CVarUnreliableRPCQueueSize(TEXT("net.UnreliableRPCQueueSize"), UnreliableRPCQueueSize, TEXT("Maximum number of unreliable RPCs queued per object. If more RPCs are queued then older ones will be dropped."));

	static int32 ReliableRPCQueueSize = 4096;
	FAutoConsoleVariableRef CVarReliableRPCQueueSize(TEXT("net.ReliableRPCQueueSize"), ReliableRPCQueueSize, TEXT("Maximum number of reliable RPCs queued per object. This is in addition to the send window size. This is to support very large RPCs that are split into smaller pieces."));

	static int32 ClientToServerUnreliableRPCQueueSize = 16;
	FAutoConsoleVariableRef CVarClientToServerUnreliableRPCQueueSize(TEXT("net.ClientToServerUnreliableRPCQueueSize"), ClientToServerUnreliableRPCQueueSize, TEXT( "Maximum number of unreliable RPCs queued for sending from the client to the server. If more RPCs are queued then older ones will be dropped."));

	static int32 MaxSimultaneousObjectsWithRPCs = 4096;
	FAutoConsoleVariableRef CVarMaxSimultaneousObjectsWithRPCs(TEXT("net.MaxSimultaneousObjectsWithRPCs"), MaxSimultaneousObjectsWithRPCs, TEXT("Maximum number of objects that can have unsent RPCs at the same time. "));
}

static const FName NetError_UnreliableQueueFull("Unreliable attachment queue full");
static const FName NetError_UnsupportedNetBlob("Unsupported NetBlob type");
static const FName NetError_TooManyObjectsWithFunctionCalls("Too many objects being targeted by RPCs at the same time.");

// SendQueue and Writer
class FNetObjectAttachmentSendQueue::FReliableSendQueue
{
public:
	FReliableSendQueue()
	: MaxPreQueueCount(AttachmentReplicationCVars::ReliableRPCQueueSize)
	{
	}

	bool HasUnsentBlobs() const
	{
		return PreQueue.Num() > 0 || ReliableQueue.HasUnsentBlobs();
	}

	bool CanSendBlobs() const
	{
		return ReliableQueue.HasUnsentBlobs();
	}

	bool IsSafeToDestroy() const
	{
		return ReliableQueue.IsSafeToDestroy();
	}

	bool IsAllSentAndAcked() const
	{
		return PreQueue.Num() == 0 && ReliableQueue.IsAllSentAndAcked();
	}

	bool IsSendWindowFull() const
	{
		return ReliableQueue.IsSendWindowFull();
	}

	uint32 GetUnsentBlobCount() const
	{
		return ReliableQueue.GetUnsentBlobCount() + static_cast<uint32>(PreQueue.Num());
	}

	bool Enqueue(TArrayView<const TRefCountPtr<FNetBlob>> Attachments)
	{
		// If we have blobs in the pre queue then continue adding to that.
		const uint32 PreQueueCount = static_cast<uint32>(PreQueue.Num());
		const uint32 AttachmentCount = static_cast<uint32>(Attachments.Num());

		// Assume that we won't fit all the attachments unless they can fit in the pre-queue.
		if (AttachmentCount + PreQueueCount > MaxPreQueueCount)
		{
			ensureMsgf(false, TEXT("%u reliable RPCs not enqueued as adding them would exceed the current MaxPreQueueCount (%u)"), AttachmentCount, MaxPreQueueCount);
			return false;
		}

		uint32 QueuedCount = 0;
		if (PreQueueCount == 0)
		{
			for (const TRefCountPtr<FNetBlob>& Attachment : Attachments)
			{
				if (!ReliableQueue.Enqueue(Attachment))
				{
					break;
				}

				++QueuedCount;
			}
		}

		const uint32 AddToPreQueueCount = AttachmentCount - QueuedCount;
		if (AddToPreQueueCount > 0)
		{
			PreQueue.Append(Attachments.GetData() + QueuedCount, AddToPreQueueCount);
		}

		return true;
	}

	uint32 Serialize(FNetSerializationContext& Context, FNetRefHandle RefHandle, FReliableNetBlobQueue::FReplicationRecord& OutRecord)
	{
		if (RefHandle.IsValid())
		{
			return ReliableQueue.SerializeWithObject(Context, RefHandle, OutRecord);
		}
		else
		{
			return ReliableQueue.Serialize(Context, OutRecord);
		}
	}

	void ProcessPacketDeliveryStatus(EPacketDeliveryStatus Status, const FReliableNetBlobQueue::FReplicationRecord& Record, bool& bOutMarkHasAttachmentsToSendDirty)
	{
		ReliableQueue.ProcessPacketDeliveryStatus(Status, Record);
		if (Status == EPacketDeliveryStatus::Delivered && Record.IsValid())
		{
			const uint32 PreUnsentBlobCount = ReliableQueue.GetUnsentBlobCount();
			PopulateQueueFromPreQueue();
			bOutMarkHasAttachmentsToSendDirty = ReliableQueue.GetUnsentBlobCount() > PreUnsentBlobCount;
		}
	}

	void CommitReplicationRecord(const FReliableNetBlobQueue::FReplicationRecord& Record)
	{
		ReliableQueue.CommitReplicationRecord(Record);
	}

	SIZE_T GetUnreliableCount() const
	{
		SIZE_T UnreliableCount = 0;

		// Count all unreliable attachments in PreQueue and ReliableQueue
		for (const TRefCountPtr<FNetBlob>& BlobRef : PreQueue)
		{
			const FNetBlob* Blob = BlobRef.GetReference();
			UnreliableCount += Blob && !Blob->IsReliable() ? 1 : 0;
		}
	
		UnreliableCount += ReliableQueue.GetUnreliableCount();

		return UnreliableCount;
	}

	void DropUnreliable()
	{
		// Drop all unreliable from pre-queue
		PreQueue.RemoveAll([](const TRefCountPtr<FNetBlob>& BlobRef)
		{
			const FNetBlob* Blob = BlobRef.GetReference();
			return Blob == nullptr || !Blob->IsReliable();
		});

		ReliableQueue.DropUnreliable();
	}

private:
	void PopulateQueueFromPreQueue()
	{
		const uint32 AttachmentCount = static_cast<uint32>(PreQueue.Num());
		if (AttachmentCount == 0)
		{
			return;
		}

		// Similar to enqueueing, but without all the checks as we've already passed them.
		TArray<TRefCountPtr<FNetBlob>> PrevPreQueue = MoveTemp(PreQueue);
		TArrayView<const TRefCountPtr<FNetBlob>> Attachments = MakeArrayView(PrevPreQueue);

		uint32 QueuedCount = 0;
		for (const TRefCountPtr<FNetBlob>& Attachment : Attachments)
		{
			if (!ReliableQueue.Enqueue(Attachment))
			{
				break;
			}

			++QueuedCount;
		}

		const uint32 AddToPreQueueCount = AttachmentCount - QueuedCount;
		if (AddToPreQueueCount > 0)
		{
			PreQueue.Append(Attachments.GetData() + QueuedCount, AddToPreQueueCount);
		}
	}

	FReliableNetBlobQueue ReliableQueue;
	TArray<TRefCountPtr<FNetBlob>> PreQueue;
	const uint32 MaxPreQueueCount;
};

FNetObjectAttachmentSendQueue::FNetObjectAttachmentSendQueue()
: ReliableQueue(nullptr)
, MaxUnreliableCount(AttachmentReplicationCVars::UnreliableRPCQueueSize)
{
}

FNetObjectAttachmentSendQueue::~FNetObjectAttachmentSendQueue()
{
	delete ReliableQueue;
}

bool FNetObjectAttachmentSendQueue::Enqueue(TArrayView<const TRefCountPtr<FNetBlob>> Attachments)
{
	const FNetBlobCreationInfo& CreationInfo = Attachments[0]->GetCreationInfo();
	if (EnumHasAnyFlags(CreationInfo.Flags, ENetBlobFlags::Reliable | ENetBlobFlags::Ordered))
	{
		if (ReliableQueue == nullptr)
		{
			ReliableQueue = new FReliableSendQueue();
		}

		return ReliableQueue->Enqueue(Attachments);
	}
	else
	{
		const SIZE_T TotalCountNeeded = UnreliableQueue.Count() + Attachments.Num();
		if (TotalCountNeeded > MaxUnreliableCount)
		{
			UE_LOG(LogIris, Verbose, TEXT("Dropping old RPCs due to too many unreliable attachments: %" SIZE_T_FMT " max: %u"), UnreliableQueue.Count(), MaxUnreliableCount);
			const SIZE_T PopCount = FPlatformMath::Min(TotalCountNeeded - MaxUnreliableCount, UnreliableQueue.Count());
			UnreliableQueue.Pop(PopCount);
		}

		for (const TRefCountPtr<FNetBlob>& Attachment : Attachments)
		{
			UnreliableQueue.Enqueue(Attachment);
		}
	}

	return true;
}

SIZE_T FNetObjectAttachmentSendQueue::GetUnreliableCount() const
{
	SIZE_T UnreliableCount = 0;

	UnreliableCount += UnreliableQueue.Count();
	if (ReliableQueue)
	{
		UnreliableCount += ReliableQueue->GetUnreliableCount();
	}

	return UnreliableCount;
}

void FNetObjectAttachmentSendQueue::DropUnreliable(bool& bOutHasUnsent)
{
	UnreliableQueue.Empty();
	if (ReliableQueue)
	{
		ReliableQueue->DropUnreliable();
		bOutHasUnsent = ReliableQueue->HasUnsentBlobs();
	}
	else
	{
		bOutHasUnsent = false;
	}
}

SIZE_T FNetObjectAttachmentSendQueue::GetReliableCount() const
{
	if (ReliableQueue == nullptr)
	{
		return 0;
	}

	return ReliableQueue->GetUnsentBlobCount();
}

bool FNetObjectAttachmentSendQueue::HasUnsent() const
{
	return !UnreliableQueue.IsEmpty() || (ReliableQueue != nullptr && ReliableQueue->HasUnsentBlobs());
}

bool FNetObjectAttachmentSendQueue::HasUnsentUnreliable() const
{
	return !UnreliableQueue.IsEmpty();
}

bool FNetObjectAttachmentSendQueue::IsAllSentAndAcked() const
{
	return UnreliableQueue.IsEmpty() && (ReliableQueue == nullptr || ReliableQueue->IsAllSentAndAcked());
}

bool FNetObjectAttachmentSendQueue::IsAllReliableSentAndAcked() const
{
	return ReliableQueue == nullptr || ReliableQueue->IsAllSentAndAcked();
}

bool FNetObjectAttachmentSendQueue::CanSendMoreReliableAttachments() const
{
	return ReliableQueue == nullptr || !ReliableQueue->IsSendWindowFull();
}

bool FNetObjectAttachmentSendQueue::IsSafeToDestroy() const
{
	return UnreliableQueue.IsEmpty() && (ReliableQueue == nullptr || ReliableQueue->IsSafeToDestroy());
}

void FNetObjectAttachmentSendQueue::SetUnreliableQueueCapacity(uint32 QueueCapacity)
{
	// MaxUnreliableCount is what prevents the queue from growing too large.
	MaxUnreliableCount = QueueCapacity;
	
	const SIZE_T UnreliableCount = UnreliableQueue.Count();
	if (QueueCapacity >= UnreliableCount)
	{
		return;
	}

	const SIZE_T DropCount = UnreliableCount - QueueCapacity;
	UE_LOG(LogIris, Warning, TEXT("Dropping %" SIZE_T_FMT " attachments due to change in unreliable queue capacity to %u"), DropCount, QueueCapacity);
	UnreliableQueue.Pop(DropCount);
}

EAttachmentWriteStatus FNetObjectAttachmentSendQueue::Serialize(FNetSerializationContext& Context, FNetRefHandle RefHandle, FNetObjectAttachmentSendQueue::FCommitRecord& OutRecord, bool& bOutHasUnsentAttachments)
{
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	FCommitRecord ReplicationRecord;
	// This count is the total number of unsent reliable blobs. If the reliable window is full no blobs can be sentuntil some have been acked.
	const uint32 UnsentReliableCount = (ReliableQueue != nullptr ? ReliableQueue->GetUnsentBlobCount() : 0U);
	const bool bCanSendReliableAttachments = UnsentReliableCount > 0 && ReliableQueue->CanSendBlobs();
	const bool bHasUnreliableAttachments = !UnreliableQueue.IsEmpty();
	if (!(bCanSendReliableAttachments || bHasUnreliableAttachments))
	{
		// Ideally we shouldn't get here, but we can handle it. Important to overflow to report a soft error.
		Writer.DoOverflow();
		OutRecord = ReplicationRecord;
		bOutHasUnsentAttachments = false;

		if (UnsentReliableCount > 0)
		{
			// We want to send reliable blobs but the window is full.
			return EAttachmentWriteStatus::ReliableWindowFull;
		}
		else
		{
			return EAttachmentWriteStatus::NoAttachments;
		}
	}

	UE_NET_TRACE_SCOPE(RPCs, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	Writer.WriteBool(bCanSendReliableAttachments);
	const uint32 HasUnreliableAttachmentsWritePos = Writer.GetPosBits();
	Writer.WriteBool(bHasUnreliableAttachments);

	if (Writer.IsOverflown())
	{
		OutRecord = ReplicationRecord;
		bOutHasUnsentAttachments = true;
		return EAttachmentWriteStatus::BitstreamOverflow;
	}

	uint32 SerializedReliableCount = 0;
	if (bCanSendReliableAttachments)
	{
		UE_NET_TRACE_SCOPE(Ordered, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		SerializedReliableCount = SerializeReliable(Context, RefHandle, ReplicationRecord.ReliableReplicationRecord);
		// If we couldn't fit any reliable attachments then don't even try unreliable
		if (SerializedReliableCount == 0)
		{
			Writer.DoOverflow();
			OutRecord = ReplicationRecord;
			bOutHasUnsentAttachments = true;
			return EAttachmentWriteStatus::BitstreamOverflow;
		}
	}

	// Assume success and change when necessary.
	EAttachmentWriteStatus WriteStatus = EAttachmentWriteStatus::Success;

	uint32 SerializedUnreliableCount = 0;
	if (bHasUnreliableAttachments)
	{
		UE_NET_TRACE_SCOPE(Unreliable, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		SerializedUnreliableCount = SerializeUnreliable(Context, RefHandle, ReplicationRecord.UnreliableCommitRecord);
		if (SerializedUnreliableCount == 0)
		{
			// If we didn't manage to send anything then inform the caller of this via overflowing the bitstream
			if (SerializedReliableCount == 0)
			{
				Writer.DoOverflow();
				// Even if we couldn't send reliable attachments due to the window being full we still probably wouldn't have fit any, so return the overflow status.
				WriteStatus = EAttachmentWriteStatus::BitstreamOverflow;
			}
			// Patch up information previously saying there were unreliable attachments because we couldn't fit any in the bitstream
			else
			{
				FNetBitStreamWriteScope WriteScope(Writer, HasUnreliableAttachmentsWritePos);
				Writer.WriteBool(false);
			}
		}
	}

	if (WriteStatus == EAttachmentWriteStatus::Success)
	{
		// Override WriteStatus with ReliableWindowFull in case we fit one or more unreliable attachments but weren't able to write any reliable ones.
		if (UnsentReliableCount > 0 && !bCanSendReliableAttachments)
		{
			WriteStatus = EAttachmentWriteStatus::ReliableWindowFull;
		}
	}

	OutRecord = ReplicationRecord;
	bOutHasUnsentAttachments = (SerializedReliableCount < UnsentReliableCount) || (SerializedUnreliableCount < UnreliableQueue.Count());
	return WriteStatus;
}

uint32 FNetObjectAttachmentSendQueue::SerializeReliable(FNetSerializationContext& Context, FNetRefHandle RefHandle, FReliableNetBlobQueue::FReplicationRecord& OutRecord)
{
	return ReliableQueue->Serialize(Context, RefHandle, OutRecord);
}

uint32 FNetObjectAttachmentSendQueue::SerializeUnreliable(FNetSerializationContext& Context, FNetRefHandle RefHandle, FUnreliableReplicationRecord& OutRecord)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	const FObjectReferenceCache* ObjectReferenceCache = Context.GetInternalContext()->ObjectReferenceCache;

	uint32 SerializedUnreliableCount = 0;
	uint32 PrevHasMoreAttachmentsWritePos = 0;
	const bool bSerializeWithObject = RefHandle.IsValid();
	for (SIZE_T AttachmentIt = 0, AttachmentEndIt = UnreliableQueue.Count(); AttachmentIt != AttachmentEndIt; ++AttachmentIt)
	{
		FNetBitStreamRollbackScope RollbackScope(*Writer);
		FNetExportRollbackScope ExportScope(Context);

		const TRefCountPtr<FNetBlob>& Attachment = UnreliableQueue.PeekAtOffsetNoCheck(AttachmentIt);

		// If we have exports, append them, if attachment is rolled back we will roll back any appended exports as well.
		FNetExportContext* ExportContext = Attachment->HasExports() ? Context.GetExportContext() : nullptr;
		if (ExportContext)
		{
			ObjectReferenceCache->AddPendingExports(Context, Attachment->CallGetNetObjectReferenceExports());
			ExportContext->AddPendingExports(Attachment->CallGetNetTokenExports());
		}

		Attachment->SerializeCreationInfo(Context, Attachment->GetCreationInfo());
		if (bSerializeWithObject)
		{
			Attachment->SerializeWithObject(Context, RefHandle);
		}
		else
		{
			Attachment->Serialize(Context);
		}

		const uint32 HasMoreAttachmentsWritePos = Writer->GetPosBits();
		const bool bHasMoreAttachments = AttachmentIt + 1 != AttachmentEndIt;
		Writer->WriteBool(bHasMoreAttachments);

		if (Writer->IsOverflown())
		{
			// Rollback exports
			ExportScope.Rollback();

			// We need to rollback before we update the continuation bit
			RollbackScope.Rollback();
			if (SerializedUnreliableCount > 0)
			{
				FNetBitStreamWriteScope WriteScope(*Writer, PrevHasMoreAttachmentsWritePos);
				Writer->WriteBool(false);
			}

			break;
		}
		else
		{
			++SerializedUnreliableCount;
			PrevHasMoreAttachmentsWritePos = HasMoreAttachmentsWritePos;
		}
	}
	FUnreliableReplicationRecord ReplicationRecord;
	ReplicationRecord.Record = SerializedUnreliableCount;
	OutRecord = ReplicationRecord;
	return SerializedUnreliableCount;
}

void FNetObjectAttachmentSendQueue::CommitReplicationRecord(const FNetObjectAttachmentSendQueue::FCommitRecord& Record)
{
	if (Record.UnreliableCommitRecord.IsValid())
	{
		UnreliableQueue.PopNoCheck(Record.UnreliableCommitRecord.Record);
	}
	if (Record.ReliableReplicationRecord.IsValid())
	{
		ReliableQueue->CommitReplicationRecord(Record.ReliableReplicationRecord);
	}
}

void FNetObjectAttachmentSendQueue::ProcessPacketDeliveryStatus(EPacketDeliveryStatus Status, const FNetObjectAttachmentSendQueue::FReliableReplicationRecord& Record, bool& bOutMarkHasAttachmentsToSendDirty)
{
	bOutMarkHasAttachmentsToSendDirty = false;
	if (ReliableQueue == nullptr)
	{
		return;
	}

	ReliableQueue->ProcessPacketDeliveryStatus(Status, Record, bOutMarkHasAttachmentsToSendDirty);
}

bool FNetObjectAttachmentsWriter::Enqueue(ENetObjectAttachmentType Type, uint32 ObjectIndex, TArrayView<const TRefCountPtr<FNetBlob>> Attachments)
{
	FNetObjectAttachmentSendQueue* Queue = GetOrCreateQueue(Type, ObjectIndex);
	if (!ensureMsgf(Queue != nullptr, TEXT("Too many objects being targeted by Attachments simultaneously: %d"), ObjectToQueue.Num()))
	{
		return false;
	}

	return Queue->Enqueue(Attachments);
}

bool FNetObjectAttachmentsWriter::HasUnsentAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	const FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		return false;
	}

	return Queue->HasUnsent();
}

bool FNetObjectAttachmentsWriter::HasUnsentUnreliableAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	const FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		return false;
	}

	return Queue->HasUnsentUnreliable();
}

bool FNetObjectAttachmentsWriter::IsAllSentAndAcked(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	const FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		return true;
	}

	return Queue->IsAllSentAndAcked();
}

bool FNetObjectAttachmentsWriter::IsAllReliableSentAndAcked(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	const FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		return true;
	}

	return Queue->IsAllSentAndAcked();
}

bool FNetObjectAttachmentsWriter::AreAllObjectsReliableSentAndAcked() const
{
	for (const auto& ObjectQueuePair : ObjectToQueue)
	{
		if (!ObjectQueuePair.Value.IsAllReliableSentAndAcked())
		{
			return false;
		}
	}

	for (const TUniquePtr<FNetObjectAttachmentSendQueue>& SpecialQueue : SpecialQueues)
	{
		if (SpecialQueue.IsValid() && !SpecialQueue->IsAllReliableSentAndAcked())
		{
			return false;
		}
	}

	return true;
}

bool FNetObjectAttachmentsWriter::CanSendMoreReliableAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	const FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		return true;
	}

	return Queue->CanSendMoreReliableAttachments();
}

bool FNetObjectAttachmentsWriter::IsSafeToDestroy(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	const FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		return true;
	}

	return Queue->IsSafeToDestroy();
}

void FNetObjectAttachmentsWriter::DropAllAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex)
{
	ObjectToQueue.Remove(ObjectIndex);
}

SIZE_T FNetObjectAttachmentsWriter::GetReliableAttachmentCount(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	const FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		return 0;
	}

	return Queue->GetReliableCount();
}

SIZE_T FNetObjectAttachmentsWriter::GetUnreliableAttachmentCount(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	const FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		return 0;
	}

	return Queue->GetUnreliableCount();
}

void FNetObjectAttachmentsWriter::DropUnreliableAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex, bool &bOutHasUnsentAttachments)
{
	FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		bOutHasUnsentAttachments = false;
		return;
	}

	Queue->DropUnreliable(bOutHasUnsentAttachments);
	if (!bOutHasUnsentAttachments && Queue->IsSafeToDestroy())
	{
		ObjectToQueue.Remove(ObjectIndex);
	}
}

EAttachmentWriteStatus FNetObjectAttachmentsWriter::Serialize(FNetSerializationContext& Context, ENetObjectAttachmentType Type, uint32 ObjectIndex, const FNetRefHandle RefHandle,  FNetObjectAttachmentsWriter::FCommitRecord& OutRecord, bool& bOutHasUnsentAttachments)
{
	FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	// If this ensure fires we have bad logic for keeping track of whether there are attachments or not
	if (ensure(Queue != nullptr))
	{
		return Queue->Serialize(Context, RefHandle, OutRecord, bOutHasUnsentAttachments);
	}
	else
	{
		OutRecord = FCommitRecord();
		bOutHasUnsentAttachments = false;
		return EAttachmentWriteStatus::NoAttachments;
	}
}

void FNetObjectAttachmentsWriter::CommitReplicationRecord(ENetObjectAttachmentType Type, uint32 ObjectIndex, const FNetObjectAttachmentsWriter::FCommitRecord& Record)
{
	FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	Queue->CommitReplicationRecord(Record);
}

void FNetObjectAttachmentsWriter::ProcessPacketDeliveryStatus(EPacketDeliveryStatus Status, ENetObjectAttachmentType Type, uint32 ObjectIndex, const FReliableReplicationRecord& Record, bool& bOutMarkHasAttachmentsToSendDirty)
{
	bOutMarkHasAttachmentsToSendDirty = false;
	FNetObjectAttachmentSendQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		return;
	}

	Queue->ProcessPacketDeliveryStatus(Status, Record, bOutMarkHasAttachmentsToSendDirty);
}

void FNetObjectAttachmentsWriter::ProcessPacketDeliveryStatus(EPacketDeliveryStatus Status, ENetObjectAttachmentType Type, uint32 ObjectIndex, const FReliableReplicationRecord& Record)
{
	bool bDummyMarkHasAttachmentsToSendDirty = false;
	ProcessPacketDeliveryStatus(Status, Type, ObjectIndex, Record, bDummyMarkHasAttachmentsToSendDirty);
}

FNetObjectAttachmentSendQueue* FNetObjectAttachmentsWriter::GetQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex)
{
	return Type == ENetObjectAttachmentType::Normal ? ObjectToQueue.Find(ObjectIndex) : SpecialQueues[uint32(Type)].Get();
}

const FNetObjectAttachmentSendQueue* FNetObjectAttachmentsWriter::GetQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	return Type == ENetObjectAttachmentType::Normal ? ObjectToQueue.Find(ObjectIndex) : SpecialQueues[uint32(Type)].Get();
}

FNetObjectAttachmentSendQueue* FNetObjectAttachmentsWriter::GetOrCreateQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex)
{
	if (Type != ENetObjectAttachmentType::Normal)
	{
		checkSlow(ObjectIndex == 0U);
		TUniquePtr<FNetObjectAttachmentSendQueue>& QueuePtr = SpecialQueues[uint32(Type)];
		if (FNetObjectAttachmentSendQueue* Queue = QueuePtr.Get())
		{
			return Queue;
		}

		QueuePtr = MakeUnique<FNetObjectAttachmentSendQueue>();

		if (Type == ENetObjectAttachmentType::OutOfBand)
		{
			QueuePtr->SetUnreliableQueueCapacity(AttachmentReplicationCVars::ClientToServerUnreliableRPCQueueSize);
		}

		return QueuePtr.Get();
	}
	else
	{
		FNetObjectAttachmentSendQueue* Queue = ObjectToQueue.Find(ObjectIndex);
		if (Queue != nullptr)
		{
			return Queue;
		}

		if (ObjectToQueue.Num() >= AttachmentReplicationCVars::MaxSimultaneousObjectsWithRPCs)
		{
			return nullptr;
		}

		Queue = &ObjectToQueue.Add(ObjectIndex);

		return Queue;
	}
}

// ReceiveQueue and Reader
class FNetObjectAttachmentReceiveQueue::FDeferredProcessingQueue
{
public:
	struct FInitParams
	{
		const UPartialNetObjectAttachmentHandler* PartialNetObjectAttachmentHandler = nullptr;
		bool bOnlyProcessUnreliable = false;
	};

	void Init(const FInitParams& InitParams)
	{
		PartialNetObjectAttachmentHandler = InitParams.PartialNetObjectAttachmentHandler;
		bOnlyProcessUnreliable = InitParams.bOnlyProcessUnreliable;
	}

	bool IsEmpty() const { return Queue.IsEmpty(); }
	bool HasUnprocessed() const { return !Queue.IsEmpty(); }
	bool IsSafeToDestroy() const
	{ 
		return Queue.IsEmpty() && !ReliableNetBlobAssembler.IsValid() && !UnreliableNetBlobAssembler.IsValid();
	}

	void Enqueue(FNetSerializationContext& Context, FNetRefHandle RefHandle, const TRefCountPtr<FNetBlob>& NetBlob, bool bIsPartialNetBlob)
	{
		const bool bIsBlobReliable = NetBlob->IsReliable();
		if (bIsBlobReliable && bOnlyProcessUnreliable)
		{
			UE_LOGF(LogIris, Error, "Received reliable blob when only unreliable is supported.");
			Context.SetError(GNetError_UnsupportedNetBlob);
			return;
		}

		if (bIsPartialNetBlob)
		{
			TUniquePtr<FNetBlobAssembler>& NetBlobAssembler = bIsBlobReliable ? ReliableNetBlobAssembler : UnreliableNetBlobAssembler;
			if (!NetBlobAssembler.IsValid())
			{
				FNetBlobAssemblerInitParams InitParams;
				InitParams.PartialNetBlobHandlerConfig = PartialNetObjectAttachmentHandler ? PartialNetObjectAttachmentHandler->GetConfig() : nullptr;
				NetBlobAssembler = MakeUnique<FNetBlobAssembler>();
				NetBlobAssembler->Init(InitParams);
			}

			NetBlobAssembler->AddPartialNetBlob(Context, RefHandle, reinterpret_cast<const TRefCountPtr<FPartialNetBlob>&>(NetBlob));
			if (NetBlobAssembler->IsReadyToAssemble() || NetBlobAssembler->IsSequenceBroken())
			{
				if (NetBlobAssembler->IsReadyToAssemble())
				{
					const TRefCountPtr<FNetBlob>& AssembledBlob = NetBlobAssembler->Assemble(Context);
					if (AssembledBlob.IsValid())
					{
						Queue.Enqueue(AssembledBlob);
					}
				}
				NetBlobAssembler.Reset();
			}
		}
		else
		{
			// If we're set to only process unreliable blobs and we're not done assembling a full set of unreliable partial blobs at this point we know we will not succeed in doing so. We want to allow the queue to be destroyed as soon as possible so let's reset the assembler.
			if (bOnlyProcessUnreliable)
			{
				UnreliableNetBlobAssembler.Reset();
			}

			Queue.Enqueue(NetBlob);
		}
	}

	const TRefCountPtr<FNetBlob>* Peek()
	{
		for (SIZE_T It = 0, EndIt = Queue.Count(); It < EndIt; ++It)
		{
			// Peek at head. We will return or pop the entry.
			TRefCountPtr<FNetBlob>& Blob = Queue.Poke();
			if (Blob.GetRefCount() > 0)
			{
				return &Blob;
			}
			else
			{
				Queue.Pop();
			}
		}

		return nullptr;
	}

	void Pop()
	{
		return Queue.Pop();
	}

	SIZE_T GetQueueCount() const
	{
		return Queue.Count();
	}

	void SetQueueCapacity(uint32 QueueCapacity)
	{
		const SIZE_T QueueCount = Queue.Count();
		const SIZE_T DropCount = QueueCount - QueueCapacity;
		UE_LOG(LogIris, Warning, TEXT("Dropping %" SIZE_T_FMT " attachments to due to change in unreliable queue capacity to %u"), DropCount, QueueCapacity);
		Queue.Pop(DropCount);
	}

private:
	TUniquePtr<FNetBlobAssembler> ReliableNetBlobAssembler;
	TUniquePtr<FNetBlobAssembler> UnreliableNetBlobAssembler;
	TResizableCircularQueue<TRefCountPtr<FNetBlob>> Queue;
	const UPartialNetObjectAttachmentHandler* PartialNetObjectAttachmentHandler = nullptr;
	/* Whether this queue is only expecting unrealiable NetBlobs or not. */
	bool bOnlyProcessUnreliable = false;
};

FNetObjectAttachmentReceiveQueue::FNetObjectAttachmentReceiveQueue()
: MaxUnreliableCount(AttachmentReplicationCVars::UnreliableRPCQueueSize)
{
}

FNetObjectAttachmentReceiveQueue::~FNetObjectAttachmentReceiveQueue()
{
	delete ReliableQueue;
	for (FDeferredProcessingQueue* Queue : MakeArrayView(DeferredProcessingQueues))
	{
		delete Queue;
	}
}

void FNetObjectAttachmentReceiveQueue::Init(const FNetObjectAttachmentReceiveQueueInitParams& InitParams)
{
	PartialNetObjectAttachmentHandler = InitParams.PartialNetObjectAttachmentHandler;
	PartialNetBlobType = (InitParams.PartialNetObjectAttachmentHandler ? InitParams.PartialNetObjectAttachmentHandler->GetNetBlobType() : InvalidNetBlobType);
}

bool FNetObjectAttachmentReceiveQueue::IsSafeToDestroy() const
{
	return IsDeferredProcessingQueueSafeToDestroy(EDeferredProcessingQueue::Unreliable) && IsDeferredProcessingQueueSafeToDestroy(EDeferredProcessingQueue::Reliable) && (ReliableQueue == nullptr || ReliableQueue->IsSafeToDestroy());
}

bool FNetObjectAttachmentReceiveQueue::HasUnprocessed() const
{
	return HasDeferredProcessingQueueUnprocessed(EDeferredProcessingQueue::Unreliable) || HasDeferredProcessingQueueUnprocessed(EDeferredProcessingQueue::Reliable);
}

const TRefCountPtr<FNetBlob>* FNetObjectAttachmentReceiveQueue::PeekReliable()
{
	if (IsDeferredProcessingQueueEmpty(EDeferredProcessingQueue::Reliable))
	{
		return nullptr;
	}
	else
	{
		return DeferredProcessingQueues[EDeferredProcessingQueue::Reliable]->Peek();
	}
}

void FNetObjectAttachmentReceiveQueue::PopReliable()
{
	DeferredProcessingQueues[EDeferredProcessingQueue::Reliable]->Pop();
}

const TRefCountPtr<FNetBlob>* FNetObjectAttachmentReceiveQueue::PeekUnreliable() const
{
	if (IsDeferredProcessingQueueEmpty(EDeferredProcessingQueue::Unreliable))
	{
		return nullptr;
	}
	else
	{
		return DeferredProcessingQueues[EDeferredProcessingQueue::Unreliable]->Peek();
	}
}

void FNetObjectAttachmentReceiveQueue::PopUnreliable()
{
	return DeferredProcessingQueues[EDeferredProcessingQueue::Unreliable]->Pop();
}

void FNetObjectAttachmentReceiveQueue::SetUnreliableQueueCapacity(uint32 QueueCapacity)
{
	MaxUnreliableCount = QueueCapacity;
	
	if (FDeferredProcessingQueue* UnreliableProcessingQueue = DeferredProcessingQueues[EDeferredProcessingQueue::Unreliable])
	{
		UnreliableProcessingQueue->SetQueueCapacity(QueueCapacity);
	}
}

bool FNetObjectAttachmentReceiveQueue::IsDeferredProcessingQueueEmpty(EDeferredProcessingQueue Queue) const
{
	return DeferredProcessingQueues[Queue] == nullptr || DeferredProcessingQueues[Queue]->IsEmpty();
}

bool FNetObjectAttachmentReceiveQueue::IsDeferredProcessingQueueSafeToDestroy(EDeferredProcessingQueue Queue) const
{
	return DeferredProcessingQueues[Queue] == nullptr || DeferredProcessingQueues[Queue]->IsSafeToDestroy();
}

bool FNetObjectAttachmentReceiveQueue::HasDeferredProcessingQueueUnprocessed(EDeferredProcessingQueue Queue) const
{
	return DeferredProcessingQueues[Queue] != nullptr && DeferredProcessingQueues[Queue]->HasUnprocessed();
}

bool FNetObjectAttachmentReceiveQueue::IsPartialNetBlob(const TRefCountPtr<FNetBlob>& Blob) const
{
	return Blob.IsValid() && Blob->GetCreationInfo().Type == PartialNetBlobType;
}

void FNetObjectAttachmentReceiveQueue::Deserialize(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	UE_NET_TRACE_SCOPE(RPCs, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	const bool bHasReliableAttachments = Reader.ReadBool();
	const bool bHasUnreliableAttachments = Reader.ReadBool();

	if (Reader.IsOverflown())
	{
		return;
	}

	if (bHasReliableAttachments)
	{
		UE_NET_TRACE_SCOPE(Ordered, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		DeserializeReliable(Context, RefHandle);
		if (Context.HasErrorOrOverflow())
		{
			return;
		}
	}

	if (bHasUnreliableAttachments)
	{
		UE_NET_TRACE_SCOPE(Unreliable, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		DeserializeUnreliable(Context, RefHandle);
		if (Context.HasErrorOrOverflow())
		{
			return;
		}
	}
}

uint32 FNetObjectAttachmentReceiveQueue::DeserializeReliable(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	if (ReliableQueue == nullptr)
	{
		ReliableQueue = new FReliableNetBlobQueue();

		checkSlow(DeferredProcessingQueues[EDeferredProcessingQueue::Reliable] == nullptr);
		DeferredProcessingQueues[EDeferredProcessingQueue::Reliable] = new FDeferredProcessingQueue();
		FDeferredProcessingQueue::FInitParams InitParams = { .PartialNetObjectAttachmentHandler = PartialNetObjectAttachmentHandler, .bOnlyProcessUnreliable = false };
		DeferredProcessingQueues[EDeferredProcessingQueue::Reliable]->Init(InitParams);
	}

	uint32 DeserializedReliableCount = 0;
	if (RefHandle.IsValid())
	{
		DeserializedReliableCount = ReliableQueue->DeserializeWithObject(Context, RefHandle);
	}
	else
	{
		DeserializedReliableCount = ReliableQueue->Deserialize(Context);
	}

	if (Context.HasErrorOrOverflow())
	{
		return DeserializedReliableCount;
	}

	FDeferredProcessingQueue* DeferredProcessingQueue = DeferredProcessingQueues[EDeferredProcessingQueue::Reliable];
	while (const TRefCountPtr<FNetBlob>* Attachment = ReliableQueue->Peek())
	{
		DeferredProcessingQueue->Enqueue(Context, RefHandle, *Attachment, IsPartialNetBlob(*Attachment));
		ReliableQueue->Pop();

		if (Context.HasErrorOrOverflow())
		{
			return DeserializedReliableCount;
		}
	}

	// Add all received unreliable attachments to prevent their processing from being blocked by reliable ones.
	{
		TArray<TRefCountPtr<FNetBlob>> UnreliableAttachments;
		UnreliableAttachments.Reserve(16);
		ReliableQueue->DequeueUnreliable(UnreliableAttachments);
		for (TRefCountPtr<FNetBlob>& UnreliableAttachment : UnreliableAttachments)
		{
			DeferredProcessingQueue->Enqueue(Context, RefHandle, UnreliableAttachment, IsPartialNetBlob(UnreliableAttachment));
			if (Context.HasErrorOrOverflow())
			{
				return DeserializedReliableCount;
			}
		}
	}

	return DeserializedReliableCount;
}

uint32 FNetObjectAttachmentReceiveQueue::DeserializeUnreliable(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	if (DeferredProcessingQueues[EDeferredProcessingQueue::Unreliable] == nullptr)
	{
		DeferredProcessingQueues[EDeferredProcessingQueue::Unreliable] = new FDeferredProcessingQueue();
		FDeferredProcessingQueue::FInitParams InitParams = { .PartialNetObjectAttachmentHandler = PartialNetObjectAttachmentHandler, .bOnlyProcessUnreliable = true };
		DeferredProcessingQueues[EDeferredProcessingQueue::Unreliable]->Init(InitParams);
	}

	FDeferredProcessingQueue* DeferredProcessingQueue = DeferredProcessingQueues[EDeferredProcessingQueue::Unreliable];

	INetBlobReceiver* BlobReceiver = Context.GetNetBlobReceiver();
	checkSlow(BlobReceiver != nullptr);

	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	uint32 DeserializedUnreliableCount = 0;
	const bool bDeserializeWithObject = RefHandle.IsValid();
	bool bHasMoreAttachments = false;
	do
	{
		// If the unreliable queue overflows this could be a carefully crafted malicious packet.
		if (DeferredProcessingQueue->GetQueueCount() == MaxUnreliableCount)
		{
			UE_LOGF(LogIris, Error, "Unreliable queue is full for %ls", *RefHandle.ToString());
			Context.SetError(NetError_UnreliableQueueFull);
			break;
		}

		FNetBlobCreationInfo CreationInfo;
		FNetBlob::DeserializeCreationInfo(Context, CreationInfo);
		const TRefCountPtr<FNetBlob>& Attachment = BlobReceiver->CreateNetBlob(CreationInfo);
		if (!Attachment.IsValid())
		{
			Context.SetError(GNetError_UnsupportedNetBlob);
			break;
		}

		if (bDeserializeWithObject)
		{
			Attachment->DeserializeWithObject(Context, RefHandle);
		}
		else
		{
			Attachment->Deserialize(Context);
		}
		
		bHasMoreAttachments = Reader.ReadBool();

		// Break if something went wrong with deserialization.
		if (Context.HasErrorOrOverflow())
		{
			UE_LOGF(LogIris, Error, "Failed to deserialize unreliable attachments for %ls", *RefHandle.ToString());
			break;
		}

		DeferredProcessingQueue->Enqueue(Context, RefHandle, Attachment, IsPartialNetBlob(Attachment));
		if (Context.HasErrorOrOverflow())
		{
			UE_LOGF(LogIris, Error, "Failed to deserialize unreliable attachments for %ls", *RefHandle.ToString());
			break;
		}

		++DeserializedUnreliableCount;
	} while (bHasMoreAttachments);

	return DeserializedUnreliableCount;
}

FNetObjectAttachmentsReader::FNetObjectAttachmentsReader()
{
}

FNetObjectAttachmentsReader::~FNetObjectAttachmentsReader()
{
}

void FNetObjectAttachmentsReader::Init(const FNetObjectAttachmentsReaderInitParams& InitParams)
{
	PartialNetObjectAttachmentHandler = InitParams.PartialNetObjectAttachmentHandler;
}

bool FNetObjectAttachmentsReader::HasUnprocessedAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	const FNetObjectAttachmentReceiveQueue* Queue = GetQueue(Type, ObjectIndex);
	if (Queue == nullptr)
	{
		return false;
	}

	return Queue->HasUnprocessed();
}

void FNetObjectAttachmentsReader::DropAllAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex)
{
	ObjectToQueue.Remove(ObjectIndex);
}

void FNetObjectAttachmentsReader::Deserialize(FNetSerializationContext& Context, ENetObjectAttachmentType Type, uint32 ObjectIndex, FNetRefHandle RefHandle)
{
	FNetObjectAttachmentReceiveQueue* Queue = GetOrCreateQueue(Type, ObjectIndex);
	if (!ensure(Queue != nullptr))
	{
		Context.SetError(NetError_TooManyObjectsWithFunctionCalls);
		return;
	}

	Queue->Deserialize(Context, RefHandle);
}

FNetObjectAttachmentReceiveQueue* FNetObjectAttachmentsReader::GetQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex)
{
	return Type == ENetObjectAttachmentType::Normal ? ObjectToQueue.Find(ObjectIndex) : SpecialQueues[uint32(Type)].Get();
}

const FNetObjectAttachmentReceiveQueue* FNetObjectAttachmentsReader::GetQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex) const
{
	return Type == ENetObjectAttachmentType::Normal ? ObjectToQueue.Find(ObjectIndex) : SpecialQueues[uint32(Type)].Get();
}

FNetObjectAttachmentReceiveQueue* FNetObjectAttachmentsReader::GetOrCreateQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex)
{
	if (Type != ENetObjectAttachmentType::Normal)
	{
		checkSlow(ObjectIndex == 0U);
		TUniquePtr<FNetObjectAttachmentReceiveQueue>& QueuePtr = SpecialQueues[uint32(Type)];
		FNetObjectAttachmentReceiveQueue* Queue = QueuePtr.Get();
		if (Queue != nullptr)
		{
			return Queue;
		}

		QueuePtr = MakeUnique<FNetObjectAttachmentReceiveQueue>();
		Queue = QueuePtr.Get();
		{
			FNetObjectAttachmentReceiveQueueInitParams InitParams;
			InitParams.PartialNetObjectAttachmentHandler = PartialNetObjectAttachmentHandler;
			Queue->Init(InitParams);
		}

		if (Type == ENetObjectAttachmentType::OutOfBand)
		{
			Queue->SetUnreliableQueueCapacity(AttachmentReplicationCVars::ClientToServerUnreliableRPCQueueSize);
		}

		return Queue;
	}
	else
	{
		FNetObjectAttachmentReceiveQueue* Queue = ObjectToQueue.Find(ObjectIndex);
		if (Queue != nullptr)
		{
			return Queue;
		}

		if (ObjectToQueue.Num() >= AttachmentReplicationCVars::MaxSimultaneousObjectsWithRPCs)
		{
			return nullptr;
		}

		Queue = &ObjectToQueue.Add(ObjectIndex);
		{
			FNetObjectAttachmentReceiveQueueInitParams InitParams;
			InitParams.PartialNetObjectAttachmentHandler = PartialNetObjectAttachmentHandler;
			Queue->Init(InitParams);
		}

		return Queue;
	}
}

}
