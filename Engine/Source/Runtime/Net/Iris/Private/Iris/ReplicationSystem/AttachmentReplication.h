// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "Iris/ReplicationSystem/NetBlob/ReliableNetBlobQueue.h"
#include "Containers/Map.h"

class UPartialNetObjectAttachmentHandler;
namespace UE::Net::Private
{
	class FNetBlobHandlerManager;
	class FNetObjectAttachmentsReader;
	class FNetObjectAttachmentsWriter;
}

namespace UE::Net::Private
{

enum class ENetObjectAttachmentType : uint32
{
	// For normal attachments a valid ObjectIndex is required.
	Normal,

	// For special attachments like OutOfBand, HugeObject and DebugObjects the ObjectIndex is assumed to be zero.
	OutOfBand,
	HugeObject,
	DebugObject,

	// For internal use only
	InternalCount
};

static constexpr uint32 NetObjectAttachmentTypeBitCount = 2;

static_assert((uint32)ENetObjectAttachmentType::InternalCount == 1U << NetObjectAttachmentTypeBitCount, "ENetObjectAttachmentType must fit in NetObjectAttachementTypeBitCount bits");

enum class EAttachmentWriteStatus : unsigned
{
	/** At least one attachment could be sent. */
	Success,
	/** There were no attachments to write. */
	NoAttachments,
	/** Due to the reliable window size being full no reliable attachments could be sent. */
	ReliableWindowFull,
	/** Writing caused bitstream overflow. */
	BitstreamOverflow,
};

class FNetObjectAttachmentSendQueue
{
public:
	typedef FReliableNetBlobQueue::FReplicationRecord FReliableReplicationRecord;
	
	struct FUnreliableReplicationRecord
	{
		bool IsValid() const { return Record != InvalidReplicationRecord; }

		static constexpr uint32 InvalidReplicationRecord = 0;
		uint32 Record = InvalidReplicationRecord;
	};

	// Commit record contains all data to be committed after serialization is committed to, i.e. will be part of a packet intended to be sent. The ReliableReplicationRecord needs to be part of the packet replication record so that we can act on packet notifications.
	struct FCommitRecord
	{
		bool IsValid() const { return UnreliableCommitRecord.IsValid() || ReliableReplicationRecord.IsValid(); }

		FReliableReplicationRecord ReliableReplicationRecord;
		FUnreliableReplicationRecord UnreliableCommitRecord;
	};

public:
	FNetObjectAttachmentSendQueue();
	~FNetObjectAttachmentSendQueue();

	bool Enqueue(TArrayView<const TRefCountPtr<FNetBlob>> Attachments);

	SIZE_T GetUnreliableCount() const;

	SIZE_T GetReliableCount() const;

	void DropUnreliable(bool &bOutHasUnsentAttachments);

	bool HasUnsent() const;

	bool HasUnsentUnreliable() const;

	bool IsSafeToDestroy() const;

	bool IsAllSentAndAcked() const;

	bool IsAllReliableSentAndAcked() const;

	bool CanSendMoreReliableAttachments() const;

	void SetUnreliableQueueCapacity(uint32 QueueCapacity);

private:
	friend FNetObjectAttachmentsWriter;
	class FReliableSendQueue;

	EAttachmentWriteStatus Serialize(FNetSerializationContext& Context, FNetRefHandle RefHandle, FCommitRecord& OutRecord, bool& bOutHasUnprocessedAttachments);
	uint32 SerializeReliable(FNetSerializationContext& Context, FNetRefHandle RefHandle, FReliableReplicationRecord& OutRecord);
	uint32 SerializeUnreliable(FNetSerializationContext& Context, FNetRefHandle RefHandle, FUnreliableReplicationRecord& OutRecord);

	void CommitReplicationRecord(const FCommitRecord& Record);

	void ProcessPacketDeliveryStatus(EPacketDeliveryStatus Status, const FReliableReplicationRecord& Record, bool& bOutMarkHasAttachmentsToSendDirty);

	FReliableSendQueue* ReliableQueue;
	TResizableCircularQueue<TRefCountPtr<FNetBlob>> UnreliableQueue;
	uint32 MaxUnreliableCount;
};

class FNetObjectAttachmentsWriter
{
public:
	typedef FNetObjectAttachmentSendQueue::FCommitRecord FCommitRecord;
	typedef FNetObjectAttachmentSendQueue::FReliableReplicationRecord FReliableReplicationRecord;

public:
	bool Enqueue(ENetObjectAttachmentType Type, uint32 ObjectIndex, TArrayView<const TRefCountPtr<FNetBlob>> Attachments);

	bool HasUnsentAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	bool HasUnsentUnreliableAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	// Whether all queued attachments have been sent and that all reliable ones have been acked.
	bool IsAllSentAndAcked(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	// Whether all queued reliable attachments have been sent and acked
	bool IsAllReliableSentAndAcked(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	// Whether all queued reliable attachments have been sent and acked for all types and objects
	bool AreAllObjectsReliableSentAndAcked() const;

	// Whether more reliable attachments can be sent now. It's possible to queue up as many attachments as you see fit, but if the queue is full it can take a while before more attachments will be replicated.
	bool CanSendMoreReliableAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	// Whether the queue can be destroyed without causing issues if more attachments are queued to this instance.
	bool IsSafeToDestroy(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	void DropAllAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex);

	// Counts all Reliable attachments for the given type
	SIZE_T GetReliableAttachmentCount(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	// Counts all unreliable attachments for the given type
	SIZE_T GetUnreliableAttachmentCount(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	void DropUnreliableAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex, bool& bOutHasUnsentAttachments);

	EAttachmentWriteStatus Serialize(FNetSerializationContext& Context, ENetObjectAttachmentType Type, uint32 ObjectIndex, FNetRefHandle RefHandle, FCommitRecord& OutRecord, bool& bOutHasUnsentAttachments);

	void CommitReplicationRecord(ENetObjectAttachmentType Type, uint32 ObjectIndex, const FCommitRecord& Record);

	void ProcessPacketDeliveryStatus(EPacketDeliveryStatus Status, ENetObjectAttachmentType Type, uint32 ObjectIndex, const FReliableReplicationRecord& Record);
	void ProcessPacketDeliveryStatus(EPacketDeliveryStatus Status, ENetObjectAttachmentType Type, uint32 ObjectIndex, const FReliableReplicationRecord& Record, bool& bOutMarkHasAttachmentsToSendDirty);

private:
	bool NetBlobMightNeedSplitting(const TRefCountPtr<FNetObjectAttachment>& Attachment) const;

	FNetObjectAttachmentSendQueue* GetQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex);
	const FNetObjectAttachmentSendQueue* GetQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	FNetObjectAttachmentSendQueue* GetOrCreateQueue(ENetObjectAttachmentType Reason, uint32 ObjectIndex);

	TMap<uint32, FNetObjectAttachmentSendQueue> ObjectToQueue;
	TUniquePtr<FNetObjectAttachmentSendQueue> SpecialQueues[uint32(ENetObjectAttachmentType::InternalCount)];
};

struct FNetObjectAttachmentReceiveQueueInitParams
{
	const UPartialNetObjectAttachmentHandler* PartialNetObjectAttachmentHandler = nullptr;
};

class FNetObjectAttachmentReceiveQueue
{
public:
	FNetObjectAttachmentReceiveQueue();
	~FNetObjectAttachmentReceiveQueue();

	void Init(const FNetObjectAttachmentReceiveQueueInitParams& InitParams);

	bool IsSafeToDestroy() const;
	bool HasUnprocessed() const;

	const TRefCountPtr<FNetBlob>* PeekReliable();
	void PopReliable();

	const TRefCountPtr<FNetBlob>* PeekUnreliable() const;
	void PopUnreliable();

	void SetUnreliableQueueCapacity(uint32 QueueCapacity);

private:
	friend FNetObjectAttachmentsReader;
	class FDeferredProcessingQueue;

	enum EDeferredProcessingQueue : unsigned
	{
		Unreliable,
		Reliable,
	};

	bool IsDeferredProcessingQueueEmpty(EDeferredProcessingQueue Queue) const;
	bool IsDeferredProcessingQueueSafeToDestroy(EDeferredProcessingQueue Queue) const;
	bool HasDeferredProcessingQueueUnprocessed(EDeferredProcessingQueue Queue) const;

	bool IsPartialNetBlob(const TRefCountPtr<FNetBlob>& Blob) const;

	void Deserialize(FNetSerializationContext& Context, FNetRefHandle RefHandle);
	uint32 DeserializeReliable(FNetSerializationContext& Context, FNetRefHandle RefHandle);
	uint32 DeserializeUnreliable(FNetSerializationContext& Context, FNetRefHandle RefHandle);

	FReliableNetBlobQueue* ReliableQueue = nullptr;
	FDeferredProcessingQueue* DeferredProcessingQueues[2] = {};
	const UPartialNetObjectAttachmentHandler* PartialNetObjectAttachmentHandler = nullptr;
	uint32 MaxUnreliableCount = 0;
	FNetBlobType PartialNetBlobType = InvalidNetBlobType;
};

struct FNetObjectAttachmentsReaderInitParams
{
	const UPartialNetObjectAttachmentHandler* PartialNetObjectAttachmentHandler = nullptr;
};

class FNetObjectAttachmentsReader
{
public:
	FNetObjectAttachmentsReader();
	~FNetObjectAttachmentsReader();

	void Init(const FNetObjectAttachmentsReaderInitParams& InitParams);

	bool HasUnprocessedAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	void DropAllAttachments(ENetObjectAttachmentType Type, uint32 ObjectIndex);

	void Deserialize(FNetSerializationContext& Context, ENetObjectAttachmentType Type, uint32 ObjectIndex, FNetRefHandle RefHandle);

	FNetObjectAttachmentReceiveQueue* GetQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex);

private:
	const FNetObjectAttachmentReceiveQueue* GetQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex) const;

	FNetObjectAttachmentReceiveQueue* GetOrCreateQueue(ENetObjectAttachmentType Type, uint32 ObjectIndex);

	TMap<uint32, FNetObjectAttachmentReceiveQueue> ObjectToQueue;
	TUniquePtr<FNetObjectAttachmentReceiveQueue> SpecialQueues[uint32(ENetObjectAttachmentType::InternalCount)];
	const UPartialNetObjectAttachmentHandler* PartialNetObjectAttachmentHandler = nullptr;
};

}
