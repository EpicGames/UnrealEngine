// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Net/Core/NetBitArray.h"

namespace UE::Net::Private
{

/**
 * Cache used to propagate captured changemasks to all connections
 * This data could be allocated using linear allocator and released in PostSendUpdate
 */
struct FChangeMaskCache
{
	// Objects that were copied this frame
	struct FCachedInfo
	{
		uint32 InternalIndex;
		uint32 StorageOffset : 24U;
		uint32 bMarkSubObjectOwnerDirty : 1U;
		uint32 bHasDirtyChangeMask : 1;
	};
	TArray<FCachedInfo, TInlineAllocator<1>> Indices;
	TArray<uint32, TInlineAllocator<1>> Storage;

	/** Prepare cache for use by reserving space for expected data*/
	inline void PrepareCache(uint32 IndexCount, uint32 StorageSize);

	/** Empty cache without freeing memory */
	inline void ResetCache();
	
	/** Empty cached data and free memory */
	inline void EmptyCache();

	/** The ref is only valid until the next Add call. */
	inline FCachedInfo& AddChangeMaskForObject(uint32 InternalIndex, uint32 BitCount);

	/** The ref is only valid until the next Add call. */
	inline FCachedInfo& AddSubObjectOwnerDirty(uint32 InternalIndex);

	/** The ref is only valid until the next Add call. */
	inline FCachedInfo& AddEmptyChangeMaskForObject(uint32 InternalIndex);

	/** The pointer is only valid until the next Add call. */
	inline uint32* GetChangeMaskStorage(const FCachedInfo& Info);

	/** Reverts the effects of the last Add call. */
	inline void PopLastEntry();
};

void FChangeMaskCache::PrepareCache(uint32 IndexCount, uint32 StorageSize)
{
	Indices.Reset(IndexCount);
	Storage.Reset(StorageSize);
}

void FChangeMaskCache::ResetCache()
{
	Indices.Reset();
	Storage.Reset();
}

void FChangeMaskCache::EmptyCache()
{
	Indices.Empty();
	Storage.Empty();
}

FChangeMaskCache::FCachedInfo& FChangeMaskCache::AddChangeMaskForObject(uint32 InternalIndex, uint32 BitCount)
{
	const uint32 StorageIndex = Storage.Num();
	const uint32 WordCount = FNetBitArrayView::CalculateRequiredWordCount(BitCount);

	FCachedInfo Info;
	Info.InternalIndex = InternalIndex;
	Info.StorageOffset = StorageIndex;
	Info.bMarkSubObjectOwnerDirty = 0U;
	Info.bHasDirtyChangeMask = 0U;

	Storage.AddZeroed(WordCount);

	return Indices.Add_GetRef(Info);
}

FChangeMaskCache::FCachedInfo& FChangeMaskCache::AddEmptyChangeMaskForObject(uint32 InternalIndex)
{
	FCachedInfo Info;
	Info.InternalIndex = InternalIndex;
	Info.StorageOffset = 0U;
	Info.bMarkSubObjectOwnerDirty = 0U;
	Info.bHasDirtyChangeMask = 0U;

	return Indices.Add_GetRef(Info);
}

uint32* FChangeMaskCache::GetChangeMaskStorage(const FCachedInfo& Info)
{
	return &Storage.GetData()[Info.StorageOffset];
}

void FChangeMaskCache::PopLastEntry()
{
	const FCachedInfo& Info = Indices.Last();

	Storage.SetNum(Info.StorageOffset, EAllowShrinking::No);
	Indices.Pop(EAllowShrinking::No);
}

inline FChangeMaskCache::FCachedInfo& FChangeMaskCache::AddSubObjectOwnerDirty(uint32 InternalIndex)
{
	FCachedInfo Info;
	Info.InternalIndex = InternalIndex;
	Info.StorageOffset = 0U;
	Info.bMarkSubObjectOwnerDirty = 1U;
	Info.bHasDirtyChangeMask = 0U;

	return Indices.Add_GetRef(Info);
}

}
