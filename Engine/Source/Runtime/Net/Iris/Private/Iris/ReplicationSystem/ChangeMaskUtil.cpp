// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ChangeMaskUtil.h"
#include "HAL/UnrealMemory.h"
#include "Misc/MemStack.h"

namespace UE::Net::Private
{

void* FGlobalChangeMaskAllocator::Alloc(uint32 Size, uint32 Alignment)
{
	return FMemory::Malloc(Size, Alignment);
}

void FGlobalChangeMaskAllocator::Free(void* Pointer)
{
	return FMemory::Free(Pointer);
}

FMemStackChangeMaskAllocator::FMemStackChangeMaskAllocator(FMemStackBase* InMemStack)
: MemStack(InMemStack)
{
}

void* FMemStackChangeMaskAllocator::Alloc(uint32 Size, uint32 Alignment)
{ 
	return MemStack->Alloc(Size, Alignment);
}

void FMemStackChangeMaskAllocator::Free(void* Pointer)
{
}

}
