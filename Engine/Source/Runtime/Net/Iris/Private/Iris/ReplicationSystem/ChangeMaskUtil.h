// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Net/Core/NetBitArray.h"

class FMemStackBase;

namespace UE::Net::Private
{

using ChangeMaskStorageType = FNetBitArrayView::StorageWordType;

struct FGlobalChangeMaskAllocator
{
	void* Alloc(uint32 Size, uint32 Alignment);
	void Free(void* Pointer);
};

struct FMemStackChangeMaskAllocator
{
	FMemStackBase* MemStack;

	FMemStackChangeMaskAllocator(FMemStackBase* InMemStack);
	void* Alloc(uint32 Size, uint32 Alignment);
	void Free(void* Pointer);
};

class FChangeMaskStorageOrPointer
{
public:
	typedef ChangeMaskStorageType StorageWordType;

	static constexpr bool UseInlinedStorage(uint32 BitCount) { return BitCount <= 64; }
	static constexpr uint32 GetStorageSize(uint32 BitCount) { return FNetBitArrayView::CalculateRequiredWordCount(BitCount) * sizeof(StorageWordType); }

	inline StorageWordType* GetPointer(uint32 BitCount);
	inline const StorageWordType* GetPointer(uint32 BitCount) const;

public:
	inline FChangeMaskStorageOrPointer() : ChangeMaskOrPointer(0) {}

	// Allocate storage for ChangeMask
	// If the changemask fits in the storage pointer no memory is allocated, if it does not fit memory is allocated from the provided Allocator
	template <typename AllocatorType>
	static void Alloc(FChangeMaskStorageOrPointer& Storage, uint32 BitCount, AllocatorType& Allocator);

	template <typename AllocatorType>
	static FNetBitArrayView AllocAndInitBitArray(FChangeMaskStorageOrPointer& Storage, uint32 BitCount, AllocatorType& Allocator);

	// Free memory (if) allocated for ChangeMask
	template <typename AllocatorType>
	static void Free(const FChangeMaskStorageOrPointer& Storage, uint32 BitCount, AllocatorType& Allocator);

private:
	uint64 ChangeMaskOrPointer;
};

struct FChangeMaskUtil
{
	// Construct a changemask from storage and bitcount
	// Storage must be allocated
	static inline FNetBitArrayView MakeChangeMask(const FChangeMaskStorageOrPointer& Storage, uint32 BitCount);
	static inline void CopyChangeMask(FChangeMaskStorageOrPointer& DestStorage, const FNetBitArrayView& ChangeMask);
	static inline void CopyChangeMask(FChangeMaskStorageOrPointer& DestStorage, FChangeMaskStorageOrPointer& SrcStorage, uint32 BitCount);
	static inline void CopyChangeMask(ChangeMaskStorageType* DstData, const ChangeMaskStorageType* SrcData, uint32 BitCount);
};

//////////////////////////////////////////////////////////////////////////
// FChangeMaskStorageOrPointer Impl
//////////////////////////////////////////////////////////////////////////

inline FChangeMaskStorageOrPointer::StorageWordType* FChangeMaskStorageOrPointer::GetPointer(uint32 BitCount)
{
	uint64* Ptr = UseInlinedStorage(BitCount) ? reinterpret_cast<uint64*>(&ChangeMaskOrPointer) : reinterpret_cast<uint64*>(ChangeMaskOrPointer);
	return reinterpret_cast<StorageWordType*>(Ptr);
}

inline const FChangeMaskStorageOrPointer::StorageWordType* FChangeMaskStorageOrPointer::GetPointer(uint32 BitCount) const
{
	const uint64* Ptr = UseInlinedStorage(BitCount) ? reinterpret_cast<const uint64*>(&ChangeMaskOrPointer) : reinterpret_cast<const uint64*>(ChangeMaskOrPointer);
	return reinterpret_cast<const StorageWordType*>(Ptr);
}

template <typename AllocatorType>
void FChangeMaskStorageOrPointer::Alloc(FChangeMaskStorageOrPointer& Storage, uint32 BitCount, AllocatorType& Allocator)
{
	if (!UseInlinedStorage(BitCount))
	{
		Storage.ChangeMaskOrPointer = (uint64)(Allocator.Alloc(GetStorageSize(BitCount), alignof(StorageWordType)));
	}
}

template <typename AllocatorType>
FNetBitArrayView FChangeMaskStorageOrPointer::AllocAndInitBitArray(FChangeMaskStorageOrPointer& Storage, uint32 BitCount, AllocatorType& Allocator)
{
	if (!UseInlinedStorage(BitCount))
	{
		Storage.ChangeMaskOrPointer = (uint64)(Allocator.Alloc(GetStorageSize(BitCount), alignof(StorageWordType)));
	}

	return FNetBitArrayView(Storage.GetPointer(BitCount), BitCount, FNetBitArrayView::ResetOnInit);
}

template <typename AllocatorType>
void FChangeMaskStorageOrPointer::Free(const FChangeMaskStorageOrPointer& Storage, uint32 BitCount, AllocatorType& Allocator)
{
	if (!UseInlinedStorage(BitCount))
	{
		Allocator.Free((void*)Storage.ChangeMaskOrPointer);
	}
}

//////////////////////////////////////////////////////////////////////////
// FChangeMaskUtil Impl
//////////////////////////////////////////////////////////////////////////

FNetBitArrayView FChangeMaskUtil::MakeChangeMask(const FChangeMaskStorageOrPointer& Storage, uint32 BitCount)
{
	return MakeNetBitArrayView(Storage.GetPointer(BitCount), BitCount);
}

void FChangeMaskUtil::CopyChangeMask(ChangeMaskStorageType* DstData, const ChangeMaskStorageType* SrcData, uint32 BitCount)
{
	FPlatformMemory::Memcpy(&DstData[0], &SrcData[0], FChangeMaskStorageOrPointer::GetStorageSize(BitCount));
}

void FChangeMaskUtil::CopyChangeMask(FChangeMaskStorageOrPointer& DestStorage, FChangeMaskStorageOrPointer& SrcStorage, uint32 BitCount)
{
	CopyChangeMask(DestStorage.GetPointer(BitCount), SrcStorage.GetPointer(BitCount), BitCount);
}

void FChangeMaskUtil::CopyChangeMask(FChangeMaskStorageOrPointer& DestStorage, const FNetBitArrayView& ChangeMask)
{
	const uint32 BitCount = ChangeMask.GetNumBits();
	FNetBitArrayView DstChangeMask(DestStorage.GetPointer(BitCount), BitCount, FNetBitArrayView::NoResetNoValidate);
	DstChangeMask.Copy(ChangeMask);
}

}
