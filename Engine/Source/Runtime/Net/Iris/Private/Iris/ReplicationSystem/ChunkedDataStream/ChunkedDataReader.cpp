// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChunkedDataReader.h"

#include "Iris/PacketControl/PacketNotification.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Misc/ScopeExit.h"

namespace UE::Net::Private
{

FChunkedDataReader::FDataChunk::FDataChunk()
: PartCount(0U)
, SequenceNumber(uint16(-1))
, PartByteCount(0U)
, bIsFirstChunk(0U)
, bIsExportChunk(0U)
{
}

const uint32 FChunkedDataReader::FDataChunk::GetPartPayloadByteCount() const
{
	// Size is encoded by a combination of PartCount and PayloadByteCount
	// The first part contains the size of the entire payload encoded as PartCount * ChunkSize + PayloadByteCount
	return (bIsFirstChunk && (PartCount > 1U)) ? FChunkedDataStreamParameters::ChunkSize : PartByteCount;
}

void FChunkedDataReader::FDataChunk::Deserialize(UE::Net::FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	bIsFirstChunk = Reader->ReadBool() ? 1U : 0U;
	if (bIsFirstChunk)
	{
		bIsExportChunk = Reader->ReadBool();
		PartCount = ReadPackedUint32(Reader);
	}
	else
	{
		bIsExportChunk = 0U;
		PartCount = 0U;
	}
			
	const bool bIsFullChunk = Reader->ReadBool();
	const uint32 ReadPartByteCount = bIsFullChunk ? FChunkedDataStreamParameters::ChunkSize : ReadPackedUint16(Reader);

	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	PartByteCount = (uint16)FMath::Min(FChunkedDataStreamParameters::ChunkSize, ReadPartByteCount);

	// Read actual payload
	UE_NET_TRACE_SCOPE(Payload, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

	// Size is encoded as a combination of PartCount and PayloadByteCount
	// The first part contains the size of the entire payload encoded as PartCount * ChunkSize + PayloadByteCount
	// So the actual payload size of the first part depends on if there are multiple parts or not.
	const uint32 PartPayloadByteCount = (bIsFirstChunk && (PartCount > 1U)) ? FChunkedDataStreamParameters::ChunkSize : PartByteCount;
	PartPayload.SetNum(PartPayloadByteCount);
	ReadBytes(Reader, PartPayload.GetData(), PartPayloadByteCount);			
}

FChunkedDataReader::FChunkedDataReader(const UDataStream::FInitParameters& InParams)
: InitParams(InParams)
, ReplicationSystem(UE::Net::GetReplicationSystem(InParams.ReplicationSystemId))
, ObjectReferenceCache(&ReplicationSystem->GetReplicationSystemInternal()->GetObjectReferenceCache())
{
	FNetTokenStore* NetTokenStore = ReplicationSystem->GetNetTokenStore();

	// Setup internal context
	ResolveContext.ConnectionId = InitParams.ConnectionId;
	ResolveContext.RemoteNetTokenStoreState = NetTokenStore->GetRemoteNetTokenStoreState(InitParams.ConnectionId);
	ResolveContext.ReplicationSystem = ReplicationSystem;

	NetTokenResolveContext.NetTokenStore = NetTokenStore;
	NetTokenResolveContext.RemoteNetTokenStoreState = ResolveContext.RemoteNetTokenStoreState;
}

void FChunkedDataReader::ResetResolvedReferences()
{
	// Make sure to release all references that we are holding on to
	if (ObjectReferenceCache)
	{
		for (const FNetRefHandle& RefHandle : ResolvedReferences)
		{
			ObjectReferenceCache->RemoveTrackedQueuedBatchObjectReference(RefHandle);
		}
	}
}

FChunkedDataReader::~FChunkedDataReader()
{
	ResetResolvedReferences();
}

bool FChunkedDataReader::ProcessExportPayload(FNetSerializationContext& Context, FRecvQueueEntry& Entry)
{
	FNetBitStreamReader ExportsReader;
	ExportsReader.InitBits(Entry.Payload.GetData(), Entry.Payload.Num() * 8U);
	FNetSerializationContext SubContext = Context.MakeSubContext(&ExportsReader);

	FNetTraceCollector* ExportsTraceCollector = nullptr;
#if UE_NET_TRACE_ENABLED
	FNetTraceCollector ExportsTraceCollectorOnStack;
	ExportsTraceCollector = &ExportsTraceCollectorOnStack;
#endif

	SubContext.SetTraceCollector(ExportsTraceCollector);

	UE_NET_TRACE_NAMED_SCOPE(ExportsTraceScope, ExportsPayload, ExportsReader, ExportsTraceCollector, ENetTraceVerbosity::Trace);

ON_SCOPE_EXIT
	{
#if UE_NET_TRACE_ENABLED
		UE_NET_TRACE_EXIT_NAMED_SCOPE(ExportsTraceScope);

		// Append huge object state at end of stream.
		if (FNetTraceCollector* TraceCollector = Context.GetTraceCollector())
		{
			FNetBitStreamReader& Reader = *Context.GetBitStreamReader();
			// Inject after all other trace events
			const uint32 LevelOffset = 3U;
			FNetTrace::FoldTraceCollector(TraceCollector, ExportsTraceCollector, GetBitStreamPositionForNetTrace(Reader) + MultiExportsPayLoadOffset, LevelOffset);

			MultiExportsPayLoadOffset += ExportsReader.GetPosBits();;
		}
#endif
	};

	// Read and process exports
	const uint32 ExportsOffset = ExportsReader.ReadBits(FChunkedDataStreamParameters::NumBitsForExportOffset);

	if (SubContext.HasErrorOrOverflow())
	{
		return false;
	}

	uint32 ExportsEndPosition = 0U;
	if (ExportsOffset != 0U)
	{
		const uint32 ReturnPos = ExportsReader.GetPosBits();
		ExportsReader.Seek(ReturnPos + ExportsOffset);

		if (!ObjectReferenceCache->ReadExports(FNetRefHandle::GetInvalid(), SubContext, &Entry.References->MustBeMappedReferences, Entry.References->IrisAsyncLoadingPriority))
		{
			return false;
		}

		ExportsEndPosition = ExportsReader.GetPosBits();
		ExportsReader.Seek(ReturnPos);
	}
	FIrisPackageMapExportsUtil::Deserialize(SubContext, Entry.References->QuantizedExports);

	// Just to get tracing to report nicely.
#if UE_NET_TRACE_ENABLED
	if (!SubContext.HasErrorOrOverflow() && ExportsOffset != 0U)
	{
		ExportsReader.Seek(ExportsEndPosition);
	}
#endif

	return !SubContext.HasErrorOrOverflow();
}

void FChunkedDataReader::AssemblePayloadsPendingAssembly(UE::Net::FNetSerializationContext& Context)
{
	// Reset ExportsPayLoadOffset
	MultiExportsPayLoadOffset = 0U;
	while (!DataChunksPendingAssembly.IsEmpty())
	{
		const FDataChunk& CurrentChunk = DataChunksPendingAssembly.First();

		if (CurrentChunk.SequenceNumber != ExpectedSeq)
		{
			break;
		}

		// If we have encountered an error, we will no longer try to assemble received chunks
		if (!HasError())
		{
			bool bLocalHasError = false;
			if (CurrentChunk.bIsFirstChunk)
			{
				FRecvQueueEntry* CurrentEntry = ReceiveQueue.IsEmpty() ? nullptr : &ReceiveQueue.Last();

				// Validate that previous entry is complete
				if (ensure(!CurrentEntry || (CurrentEntry->RemainingByteCount == 0U)))
				{
					// If the last received payload was an export we append to the same entry.
					if (!CurrentEntry || CurrentChunk.bIsExportChunk || !(CurrentEntry->bHasProcessedExports && CurrentEntry->Payload.Num() == 0U))
					{
						// Create new chunk to dispatch and start assembling payload
						CurrentEntry = &ReceiveQueue.Emplace_GetRef();							

						if (CurrentChunk.bIsExportChunk)
						{
							CurrentEntry->References = MakeUnique<FReferencesForImport>();					
						}
					}

					CurrentEntry->RemainingByteCount = ((CurrentChunk.PartCount - 1U) * FChunkedDataStreamParameters::ChunkSize) + CurrentChunk.PartByteCount;
						
					UE_LOG_CHUNKEDDATASTREAM_CONN(Verbose, TEXT("AssemblePayloadsPendingAssembly Size: %u PartCount: %u IsExportPayload: %u"), CurrentEntry->RemainingByteCount, CurrentChunk.PartCount, CurrentChunk.bIsExportChunk);

					if (CurrentUndispatchedPayloadBytes + CurrentEntry->RemainingByteCount > MaxUndispatchedPayloadBytes)
					{
						SetError(TEXT("Error: MaxUndispatchedPayloadBytes exceeded."));
					}
					else
					{
						CurrentEntry->Payload.Reserve(CurrentEntry->RemainingByteCount);
						CurrentUndispatchedPayloadBytes += CurrentEntry->RemainingByteCount;
					}
				}
				else
				{
					SetError(TEXT("Error: Encountered new payload when previous one still is not fully received, DataStream will be closed."));
				}
			}
			if (!HasError())
			{
				FRecvQueueEntry& CurrentEntry = ReceiveQueue.Last();
				if (ensure((uint32)CurrentChunk.PartPayload.Num() <= CurrentEntry.RemainingByteCount))
				{
					CurrentEntry.Payload.Append(CurrentChunk.PartPayload);
					CurrentEntry.RemainingByteCount -= CurrentChunk.PartPayload.Num();
						
					if (CurrentEntry.RemainingByteCount == 0 && !CurrentEntry.bHasProcessedExports && CurrentEntry.References)
					{
						// Read and process exports as soon as export payload is assembled
						if (ProcessExportPayload(Context, CurrentEntry))
						{
							// If the export payload has been processed, we reset the payload so that we can reuse the same entry for the actual data payload
							CurrentUndispatchedPayloadBytes -= CurrentEntry.Payload.Num();
							CurrentEntry.Payload.Reset();
							CurrentEntry.bHasProcessedExports = true;
						}
						else
						{
							SetError(TEXT("Error: Failed to ProcessExportPayload, DataStream will be closed."));
							bLocalHasError = true;
						}
					}
				}
				else
				{
					SetError(TEXT("Error: Received more data than expected when assembling payload, DataStream will be closed."));
				}
			}
		}

		// We are done with this chunk
		++ExpectedSeq;
		DataChunksPendingAssembly.PopFront();
	}

	if (DataChunksPendingAssembly.IsEmpty())
	{
		DataChunksPendingAssembly.Trim();
	}
}

bool FChunkedDataReader::TryResolveUnresolvedMustBeMappedReferences(TArray<FNetRefHandle>& MustBeMappedReferences, EIrisAsyncLoadingPriority IrisAsyncLoadingPriority)
{
	// Resolve
	TArray<FNetRefHandle> Unresolved;
	TArray<TPair<FNetRefHandle, UObject*>, TInlineAllocator<4>> QueuedObjectsToTrack;

	Unresolved.Reserve(MustBeMappedReferences.Num());
	QueuedObjectsToTrack.SetNum(MustBeMappedReferences.Num());

	ResolveContext.AsyncLoadingPriority = ConvertAsyncLoadingPriority(IrisAsyncLoadingPriority);
	ON_SCOPE_EXIT
	{
		ResolveContext.AsyncLoadingPriority = INDEX_NONE;
	};

	// Try to resolve references
	for (FNetRefHandle Handle : MustBeMappedReferences)
	{
		UObject* ResolvedObject = nullptr;
		ENetObjectReferenceResolveResult ResolveResult = ObjectReferenceCache->ResolveObjectReference(FObjectReferenceCache::MakeNetObjectReference(Handle), ResolveContext, ResolvedObject);
		if (EnumHasAnyFlags(ResolveResult, ENetObjectReferenceResolveResult::HasUnresolvedMustBeMappedReferences) && !ObjectReferenceCache->IsNetRefHandleBroken(Handle, true))
		{
			Unresolved.Add(Handle);
		}
		else if (ResolveResult == ENetObjectReferenceResolveResult::None)
		{
			QueuedObjectsToTrack.Emplace(Handle, ResolvedObject);
		}
	}

	// If more references are resolved, add them to tracking list
	for (TPair<FNetRefHandle, UObject*>& NetRefHandleObjectPair : QueuedObjectsToTrack)
	{
		if (!ResolvedReferences.Contains(NetRefHandleObjectPair.Key))
		{
			ResolvedReferences.Add(NetRefHandleObjectPair.Key);
			ObjectReferenceCache->AddTrackedQueuedBatchObjectReference(NetRefHandleObjectPair.Key, NetRefHandleObjectPair.Value);
		}
	}

	if (Unresolved.Num())
	{
		// Upate status
		MustBeMappedReferences = MoveTemp(Unresolved);
	
		return false;
	}

	// Nothing more to do.
	MustBeMappedReferences.Reset();

	return true;	
}

UChunkedDataStream::EDispatchResult FChunkedDataReader::DispatchReceivedPayload(TFunctionRef<void(TConstArrayView64<uint8>)> DispatchPayloadFunction)
{
	if (ReceiveQueue.IsEmpty() || HasError())
	{
		return UChunkedDataStream::EDispatchResult::NothingToDispatch;
	}

	FRecvQueueEntry& Entry = ReceiveQueue.First();
	if (Entry.RemainingByteCount != 0U)
	{
		return UChunkedDataStream::EDispatchResult::NothingToDispatch;
	}

	const bool bProcessReferences = Entry.References.IsValid();
	if (bProcessReferences)
	{					
		if (ObjectReferenceCache->ShouldAsyncLoad() && !TryResolveUnresolvedMustBeMappedReferences(Entry.References->MustBeMappedReferences, Entry.References->IrisAsyncLoadingPriority))
		{
			// Wait for async loading to complete if we have any mustbemapped entries
			UE_LOG_CHUNKEDDATASTREAM_CONN(Verbose, TEXT("Waiting for %d MustBeMapped references to be resolvable"), Entry.References->MustBeMappedReferences.Num());
			return UChunkedDataStream::EDispatchResult::WaitingForMustBeMappedReferences;
		}

		// Setup context for dispatch
		FInternalNetSerializationContext InternalContext;
		FInternalNetSerializationContext::FInitParameters InternalContextInitParams;
		InternalContextInitParams.ReplicationSystem = ReplicationSystem;
		InternalContextInitParams.ObjectResolveContext = ResolveContext;
		InternalContext.Init(InternalContextInitParams);

		FNetSerializationContext Context;
		Context.SetLocalConnectionId(InitParams.ConnectionId);
		Context.SetInternalContext(&InternalContext);

		// Dequantize exports
		FIrisPackageMapExportsUtil::Dequantize(Context, Entry.References->QuantizedExports, PackageMapExports);
	}

	if (Entry.Payload.Num())
	{
		UE_LOG_CHUNKEDDATASTREAM_CONN(Verbose, TEXT("Dispatching payload with %u bytes and %u potential exports"), Entry.Payload.Num(), PackageMapExports.References.Num());
		DispatchPayloadFunction(MakeConstArrayView(Entry.Payload));
		CurrentUndispatchedPayloadBytes -= Entry.Payload.Num();
	}

	if (!bProcessReferences)
	{
		ResetResolvedReferences();
	}
	ReceiveQueue.PopFront();

	if (ReceiveQueue.IsEmpty())
	{
		ReceiveQueue.Trim();
	}

	return UChunkedDataStream::EDispatchResult::Ok;
}

UChunkedDataStream::EDispatchResult FChunkedDataReader::DispatchReceivedPayloads(TFunctionRef<void(TConstArrayView64<uint8>)> DispatchPayloadFunction)
{		
	UChunkedDataStream::EDispatchResult Result = UChunkedDataStream::EDispatchResult::Ok;
	for (; Result == UChunkedDataStream::EDispatchResult::Ok; Result = DispatchReceivedPayload(DispatchPayloadFunction))
	{
	}

	return Result;
}

uint32 FChunkedDataReader::GetNumReceivedPayloadsPendingDispatch() const
{
	uint32 ReadyToDispatchCount = 0U;
	for (const FRecvQueueEntry& Entry : ReceiveQueue)
	{
		ReadyToDispatchCount += Entry.RemainingByteCount == 0U ? 1U : 0U;
	}
	return ReadyToDispatchCount;
}

void FChunkedDataReader::ReadData(UE::Net::FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// $TODO: set this up in DataStreamManager instead. UE-243627
	FInternalNetSerializationContext InternalContext;
	FInternalNetSerializationContext::FInitParameters InternalContextInitParams;
	InternalContextInitParams.ReplicationSystem = ReplicationSystem;
	InternalContextInitParams.ObjectResolveContext = ResolveContext;
	InternalContext.Init(InternalContextInitParams);

	Context.SetLocalConnectionId(InitParams.ConnectionId);
	Context.SetInternalContext(&InternalContext);

	uint16 LastReadSeq = uint16(-1);
	while (Reader->ReadBool())
	{
		if (Reader->IsOverflown())
		{
			break;
		}

		UE_NET_TRACE_SCOPE(DataChunk, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

		UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(SequenceScope, static_cast<const TCHAR*>(nullptr), *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		// Read sequence number
		const bool bIsInSequence = Reader->ReadBool();
		uint16 ReadSeq = 0U;
		if (bIsInSequence)
		{
			ReadSeq = (LastReadSeq + 1U) & FChunkedDataStreamParameters::SequenceBitMask;
		}
		else
		{
			ReadSeq = (uint16)Reader->ReadBits(FChunkedDataStreamParameters::SequenceBitCount);
		}

		if (Reader->IsOverflown())
		{
			break;
		}

		LastReadSeq = ReadSeq;

		const uint16 Seq = LastReadSeq;
		const uint16 SeqDelta = (Seq - ExpectedSeq) & FChunkedDataStreamParameters::SequenceBitMask;

		// Make room to store missing sequence numbers.
		{
			DataChunksPendingAssembly.Reserve(FMath::Max(DataChunksPendingAssembly.Num(), (int32)(SeqDelta + 1)));
			while ((uint32)DataChunksPendingAssembly.Num() <= SeqDelta)
			{
				DataChunksPendingAssembly.Add(FDataChunk());
			}
		}

		FDataChunk& Chunk = DataChunksPendingAssembly[SeqDelta];
		Chunk.SequenceNumber = ExpectedSeq + SeqDelta;
		Chunk.Deserialize(Context);

#if UE_NET_TRACE_ENABLED
		if (FNetTrace::GetNetTraceVerbosityEnabled(ENetTraceVerbosity::VeryVerbose))
		{
			TStringBuilder<64> Builder;
			if (Chunk.bIsFirstChunk)
			{
				Builder.Appendf(TEXT("Seq %u First part of %u"), Chunk.SequenceNumber, Chunk.PartCount);
			}
			else
			{
				Builder.Appendf(TEXT("Seq %u"), Chunk.SequenceNumber);
			}
			UE_NET_TRACE_SET_SCOPE_NAME(SequenceScope, Builder.ToString());
		}
#endif // UE_NET_TRACE_ENABLED

		UE_LOG_CHUNKEDDATASTREAM_CONN(Verbose, TEXT("Deserialize Seq:%u (local:%u), Expected %u"), Seq, Chunk.SequenceNumber, ExpectedSeq);
	}

	// Assemble data chunks that we have received
	AssemblePayloadsPendingAssembly(Context);

	// Remove dangling reference to InternalContext on the stack
	Context.SetInternalContext(nullptr);
}

void FChunkedDataReader::SetError(const FString& InErrorMessage)
{
	UE_LOG_CHUNKEDDATASTREAM_CONN(Error, TEXT("FChunkedDataReader::ErrorEncountered() %s"), *InErrorMessage);
	if (!bHasError)
	{
		bHasError = true;
	}
}

bool FChunkedDataReader::HasError() const
{
	return bHasError;
}

} // End of namespace(s)
