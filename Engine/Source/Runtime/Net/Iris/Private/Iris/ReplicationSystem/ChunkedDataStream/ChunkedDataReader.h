// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChunkedDataStreamCommon.h"
#include "Containers/Array.h"
#include "Containers/RingBuffer.h"

#include "Iris/DataStream/DataStream.h"

#include "Iris/ReplicationSystem/ChunkedDataStream/ChunkedDataStream.h"
#include "Iris/ReplicationSystem/ObjectReferenceCacheFwd.h"
#include "Iris/ReplicationSystem/ObjectReferenceTypes.h"

#include "Iris/Serialization/IrisObjectReferencePackageMap.h"
#include "Iris/Serialization/IrisPackageMapExportUtil.h"
#include "Iris/Serialization/NetExportContext.h"

#include "Net/Core/NetToken/NetTokenExportContext.h"

namespace UE::Net::Private
{

// Used by ChunkedDataStream to reading and dispatching incoming data
class FChunkedDataReader
{
public:

	using EWriteResult = UDataStream::EWriteResult;
	using FBeginWriteParameters = UDataStream::FBeginWriteParameters;
	using FInitParameters = UDataStream::FInitParameters;

	struct FReferencesForImport
	{
		FIrisPackageMapExportsQuantizedType QuantizedExports;
		TArray<FNetRefHandle> MustBeMappedReferences;
		EIrisAsyncLoadingPriority IrisAsyncLoadingPriority = EIrisAsyncLoadingPriority::Default;

		~FReferencesForImport()
		{
			FIrisPackageMapExportsUtil::FreeDynamicState(QuantizedExports);
		}
	};

	// RecvQueue entry
	struct FRecvQueueEntry
	{
		// Returns true if this is a processed export payload
		bool GetIsProcessedExportPayload() const
		{
			return bHasProcessedExports;
		}

		TArray<uint8, TAlignedHeapAllocator<4>> Payload;
		TUniquePtr<FReferencesForImport> References;
		uint32 RemainingByteCount = 0U;
		bool bHasProcessedExports = false;
	};

	struct FDataChunk
	{
		FDataChunk();

		const uint32 GetPartPayloadByteCount() const;
		void Deserialize(UE::Net::FNetSerializationContext& Context);

		TArray<uint8> PartPayload;
		uint32 PartCount;
		uint16 SequenceNumber;
		uint16 PartByteCount : 14;
		uint16 bIsFirstChunk : 1;
		uint16 bIsExportChunk : 1;
	};

public:

	FChunkedDataReader(const UDataStream::FInitParameters& InParams);
	~FChunkedDataReader();

	bool ProcessExportPayload(FNetSerializationContext& Context, FRecvQueueEntry& Entry);
	void AssemblePayloadsPendingAssembly(UE::Net::FNetSerializationContext& Context);
	bool TryResolveUnresolvedMustBeMappedReferences(TArray<FNetRefHandle>& MustBeMappedReferences, EIrisAsyncLoadingPriority IrisAsyncLoadingPriority);
	UChunkedDataStream::EDispatchResult DispatchReceivedPayload(TFunctionRef<void(TConstArrayView64<uint8>)> DispatchPayloadFunction);
	UChunkedDataStream::EDispatchResult DispatchReceivedPayloads(TFunctionRef<void(TConstArrayView64<uint8>)> DispatchPayloadFunction);

	uint32 GetNumReceivedPayloadsPendingDispatch() const;
	void ReadData(UE::Net::FNetSerializationContext& Context);
	void SetError(const FString& InErrorMessage);
	bool HasError() const;
	void ResetResolvedReferences();

public:

	friend class FChunkedDataStreamExportReadScope;

	// Incoming data
	TRingBuffer<FDataChunk> DataChunksPendingAssembly;

	// Received data, ready to dispatch
	TRingBuffer<FRecvQueueEntry> ReceiveQueue;

	// Next expected sequence number
	uint16 ExpectedSeq = 0;

	// We have encountered and error, and should close the DataStream
	bool bHasError = false;

	// Cached on init
	FInitParameters InitParams;
	UReplicationSystem* ReplicationSystem = nullptr;
	FObjectReferenceCache* ObjectReferenceCache = nullptr;
	FNetObjectResolveContext ResolveContext;
	FNetTokenResolveContext NetTokenResolveContext;

	// Resolved references for which we have are holding on to references to avoid GC, must be released on exit
	TArray<FNetRefHandle, TInlineAllocator<4>> ResolvedReferences;

	// Exports
	UE::Net::FIrisPackageMapExports PackageMapExports;

	// Maximum undispatched payload bytes, if this is overflown datastream will be put in error state and closed
	uint64 MaxUndispatchedPayloadBytes = 10485760U;

	// Current number of received payload bytes ready to dispatch
	uint64 CurrentUndispatchedPayloadBytes = 0U;

	// Offset used when folding multiple exports payload processed after reading the same pacekt
	uint32 MultiExportsPayLoadOffset = 0U;
};

} // End of namespace(s)
