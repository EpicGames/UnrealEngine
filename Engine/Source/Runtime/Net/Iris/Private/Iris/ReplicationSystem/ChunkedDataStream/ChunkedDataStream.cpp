// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ChunkedDataStream/ChunkedDataStream.h"
#include "ChunkedDataReader.h"
#include "ChunkedDataWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChunkedDataStream)

DEFINE_LOG_CATEGORY(LogIrisChunkedDataStream);

bool UChunkedDataStream::EnqueuePayload(const TSharedPtr<TArray64<uint8>>& Payload)
{
	return ChunkedWriter->EnqueuePayload(Payload);
}

void UChunkedDataStream::Init(const UDataStream::FInitParameters& Params)
{
	Super::Init(Params);

	using namespace UE::Net::Private;

	PackageMap = TObjectPtr<UIrisObjectReferencePackageMap>(NewObject<UIrisObjectReferencePackageMap>());

	ChunkedWriter = new FChunkedDataWriter(Params);
	ChunkedReader = new FChunkedDataReader(Params);
}

void UChunkedDataStream::Deinit()
{
	Super::Deinit();

	if (ChunkedWriter)
	{
		delete ChunkedWriter;
		ChunkedWriter = nullptr;
	}
	if (ChunkedReader)
	{
		delete ChunkedReader;
		ChunkedReader = nullptr;
	}

	// Release Packagemap
	PackageMap = nullptr;
}

UDataStream::EWriteResult UChunkedDataStream::BeginWrite(const FBeginWriteParameters& Params)
{
	if (ChunkedWriter != nullptr)
	{
		return ChunkedWriter->BeginWrite(Params);
	}

	return EWriteResult::NoData;
}

UDataStream::EWriteResult UChunkedDataStream::WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord)
{
	if (ChunkedWriter != nullptr)
	{
		return ChunkedWriter->WriteData(Context, OutRecord);
	}

	return EWriteResult::NoData;
}

void UChunkedDataStream::ReadData(UE::Net::FNetSerializationContext& Context)
{
	if (ChunkedReader != nullptr)
	{
		return ChunkedReader->ReadData(Context);
	}
}

void UChunkedDataStream::ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* InRecord)
{
	if (ChunkedWriter)
	{
		ChunkedWriter->ProcessPacketDeliveryStatus(Status, InRecord);
	}
}

bool UChunkedDataStream::HasAcknowledgedAllReliableData() const
{
	if (ChunkedWriter)
	{
		return ChunkedWriter->HasAcknowledgedAllReliableData();
	}

	return true;
}

UChunkedDataStream::EDispatchResult UChunkedDataStream::DispatchReceivedPayloads(TFunctionRef<void(TConstArrayView64<uint8>)> DispatchPayloadFunction)
{
	if (!ChunkedReader)
	{
		return EDispatchResult::NothingToDispatch;
	}

	return ChunkedReader->DispatchReceivedPayloads(DispatchPayloadFunction);
}

UChunkedDataStream::EDispatchResult UChunkedDataStream::DispatchReceivedPayload(TFunctionRef<void(TConstArrayView64<uint8>)> DispatchPayloadFunction)
{
	if (!ChunkedReader)
	{
		return EDispatchResult::NothingToDispatch;
	}

	return ChunkedReader->DispatchReceivedPayload(DispatchPayloadFunction);
}

uint32 UChunkedDataStream::GetNumReceivedPayloadsPendingDispatch() const
{
	if (ChunkedReader)
	{
		return ChunkedReader->GetNumReceivedPayloadsPendingDispatch();
	}

	return 0U;
}

UIrisObjectReferencePackageMap* UChunkedDataStream::GetPackageMap()
{
	return PackageMap.Get();
}

const UIrisObjectReferencePackageMap* UChunkedDataStream::GetPackageMap() const
{
	return PackageMap.Get();
}

FChunkedDataStreamExportWriteScope::FChunkedDataStreamExportWriteScope(UChunkedDataStream* DataStream)
: WriteScope(DataStream ? DataStream->GetPackageMap() : nullptr, DataStream && DataStream->ChunkedWriter ? &DataStream->ChunkedWriter->PackageMapExports : nullptr)
{
}

UIrisObjectReferencePackageMap* FChunkedDataStreamExportWriteScope::GetPackageMap()
{
	return WriteScope.GetPackageMap();
}

FChunkedDataStreamExportReadScope::FChunkedDataStreamExportReadScope(UChunkedDataStream* DataStream)
: ReadScope(DataStream ? DataStream->GetPackageMap() : nullptr, DataStream && DataStream->ChunkedReader ? &DataStream->ChunkedReader->PackageMapExports : nullptr, DataStream && DataStream->ChunkedReader ? &DataStream->ChunkedReader->NetTokenResolveContext : nullptr)
{
}

UIrisObjectReferencePackageMap* FChunkedDataStreamExportReadScope::GetPackageMap()
{
	return ReadScope.GetPackageMap();
}

uint32 UChunkedDataStream::GetQueuedByteCount() const
{
	if (ChunkedWriter)
	{
		return ChunkedWriter->GetQueuedBytes();
	}

	return 0U;
}

void UChunkedDataStream::SetMaxUndispatchedPayloadBytes(uint32  MaxUndispatchedPayloadBytes)
{
	if (ChunkedReader)
	{
		ChunkedReader->MaxUndispatchedPayloadBytes = MaxUndispatchedPayloadBytes;
	}
}

void UChunkedDataStream::SetMaxEnqueuedPayloadBytes(uint32 MaxEnqueuedPayloadBytes)
{
	if (ChunkedWriter)
	{
		ChunkedWriter->SendBufferMaxSize = MaxEnqueuedPayloadBytes;
	}
}

bool UChunkedDataStream::HasError() const
{
	if (ChunkedReader)
	{
		return ChunkedReader->HasError();
	}

	return false;
}
