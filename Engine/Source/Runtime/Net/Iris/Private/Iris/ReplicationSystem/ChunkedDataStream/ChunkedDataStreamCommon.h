// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Core/IrisLog.h"
#include "Net/Core/Trace/NetTrace.h"

#ifndef UE_NET_CHUNKEDDATASTREAM_LOG_COMPILE_VERBOSITY
// Don't compile verbose logs in Shipping builds	
#if UE_BUILD_SHIPPING
#	define UE_NET_CHUNKEDDATASTREAM_LOG_COMPILE_VERBOSITY Log
#else
#	define UE_NET_CHUNKEDDATASTREAM_LOG_COMPILE_VERBOSITY All
#endif
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogIrisChunkedDataStream, Log, UE_NET_CHUNKEDDATASTREAM_LOG_COMPILE_VERBOSITY);
#define UE_LOG_CHUNKEDDATASTREAM_CONN(Verbosity, Format, ...)  UE_LOG(LogIrisChunkedDataStream, Verbosity, TEXT("ChunkedDataStream: R:%u :C%u ") Format, InitParams.ReplicationSystemId, InitParams.ConnectionId, ##__VA_ARGS__)

namespace UE::Net::Private
{

struct FChunkedDataStreamParameters
{	
	static constexpr uint32 SequenceBitCount = 11U;
	static constexpr uint32 MaxUnackedDataChunkCount = 1 << SequenceBitCount;
	static constexpr uint32 SequenceBitMask = (1U << SequenceBitCount) - 1U;
	static constexpr uint32 ChunkSize = 192U;

	static constexpr uint32 NumBitsForExportOffset = 32U;
};

} // End of namespace(s)
