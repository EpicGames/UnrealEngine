// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChunkedDataWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"

#include "Iris/PacketControl/PacketNotification.h"
#include "Iris/ReplicationSystem/NetExports.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Containers/ContainersFwd.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Misc/ScopeExit.h"

namespace UE::Net::Private
{

FChunkedDataWriter::FDataChunk::FDataChunk()
: PayloadByteOffset(0U)
, PartCount(0U)
, SequenceNumber(uint16(-1))
, PartByteCount(0U)
, bIsFirstChunk(0U)
, bIsExportChunk(0U)
{
}

void FChunkedDataWriter::FDataChunk::Serialize(UE::Net::FNetSerializationContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// Size is encoded as a combination of PartCount and PayloadByteCount
	// The first part contains the size of the entire payload encoded as PartCount * ChunkSize + PayloadByteCount
	uint32 PartPayLoadBytesToWrite = PartByteCount;
	if (Writer->WriteBool(bIsFirstChunk))
	{
		Writer->WriteBool(bIsExportChunk);
		WritePackedUint32(Writer, PartCount);
		PartPayLoadBytesToWrite = PartCount > 1U ? FChunkedDataStreamParameters::ChunkSize : PartByteCount;
	}
	const bool bIsFullChunk = PartByteCount == FChunkedDataStreamParameters::ChunkSize;
	if (!Writer->WriteBool(bIsFullChunk))
	{
		WritePackedUint16(Writer, PartByteCount);
	}

	// Write actual payload
	UE_NET_TRACE_SCOPE(Payload, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
#if UE_NET_TRACE_ENABLED
		if (bIsFirstChunk && bIsExportChunk)
		{
			if (FNetTraceCollector* Collector = SrcEntry->References->TraceCollector)
			{
				FNetTrace::FoldTraceCollector(Context.GetTraceCollector(), Collector, GetBitStreamPositionForNetTrace(*Writer));
			}
		}
#endif // UE_NET_TRACE_ENABLED

	const uint8* PayloadData = bIsExportChunk ? SrcEntry->References->ExportsPayload.GetData() : SrcEntry->Payload->GetData();
	WriteBytes(Writer, PayloadData + PayloadByteOffset, PartPayLoadBytesToWrite);
}

FChunkedDataWriter::FChunkedDataWriter(const FInitParameters& InParams)
: InitParams(InParams)
, ReplicationSystem(UE::Net::GetReplicationSystem(InParams.ReplicationSystemId))
, ObjectReferenceCache(&ReplicationSystem->GetReplicationSystemInternal()->GetObjectReferenceCache())
{
}

bool FChunkedDataWriter::SplitPayload(FSendQueueEntry& SrcEntry, TConstArrayView<uint8> Payload, bool bIsExportPayload)
{
	const uint8* SrcPayload = Payload.GetData();
	const uint32 SrcPayloadBytes = Payload.Num();

	const uint32 ChunkCount = (SrcPayloadBytes + FChunkedDataStreamParameters::ChunkSize - 1U) / FChunkedDataStreamParameters::ChunkSize;

	DataChunksPendingSend.Reserve(DataChunksPendingSend.Num() + int32(ChunkCount));

	UE_LOG_CHUNKEDDATASTREAM_CONN(Verbose, TEXT("Split Payload %u Bytes into %u chunks"), SrcPayloadBytes, ChunkCount);
	
	uint32 CurrentOffset = 0U;
	for (uint32 ChunkIt = 0, ChunkEndIt = ChunkCount; ChunkIt != ChunkEndIt; ++ChunkIt)
	{
		const bool bIsFirstChunk = ChunkIt == 0U;

		FDataChunk Chunk;
			
		Chunk.SrcEntry = &SrcEntry;
		Chunk.PartCount = ChunkCount;
		Chunk.bIsFirstChunk = bIsFirstChunk ? 1U : 0U;
		Chunk.bIsExportChunk = bIsExportPayload ? 1U : 0U;
		Chunk.SequenceNumber = NextSequenceNumber++;

		// Copy relevant data from our temporary buffer.
		Chunk.PayloadByteOffset = CurrentOffset;
		Chunk.PartByteCount = (uint16)FPlatformMath::Min(SrcPayloadBytes - CurrentOffset, FChunkedDataStreamParameters::ChunkSize);
		CurrentOffset += Chunk.PartByteCount;

		// We encode full payload size in first part as (SrcPayloadBytes - ((PartCount - 1) * FChunkedDataStreamParameters::ChunkSize))
		if (bIsFirstChunk && Chunk.PartCount > 1U)
		{
			const uint16 LastPartByteCount = SrcPayloadBytes % FChunkedDataStreamParameters::ChunkSize;
			Chunk.PartByteCount = LastPartByteCount != 0 ? LastPartByteCount : FChunkedDataStreamParameters::ChunkSize;
		}
			
		DataChunksPendingSend.Add(MoveTemp(Chunk));
	}

	return ChunkCount > 0;
}

FChunkedDataWriter::FReferencesForExport* FChunkedDataWriter::CreateExportPayload()
{
	if (!PackageMapExports.IsEmpty() && NetTokensPendingExport.IsEmpty())
	{
		TUniquePtr<FReferencesForExport> Result = MakeUnique<FReferencesForExport>();
		Result->ExportsPayload.AddUninitialized(ExportsBufferMaxSize);

		uint32 ExportsPayloadBytes = 0;
		{
			FNetBitStreamWriter ExportsWriter;
			const uint32 MaxExportsPayloadBytes = Result->ExportsPayload.Num();
			ExportsWriter.InitBytes(Result->ExportsPayload.GetData(), MaxExportsPayloadBytes);

			// Create context
			FNetSerializationContext Context(&ExportsWriter);
			FInternalNetSerializationContext InternalContext(ReplicationSystem);
			Context.SetInternalContext(&InternalContext);
			Context.SetLocalConnectionId(InitParams.ConnectionId);

			// Temporary quantized state
			FIrisPackageMapExportsQuantizedType QuantizedExports = {};

			// We need to release dynamic data on exit
			ON_SCOPE_EXIT
			{
				FIrisPackageMapExportsUtil::FreeDynamicState(QuantizedExports);
			};

			FIrisPackageMapExportsUtil::Quantize(Context, PackageMapExports, NetTokensPendingExport, QuantizedExports);

			// Setup export scope
			FNetExports::FExportScope ExportScope = InitParams.NetExports->MakeExportScope(Context, Result->BatchExports);

#if UE_NET_TRACE_ENABLED
			FNetTraceCollector* LocalTraceCollector = UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace);
			Context.SetTraceCollector(LocalTraceCollector);
#endif

			const uint32 ExportHeaderPos = ExportsWriter.GetPosBits();

			UE_NET_TRACE_SCOPE(ExportPayload, ExportsWriter, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

			// Header
			ExportsWriter.WriteBits(0, FChunkedDataStreamParameters::NumBitsForExportOffset);

			// Append potential exports
			ObjectReferenceCache->AddPendingExports(Context, MakeArrayView(QuantizedExports.ObjectReferenceStorage.GetData(), QuantizedExports.ObjectReferenceStorage.Num()));

			// Serialize the reference data
			FIrisPackageMapExportsUtil::Serialize(Context, QuantizedExports);

			const uint32 WrittenBitsInBatch = (ExportsWriter.GetPosBits() - ExportHeaderPos) - FChunkedDataStreamParameters::NumBitsForExportOffset;

			// Serialize exports if there are any
			const FObjectReferenceCache::EWriteExportsResult WriteExportResult = ObjectReferenceCache->WritePendingExports(Context, 0);
			if (WriteExportResult == FObjectReferenceCache::EWriteExportsResult::BitStreamOverflow)
			{
				return nullptr;
			}
			else if (WriteExportResult == FObjectReferenceCache::EWriteExportsResult::WroteExports)
			{
				// Go back and update header that we have exports to read.
				FNetBitStreamWriteScope SizeScope(ExportsWriter, ExportHeaderPos);
				ExportsWriter.WriteBits(WrittenBitsInBatch, FChunkedDataStreamParameters::NumBitsForExportOffset);
			}
			ExportsWriter.CommitWrites();

			ExportsPayloadBytes = ExportsWriter.GetPosBytes();

#if UE_NET_TRACE_ENABLED
			Result->TraceCollector = LocalTraceCollector;
			LocalTraceCollector = nullptr;
#endif
		}

		// Trim down the payload.
		Result->ExportsPayload.SetNum(ExportsPayloadBytes, EAllowShrinking::Yes);

		return Result.Release();
	}

	return nullptr;
}

void FChunkedDataWriter::ResetExports()
{
	PackageMapExports.Reset();
	NetTokensPendingExport.Reset();
}

bool FChunkedDataWriter::EnqueuePayload(const TSharedPtr<TArray64<uint8>>& Payload)
{
	uint32 TotalPayloadByteCount = (uint32)Payload->Num();

	// Nothing to send
	if (TotalPayloadByteCount == 0U)
	{
		return true;
	}

	bool bCanEnqueueData = (TotalPayloadByteCount + CurrentBytesInSendQueue) <= SendBufferMaxSize;

	// Do we have exports
	TUniquePtr<FReferencesForExport> Exports(bCanEnqueueData ? CreateExportPayload() : nullptr);
	if (Exports)
	{
		TotalPayloadByteCount += Exports->ExportsPayload.Num();
		bCanEnqueueData = (TotalPayloadByteCount + CurrentBytesInSendQueue) <= SendBufferMaxSize;
	}

	if (!bCanEnqueueData)
	{
		UE_LOG_CHUNKEDDATASTREAM_CONN(Warning, TEXT("EnqueuePayload SendBufferFull: Cannot enqueue payload with %u Bytes, CurrentBytesInSendQueue %u"), TotalPayloadByteCount, CurrentBytesInSendQueue);
		return false;
	}

	TUniquePtr<FSendQueueEntry>& NewEntry = SendQueue.Emplace_GetRef(MakeUnique<FSendQueueEntry>(Payload));
	NewEntry->References = MoveTemp(Exports);
	CurrentBytesInSendQueue += TotalPayloadByteCount;

	UE_LOG_CHUNKEDDATASTREAM_CONN(Verbose, TEXT("EnqueuePayload NewEntry %" INT64_FMT " Bytes, %u exports CurrentBytesInSendQueue %u"), Payload->Num(), PackageMapExports.References.Num(), CurrentBytesInSendQueue);

	ResetExports();

	return true;
}

bool FChunkedDataWriter::CanSend() const
{
	return (uint32)DataChunksPendingAck.Num() < FChunkedDataStreamParameters::MaxUnackedDataChunkCount;
}

UDataStream::EWriteResult FChunkedDataWriter::BeginWrite(const FBeginWriteParameters& Params)
{
	if (SendQueue.Num() && CanSend())
	{
		return EWriteResult::HasMoreData;
	}

	return EWriteResult::NoData;
}

bool FChunkedDataWriter::HasAcknowledgedAllReliableData() const
{
	return SendQueue.IsEmpty();
}

bool FChunkedDataWriter::UpdateSendQueue()
{
	for (TUniquePtr<FSendQueueEntry>& Entry : SendQueue)
	{
		if (Entry->RefCount == 0U)
		{
			// Should we enqueue exports as separate entry?
			if (Entry->References)
			{
				const bool bIsExportPayload = true;
				SplitPayload(*Entry, Entry->References->ExportsPayload, bIsExportPayload);
			}
			SplitPayload(*Entry, *Entry->Payload.Get());
			return DataChunksPendingSend.Num() > 0;
		}
	}

	if (!DataChunksPendingSend.IsEmpty())
	{
		return true;
	}	
		
	return false;
}

UDataStream::EWriteResult FChunkedDataWriter::WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	
	// Write chunks, we need at least 1 bit free...
	if (SendQueue.Num() == 0 || Writer->GetBitsLeft() < 1U)
	{
		// If we have no pending data in-flight we can trim down our storage
		if (SendQueue.Num() == 0)
		{
			DataChunksPendingSend.Trim();
			DataChunksPendingAck.Trim();
		}
		
		return EWriteResult::NoData;
	}

	// Write data until we have no more data to write or it does not fit
	uint32 WrittenCount = 0;

	// Setup a substream and context for writing data
	FNetBitStreamWriter SubStream = Writer->CreateSubstream(Writer->GetBitsLeft() - 1U);
	FNetSerializationContext SubContext = Context.MakeSubContext(&SubStream);

	uint16 PrevWrittenSeq = uint16(-1);
	int32 CurrentChunkIndex = 0;
	bool bHasMoreDataToSend = false;
	for (;;)
	{
		bHasMoreDataToSend = CanSend() && UpdateSendQueue();
		if (!bHasMoreDataToSend || CurrentChunkIndex >= DataChunksPendingSend.Num())
		{
			break;
		}

		const FDataChunk& Chunk = DataChunksPendingSend.GetAtIndexNoCheck(CurrentChunkIndex);
		const uint16 CurrentSeq = Chunk.SequenceNumber;
				
		if (IsSequenceSent(CurrentSeq))
		{
			++CurrentChunkIndex;
			continue;
		}

		{
			UE_NET_TRACE_SCOPE(DataChunk, SubStream, SubContext.GetTraceCollector(), ENetTraceVerbosity::Verbose);

			FNetBitStreamRollbackScope SequenceRollback(SubStream);

			// Continuation marker
			SubStream.WriteBool(true);

			// Write sequence number, only if it differs from previous written one.
			const uint16 Seq = Chunk.SequenceNumber & FChunkedDataStreamParameters::SequenceBitMask;
			const bool bIsInSequence = Seq == ((PrevWrittenSeq + 1U) & FChunkedDataStreamParameters::SequenceBitMask);
			PrevWrittenSeq = Seq;

			UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(SequenceScope, static_cast<const TCHAR*>(nullptr), SubStream, SubContext.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

			if (!SubStream.WriteBool(bIsInSequence))
			{
				SubStream.WriteBits(Seq, FChunkedDataStreamParameters::SequenceBitCount);
			}

			// Write chunk
			Chunk.Serialize(SubContext);

			if (SubStream.IsOverflown())
			{
				break;
			}

#if UE_NET_TRACE_ENABLED
			if (FNetTrace::GetNetTraceVerbosityEnabled(ENetTraceVerbosity::VeryVerbose))
			{
				TStringBuilder<64> Builder;
				if (Chunk.bIsFirstChunk)
				{
					Builder.Appendf(TEXT("Seq %u First part of %u"), Chunk.SequenceNumber, Chunk.PartCount);
				}
				else
				{
					Builder.Appendf(TEXT("Seq %u"), Chunk.SequenceNumber);
				}
				UE_NET_TRACE_SET_SCOPE_NAME(SequenceScope, Builder.ToString());
			}
#endif // UE_NET_TRACE_ENABLED

			UE_LOG_CHUNKEDDATASTREAM_CONN(Verbose, TEXT("Serialized Seq:%u (local:%u)"), Seq, CurrentSeq);

		}

		// Enqueue in our record as well for resending if we drop data
		DataChunksPendingAck.Add((uint16)CurrentSeq);
		SetSequenceIsSent(CurrentSeq);
		++WrittenCount;
		++CurrentChunkIndex;
	}

	// Commit substream
	if (WrittenCount)
	{
		Writer->CommitSubstream(SubStream);
		Writer->WriteBool(false);

		// Store number of written batches in the external record pointer
		UPTRINT& OutRecordCount = *reinterpret_cast<UPTRINT*>(&OutRecord);
		OutRecordCount = WrittenCount;

		return bHasMoreDataToSend ? EWriteResult::HasMoreData : EWriteResult::Ok;
	}
	else
	{
		Writer->DiscardSubstream(SubStream);

		return bHasMoreDataToSend ? EWriteResult::HasMoreData : EWriteResult::NoData;
	}
}

void FChunkedDataWriter::RemoveSendQueueEntry(FSendQueueEntry* SendQueueEntry)
{
	const int32 Index = SendQueue.IndexOfByPredicate([SendQueueEntry](const TUniquePtr<FSendQueueEntry>& Entry) { return Entry.Get() == SendQueueEntry; });
	if (Index != INDEX_NONE)
	{
		SendQueue.RemoveAt(Index);
	}
}

void FChunkedDataWriter::PopDeliveredChunks()
{
	while (!DataChunksPendingSend.IsEmpty())
	{
		const FDataChunk& CurrentChunk = DataChunksPendingSend.First();

		FSendQueueEntry* SendQueueEntry = CurrentChunk.SrcEntry;

		const uint32 Index = SequenceToIndex(CurrentChunk.SequenceNumber);
		if (!IsIndexAcked(Index))
		{
			break;
		}

		ClearIndexIsAcked(Index);
		ClearIndexIsSent(Index);

		DataChunksPendingSend.PopFrontNoCheck();

		// NOTE: it is intentional that we wait with removing the SendQueue entry until we know that all data before it has
		// been delivered to the client to ensure that potential exports has been processed before we acknowledge them
		// If refcount is zero, we have delivered our payload and can treat exports as exported
		if (SendQueueEntry->RefCount == 0U)
		{
			UE_LOG_CHUNKEDDATASTREAM_CONN(Verbose, TEXT("Completed SendQueueEntry 0x%p"), SendQueueEntry);
			// We need to explicitly acknowledge exports made through the huge object batch
			if (SendQueueEntry->References)
			{
				InitParams.NetExports->AcknowledgeBatchExports(SendQueueEntry->References->BatchExports);
				CurrentBytesInSendQueue -= SendQueueEntry->References->ExportsPayload.Num();
			}
			CurrentBytesInSendQueue -= (uint32)SendQueueEntry->Payload->Num();
			RemoveSendQueueEntry(SendQueueEntry);
		}
	}
}

void FChunkedDataWriter::ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record)
{
	// The Record pointer is used as storage for the number of batches to process
	uint32 RecordCount = (uint32)reinterpret_cast<UPTRINT>(Record);

	if (Status == UE::Net::EPacketDeliveryStatus::Lost)
	{
		while (RecordCount)
		{
			// Mark entries as not sent
			const uint32 LostSeq = DataChunksPendingAck.PopFrontValue();
			ClearSequenceIsSent(LostSeq);
			--RecordCount;
			UE_LOG_CHUNKEDDATASTREAM_CONN(Verbose, TEXT("Notified Dropped Seq %u"), LostSeq);
		}
	}
	else
	{
		while (RecordCount)
		{
			const uint32 DeliveredSeq = DataChunksPendingAck.PopFrontValue();
			// Mark entries as not sent
			SetSequenceIsAcked(DeliveredSeq);
			--RecordCount;
			UE_LOG_CHUNKEDDATASTREAM_CONN(Verbose, TEXT("Notified Delivered Seq %u"), DeliveredSeq);
		}
		PopDeliveredChunks();
	}
}

} // End of namespace(s)
