// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChunkedDataStreamCommon.h"
#include "Containers/Array.h"
#include "Containers/RingBuffer.h"
#include "Iris/DataStream/DataStream.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"
#include "Iris/Serialization/IrisPackageMapExportUtil.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Net/Core/NetToken/NetTokenExportContext.h"

namespace UE::Net::Private
{

// Used by ChunkedDataStream to capture and send payloads and exports
class FChunkedDataWriter
{
public:
	using EWriteResult = UDataStream::EWriteResult;
	using FBeginWriteParameters = UDataStream::FBeginWriteParameters;
	using FInitParameters = UDataStream::FInitParameters;
	FChunkedDataWriter(const FInitParameters& InParams);

	// Tracks references associated with an enqueued payload
	struct FReferencesForExport
	{
		TArray<uint8, TAlignedHeapAllocator<>> ExportsPayload;
		FNetExportContext::FBatchExports BatchExports;

		~FReferencesForExport()
		{
			if (TraceCollector)
			{
				UE_NET_TRACE_DESTROY_COLLECTOR(TraceCollector);
			}
		}
		FNetTraceCollector* TraceCollector = nullptr;
	};
	
	// Enqueued payload to send
	struct FSendQueueEntry
	{
		FSendQueueEntry() {}
		FSendQueueEntry(FSendQueueEntry&& Other)
		: Payload(MoveTemp(Other.Payload))
		, References(MoveTemp(Other.References))
		, RefCount(Other.RefCount)
		{
			Other.RefCount = 0;
		}

		FSendQueueEntry(const TSharedPtr<TArray64<uint8>>& InPayload)
		: Payload(InPayload)
		{
		}

		void AddRef() const
		{
			++RefCount;
		}

		void Release() const
		{
			--RefCount;
		}
	
		TSharedPtr<TArray64<uint8>> Payload;
		TUniquePtr<FReferencesForExport> References;
		mutable int32 RefCount = 0;
	};

	// Split chunk of data, referencing source
	struct FDataChunk
	{
		FDataChunk();
		void Serialize(UE::Net::FNetSerializationContext& Context) const;
		
		// Hold a reference to the datachunk as source data is shared
		TRefCountPtr<FSendQueueEntry> SrcEntry;
		uint32 PayloadByteOffset;
		uint32 PartCount;
		uint16 SequenceNumber;
		uint16 PartByteCount : 14U;
		uint16 bIsFirstChunk : 1U;
		uint16 bIsExportChunk : 1U;
	};

public:
	uint32 SequenceToIndex(uint32 Seq) const
	{
		return Seq % FChunkedDataStreamParameters::MaxUnackedDataChunkCount;
	}

	bool IsIndexAcked(uint32 Index) const
	{
		return (Acked[Index >> 5U] & (1U << (Index & 31U))) != 0U;
	}

	void SetIndexIsAcked(uint32 Index)
	{
		Acked[Index >> 5U] |= (1U << (Index & 31U));
	}

	void SetSequenceIsAcked(uint32 Seq)
	{
		return SetIndexIsAcked(SequenceToIndex(Seq));
	}

	void ClearIndexIsAcked(uint32 Index)
	{
		Acked[Index >> 5U] &= ~(1U << (Index & 31U));
	}

	bool IsIndexSent(uint32 Index) const
	{
		return (Sent[Index >> 5U] & (1U << (Index & 31U))) != 0U;
	}

	bool IsSequenceSent(uint32 Seq) const
	{
		return IsIndexSent(SequenceToIndex(Seq));
	}

	void SetIndexIsSent(uint32 Index)
	{
		Sent[Index >> 5U] |= (1U << (Index & 31U));
	}

	void SetSequenceIsSent(uint32 Seq)
	{
		return SetIndexIsSent(SequenceToIndex(Seq));
	}

	void ClearIndexIsSent(uint32 Index)
	{
		Sent[Index >> 5U] &= ~(1U << (Index & 31U));
	}

	void ClearSequenceIsSent(uint32 Seq)
	{
		return ClearIndexIsSent(SequenceToIndex(Seq));
	}

	bool SplitPayload(FSendQueueEntry& SrcEntry, TConstArrayView<uint8> Payload, bool bIsExportPayload = false);
	FReferencesForExport* CreateExportPayload();
	void ResetExports();
	bool EnqueuePayload(const TSharedPtr<TArray64<uint8>>& Payload);
	bool CanSend() const;
	bool UpdateSendQueue();
	EWriteResult BeginWrite(const FBeginWriteParameters& Params);
	EWriteResult WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord);
	bool HasAcknowledgedAllReliableData() const;
	void PopDeliveredChunks();
	void ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record);
	void RemoveSendQueueEntry(FSendQueueEntry* SendQueueEntry);

	// This could be more precise by by updating CurrentBytesInSendQueue based on individual datachunks.
	uint32 GetQueuedBytes() const
	{
		return CurrentBytesInSendQueue;
	}

public:
	// Payload data
	TArray<TUniquePtr<FSendQueueEntry>> SendQueue;

	// Split data chunks
	TRingBuffer<FDataChunk> DataChunksPendingSend;

	// In-flight data chunks pending ack
	TRingBuffer<uint16> DataChunksPendingAck;

	// Track status of entries in the DataChunksPendingSend
	uint32 Sent[(FChunkedDataStreamParameters::MaxUnackedDataChunkCount + 31)/32] = {};
	uint32 Acked[(FChunkedDataStreamParameters::MaxUnackedDataChunkCount + 31)/32] = {};
	uint16 NextSequenceNumber = 0U;
	friend class FChunkedDataStreamExportWriteScope;

	// Cached copy of DataStream init params
	FInitParameters InitParams;
	UReplicationSystem* ReplicationSystem = nullptr;
	FObjectReferenceCache* ObjectReferenceCache = nullptr;

	// Total number of bytes in send queue.
	uint32 CurrentBytesInSendQueue = 0U;

	// Just for sanity
	uint32 ExportsBufferMaxSize = 524288U;

	// We do not allow more paylaod bytes to be enqueued than this.
	uint32 SendBufferMaxSize = 10485760U;

	// Exports
	UE::Net::FIrisPackageMapExports PackageMapExports;
	UE::Net::FNetTokenExportContext::FNetTokenExports NetTokensPendingExport;
};

} // End of namespace(s)
