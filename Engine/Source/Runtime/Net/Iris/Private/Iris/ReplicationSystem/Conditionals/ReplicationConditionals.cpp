// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Conditionals/ReplicationConditionals.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationConnections.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationSystem/Conditionals/ReplicationCondition.h"
#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineInvalidationTracker.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectGroups.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFiltering.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/ReplicationSystem/ReplicationWriter.h"
#include "Containers/ArrayView.h"
#include "UObject/CoreNetTypes.h"
#include "Net/Core/Connection/NetEnums.h"
#include "Net/Core/NetHandle/NetHandleManager.h"
#include "Net/Core/PropertyConditions/RepChangedPropertyTracker.h"
#include "Net/Core/PropertyConditions/PropertyConditions.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogIrisConditionals, Log, All);

namespace UE::Net::Private
{

static bool bEnableUpdateObjectsWithDirtyConditionals = true;
static FAutoConsoleVariableRef CVarEnableUpdateObjectsWithDirtyConditionals(
	TEXT("net.Iris.EnableUpdateObjectsWithDirtyConditionals"),
	bEnableUpdateObjectsWithDirtyConditionals,
	TEXT("Enable the updating subobjects with conditionals."));

FReplicationConditionals::FReplicationConditionals()
{
}

void FReplicationConditionals::Init(FReplicationConditionalsInitParams& Params)
{
#if DO_CHECK
	// Verify we can handle max connection count
	{
		const FPerObjectInfo ObjectInfo{static_cast<decltype(FPerObjectInfo::AutonomousConnectionId)>(Params.MaxConnectionCount)};
		check(ObjectInfo.AutonomousConnectionId == Params.MaxConnectionCount);
	}
#endif

	NetRefHandleManager = Params.NetRefHandleManager;
	ReplicationFiltering = Params.ReplicationFiltering;
	ReplicationConnections = Params.ReplicationConnections;
	BaselineInvalidationTracker = Params.BaselineInvalidationTracker;
	NetObjectGroups = Params.NetObjectGroups;
	MaxInternalNetRefIndex = Params.MaxInternalNetRefIndex;
	MaxConnectionCount = Params.MaxConnectionCount;

	PerObjectInfos.SetNumZeroed(MaxInternalNetRefIndex);
	ConnectionInfos.SetNum(MaxConnectionCount + 1U);
	ObjectsWithDirtyLifetimeConditionals.Init(Params.MaxInternalNetRefIndex);
}

void FReplicationConditionals::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	MaxInternalNetRefIndex = NewMaxInternalIndex;

	PerObjectInfos.SetNumZeroed(NewMaxInternalIndex);

	ObjectsWithDirtyLifetimeConditionals.SetNumBits(NewMaxInternalIndex);

	for (FPerConnectionInfo& ConnectionInfo : ConnectionInfos)
	{
		// Resize the ObjectConditionals list for valid connections
		if (!ConnectionInfo.ObjectConditionals.IsEmpty())
		{
			ConnectionInfo.ObjectConditionals.SetNumZeroed(NewMaxInternalIndex);
		}
	}
}

void FReplicationConditionals::OnInternalNetRefIndicesFreed(const TConstArrayView<FInternalNetRefIndex>& FreedIndices)
{
	IRIS_PROFILER_SCOPE(FReplicationConditionals_OnInternalNetRefIndicesFreed);
	for (const FInternalNetRefIndex ObjectIndex : FreedIndices)
	{
		ClearPerObjectInfo(ObjectIndex);
	}

	const FNetBitArray& ValidConnections = ReplicationConnections->GetValidConnections();
	if (ValidConnections.FindLastOne() != FNetBitArrayBase::InvalidIndex)
	{
		for (const FInternalNetRefIndex ObjectIndex : FreedIndices)
		{
			ClearConnectionInfosForObject(ValidConnections, ObjectIndex);
		}
	}
}

void FReplicationConditionals::MarkLifeTimeConditionalsDirtyForObjectsInGroup(FNetObjectGroupHandle GroupHandle)
{
	IRIS_PROFILER_SCOPE(MarkLifeTimeConditionalsDirtyForObjectsInGroup)

	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();
	if (GroupHandle.IsReservedNetObjectGroup())
	{
		UE_LOGF(LogIris, Warning, "FReplicationConditionals::MarkLifeTimeConditionalsDirtyForObjectsInGroup - Marking reserved group dirty is not allowed. GroupIndex: %u which is not allowed.", GroupIndex);
		return;
	}

	if (const FNetObjectGroup* Group = NetObjectGroups->GetGroup(GroupHandle))
	{
		for (FInternalNetRefIndex InternalObjectIndex : Group->Members)
		{
			ObjectsWithDirtyLifetimeConditionals.SetBit(InternalObjectIndex);
		}
	}
}

bool FReplicationConditionals::SetConditionConnectionFilter(FInternalNetRefIndex ObjectIndex, EReplicationCondition Condition, uint32 ConnectionId, bool bEnable)
{
	if (ConnectionId >= MaxConnectionCount)
	{
		return false;
	}

	if (!ensure(Condition == EReplicationCondition::RoleAutonomous))
	{
		UE_LOGF(LogIris, Error, "Only EReplicationCondition::RoleAutonomous supports connection filtering, got '%u'.", uint32(Condition));
		return false;
	}

	const uint32 AutonomousConnectionId = (ConnectionId == 0U || !bEnable) ? 0U : ConnectionId;
	FPerObjectInfo* ObjectInfo = GetPerObjectInfo(ObjectIndex);
	if (ObjectInfo->AutonomousConnectionId != AutonomousConnectionId)
	{
		UE_LOGF(LogIrisConditionals, Verbose, "SetConditionConnectionFilter %ls. AutonomousConnectionId: %u", *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), AutonomousConnectionId);

		const uint32 ConnIdForBaselineInvalidation = (bEnable ? ConnectionId : ObjectInfo->AutonomousConnectionId);
		ObjectInfo->AutonomousConnectionId = uint16(AutonomousConnectionId);

		MarkRemoteRoleDirty(ObjectIndex);
		// Mark object as having dirty global conditional that should be evaluated before next send
		ObjectsWithDirtyLifetimeConditionals.SetBit(ObjectIndex);

		InvalidateBaselinesForObjectHierarchy(ObjectIndex, TConstArrayView<uint32>(&ConnIdForBaselineInvalidation, 1));
	}

	return true;
}

void FReplicationConditionals::SetOwningConnection(FInternalNetRefIndex ObjectIndex, uint32 OwningConnectionId)
{
	const uint32 OldOwningConnectionId = ReplicationFiltering->GetOwningConnection(ObjectIndex);
	if (OldOwningConnectionId != OwningConnectionId && (OwningConnectionId == InvalidConnectionId || ReplicationConnections->IsValidConnection(OwningConnectionId)))
	{
		UE_LOGF(LogIrisConditionals, Verbose, "SetOwningConnection on object %u. Connection: %u", ObjectIndex, OwningConnectionId);

		// Mark object as having dirty global conditional that should be evaluated before next send
		ObjectsWithDirtyLifetimeConditionals.SetBit(ObjectIndex);

		// Invalidate baselines for connections affected by the owner change.
		{
			const uint32 ConnectionIdToInvalidateCandidates[] = {OldOwningConnectionId, OwningConnectionId};
			uint32 ConnectionIdsToInvalidate[UE_ARRAY_COUNT(ConnectionIdToInvalidateCandidates)];
			uint32 ConnectionIdCount = 0;
			for (uint32 ConnectionId : ConnectionIdToInvalidateCandidates)
			{
				if (ConnectionId != InvalidConnectionId)
				{
					ConnectionIdsToInvalidate[ConnectionIdCount++] = ConnectionId;
				}
			}
			InvalidateBaselinesForObjectHierarchy(ObjectIndex, TConstArrayView<uint32>(ConnectionIdsToInvalidate, ConnectionIdCount));
		}
	}
}

void FReplicationConditionals::AddConnection(uint32 ConnectionId, ENetConnectionTraits NetConnectionTraits)
{
	// Init connection info
	FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
	ConnectionInfo.ObjectConditionals.SetNumZeroed(MaxInternalNetRefIndex);
	ConnectionInfo.NetConnectionTraits = NetConnectionTraits;
}

void FReplicationConditionals::RemoveConnection(uint32 ConnectionId)
{
	// Reset connection info
	FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
	ConnectionInfo.ObjectConditionals.Empty();
	ConnectionInfo.NetConnectionTraits = ENetConnectionTraits::None;
}

void FReplicationConditionals::SetAutonomousAsSimulated(uint32 ConnectionId, bool bEnabled)
{
	if (ConnectionInfos.IsValidIndex(ConnectionId))
	{
		FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
		ConnectionInfo.bAutonomousAsSimulated = bEnabled;
	}
	else
	{
		UE_LOGF(LogIris, Warning, "Attempting to set AutonomousAsSimulated on an invalid connection id %d", ConnectionId);
	}
}

bool FReplicationConditionals::SetCondition(FInternalNetRefIndex ObjectIndex, EReplicationCondition Condition, bool bEnable)
{
	if (!ensure(Condition != EReplicationCondition::RoleAutonomous))
	{
		UE_LOGF(LogIris, Error, "%ls", TEXT("EReplicationCondition::RoleAutonomous requires a connection."));
		return false;
	}

	if (Condition == EReplicationCondition::ReplicatePhysics)
	{
		FPerObjectInfo* ObjectInfo = GetPerObjectInfo(ObjectIndex);
		if (bEnable && !ObjectInfo->bRepPhysics)
		{
			UE_LOGF(LogIrisConditionals, Verbose, "SetCondition object %ls. EReplicationCondition::ReplicatePhysics: %u", *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), bEnable ? 1U : 0U);

			// We only care to track this change if the condition is enabled.
			const uint32 ConnIdForBaselineInvalidation = BaselineInvalidationTracker->InvalidateBaselineForAllConnections;
			InvalidateBaselinesForObjectHierarchy(ObjectIndex, TConstArrayView<uint32>(&ConnIdForBaselineInvalidation, 1));

			// Mark object as having dirty global conditional that should be evaluated before next send
			ObjectsWithDirtyLifetimeConditionals.SetBit(ObjectIndex);
		}
		ObjectInfo->bRepPhysics = bEnable ? 1U : 0U;
		return true;
	}

	ensureMsgf(false, TEXT("Unhandled EReplicationCondition '%u'"), uint32(Condition));
	return false;
}

void FReplicationConditionals::InitPropertyCustomConditions(FInternalNetRefIndex ObjectIndex)
{
	IRIS_PROFILER_SCOPE(FReplicationConditionals_InitPropertyCustomConditions);

	const FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
	const FReplicationProtocol* Protocol = ReplicatedObjectData.Protocol;

	if (NetRefHandleManager->GetReplicatedObjectStateBufferNoCheck(ObjectIndex) == nullptr || !EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasLifetimeConditionals))
	{
		return;
	}

	// Set up fragment owner collector once, currently we only support a single owner
	constexpr uint32 MaxFragmentOwnerCount = 1U;
	UObject* FragmentOwners[MaxFragmentOwnerCount] = {};
	FReplicationStateOwnerCollector FragmentOwnerCollector(FragmentOwners, MaxFragmentOwnerCount);

	FReplicationFragment* const * Fragments = ReplicatedObjectData.InstanceProtocol->Fragments;
	const FReplicationInstanceProtocol* InstanceProtocol = ReplicatedObjectData.InstanceProtocol;
	const uint32 FragmentCount = InstanceProtocol->FragmentCount;
	const uint32 FirstRelevantStateIndex = Protocol->FirstLifetimeConditionalsStateIndex;

	const UObject* LastOwner = nullptr;
	for (const FReplicationStateDescriptor*& StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors + FirstRelevantStateIndex, static_cast<int32>(Protocol->ReplicationStateCount - FirstRelevantStateIndex)))
	{
		if (EnumHasAnyFlags(StateDescriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals))
		{
			const SIZE_T StateIndex = &StateDescriptor - Protocol->ReplicationStateDescriptors;

			// Get Owner
			const FReplicationFragment* ReplicationFragment = InstanceProtocol->Fragments[StateIndex];
			FragmentOwnerCollector.Reset();
			ReplicationFragment->CollectOwner(&FragmentOwnerCollector);
			if (FragmentOwnerCollector.GetOwnerCount() == 0U)
			{
				// We have no owner
				continue;
			}

			const UObject* CurrentOwner = FragmentOwnerCollector.GetOwners()[0];
			TSharedPtr<FRepChangedPropertyTracker> ChangedPropertyTracker;
			if (CurrentOwner != LastOwner)
			{				
				ChangedPropertyTracker = FNetPropertyConditionManager::Get().FindOrCreatePropertyTracker(CurrentOwner);
			}

			const FReplicationInstanceProtocol::FFragmentData& Fragment = InstanceProtocol->FragmentData[StateIndex];
			FNetBitArrayView ConditionalChangeMask = GetMemberConditionalChangeMask(Fragment.ExternalSrcBuffer, StateDescriptor);

			// Initialize conditionals based on the state of the ChangedPropertyTracker
			const FRepChangedPropertyTracker* Tracker = ChangedPropertyTracker.Get();
			for (const FReplicationStateMemberLifetimeConditionDescriptor& MemberLifeTimeConditionDescriptor : MakeArrayView(StateDescriptor->MemberLifetimeConditionDescriptors, StateDescriptor->MemberCount))
			{
				const SIZE_T MemberIndex = &MemberLifeTimeConditionDescriptor - StateDescriptor->MemberLifetimeConditionDescriptors;
				const uint16 RepIndex = StateDescriptor->MemberProperties[MemberIndex]->RepIndex;
				if (!Tracker->IsParentActive(RepIndex))
				{
					const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = StateDescriptor->MemberChangeMaskDescriptors[MemberIndex];
					ConditionalChangeMask.ClearBits(MemberChangeMaskDescriptor.BitOffset, MemberChangeMaskDescriptor.BitCount);
				}

				if (MemberLifeTimeConditionDescriptor.Condition == COND_Dynamic)
				{
					ELifetimeCondition Condition = Tracker->GetDynamicCondition(RepIndex);
					if (Condition != COND_Dynamic)
					{
						SetDynamicCondition(ObjectIndex, RepIndex, Condition);
					}
				}
			}
		}
	}
}

// N.B. Calls can come for properties that have been disabled. We must handle such cases gracefully.
bool FReplicationConditionals::SetPropertyCustomCondition(FInternalNetRefIndex ObjectIndex, const void* Owner, uint16 RepIndex, bool bIsActive)
{
	IRIS_PROFILER_SCOPE(FReplicationConditionals_SetPropertyCustomCondition);

	const FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
	const FReplicationProtocol* Protocol = ReplicatedObjectData.Protocol;

	if (NetRefHandleManager->GetReplicatedObjectStateBufferNoCheck(ObjectIndex) == nullptr || !EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasLifetimeConditionals))
	{
		return false;
	}

	if (Protocol->LifetimeConditionalsStateCount == 1U)
	{
		const SIZE_T StateIndex = Protocol->FirstLifetimeConditionalsStateIndex;
		const FReplicationInstanceProtocol* InstanceProtocol = ReplicatedObjectData.InstanceProtocol;
		const FReplicationInstanceProtocol::FFragmentData& Fragment = InstanceProtocol->FragmentData[StateIndex];
		const FReplicationStateDescriptor* StateDescriptor = Protocol->ReplicationStateDescriptors[StateIndex];

		// Note: In this optimized code path we assume the passed Owner is the fragment owner. No checks.
		if (RepIndex >= StateDescriptor->RepIndexCount)
		{
			UE_LOGF(LogIris, Warning, "Trying to change non-existing custom conditional for RepIndex %u in protocol %ls", RepIndex, ToCStr(Protocol->DebugName));
			return false;
		}

		// Modify the external state changemasks accordingly.
		const FReplicationStateMemberRepIndexToMemberIndexDescriptor& RepIndexToMemberIndexDescriptor = StateDescriptor->MemberRepIndexToMemberIndexDescriptors[RepIndex];
		if (RepIndexToMemberIndexDescriptor.MemberIndex == FReplicationStateMemberRepIndexToMemberIndexDescriptor::InvalidEntry)
		{
			UE_LOGF(LogIris, Warning, "Trying to change non-existing custom conditional for RepIndex %u in protocol %ls", RepIndex, ToCStr(Protocol->DebugName));
			return false;
		}

		const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskDescriptor = StateDescriptor->MemberChangeMaskDescriptors[RepIndexToMemberIndexDescriptor.MemberIndex];
		FNetBitArrayView ConditionalChangeMask = UE::Net::Private::GetMemberConditionalChangeMask(Fragment.ExternalSrcBuffer, StateDescriptor);

		if (bIsActive)
		{
			ConditionalChangeMask.SetBits(ChangeMaskDescriptor.BitOffset, ChangeMaskDescriptor.BitCount);

			// If a condition is enabled we also mark the corresponding regular changemask as dirty.
			FNetBitArrayView MemberChangeMask = UE::Net::Private::GetMemberChangeMask(Fragment.ExternalSrcBuffer, StateDescriptor);
			FReplicationStateHeader& ReplicationStateHeader = UE::Net::Private::GetReplicationStateHeader(Fragment.ExternalSrcBuffer, StateDescriptor);
			MarkDirty(ReplicationStateHeader, MemberChangeMask, ChangeMaskDescriptor);

			// Enabled conditions causes new properties to be replicated which most likely have incorrect values at the receiving end.
			BaselineInvalidationTracker->InvalidateBaselines(ObjectIndex, BaselineInvalidationTracker->InvalidateBaselineForAllConnections);
		}
		else
		{
			ConditionalChangeMask.ClearBits(ChangeMaskDescriptor.BitOffset, ChangeMaskDescriptor.BitCount);
		}
	}
	else
	{
		// Set up fragment owner collector once.
		constexpr uint32 MaxFragmentOwnerCount = 1U;
		UObject* FragmentOwners[MaxFragmentOwnerCount] = {};
		FReplicationStateOwnerCollector FragmentOwnerCollector(FragmentOwners, MaxFragmentOwnerCount);

		const FReplicationInstanceProtocol* InstanceProtocol = ReplicatedObjectData.InstanceProtocol;

		const uint32 FirstRelevantStateIndex = Protocol->FirstLifetimeConditionalsStateIndex;
		for (const FReplicationStateDescriptor*& StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors + FirstRelevantStateIndex, static_cast<int32>(Protocol->ReplicationStateCount - FirstRelevantStateIndex)))
		{
			if (EnumHasAnyFlags(StateDescriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals))
			{
				const SIZE_T StateIndex = &StateDescriptor - Protocol->ReplicationStateDescriptors;

				// Is the passed Owner the owner of the fragment?
				{
					const FReplicationFragment* ReplicationFragment = InstanceProtocol->Fragments[StateIndex];

					FragmentOwnerCollector.Reset();
					ReplicationFragment->CollectOwner(&FragmentOwnerCollector);
					if (FragmentOwnerCollector.GetOwnerCount() == 0U || FragmentOwnerCollector.GetOwners()[0] != Owner)
					{
						// Not the right owner.
						continue;
					}
				}

				// Can this state contain this property?
				if (RepIndex >= StateDescriptor->RepIndexCount)
				{
					continue;
				}

				// Does this state contain this property?
				const FReplicationStateMemberRepIndexToMemberIndexDescriptor& RepIndexToMemberIndexDescriptor = StateDescriptor->MemberRepIndexToMemberIndexDescriptors[RepIndex];
				if (RepIndexToMemberIndexDescriptor.MemberIndex == FReplicationStateMemberRepIndexToMemberIndexDescriptor::InvalidEntry)
				{
					continue;
				}

				// We found the relevant state. Modify the external state changemasks.
				const FReplicationInstanceProtocol::FFragmentData& Fragment = InstanceProtocol->FragmentData[StateIndex];

				const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskDescriptor = StateDescriptor->MemberChangeMaskDescriptors[RepIndexToMemberIndexDescriptor.MemberIndex];
				FNetBitArrayView ConditionalChangeMask = GetMemberConditionalChangeMask(Fragment.ExternalSrcBuffer, StateDescriptor);

				if (bIsActive)
				{
					ConditionalChangeMask.SetBits(ChangeMaskDescriptor.BitOffset, ChangeMaskDescriptor.BitCount);

					// If a condition is enabled we also mark the corresponding regular changemask as dirty.
					FNetBitArrayView MemberChangeMask = GetMemberChangeMask(Fragment.ExternalSrcBuffer, StateDescriptor);
					FReplicationStateHeader& Header = GetReplicationStateHeader(Fragment.ExternalSrcBuffer, StateDescriptor);

					MarkDirty(Header, MemberChangeMask, ChangeMaskDescriptor);

					// Enabled conditions causes new properties to be replicated which most likely have incorrect values at the receiving end.
					BaselineInvalidationTracker->InvalidateBaselines(ObjectIndex, BaselineInvalidationTracker->InvalidateBaselineForAllConnections);
				}
				else
				{
					ConditionalChangeMask.ClearBits(ChangeMaskDescriptor.BitOffset, ChangeMaskDescriptor.BitCount);
				}

				return true;
			}
		}

		UE_LOGF(LogIris, Warning, "Trying to change non-existing custom conditional for RepIndex %u in protocol %ls", RepIndex, ToCStr(Protocol->DebugName));
	}

	return false;
}

bool FReplicationConditionals::SetPropertyDynamicCondition(FInternalNetRefIndex ObjectIndex, const void* Owner, uint16 RepIndex, ELifetimeCondition Condition)
{
	IRIS_PROFILER_SCOPE(FReplicationConditionals_SetPropertyDynamicCondition);

	const FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
	const FReplicationProtocol* Protocol = ReplicatedObjectData.Protocol;

	if (NetRefHandleManager->GetReplicatedObjectStateBufferNoCheck(ObjectIndex) == nullptr || !EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasLifetimeConditionals))
	{
		return false;
	}

	if (Protocol->LifetimeConditionalsStateCount == 1U)
	{
		const SIZE_T StateIndex = Protocol->FirstLifetimeConditionalsStateIndex;
		const FReplicationInstanceProtocol* InstanceProtocol = ReplicatedObjectData.InstanceProtocol;
		const FReplicationInstanceProtocol::FFragmentData& Fragment = InstanceProtocol->FragmentData[StateIndex];
		const FReplicationStateDescriptor* StateDescriptor = Protocol->ReplicationStateDescriptors[StateIndex];

		if (RepIndex >= StateDescriptor->RepIndexCount)
		{
			UE_LOGF(LogIris, Warning, "Trying to change non-existing dynamic conditional for RepIndex %u in protocol %ls", RepIndex, ToCStr(Protocol->DebugName));
			return false;
		}

		const FReplicationStateMemberRepIndexToMemberIndexDescriptor& RepIndexToMemberIndexDescriptor = StateDescriptor->MemberRepIndexToMemberIndexDescriptors[RepIndex];
		const uint16 MemberIndex = RepIndexToMemberIndexDescriptor.MemberIndex;
		if ((MemberIndex == FReplicationStateMemberRepIndexToMemberIndexDescriptor::InvalidEntry))
		{
			UE_LOGF(LogIris, Warning, "Trying to change non-existing dynamic conditional for RepIndex %u in protocol %ls", RepIndex, ToCStr(Protocol->DebugName));
			return false;
		}

		if (StateDescriptor->MemberLifetimeConditionDescriptors[MemberIndex].Condition != COND_Dynamic)
		{
			UE_LOGF(LogIris, Warning, "Trying to change condition for member %ls with wrong condition in protocol %ls", ToCStr(StateDescriptor->MemberDebugDescriptors[MemberIndex].DebugName), ToCStr(Protocol->DebugName));
			return false;
		}

		const ELifetimeCondition OldCondition = GetDynamicCondition(ObjectIndex, RepIndex);
		SetDynamicCondition(ObjectIndex, RepIndex, Condition);

		// If a condition may cause something to go from not replicated to replicated we mark the changemask as dirty and invalidate baselines.
		if (DynamicConditionChangeRequiresBaselineInvalidation(OldCondition, Condition))
		{
			const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskDescriptor = StateDescriptor->MemberChangeMaskDescriptors[MemberIndex];

			FNetBitArrayView MemberChangeMask = UE::Net::Private::GetMemberChangeMask(Fragment.ExternalSrcBuffer, StateDescriptor);
			FReplicationStateHeader& ReplicationStateHeader = UE::Net::Private::GetReplicationStateHeader(Fragment.ExternalSrcBuffer, StateDescriptor);
			MarkDirty(ReplicationStateHeader, MemberChangeMask, ChangeMaskDescriptor);

			// $TODO Consider more extensive checking to see if only a single connection requires baseline invalidation.
			BaselineInvalidationTracker->InvalidateBaselines(ObjectIndex, BaselineInvalidationTracker->InvalidateBaselineForAllConnections);
		}

		return true;
	}
	else
	{
		constexpr uint32 MaxFragmentOwnerCount = 1U;
		UObject* FragmentOwners[MaxFragmentOwnerCount] = {};
		FReplicationStateOwnerCollector FragmentOwnerCollector(FragmentOwners, MaxFragmentOwnerCount);

		const FReplicationInstanceProtocol* InstanceProtocol = ReplicatedObjectData.InstanceProtocol;

		const uint32 FirstRelevantStateIndex = Protocol->FirstLifetimeConditionalsStateIndex;
		for (const FReplicationStateDescriptor*& StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors + FirstRelevantStateIndex, static_cast<int32>(Protocol->ReplicationStateCount - FirstRelevantStateIndex)))
		{
			if (EnumHasAnyFlags(StateDescriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals))
			{
				const SIZE_T StateIndex = &StateDescriptor - Protocol->ReplicationStateDescriptors;

				// Is the passed Owner the owner of the fragment?
				{
					const FReplicationFragment* ReplicationFragment = InstanceProtocol->Fragments[StateIndex];

					FragmentOwnerCollector.Reset();
					ReplicationFragment->CollectOwner(&FragmentOwnerCollector);
					if (FragmentOwnerCollector.GetOwnerCount() == 0U || FragmentOwnerCollector.GetOwners()[0] != Owner)
					{
						// Not the right owner.
						continue;
					}
				}

				// Can this state contain this property?
				if (RepIndex >= StateDescriptor->RepIndexCount)
				{
					continue;
				}

				// Does this state contain this property?
				const FReplicationStateMemberRepIndexToMemberIndexDescriptor& RepIndexToMemberIndexDescriptor = StateDescriptor->MemberRepIndexToMemberIndexDescriptors[RepIndex];
				const uint16 MemberIndex = RepIndexToMemberIndexDescriptor.MemberIndex;
				if (MemberIndex == FReplicationStateMemberRepIndexToMemberIndexDescriptor::InvalidEntry)
				{
					continue;
				}

				// We've found the relevant state. Verify it's a dynamic condition member.
				if (StateDescriptor->MemberLifetimeConditionDescriptors[MemberIndex].Condition != COND_Dynamic)
				{
					UE_LOGF(LogIris, Warning, "Trying to change condition for member %ls with wrong condition in protocol %ls", ToCStr(StateDescriptor->MemberDebugDescriptors[MemberIndex].DebugName), ToCStr(Protocol->DebugName));
					return false;
				}

				const ELifetimeCondition OldCondition = GetDynamicCondition(ObjectIndex, RepIndex);
				SetDynamicCondition(ObjectIndex, RepIndex, Condition);

				// If a condition may cause something to go from not replicated to replicated we mark the changemask as dirty and invalidate baselines.
				if (DynamicConditionChangeRequiresBaselineInvalidation(OldCondition, Condition))
				{
					const FReplicationInstanceProtocol::FFragmentData& Fragment = InstanceProtocol->FragmentData[StateIndex];
					const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskDescriptor = StateDescriptor->MemberChangeMaskDescriptors[MemberIndex];

					FNetBitArrayView MemberChangeMask = UE::Net::Private::GetMemberChangeMask(Fragment.ExternalSrcBuffer, StateDescriptor);
					FReplicationStateHeader& ReplicationStateHeader = UE::Net::Private::GetReplicationStateHeader(Fragment.ExternalSrcBuffer, StateDescriptor);
					MarkDirty(ReplicationStateHeader, MemberChangeMask, ChangeMaskDescriptor);

					BaselineInvalidationTracker->InvalidateBaselines(ObjectIndex, BaselineInvalidationTracker->InvalidateBaselineForAllConnections);
				}

				return true;
			}
		}
	}

	return false;
}

void FReplicationConditionals::Update()
{
	if (bEnableUpdateObjectsWithDirtyConditionals)
	{
		UpdateAndResetObjectsWithDirtyConditionals();
	}
}

void FReplicationConditionals::GetChildSubObjectsToReplicate(uint32 ReplicatingConnectionId, const FConditionalsMask& LifetimeConditionals,  const FInternalNetRefIndex ParentObjectIndex, const FChildSubObjectsInfo& SubObjectsInfo, FSubObjectsToReplicateArray& OutSubObjectsToReplicate) const
{
	if (SubObjectsInfo.SubObjectLifeTimeConditions == nullptr)
	{
		for (uint32 ArrayIndex = 0; ArrayIndex < SubObjectsInfo.NumSubObjects; ++ArrayIndex)
		{
			FInternalNetRefIndex SubObjectIndex = SubObjectsInfo.ChildSubObjects[ArrayIndex];

			FChildSubObjectsInfo ChildSubObjectsInfo;
			if (NetRefHandleManager->GetChildSubObjects(SubObjectIndex, ChildSubObjectsInfo))
			{
				// If SubObject has ChildSubObjects we need to consider replication order.
				if (ChildSubObjectsInfo.bReplicateChildSubObjectsAfterParent)
				{
					OutSubObjectsToReplicate.Add(SubObjectIndex);
					GetChildSubObjectsToReplicate(ReplicatingConnectionId, LifetimeConditionals, SubObjectIndex, ChildSubObjectsInfo, OutSubObjectsToReplicate);
				}
				else
				{
					GetChildSubObjectsToReplicate(ReplicatingConnectionId, LifetimeConditionals, SubObjectIndex, ChildSubObjectsInfo, OutSubObjectsToReplicate);
					OutSubObjectsToReplicate.Add(SubObjectIndex);
				}
			}
			else
			{
				OutSubObjectsToReplicate.Add(SubObjectIndex);
			}
		}
	}
	else
	{
		// Append child subobjects that fulfill the condition
		for (uint32 ArrayIndex = 0; ArrayIndex < SubObjectsInfo.NumSubObjects; ++ArrayIndex)
		{
			const FInternalNetRefIndex SubObjectIndex = SubObjectsInfo.ChildSubObjects[ArrayIndex];
			const ELifetimeCondition LifeTimeCondition = (ELifetimeCondition)SubObjectsInfo.SubObjectLifeTimeConditions[ArrayIndex];
			if (LifeTimeCondition == COND_NetGroup)
			{
				bool bShouldReplicateSubObject = false;
				const TArrayView<const FNetObjectGroupHandle::FGroupIndexType> GroupIndexes = NetObjectGroups->GetGroupIndexesOfNetObject(SubObjectIndex);
				for (const FNetObjectGroupHandle::FGroupIndexType GroupIndex : GroupIndexes)
				{
					const FNetObjectGroupHandle NetGroup = NetObjectGroups->GetHandleFromIndex(GroupIndex);

					if (NetGroup.IsNetGroupOwnerNetObjectGroup())
					{
						bShouldReplicateSubObject = LifetimeConditionals.IsConditionEnabled(COND_OwnerOnly);
					}
					else if (NetGroup.IsNetGroupReplayNetObjectGroup())
					{
						bShouldReplicateSubObject = LifetimeConditionals.IsConditionEnabled(COND_ReplayOnly);
					}
					else
					{
						ENetFilterStatus ReplicationStatus = ENetFilterStatus::Disallow;
						ensureMsgf(ReplicationFiltering->GetSubObjectFilterStatus(NetGroup, ReplicatingConnectionId, ReplicationStatus), TEXT("FReplicationConditionals::GetChildSubObjectsToReplicat Trying to filter with group %u that is not a SubObjectFilterGroup"), NetGroup.GetGroupIndex());
						bShouldReplicateSubObject = ReplicationStatus != ENetFilterStatus::Disallow;
					}
						
					if (bShouldReplicateSubObject)
					{
						FChildSubObjectsInfo ChildSubObjectsInfo;
						if (NetRefHandleManager->GetChildSubObjects(SubObjectIndex, ChildSubObjectsInfo))
						{
							// If SubObject has ChildSubObjects we need to consider replication order.
							if (ChildSubObjectsInfo.bReplicateChildSubObjectsAfterParent)
							{
								OutSubObjectsToReplicate.Add(SubObjectIndex);
								GetChildSubObjectsToReplicate(ReplicatingConnectionId, LifetimeConditionals, SubObjectIndex, ChildSubObjectsInfo, OutSubObjectsToReplicate);
							}
							else
							{
								GetChildSubObjectsToReplicate(ReplicatingConnectionId, LifetimeConditionals, SubObjectIndex, ChildSubObjectsInfo, OutSubObjectsToReplicate);
								OutSubObjectsToReplicate.Add(SubObjectIndex);
							}
						}
						else
						{
							OutSubObjectsToReplicate.Add(SubObjectIndex);
						}
						break;
					}
				}

				UE_CLOGF(!bShouldReplicateSubObject, LogIrisConditionals, VeryVerbose, "%ls Filtered out by COND_NetGroup", *NetRefHandleManager->PrintObjectFromIndex(SubObjectIndex));
			}
			else if (LifetimeConditionals.IsConditionEnabled(LifeTimeCondition))
			{
				FChildSubObjectsInfo ChildSubObjectsInfo;
				if (NetRefHandleManager->GetChildSubObjects(SubObjectIndex, ChildSubObjectsInfo))
				{
					// If SubObject has ChildSubObjects we need to consider replication order.
					if (ChildSubObjectsInfo.bReplicateChildSubObjectsAfterParent)
					{
						OutSubObjectsToReplicate.Add(SubObjectIndex);
						GetChildSubObjectsToReplicate(ReplicatingConnectionId, LifetimeConditionals, SubObjectIndex, ChildSubObjectsInfo, OutSubObjectsToReplicate);
					}
					else
					{
						GetChildSubObjectsToReplicate(ReplicatingConnectionId, LifetimeConditionals, SubObjectIndex, ChildSubObjectsInfo, OutSubObjectsToReplicate);
						OutSubObjectsToReplicate.Add(SubObjectIndex);
					}
				}
				else
				{
					OutSubObjectsToReplicate.Add(SubObjectIndex);
				}
			}
			else
			{
				UE_LOGF(LogIrisConditionals, VeryVerbose, "%ls Filtered out by %ls", *NetRefHandleManager->PrintObjectFromIndex(SubObjectIndex), *UEnum::GetValueAsString(LifeTimeCondition));
			}
		}
	}
}

void FReplicationConditionals::GetSubObjectsToReplicate(uint32 ReplicationConnectionId, FInternalNetRefIndex RootObjectIndex, FSubObjectsToReplicateArray& OutSubObjectsToReplicate)
{
	//IRIS_PROFILER_SCOPE_VERBOSE(FReplicationConditionals_GetSubObjectsToReplicate);

	constexpr bool bInitialState = false;
	const FConditionalsMask LifetimeConditionals = GetLifetimeConditionals(ReplicationConnectionId, RootObjectIndex, bInitialState);

	FChildSubObjectsInfo SubObjectsInfo;

	if (NetRefHandleManager->GetChildSubObjects(RootObjectIndex, SubObjectsInfo))
	{
		GetChildSubObjectsToReplicate(ReplicationConnectionId, LifetimeConditionals, RootObjectIndex, SubObjectsInfo, OutSubObjectsToReplicate);
	}
}

bool FReplicationConditionals::ApplyConditionalsToChangeMask(uint32 ReplicatingConnectionId, bool bIsInitialState, FInternalNetRefIndex ParentObjectIndex, FInternalNetRefIndex ObjectIndex, uint32* ChangeMaskData, const uint32* ConditionalChangeMaskData, const FReplicationProtocol* Protocol)
{
	//IRIS_PROFILER_SCOPE_VERBOSE(FReplicationConditionals_ApplyConditionalsToChangeMask);

	bool bMaskWasModified = false;

	// Assume we need all information regarding connection filtering and replication conditionals.
	FNetBitArrayView ChangeMask = MakeNetBitArrayView(ChangeMaskData, Protocol->ChangeMaskBitCount);

	// Legacy lifetime conditionals support.
	if (EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasLifetimeConditionals))
	{
		const FConditionalsMask LifetimeConditionals = GetLifetimeConditionals(ReplicatingConnectionId, ParentObjectIndex, bIsInitialState);

		FConditionalsMask PrevLifeTimeConditions = ConnectionInfos[ReplicatingConnectionId].ObjectConditionals[ObjectIndex];
		if (PrevLifeTimeConditions.IsUninitialized())
		{
			PrevLifeTimeConditions = LifetimeConditionals;
		}
		ConnectionInfos[ReplicatingConnectionId].ObjectConditionals[ObjectIndex] = LifetimeConditionals;

		// Optimized path for single lifetime conditional state
		if (Protocol->LifetimeConditionalsStateCount == 1U)
		{
			const FReplicationStateDescriptor* StateDescriptor = Protocol->ReplicationStateDescriptors[Protocol->FirstLifetimeConditionalsStateIndex];
			const uint32 ChangeMaskBitOffset = Protocol->FirstLifetimeConditionalsChangeMaskOffset;

			const FReplicationStateMemberChangeMaskDescriptor* ChangeMaskDescriptors = StateDescriptor->MemberChangeMaskDescriptors;
			const FReplicationStateMemberLifetimeConditionDescriptor* LifetimeConditionDescriptors = StateDescriptor->MemberLifetimeConditionDescriptors;
			for (uint32 MemberIt = 0U, MemberEndIt = StateDescriptor->MemberCount; MemberIt != MemberEndIt; ++MemberIt)
			{
				const FReplicationStateMemberLifetimeConditionDescriptor& LifetimeConditionDescriptor = LifetimeConditionDescriptors[MemberIt];
				ELifetimeCondition Condition = static_cast<ELifetimeCondition>(LifetimeConditionDescriptor.Condition);
				if (Condition == COND_Dynamic)
				{
					const FProperty* Property = StateDescriptor->MemberProperties[MemberIt];
					if (ensure(Property != nullptr))
					{
						Condition = GetDynamicCondition(ObjectIndex, Property->RepIndex);
					}
				}
				
				// If condition was enabled we need to dirty changemask of relevant members. If it was disabled we clear the changemask of relevant members.
				if (LifetimeConditionals.IsConditionEnabled(Condition))
				{
					if (!PrevLifeTimeConditions.IsConditionEnabled(Condition))
					{
						UE_LOGF(LogIrisConditionals, Verbose, "Dirtying member %ls %ls:%ls due to condition %ls", *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), ToCStr(StateDescriptor->DebugName), ToCStr(StateDescriptor->MemberDebugDescriptors[MemberIt].DebugName), *UEnum::GetValueAsString(Condition));

						const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskDescriptor = ChangeMaskDescriptors[MemberIt];
						for (uint32 BitIt = ChangeMaskBitOffset + ChangeMaskDescriptor.BitOffset, BitEndIt = BitIt + ChangeMaskDescriptor.BitCount; BitIt != BitEndIt; ++BitIt)
						{
							bMaskWasModified |= !ChangeMask.GetBit(BitIt);
							ChangeMask.SetBit(BitIt);
						}
					}
				}
				else
				{
					const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskDescriptor = ChangeMaskDescriptors[MemberIt];
					if (ChangeMask.IsAnyBitSet(ChangeMaskBitOffset + ChangeMaskDescriptor.BitOffset, ChangeMaskDescriptor.BitCount))
					{
						UE_LOGF(LogIrisConditionals, VeryVerbose, "Filtering out member %ls %ls:%ls due to condition %ls", *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), ToCStr(StateDescriptor->DebugName), ToCStr(StateDescriptor->MemberDebugDescriptors[MemberIt].DebugName), *UEnum::GetValueAsString(Condition));
						ChangeMask.ClearBits(ChangeMaskBitOffset + ChangeMaskDescriptor.BitOffset, ChangeMaskDescriptor.BitCount);
						bMaskWasModified = true;
					}
				}
			}
		}
		else
		{
			uint32 CurrentChangeMaskBitOffset = 0U;
			uint32 LifetimeConditionalsStateIt = 0U;
			const uint32 LifetimeConditionalsStateEndIt = Protocol->LifetimeConditionalsStateCount;
			for (const FReplicationStateDescriptor* StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
			{
				if (EnumHasAnyFlags(StateDescriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals))
				{
					const FReplicationStateMemberChangeMaskDescriptor* ChangeMaskDescriptors = StateDescriptor->MemberChangeMaskDescriptors;
					const FReplicationStateMemberLifetimeConditionDescriptor* LifetimeConditionDescriptors = StateDescriptor->MemberLifetimeConditionDescriptors;
					for (uint32 MemberIt = 0U, MemberEndIt = StateDescriptor->MemberCount; MemberIt != MemberEndIt; ++MemberIt)
					{
						const FReplicationStateMemberLifetimeConditionDescriptor& LifetimeConditionDescriptor = LifetimeConditionDescriptors[MemberIt];
						ELifetimeCondition Condition = static_cast<ELifetimeCondition>(LifetimeConditionDescriptor.Condition);
						if (Condition == COND_Dynamic)
						{
							const FProperty* Property = StateDescriptor->MemberProperties[MemberIt];
							if (ensure(Property != nullptr))
							{
								Condition = GetDynamicCondition(ObjectIndex, Property->RepIndex);
							}
						}

						// If the condition is fulfilled the changemask will remain intact so we can continue to the next member.
						if (LifetimeConditionals.IsConditionEnabled(Condition))
						{
							if (!PrevLifeTimeConditions.IsConditionEnabled(Condition))
							{
								UE_LOGF(LogIrisConditionals, Verbose, "Dirtying member %ls %ls:%ls due to condition %ls", *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), ToCStr(StateDescriptor->DebugName), ToCStr(StateDescriptor->MemberDebugDescriptors[MemberIt].DebugName), *UEnum::GetValueAsString(Condition));
								const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskDescriptor = ChangeMaskDescriptors[MemberIt];
								for (uint32 BitIt = CurrentChangeMaskBitOffset + ChangeMaskDescriptor.BitOffset, BitEndIt = BitIt + ChangeMaskDescriptor.BitCount; BitIt != BitEndIt; ++BitIt)
								{
									bMaskWasModified |= !ChangeMask.GetBit(BitIt);
									ChangeMask.SetBit(BitIt);
								}
							}
						}
						else
						{
							const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskDescriptor = ChangeMaskDescriptors[MemberIt];
							if (ChangeMask.IsAnyBitSet(CurrentChangeMaskBitOffset + ChangeMaskDescriptor.BitOffset, ChangeMaskDescriptor.BitCount))
							{
								UE_LOGF(LogIrisConditionals, VeryVerbose, "Filtering out member %ls %ls:%ls due to condition %ls", *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), ToCStr(StateDescriptor->DebugName), ToCStr(StateDescriptor->MemberDebugDescriptors[MemberIt].DebugName), *UEnum::GetValueAsString(Condition));

								ChangeMask.ClearBits(CurrentChangeMaskBitOffset + ChangeMaskDescriptor.BitOffset, ChangeMaskDescriptor.BitCount);
								bMaskWasModified = true;
							}
						}
					}

					// Done processing all states with lifetime conditionals?
					++LifetimeConditionalsStateIt;
					if (LifetimeConditionalsStateIt == LifetimeConditionalsStateEndIt)
					{
						break;
					}
				}

				CurrentChangeMaskBitOffset += StateDescriptor->ChangeMaskBitCount;
			}
		}
	}

	// Apply custom conditionals by word operations.
	if (ConditionalChangeMaskData != nullptr)
	{
		const uint32 WordCount = ChangeMask.GetNumWords();
		uint32 ChangedBits = 0;
		for (uint32 WordIt = 0; WordIt != WordCount; ++WordIt)
		{
			const uint32 OldMask = ChangeMaskData[WordIt];
			const uint32 ConditionalMask = ConditionalChangeMaskData[WordIt];
			const uint32 NewMask = OldMask & ConditionalMask;
			ChangeMaskData[WordIt] = NewMask;

			ChangedBits |= OldMask & ~ConditionalMask;
		}

		bMaskWasModified = bMaskWasModified | (ChangedBits != 0U);
	}

	return bMaskWasModified;
}

void FReplicationConditionals::UpdateAndResetObjectsWithDirtyConditionals()
{
	IRIS_PROFILER_SCOPE(FReplicationConditionals_UpdateAndResetObjectsWithDirtyConditionals);

	const FNetBitArray& ValidConnections = ReplicationConnections->GetValidConnections();

	// We do not expect many objects with dirty global lifetime conditionals each frame
	const uint32 MaxBatchObjectCount = 128U;
	FInternalNetRefIndex ObjectIndices[MaxBatchObjectCount];

	const uint32 BitCount = ~0U;
	for (uint32 ObjectCount, StartIndex = 0; (ObjectCount = ObjectsWithDirtyLifetimeConditionals.GetSetBitIndices(StartIndex, BitCount, ObjectIndices, MaxBatchObjectCount)) > 0; )
	{
		for (uint32 ConnectionId : ValidConnections)
		{
			UE::Net::Private::FReplicationConnection* Connection = ReplicationConnections->GetConnection(ConnectionId);
			Connection->ReplicationWriter->UpdateDirtyGlobalLifetimeConditionals(MakeArrayView(ObjectIndices, ObjectCount));
		}

		StartIndex = ObjectIndices[ObjectCount - 1] + 1U;
		if ((StartIndex == ObjectsWithDirtyLifetimeConditionals.GetNumBits()) | (ObjectCount < MaxBatchObjectCount))
		{
			break;
		}
	}	

	ObjectsWithDirtyLifetimeConditionals.ClearAllBits();
}

FReplicationConditionals::FConditionalsMask FReplicationConditionals::GetLifetimeConditionals(uint32 ReplicatingConnectionId, FInternalNetRefIndex ParentObjectIndex, bool bIsInitialState) const
{
	FConditionalsMask ConditionalsMask{0};

	const uint32 ObjectOwnerConnectionId = ReplicationFiltering->GetOwningConnection(ParentObjectIndex);
	const bool bIsReplicatingToOwner = (ReplicatingConnectionId == ObjectOwnerConnectionId);

	const FReplicationConditionals::FPerObjectInfo* ObjectInfo = GetPerObjectInfo(ParentObjectIndex);
	bool bRoleSimulated = ReplicatingConnectionId != ObjectInfo->AutonomousConnectionId;
	const bool bRoleAutonomous = ReplicatingConnectionId == ObjectInfo->AutonomousConnectionId;
	const bool bRepPhysics = ObjectInfo->bRepPhysics;
	const bool bIsReplay = EnumHasAnyFlags(ConnectionInfos[ReplicatingConnectionId].NetConnectionTraits, ENetConnectionTraits::ReplayPlayback | ENetConnectionTraits::ReplayRecording);
	if (ConnectionInfos.IsValidIndex(ObjectOwnerConnectionId))
	{
		const FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ObjectOwnerConnectionId];
		if (bRoleAutonomous && ConnectionInfo.bAutonomousAsSimulated)
		{
			bRoleSimulated = true;
		}
	}

	ConditionalsMask.SetConditionEnabled(COND_None, true);
	ConditionalsMask.SetConditionEnabled(COND_InitialOnly, bIsInitialState);
	ConditionalsMask.SetConditionEnabled(COND_OwnerOnly, bIsReplicatingToOwner);
	ConditionalsMask.SetConditionEnabled(COND_SkipOwner, !bIsReplicatingToOwner);
	ConditionalsMask.SetConditionEnabled(COND_SimulatedOnly, bRoleSimulated);
	ConditionalsMask.SetConditionEnabled(COND_AutonomousOnly, bRoleAutonomous);
	ConditionalsMask.SetConditionEnabled(COND_SimulatedOrPhysics, bRoleSimulated | bRepPhysics);
	ConditionalsMask.SetConditionEnabled(COND_InitialOrOwner, bIsReplicatingToOwner | bIsInitialState);
	ConditionalsMask.SetConditionEnabled(COND_Custom, true);
	ConditionalsMask.SetConditionEnabled(COND_ReplayOrOwner, bIsReplicatingToOwner | bIsReplay);
	ConditionalsMask.SetConditionEnabled(COND_ReplayOnly, bIsReplay);
	ConditionalsMask.SetConditionEnabled(COND_SimulatedOnlyNoReplay, bRoleSimulated & !bIsReplay);
	ConditionalsMask.SetConditionEnabled(COND_SimulatedOrPhysicsNoReplay, (bRoleSimulated | bRepPhysics) & !bIsReplay);
	ConditionalsMask.SetConditionEnabled(COND_SkipReplay, !bIsReplay);
	ConditionalsMask.SetConditionEnabled(COND_Dynamic, true);
	//ConditionalsMask.SetConditionEnabled(COND_Never, false); Ctor sets its false by default
	//ConditionalsMask.SetConditionEnabled(COND_NetGroup, Not Known); NetGroup is evaluated in FReplicationConditionals::GetChildSubObjectsToReplicate
	
	return ConditionalsMask;
}

void FReplicationConditionals::ClearPerObjectInfo(FInternalNetRefIndex ObjectIndex)
{
	FPerObjectInfo& PerObjectInfo = PerObjectInfos.GetData()[ObjectIndex];
	PerObjectInfo = {};

	// Remove any dynamic conditions information stored.
	DynamicConditions.Remove(ObjectIndex);
}

void FReplicationConditionals::ClearConnectionInfosForObject(const FNetBitArray& ValidConnections, FInternalNetRefIndex ObjectIndex)
{
	IRIS_PROFILER_SCOPE(FReplicationConditionals_ClearConnectionInfosForObject);

	for (uint32 ConnectionId : ValidConnections)
	{
		FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
		ConnectionInfo.ObjectConditionals[ObjectIndex] = FConditionalsMask{};
	}
}

ELifetimeCondition FReplicationConditionals::GetDynamicCondition(FInternalNetRefIndex ObjectIndex, uint16 RepIndex) const
{
	const FObjectDynamicConditions* ObjectConditions = DynamicConditions.Find(ObjectIndex);
	if (ObjectConditions == nullptr)
	{
		return COND_Dynamic;
	}

	const int16* Condition = ObjectConditions->DynamicConditions.Find(RepIndex);
	if (Condition == nullptr)
	{
		return COND_Dynamic;
	}

	return static_cast<const ELifetimeCondition>(*Condition);
}

void FReplicationConditionals::SetDynamicCondition(FInternalNetRefIndex ObjectIndex, uint16 RepIndex, ELifetimeCondition Condition)
{
	FObjectDynamicConditions& ObjectConditions = DynamicConditions.FindOrAdd(ObjectIndex);
	ObjectConditions.DynamicConditions.Emplace(RepIndex, static_cast<int16>(Condition));
}

bool FReplicationConditionals::DynamicConditionChangeRequiresBaselineInvalidation(ELifetimeCondition OldCondition, ELifetimeCondition NewCondition) const
{
	// If the old condition didn't cause the member to always be replicated it could have been not replicated to one or more connections.
	const bool OldConditionMayHaveBeenDisabled = !(OldCondition == COND_None || OldCondition == COND_Dynamic);

	// If the new condition is something other than never replicating then it may be replicated.
	const bool NewConditionMayBeEnabled = (NewCondition != COND_Never);

	return OldConditionMayHaveBeenDisabled && NewConditionMayBeEnabled;
}

void FReplicationConditionals::MarkRemoteRoleDirty(FInternalNetRefIndex ObjectIndex)
{
	const FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
	const FReplicationProtocol* Protocol = ReplicatedObjectData.Protocol;

	if (NetRefHandleManager->GetReplicatedObjectStateBufferNoCheck(ObjectIndex) == nullptr)
	{
		return;
	}

	if (!ReplicatedObjectData.NetHandle.IsValid())
	{
		return;
	}

	const uint16 RepIndex = GetRemoteRoleRepIndex(Protocol);
	if (RepIndex == InvalidRepIndex)
	{
		return;
	}

	MarkPropertyDirty(ObjectIndex, RepIndex);
}

uint16 FReplicationConditionals::GetRemoteRoleRepIndex(const FReplicationProtocol* Protocol)
{
	if (CachedRemoteRoleRepIndex != InvalidRepIndex)
	{
		return CachedRemoteRoleRepIndex;
	}
	
	const FNetSerializer* NetRoleNetSerializer = &UE_NET_GET_SERIALIZER(FNetRoleNetSerializer);

	// Loop through all state descriptors end their properties to find the RemoteRole
	for (const FReplicationStateDescriptor* StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, static_cast<int32>(Protocol->ReplicationStateCount)))
	{
		for (const FReplicationStateMemberSerializerDescriptor& SerializerDescriptor : MakeArrayView(StateDescriptor->MemberSerializerDescriptors, StateDescriptor->MemberCount))
		{
			if (SerializerDescriptor.Serializer != NetRoleNetSerializer)
			{
				continue;
			}

			const SIZE_T MemberIndex = &SerializerDescriptor - StateDescriptor->MemberSerializerDescriptors;
			const FProperty* Property = StateDescriptor->MemberProperties[MemberIndex];
			if (Property && Property->GetFName() == NAME_RemoteRole)
			{
				CachedRemoteRoleRepIndex = Property->RepIndex;
				return Property->RepIndex;
			}
		}
	}

	return InvalidRepIndex;
}

void FReplicationConditionals::MarkPropertyDirty(FInternalNetRefIndex ObjectIndex, uint16 RepIndex)
{
	const FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);

	const FNetHandle OwnerHandle = ReplicatedObjectData.NetHandle;
	if (!OwnerHandle.IsValid())
	{
		return;
	}

	if (NetRefHandleManager->GetReplicatedObjectStateBufferNoCheck(ObjectIndex) == nullptr)
	{
		return;
	}

	const FReplicationProtocol* Protocol = ReplicatedObjectData.Protocol;
	const FReplicationInstanceProtocol* InstanceProtocol = ReplicatedObjectData.InstanceProtocol;

	constexpr uint32 MaxFragmentOwnerCount = 1U;
	UObject* FragmentOwners[MaxFragmentOwnerCount] = {};
	FReplicationStateOwnerCollector FragmentOwnerCollector(FragmentOwners, MaxFragmentOwnerCount);

	for (const FReplicationStateDescriptor*& StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, static_cast<int32>(Protocol->ReplicationStateCount)))
	{
		const SIZE_T StateIndex = &StateDescriptor - Protocol->ReplicationStateDescriptors;

		// Is the passed Owner the owner of the fragment?
		{
			const FReplicationFragment* ReplicationFragment = InstanceProtocol->Fragments[StateIndex];

			FragmentOwnerCollector.Reset();
			ReplicationFragment->CollectOwner(&FragmentOwnerCollector);
			if (FragmentOwnerCollector.GetOwnerCount() == 0U || FNetHandleManager::GetNetHandle(FragmentOwnerCollector.GetOwners()[0]) != OwnerHandle)
			{
				// Not the right owner.
				continue;
			}
		}

		// Can this state contain this property?
		if (RepIndex >= StateDescriptor->RepIndexCount)
		{
			continue;
		}

		// Does this state contain this property?
		const FReplicationStateMemberRepIndexToMemberIndexDescriptor& RepIndexToMemberIndexDescriptor = StateDescriptor->MemberRepIndexToMemberIndexDescriptors[RepIndex];
		if (RepIndexToMemberIndexDescriptor.MemberIndex == FReplicationStateMemberRepIndexToMemberIndexDescriptor::InvalidEntry)
		{
			continue;
		}

		// We found the relevant state. Modify the external state changemask.
		const FReplicationInstanceProtocol::FFragmentData& Fragment = InstanceProtocol->FragmentData[StateIndex];
		const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskDescriptor = StateDescriptor->MemberChangeMaskDescriptors[RepIndexToMemberIndexDescriptor.MemberIndex];
		FNetBitArrayView MemberChangeMask = GetMemberChangeMask(Fragment.ExternalSrcBuffer, StateDescriptor);
		FReplicationStateHeader& Header = GetReplicationStateHeader(Fragment.ExternalSrcBuffer, StateDescriptor);
		MarkDirty(Header, MemberChangeMask, ChangeMaskDescriptor);

		return;
	}

	UE_LOGF(LogIris, Warning, "Trying to mark non-existing property with RepIndex %u in protocol %ls as dirty", RepIndex, ToCStr(Protocol->DebugName));
}

void FReplicationConditionals::InvalidateBaselinesForObjectHierarchy(uint32 ObjectIndex, const TConstArrayView<uint32>& ConnectionsToInvalidate)
{
	// Invalidate baselines for root object
	{
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
		if (EnumHasAnyFlags(ObjectData.Protocol->ProtocolTraits, EReplicationProtocolTraits::HasLifetimeConditionals))
		{
			for (const uint32 ConnId : ConnectionsToInvalidate)
			{
				BaselineInvalidationTracker->InvalidateBaselines(ObjectIndex, ConnId);
			}
		}
	}

	// Invalidate baselines for subobjects
	for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
	{
		const FNetRefHandleManager::FReplicatedObjectData& SubObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(SubObjectIndex);
		if (EnumHasAnyFlags(SubObjectData.Protocol->ProtocolTraits, EReplicationProtocolTraits::HasLifetimeConditionals))
		{
			for (const uint32 ConnId : ConnectionsToInvalidate)
			{
				BaselineInvalidationTracker->InvalidateBaselines(SubObjectIndex, ConnId);
			}
		}
	}
}

}
