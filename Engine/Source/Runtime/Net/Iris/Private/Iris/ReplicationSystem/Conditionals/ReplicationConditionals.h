// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Iris/ReplicationSystem/NetObjectGroupHandle.h"
#include "Net/Core/NetBitArray.h"

enum ELifetimeCondition : int;
namespace UE::Net
{
	enum class ENetConnectionTraits : uint32;
	enum class EReplicationCondition : uint32;

	typedef uint32 FInternalNetRefIndex;

	struct FReplicationProtocol;

	namespace Private
	{
		class FDeltaCompressionBaselineInvalidationTracker;
		class FNetRefHandleManager;
		class FReplicationConnections;
		class FNetObjectGroups;
		class FReplicationFiltering;
		class FNetObjectGroups;
		struct FChildSubObjectsInfo;
	}
}

namespace UE::Net::Private
{

struct FReplicationConditionalsInitParams
{
	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	const FReplicationFiltering* ReplicationFiltering = nullptr;
	FReplicationConnections* ReplicationConnections = nullptr;
	const FNetObjectGroups* NetObjectGroups = nullptr;
	FDeltaCompressionBaselineInvalidationTracker* BaselineInvalidationTracker = nullptr;
	FInternalNetRefIndex MaxInternalNetRefIndex = 0;
	uint32 MaxConnectionCount = 0;
};

/**
 * NOTE: Almost everything in this class is for backward compatibility mode only. As such some functions are
 * fairly slow. For declared replication states this overhead will be eliminated because the state
 * itself can do all the relevant tracking locally. The one thing that will possibly remain is the unmasking
 * of changemasks. The unmasking could then either be done on the sending side, CPU vs bandwidth trade-off,
 * or on the receiving side, not so CPU costly as there's typically one connection and maybe tens of objects.
 * The unmasking is then likely to move to protocol operations.
 */
class FReplicationConditionals
{
public:
	FReplicationConditionals();

	void Init(FReplicationConditionalsInitParams& Params);

	/** Called when the maximum InternalNetRefIndex increased and we need to realloc our lists */
	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

	/** Called when when one or more NetRefInternalIndices have been freed and can be re-assigned to new objects. */
	void OnInternalNetRefIndicesFreed(const TConstArrayView<FInternalNetRefIndex>& FreedIndices);

	void AddConnection(uint32 ConnectionId, ENetConnectionTraits NetConnectionTraits);
	void RemoveConnection(uint32 ConnectionId);

	/** 
	 * When considering replication conditions that require an owning object's role to be ROLE_SimulatedProxy, and the
	 * owning object's role is ROLE_AutonomousProxy, treat its role as ROLE_SimulatedProxy if this option is set to true.
	 * 
	 * Example:
	 *	If an object has the role ROLE_AutonomousProxy but it has a property with the condition COND_SimulatedOnly, the
	 *	property would still be replicated to clients. Other properties that would have COND_AutonomousOnly will still
	 *	be replicated.
	 */
	void SetAutonomousAsSimulated(uint32 ConnectionId, bool bEnabled);

	bool SetConditionConnectionFilter(FInternalNetRefIndex ObjectIndex, EReplicationCondition Condition, uint32 ConnectionId, bool bEnable);
	bool SetCondition(FInternalNetRefIndex ObjectIndex, EReplicationCondition Condition, bool bEnable);
	void SetOwningConnection(FInternalNetRefIndex ObjectIndex, uint32 ConnectionId);

	// For property custom conditions only
	void InitPropertyCustomConditions(FInternalNetRefIndex ObjectIndex);
	bool SetPropertyCustomCondition(FInternalNetRefIndex ObjectIndex, const void* Owner, uint16 RepIndex, bool bIsActive);
	bool SetPropertyDynamicCondition(FInternalNetRefIndex ObjectIndex, const void* Owner, uint16 RepIndex, ELifetimeCondition Condition);

	void MarkLifeTimeConditionalsDirtyForObjectsInGroup(FNetObjectGroupHandle GroupHandle);
	
	/** Unconditionally marks a property as dirty, causing it to replicate with the object at the earliest convenience. */
	void MarkPropertyDirty(FInternalNetRefIndex ObjectIndex, uint16 RepIndex);

	void Update();

	struct FConditionalsMask
	{
		bool IsUninitialized() const { return ConditionalsMask == 0; }
		bool IsConditionEnabled(int Condition) const { checkSlow(Condition < 16); return ConditionalsMask & (uint16(1) << unsigned(Condition)); }
		bool SetConditionEnabled(int Condition, bool bEnabled) { checkSlow(Condition < 16); return ConditionalsMask |= (uint16(bEnabled ? 1 : 0) << unsigned(Condition)); }

		// Each LifetimeCondition is represented in this member via (1U << ELifetimeCondition)
		uint16 ConditionalsMask;
	};

	bool ApplyConditionalsToChangeMask(uint32 ReplicatingConnectionId, bool bIsInitialState, FInternalNetRefIndex ParentObjectIndex, FInternalNetRefIndex ObjectIndex, uint32* ChangeMaskData, const uint32* ConditionalChangeMaskData, const FReplicationProtocol* Protocol);

	using FSubObjectsToReplicateArray = TArray<FInternalNetRefIndex, TInlineAllocator<32>>;
	void GetSubObjectsToReplicate(uint32 ReplicationConnectionId, FInternalNetRefIndex ParentObjectIndex, FSubObjectsToReplicateArray& OutSubObjectsToReplicate);

private:
	struct FPerObjectInfo
	{
		// Assume there can only be one connection which has the role autonomous connection and all else are simulated
		uint16 AutonomousConnectionId : 15;
		uint16 bRepPhysics : 1;
	};

	struct FPerConnectionInfo
	{
		TArray<FConditionalsMask> ObjectConditionals;
		ENetConnectionTraits NetConnectionTraits = {};
		bool bAutonomousAsSimulated = false;
	};

	struct FSubObjectConditionInfo
	{
		// This is a more compact storage form of ELifetimeCondition
		int8 Condition;
	};

	struct FObjectDynamicConditions
	{
		// RepIndex and ELifetimeCondition expressed as int16
		TMap<uint16, int16> DynamicConditions;
	};

private:
	void UpdateAndResetObjectsWithDirtyConditionals();

	FConditionalsMask GetLifetimeConditionals(uint32 ReplicatingConnectionId, FInternalNetRefIndex ParentObjectIndex, bool bInitialState) const;

	FPerObjectInfo* GetPerObjectInfo(FInternalNetRefIndex ObjectIndex);
	const FPerObjectInfo* GetPerObjectInfo(FInternalNetRefIndex ObjectIndex) const;
	void ClearPerObjectInfo(FInternalNetRefIndex ObjectIndex);
	void ClearConnectionInfosForObject(const FNetBitArray& ValidConnections, FInternalNetRefIndex ObjectIndex);

	void GetChildSubObjectsToReplicate(uint32 ReplicatingConnectionId, const FConditionalsMask& LifetimeConditionals, const FInternalNetRefIndex ParentObjectIndex, const FChildSubObjectsInfo& SubObjectsInfo, FSubObjectsToReplicateArray& OutSubObjectsToReplicate) const;

	ELifetimeCondition GetDynamicCondition(FInternalNetRefIndex ObjectIndex, uint16 RepIndex) const;
	void SetDynamicCondition(FInternalNetRefIndex ObjectIndex, uint16 RepIndex, ELifetimeCondition Condition);
	bool DynamicConditionChangeRequiresBaselineInvalidation(ELifetimeCondition OldCondition, ELifetimeCondition NewCondition) const;

	void MarkRemoteRoleDirty(FInternalNetRefIndex ObjectIndex);
	uint16 GetRemoteRoleRepIndex(const FReplicationProtocol* Protocol);

	// Invalidates baselines for root object and subobjects with lifetime conditionals.
	void InvalidateBaselinesForObjectHierarchy(uint32 ObjectIndex, const TConstArrayView<uint32>& ConnectionsToInvalidate);

private:
	static constexpr uint16 InvalidRepIndex = 65535U;

	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	const FReplicationFiltering* ReplicationFiltering = nullptr;
	FReplicationConnections* ReplicationConnections = nullptr;
	FDeltaCompressionBaselineInvalidationTracker* BaselineInvalidationTracker = nullptr;
	const FNetObjectGroups* NetObjectGroups = nullptr;

	TArray<FPerObjectInfo> PerObjectInfos;
	TArray<FPerConnectionInfo> ConnectionInfos;
	TMap<FInternalNetRefIndex, FObjectDynamicConditions> DynamicConditions;
	FNetBitArray ObjectsWithDirtyLifetimeConditionals;

	FInternalNetRefIndex MaxInternalNetRefIndex = 0;
	uint32 MaxConnectionCount = 0;
	uint16 CachedRemoteRoleRepIndex = InvalidRepIndex;
};

inline FReplicationConditionals::FPerObjectInfo* FReplicationConditionals::GetPerObjectInfo(FInternalNetRefIndex ObjectIndex)
{
	return PerObjectInfos.GetData() + ObjectIndex;
}

inline const FReplicationConditionals::FPerObjectInfo* FReplicationConditionals::GetPerObjectInfo(FInternalNetRefIndex ObjectIndex) const
{
	return PerObjectInfos.GetData() + ObjectIndex;
}

}
