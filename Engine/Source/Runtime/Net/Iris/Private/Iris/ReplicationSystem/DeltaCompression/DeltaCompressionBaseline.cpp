// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaseline.h"

namespace UE::Net::Private
{

}
