// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/ChangeMaskUtil.h"

namespace UE::Net::Private
{

class FDeltaCompressionBaseline
{
public:
	bool IsValid() const { return StateBuffer != nullptr; }

	const ChangeMaskStorageType* ChangeMask = nullptr;
	const uint8* StateBuffer = nullptr;
};

}
