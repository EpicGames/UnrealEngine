// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineInvalidationTracker.h"
#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineManager.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"

namespace UE::Net::Private
{

FDeltaCompressionBaselineInvalidationTracker::FDeltaCompressionBaselineInvalidationTracker()
{
}

FDeltaCompressionBaselineInvalidationTracker::~FDeltaCompressionBaselineInvalidationTracker()
{
}

void FDeltaCompressionBaselineInvalidationTracker::Init(FDeltaCompressionBaselineInvalidationTrackerInitParams& InitParams)
{
	BaselineManager = InitParams.BaselineManager;
	InvalidatedObjects.Init(InitParams.MaxInternalNetRefIndex);
}

void FDeltaCompressionBaselineInvalidationTracker::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	InvalidatedObjects.SetNumBits(NewMaxInternalIndex);
}

void FDeltaCompressionBaselineInvalidationTracker::InvalidateBaselines(FInternalNetRefIndex ObjectIndex, uint32 ConnId)
{
	if (InvalidatedObjects.GetBit(ObjectIndex))
	{
		return;
	}

	if (BaselineManager->GetDeltaCompressionStatus(ObjectIndex) != ENetObjectDeltaCompressionStatus::Allow)
	{
		return;
	}

	if (ConnId == InvalidateBaselineForAllConnections)
	{
		InvalidatedObjects.SetBit(ObjectIndex);
	}

	if (InvalidationInfos.GetSlack() == 0)
	{
		InvalidationInfos.Reserve(InvalidationInfos.Num() + InvalidationInfoGrowCount);
	}

	InvalidationInfos.Emplace(FInvalidationInfo{ConnId, ObjectIndex});
}

void FDeltaCompressionBaselineInvalidationTracker::PreSendUpdate()
{
	// Could consider sorting of object indices here.
}

void FDeltaCompressionBaselineInvalidationTracker::PostSendUpdate()
{
	if (InvalidationInfos.Num() > 0)
	{
		InvalidationInfos.Empty();
		InvalidatedObjects.ClearAllBits();
	}
}

}
