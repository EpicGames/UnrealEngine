// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaseline.h"
#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineStorage.h"

namespace UE::Net
{
	typedef uint32 FInternalNetRefIndex;
}

namespace UE::Net::Private
{
	class FDeltaCompressionBaselineManager;
}

namespace UE::Net::Private
{

struct FDeltaCompressionBaselineInvalidationTrackerInitParams
{
	const FDeltaCompressionBaselineManager* BaselineManager = nullptr;
	FInternalNetRefIndex MaxInternalNetRefIndex = 0;
};

class FDeltaCompressionBaselineInvalidationTracker
{
public:
	enum Constants : uint32
	{
		InvalidateBaselineForAllConnections = 0U,
	};

	struct FInvalidationInfo
	{
		uint32 ConnId = InvalidateBaselineForAllConnections;
		FInternalNetRefIndex ObjectIndex = 0U;
	};

public:
	FDeltaCompressionBaselineInvalidationTracker();
	~FDeltaCompressionBaselineInvalidationTracker();

	void Init(FDeltaCompressionBaselineInvalidationTrackerInitParams& InitParams);

	// It is up to calling code to also do this for eventual subobjects.
	void InvalidateBaselines(FInternalNetRefIndex ObjectIndex, uint32 ConnId);

	// Returns an array of objects with enabled conditions such that DC baselines need to be invalidated.
	TArrayView<const FInvalidationInfo> GetBaselineInvalidationInfos() const;

	void PreSendUpdate();
	void PostSendUpdate();

	/** Called when the maximum InternalNetRefIndex increased and we need to realloc our lists */
	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

private:
	enum : unsigned
	{
		InvalidationInfoGrowCount = 256,
	};

	TArray<FInvalidationInfo> InvalidationInfos;
	FNetBitArray InvalidatedObjects;

	const FDeltaCompressionBaselineManager* BaselineManager = nullptr;
};

inline TArrayView<const FDeltaCompressionBaselineInvalidationTracker::FInvalidationInfo> FDeltaCompressionBaselineInvalidationTracker::GetBaselineInvalidationInfos() const
{
	return MakeArrayView(InvalidationInfos);
}

}
