// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineManager.h"
#include "Iris/IrisConfigInternal.h"
#include "Iris/Core/BitTwiddling.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisMemoryTracker.h"
#include "Iris/ReplicationSystem/ChangeMaskCache.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationConnections.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"
#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineInvalidationTracker.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "HAL/IConsoleManager.h"
#include "Net/Core/Trace/NetTrace.h"
#include <limits>

#if UE_NET_VALIDATE_DC_BASELINES
#define UE_NET_DC_BASELINE_CHECK(...) check(__VA_ARGS__)
#else
#define UE_NET_DC_BASELINE_CHECK(...) 
#endif

namespace UE::Net::Private
{

// Delta compression kill switch
static bool bIsDeltaCompressionEnabled = true;
static FAutoConsoleVariableRef CVarEnableDeltaCompression(
	TEXT("net.Iris.EnableDeltaCompression"),
	bIsDeltaCompressionEnabled,
	TEXT("Enable delta compression for replicated objects. Default is true.")
);

// Delta compression baseline throttling.
static int32 MinimumNumberOfFramesBetweenBaselines = 60;
static FAutoConsoleVariableRef CVarMinimumNumberOfFramesBetweenBaselines(
	TEXT("net.Iris.MinimumNumberOfFramesBetweenBaselines"),
	MinimumNumberOfFramesBetweenBaselines,
	TEXT("Minimum number of frames between creation of new delta compression baselines for an object. Default is 60.")
);

bool IsDeltaCompressionEnabled()
{
	return bIsDeltaCompressionEnabled;
}

}

namespace UE::Net::Private
{

FDeltaCompressionBaselineManager::FDeltaCompressionBaselineManager()
{
	// If alignment of storage type for FPerObjectInfo is too small then it's totally unsafe to cast such pointers to FPerObjectInfo.
	static_assert(InvalidObjectInfoIndex == 0, "InvalidObjectInfoIndex needs to be zero in order for current code to function properly.");
}

FDeltaCompressionBaselineManager::~FDeltaCompressionBaselineManager()
{
}

void FDeltaCompressionBaselineManager::Init(FDeltaCompressionBaselineManagerInitParams& InitParams)
{
	Connections = InitParams.Connections;
	NetRefHandleManager = InitParams.NetRefHandleManager;
	BaselineInvalidationTracker = InitParams.BaselineInvalidationTracker;
	ReplicationSystem = InitParams.ReplicationSystem;
	MaxConnectionCount = InitParams.Connections->GetMaxConnectionCount();
	MaxDeltaCompressedObjectCount = FPlatformMath::Min(InitParams.MaxNetObjectCount, InitParams.MaxDeltaCompressedObjectCount);

	DeltaCompressionEnabledObjects.Init(InitParams.MaxInternalNetRefIndex);

	// PerObjectInfo initialization
	{
		check(MaxDeltaCompressedObjectCount < std::numeric_limits<ObjectInfoIndexType>::max());
		UsedPerObjectInfos.Init(MaxDeltaCompressedObjectCount + 1U);
		UsedPerObjectInfos.SetBit(InvalidObjectInfoIndex);
		ObjectIndexToObjectInfoIndex.SetNumZeroed(InitParams.MaxInternalNetRefIndex);

		const uint32 BytesPerConnection = sizeof(FObjectBaselineInfo);
		// The PerObjectInfo already contains the connection specific data for one connection,
		// but we need to accommodate for MaxConnectionCount.
		BytesPerObjectInfo = static_cast<uint32>(Align(sizeof(FPerObjectInfo) + BytesPerConnection*(FPlatformMath::Max(MaxConnectionCount, 1U) - 1U), alignof(FPerObjectInfo)));
	}

	BaselineCounts.SetNumZeroed(MaxDeltaCompressedObjectCount + 1U);

	// Init baseline state manager
	{
		FDeltaCompressionBaselineStorageInitParams BaselineStorageInitParams;
		BaselineStorageInitParams.ReplicationStateStorage = InitParams.ReplicationStateStorage;
		BaselineStorageInitParams.MaxBaselineCount = InitParams.MaxDeltaCompressedObjectCount*MaxConnectionCount*2U;
		BaselineStorage.Init(BaselineStorageInitParams);
	}
}

void FDeltaCompressionBaselineManager::Deinit()
{
	// The BaselineStateManager will release all baselines in its destructor.
	FreeAllPerObjectInfos();
	BaselineStorage.Deinit();
}

void FDeltaCompressionBaselineManager::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	DeltaCompressionEnabledObjects.SetNumBits(NewMaxInternalIndex);
	ObjectIndexToObjectInfoIndex.SetNumZeroed(NewMaxInternalIndex);
}

void FDeltaCompressionBaselineManager::PreSendUpdate(FDeltaCompressionBaselineManagerPreSendUpdateParams& UpdateParams)
{
	IRIS_PROFILER_SCOPE(DeltaCompressionBaselineManager_PreSendUpdate);

	++FrameCounter;

	UpdateScope();

	UpdateDirtyStateMasks(UpdateParams.ChangeMaskCache);

	InvalidateBaselinesDueToModifiedConditionals();

	ConstructBaselineSharingContext(BaselineSharingContext);
}

void FDeltaCompressionBaselineManager::PostSendUpdate(FDeltaCompressionBaselineManagerPostSendUpdateParams& UpdateParams)
{
	IRIS_PROFILER_SCOPE(DeltaCompressionBaselineManager_PostSendUpdate);

	DestructBaselineSharingContext(BaselineSharingContext);
}

void FDeltaCompressionBaselineManager::AddConnection(uint32 ConnectionId, ENetConnectionTraits NetConnectionTraits)
{
}

void FDeltaCompressionBaselineManager::RemoveConnection(uint32 ConnectionId)
{
	ReleaseBaselinesForConnection(ConnectionId);
}

ENetObjectDeltaCompressionStatus FDeltaCompressionBaselineManager::GetDeltaCompressionStatus(FInternalNetRefIndex Index) const
{
	return DeltaCompressionEnabledObjects.GetBit(Index) ? ENetObjectDeltaCompressionStatus::Allow : ENetObjectDeltaCompressionStatus::Disallow;
}

void FDeltaCompressionBaselineManager::SetDeltaCompressionStatus(FInternalNetRefIndex ObjectIndex, ENetObjectDeltaCompressionStatus Status)
{
	// If DC is disabled or the object doesn't support it then mark the object as not DC enabled, regardless of the user's wishes.
	bool bEnableDCForObject = (Status == ENetObjectDeltaCompressionStatus::Allow ? true : false);
	bEnableDCForObject = bEnableDCForObject && bIsDeltaCompressionEnabled;
	bEnableDCForObject = bEnableDCForObject && DoesObjectSupportDeltaCompression(ObjectIndex);

	DeltaCompressionEnabledObjects.SetBitValue(ObjectIndex, bEnableDCForObject);
}

void FDeltaCompressionBaselineManager::UpdateScope()
{
	IRIS_PROFILER_SCOPE(DeltaCompressionBaselineManager_UpdateScope);

	using WordType = FNetBitArray::StorageWordType;
	using SignedWordType = TSignedIntType<sizeof(WordType)>::Type;
	constexpr SIZE_T BitsPerWord = sizeof(WordType)*8U;

	const FNetBitArrayView ObjectsInScope = NetRefHandleManager->GetCurrentFrameScopableInternalIndices();
	const FNetBitArrayView PrevObjectsInScope = NetRefHandleManager->GetPrevFrameScopableInternalIndices();

	const WordType* ObjectsInScopeStorage = ObjectsInScope.GetData();
	const WordType* PrevObjectsInScopeStorage = PrevObjectsInScope.GetData();
	WordType* DeltaCompressionEnabledObjectsStorage = DeltaCompressionEnabledObjects.GetData();
	for (uint32 WordEndIt = ObjectsInScope.GetNumWords(), WordIt = 0; WordIt != WordEndIt; ++WordIt)
	{
		const WordType ObjectsInScopeWord = ObjectsInScopeStorage[WordIt];
		const WordType PrevObjectsInScopeWord = PrevObjectsInScopeStorage[WordIt];
		const WordType DeltaCompressionEnabledObjectsWord = DeltaCompressionEnabledObjectsStorage[WordIt];

		// We're only interested in processing delta compressed objects.
		const WordType ModifiedObjectsWord = (ObjectsInScopeWord ^ PrevObjectsInScopeWord) & DeltaCompressionEnabledObjectsWord;

		// If we have DC enabled objects a same frame add+remove could cause bits to be set for non-existing objects.
		// So we enter the update loop if there are either modified objects or DC enabled objects.
		if ((DeltaCompressionEnabledObjectsWord | ModifiedObjectsWord) == 0)
		{
			continue;
		}

		// Clear delta compression status for objects no longer in scope
		DeltaCompressionEnabledObjectsStorage[WordIt] = DeltaCompressionEnabledObjectsWord & ObjectsInScopeWord;

		const WordType BitOffset = WordIt*BitsPerWord;

		// Process removed objects
		for (WordType RemovedObjects = PrevObjectsInScopeWord & ModifiedObjectsWord; RemovedObjects; )
		{
			const WordType LeastSignificantBit = GetLeastSignificantBit(RemovedObjects);
			RemovedObjects ^= LeastSignificantBit;

			const WordType ObjectIndex = BitOffset + FPlatformMath::CountTrailingZeros(LeastSignificantBit);

			// Free object info if there are no baselines requiring it to persist.
			{
				ObjectInfoIndexType& ObjectInfoIndex = ObjectIndexToObjectInfoIndex[ObjectIndex];
				if (ObjectInfoIndex != InvalidObjectInfoIndex && BaselineCounts[ObjectInfoIndex] == 0)
				{
					FreePerObjectInfo(ObjectInfoIndex);
					ObjectInfoIndex = InvalidObjectInfoIndex;
				}
			}
		}

		// Process added objects
		for (WordType AddedObjects = ObjectsInScopeWord & ModifiedObjectsWord; AddedObjects; )
		{
			const WordType LeastSignificantBit = GetLeastSignificantBit(AddedObjects);
			AddedObjects ^= LeastSignificantBit;

			const WordType ObjectIndex = BitOffset + FPlatformMath::CountTrailingZeros(LeastSignificantBit);
			AllocPerObjectInfoForObject(ObjectIndex);
		}
	}

	UE_NET_TRACE_FRAME_STATSCOUNTER(ReplicationSystem->GetId(), ReplicationSystem.ObjectWithDC, DeltaCompressionEnabledObjects.CountSetBits(), ENetTraceVerbosity::Verbose);
}

void FDeltaCompressionBaselineManager::UpdateDirtyStateMasks(const FChangeMaskCache* ChangeMaskCache)
{
	IRIS_PROFILER_SCOPE(DeltaCompressionBaselineManager_UpdateDirtyStateMasks);

	// If there aren't any dirty objects there's nothing for us to do.
	if (ChangeMaskCache->Indices.Num() <= 0)
	{
		return;
	}

	const FNetBitArray& ValidConnections = Connections->GetValidConnections();
	const uint32 MaxConnectionId = ValidConnections.FindLastOne();
	// If there are no connections there are no objects that need baseline updates.
	if (MaxConnectionId == ~0U)
	{
		return;
	}

	const ChangeMaskStorageType* DirtyChangeMaskStoragePtr = ChangeMaskCache->Storage.GetData();
	for (const auto& Entry : ChangeMaskCache->Indices)
	{
		if (!Entry.bHasDirtyChangeMask)
		{
			continue;
		}

		if (!DeltaCompressionEnabledObjects.GetBit(Entry.InternalIndex))
		{
			continue;
		}

		FPerObjectInfo* ObjectInfo = GetPerObjectInfoForObject(Entry.InternalIndex);
		if (ObjectInfo == nullptr)
		{
			continue;
		}

		ChangeMaskStorageType*const ConnectionChangeMaskStoragePtr = ObjectInfo->ChangeMasksForConnections;
		ChangeMaskStorageType const*const UpdatedChanges = DirtyChangeMaskStoragePtr + Entry.StorageOffset;
		const uint32 ChangeMaskStridePerConnection = ObjectInfo->ChangeMaskStride;
		for (uint32 ConnectionId : ValidConnections)
		{
			const SIZE_T ChangeMaskOffset = ChangeMaskStridePerConnection*ConnectionId;
			ChangeMaskStorageType* ConnectionChangeMask = ConnectionChangeMaskStoragePtr + ChangeMaskOffset;
			for (uint32 WordIt = 0, WordEndIt = ChangeMaskStridePerConnection; WordIt != WordEndIt; ++WordIt)
			{
				ConnectionChangeMask[WordIt] |= UpdatedChanges[WordIt];
			}
		}
	}
}

FDeltaCompressionBaselineManager::FPerObjectInfo* FDeltaCompressionBaselineManager::AllocPerObjectInfoForObject(uint32 ObjectIndex)
{
	const ObjectInfoIndexType ObjectInfoIndex = AllocPerObjectInfo();
	ObjectIndexToObjectInfoIndex.GetData()[ObjectIndex] = ObjectInfoIndex;

	if (ObjectInfoIndex != InvalidObjectInfoIndex)
	{
		FPerObjectInfo* ObjectInfo = GetPerObjectInfo(ObjectInfoIndex);
		ObjectInfo->ObjectIndex = ObjectIndex;
		
		// Allocate storage for changemasks for all connections
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
		const FReplicationProtocol* Protocol = ObjectData.Protocol;
		const uint32 ChangeMaskStride = Align(Protocol->ChangeMaskBitCount, sizeof(ChangeMaskStorageType)*8U)/(sizeof(ChangeMaskStorageType)*8U);
		check(ChangeMaskStride < 65536U);
		if (ChangeMaskStride > 0)
		{
			const uint32 AllocSize = ChangeMaskStride*3*MaxConnectionCount*sizeof(ChangeMaskStorageType);
			ChangeMaskStorageType* ChangeMasksForConnections = static_cast<ChangeMaskStorageType*>(ChangeMaskAllocator.Alloc(AllocSize, alignof(ChangeMaskStorageType)));
			FPlatformMemory::Memzero(ChangeMasksForConnections, AllocSize);

			ObjectInfo->ChangeMaskStride = static_cast<uint16>(ChangeMaskStride);
			ObjectInfo->ChangeMasksForConnections = ChangeMasksForConnections;
		}

		return ObjectInfo;
	}
	else
	{
		return nullptr;
	}
}

void FDeltaCompressionBaselineManager::FreePerObjectInfoForObject(uint32 ObjectIndex)
{
	ObjectInfoIndexType& ObjectInfoIndex = ObjectIndexToObjectInfoIndex.GetData()[ObjectIndex];
	if (ObjectInfoIndex != InvalidObjectInfoIndex)
	{
		FreePerObjectInfo(ObjectInfoIndex);
		ObjectInfoIndex = InvalidObjectInfoIndex;
	}
}

void FDeltaCompressionBaselineManager::ConstructPerObjectInfo(FPerObjectInfo* ObjectInfo) const
{
	ObjectInfo->ChangeMasksForConnections = nullptr;
	ObjectInfo->ObjectIndex = 0;
	ObjectInfo->PrevBaselineCreationFrame = 0;
	ObjectInfo->ChangeMaskStride = 0;
	for (SIZE_T ConnIt = 0, ConnEndIt = MaxConnectionCount; ConnIt != ConnEndIt; ++ConnIt)
	{
		new (&ObjectInfo->BaselinesForConnections[ConnIt]) FObjectBaselineInfo();
	}
}

void FDeltaCompressionBaselineManager::DestructPerObjectInfo(FPerObjectInfo* ObjectInfo)
{
	if (ObjectInfo->ChangeMasksForConnections != nullptr)
	{
		ChangeMaskAllocator.Free(ObjectInfo->ChangeMasksForConnections);
	}

	for (SIZE_T ConnIt = 0, ConnEndIt = MaxConnectionCount; ConnIt != ConnEndIt; ++ConnIt)
	{
		ObjectInfo->BaselinesForConnections[ConnIt].~FObjectBaselineInfo();
	}
}

FDeltaCompressionBaselineManager::ObjectInfoIndexType FDeltaCompressionBaselineManager::AllocPerObjectInfo()
{
	const uint32 ObjectInfoIndex = UsedPerObjectInfos.FindFirstZero();
	if (ObjectInfoIndex != FNetBitArray::InvalidIndex)
	{
		UsedPerObjectInfos.SetBit(ObjectInfoIndex);

		const SIZE_T ObjectInfoStorageIndex = ObjectInfoIndex*BytesPerObjectInfo;
		if (ObjectInfoStorageIndex >= ObjectInfoStorage.Num())
		{
			LLM_SCOPE_BYTAG(Iris);
			ObjectInfoStorage.AddUninitialized(ObjectInfoGrowCount*BytesPerObjectInfo);
		}

		FPerObjectInfo* ObjectInfo = reinterpret_cast<FPerObjectInfo*>(ObjectInfoStorage.GetData() + ObjectInfoStorageIndex);
		ConstructPerObjectInfo(ObjectInfo);

		return static_cast<ObjectInfoIndexType>(ObjectInfoIndex);
	}

	return InvalidObjectInfoIndex;
}

void FDeltaCompressionBaselineManager::FreePerObjectInfo(ObjectInfoIndexType Index)
{
	UsedPerObjectInfos.ClearBit(Index);

	FPerObjectInfo* ObjectInfo = GetPerObjectInfo(Index);
	DestructPerObjectInfo(ObjectInfo);
}

void FDeltaCompressionBaselineManager::FreeAllPerObjectInfos()
{
	auto DestructObjectInfo = [this](uint32 ObjectInfoIndex)
	{
		FPerObjectInfo* ObjectInfo = GetPerObjectInfo(static_cast<ObjectInfoIndexType>(ObjectInfoIndex));
		DestructPerObjectInfo(ObjectInfo);
	};

	UsedPerObjectInfos.ClearBit(InvalidObjectInfoIndex);
	UsedPerObjectInfos.ForAllSetBits(DestructObjectInfo);
}

FDeltaCompressionBaseline FDeltaCompressionBaselineManager::CreateBaseline(uint32 ConnId, uint32 ObjectIndex, uint32 BaselineIndex)
{
	IRIS_PROFILER_SCOPE(FDeltaCompressionBaselineManager_CreateBaseline);
	UE_NET_DC_BASELINE_CHECK(BaselineSharingContext.ObjectInfoIndicesWithNewBaseline.GetNumBits() > 0);
	
	FDeltaCompressionBaseline Baseline;
	if (!DeltaCompressionEnabledObjects.GetBit(ObjectIndex))
	{
		return Baseline;
	}

	const ObjectInfoIndexType ObjectInfoIndex = ObjectIndexToObjectInfoIndex[ObjectIndex];
	if (ObjectInfoIndex == InvalidObjectInfoIndex)
	{
		return Baseline;
	}

	FPerObjectInfo* ObjectInfo = GetPerObjectInfo(ObjectInfoIndex);
	FObjectBaselineInfo& BaselineInfo = ObjectInfo->BaselinesForConnections[ConnId];
	FInternalBaseline& InternalBaseline = BaselineInfo.Baselines[BaselineIndex];
	int16 BaselineCountAdjustment = 0;
	if (InternalBaseline.IsValid())
	{
		--BaselineCountAdjustment;
		ReleaseInternalBaseline(InternalBaseline);
	}

	FDeltaCompressionBaselineStateInfo BaselineStateInfo;
	if (BaselineSharingContext.ObjectInfoIndicesWithNewBaseline.GetBit(ObjectInfoIndex))
	{
		const DeltaCompressionBaselineStateInfoIndexType BaselineStateInfoIndex = BaselineSharingContext.ObjectInfoIndexToBaselineInfoIndex[ObjectInfoIndex];
		BaselineStateInfo = BaselineStorage.GetBaselineReservationForCurrentState(BaselineStateInfoIndex);
	}
	else if (IsAllowedToCreateBaselineForObject(ConnId, ObjectIndex, ObjectInfo, ObjectInfoIndex))
	{
		BaselineStateInfo = BaselineStorage.ReserveBaselineForCurrentState(ObjectIndex);
		if (BaselineStateInfo.IsValid())
		{
			++BaselineSharingContext.CreatedBaselineCount;
			BaselineSharingContext.ObjectInfoIndicesWithNewBaseline.SetBit(ObjectInfoIndex);
			BaselineSharingContext.ObjectInfoIndexToBaselineInfoIndex[ObjectInfoIndex] = BaselineStateInfo.StateInfoIndex;
		}
	}

	if (BaselineStateInfo.IsValid())
	{
		ObjectInfo->PrevBaselineCreationFrame = FrameCounter;

		BaselineStorage.AddRefBaseline(BaselineStateInfo.StateInfoIndex);

		++BaselineCountAdjustment;

		// 
		InternalBaseline.BaselineStateInfoIndex = BaselineStateInfo.StateInfoIndex;
		InternalBaseline.ChangeMask = GetChangeMaskPointerForBaseline(ObjectInfo, ConnId, BaselineIndex);
		
		// Copy connection specific changemask since last baseline and clear it.
		{
			const uint32 ChangeMaskStride = ObjectInfo->ChangeMaskStride;
			const SIZE_T ChangeMaskOffset = ChangeMaskStride*ConnId;
			ChangeMaskStorageType* ConnectionChangeMask = ObjectInfo->ChangeMasksForConnections + ChangeMaskOffset;
			for (uint32 WordIt = 0, WordEndIt = ChangeMaskStride; WordIt != WordEndIt; ++WordIt)
			{
				InternalBaseline.ChangeMask[WordIt] = ConnectionChangeMask[WordIt];
				ConnectionChangeMask[WordIt] = 0;
			}
		}

		// Fill in return value
		Baseline.ChangeMask = InternalBaseline.ChangeMask;
		Baseline.StateBuffer = BaselineStateInfo.StateBuffer;
	}

	// Check for baseline count overflow.
	UE_NET_DC_BASELINE_CHECK(BaselineCountAdjustment <= 0 || (uint16(BaselineCounts[ObjectInfoIndex] + BaselineCountAdjustment) > 0));

	BaselineCounts[ObjectInfoIndex] += BaselineCountAdjustment;
	return Baseline;
}

void FDeltaCompressionBaselineManager::DestroyBaseline(uint32 ConnId, uint32 ObjectIndex, uint32 BaselineIndex)
{
	DestroyBaseline(ConnId, ObjectIndex, BaselineIndex, EChangeMaskBehavior::Discard);
}

void FDeltaCompressionBaselineManager::LostBaseline(uint32 ConnId, uint32 ObjectIndex, uint32 BaselineIndex)
{
	DestroyBaseline(ConnId, ObjectIndex, BaselineIndex, EChangeMaskBehavior::Merge);
}

FDeltaCompressionBaseline FDeltaCompressionBaselineManager::GetBaseline(uint32 ConnId, uint32 ObjectIndex, uint32 BaselineIndex) const
{
	FDeltaCompressionBaseline Baseline;

	if (BaselineIndex == InvalidBaselineIndex)
	{
		return Baseline;
	}

	const FPerObjectInfo* ObjectInfo = GetPerObjectInfoForObject(ObjectIndex);
	if (ObjectInfo == nullptr)
	{
		return Baseline;
	}

	UE_NET_DC_BASELINE_CHECK(BaselineIndex < MaxBaselineCount);

	const FObjectBaselineInfo& BaselineInfo = ObjectInfo->BaselinesForConnections[ConnId];
	const FInternalBaseline& InternalBaseline = BaselineInfo.Baselines[BaselineIndex];

	if (InternalBaseline.IsValid())
	{
		const FDeltaCompressionBaselineStateInfo& BaselineStateInfo = BaselineStorage.GetBaseline(InternalBaseline.BaselineStateInfoIndex);
		Baseline.ChangeMask = InternalBaseline.ChangeMask;
		Baseline.StateBuffer = BaselineStateInfo.StateBuffer;
	}

	return Baseline;
}

void FDeltaCompressionBaselineManager::ReleaseInternalBaseline(FInternalBaseline& InternalBaseline)
{
	BaselineStorage.ReleaseBaseline(InternalBaseline.BaselineStateInfoIndex);
	InternalBaseline.BaselineStateInfoIndex = InvalidBaselineStateInfoIndex;
}

void FDeltaCompressionBaselineManager::AdjustBaselineCount(const FPerObjectInfo* ObjectInfo, ObjectInfoIndexType ObjectInfoIndex, int16 Adjustment)
{
	if (!Adjustment)
	{
		return;
	}

	const uint16 NewBaselineCount = static_cast<uint16>(BaselineCounts[ObjectInfoIndex] + Adjustment);
	UE_NET_DC_BASELINE_CHECK(NewBaselineCount <= BaselineCounts[ObjectInfoIndex]);
	BaselineCounts[ObjectInfoIndex] = NewBaselineCount;

	if (NewBaselineCount == 0)
	{
		// If object is no longer in scope or no longer has DC enabled we can free the associated PerObjectInfo.
		const uint32 ObjectIndex = ObjectInfo->ObjectIndex;
		if (!DeltaCompressionEnabledObjects.GetBit(ObjectIndex))
		{
			FreePerObjectInfo(ObjectInfoIndex);
			ObjectIndexToObjectInfoIndex[ObjectIndex] = InvalidObjectInfoIndex;
		}
	}
}

void FDeltaCompressionBaselineManager::ReleaseBaselinesForConnection(uint32 ConnId)
{
	auto ReleaseObjectBaselines = [this, ConnId](uint32 InObjectInfoIndex)
	{
		const uint16 ObjectInfoIndex = static_cast<uint16>(InObjectInfoIndex);
		FPerObjectInfo* ObjectInfo = GetPerObjectInfo(ObjectInfoIndex);
		FObjectBaselineInfo& BaselineInfo = ObjectInfo->BaselinesForConnections[ConnId];

		int16 BaselineCountAdjustment = 0;
		for (const uint32 BaselineIndex : {0, 1})
		{
			FInternalBaseline& InternalBaseline = BaselineInfo.Baselines[BaselineIndex];
			if (InternalBaseline.IsValid())
			{
				ReleaseInternalBaseline(InternalBaseline);
				--BaselineCountAdjustment;
			}
		}

		AdjustBaselineCount(ObjectInfo, ObjectInfoIndex, BaselineCountAdjustment);
	};

	// The releasing of baselines can cause PerObjectInfos to be released.
	FNetBitArray ConnUsedObjectInfos(UsedPerObjectInfos);
	ConnUsedObjectInfos.ClearBit(InvalidObjectInfoIndex);
	ConnUsedObjectInfos.ForAllSetBits(ReleaseObjectBaselines);

}

void FDeltaCompressionBaselineManager::ConstructBaselineSharingContext(FBaselineSharingContext& SharingContext)
{
	IRIS_PROFILER_SCOPE(FDeltaCompressionBaselineManager_ConstructBaselineSharingContext);

#if UE_SUPPORT_PARALLEL_IRIS
	if (ReplicationSystem)
	{
		ensureMsgf(!ReplicationSystem->IsInParallelPhase(), TEXT("ConstructBaselineSharingContext should never be called from ParallelPhase"));
	}
#endif // UE_SUPPORT_PARALLEL_IRIS

	SharingContext.CreatedBaselineCount = 0;
	SharingContext.ObjectInfoIndicesWithNewBaseline.Init(UsedPerObjectInfos.GetNumBits());
	SharingContext.ObjectInfoIndexToBaselineInfoIndex.SetNumZeroed(UsedPerObjectInfos.GetNumBits());
}

void FDeltaCompressionBaselineManager::DestructBaselineSharingContext(FBaselineSharingContext& SharingContext)
{
	IRIS_PROFILER_SCOPE(FDeltaCompressionBaselineManager_DestructBaselineSharingContext);

#if UE_SUPPORT_PARALLEL_IRIS
	if (ReplicationSystem)
	{
		ensureMsgf(!ReplicationSystem->IsInParallelPhase(), TEXT("DestructBaselineSharingContext should never be called from ParallelPhase"));
	}
#endif // UE_SUPPORT_PARALLEL_IRIS

	auto ReleaseBaseline = [this, SharingContext](uint32 ObjectInfoIndex)
	{
		const DeltaCompressionBaselineStateInfoIndexType BaselineStateInfoIndex = SharingContext.ObjectInfoIndexToBaselineInfoIndex[ObjectInfoIndex];
		BaselineStorage.OptionallyCommitAndDoReleaseBaseline(BaselineStateInfoIndex);
	};

	if (SharingContext.CreatedBaselineCount > 0)
	{
		SharingContext.ObjectInfoIndicesWithNewBaseline.ForAllSetBits(ReleaseBaseline);
	}

	// Reset context
	SharingContext = FBaselineSharingContext();
}

bool FDeltaCompressionBaselineManager::DoesObjectSupportDeltaCompression(uint32 ObjectIndex) const
{
	const FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
	const FReplicationProtocol* Protocol = ReplicatedObjectData.Protocol;
	return EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::SupportsDeltaCompression);
}

bool FDeltaCompressionBaselineManager::IsAllowedToCreateBaselineForObject(uint32 ConnId, uint32 ObjectIndex, const FPerObjectInfo* ObjectInfo, ObjectInfoIndexType InfoIndex) const
{
	// If no baselines are created we allow one to be created.
	if (BaselineCounts[InfoIndex] == 0)
	{
		return true;
	}

	const uint32 FramesSincePrevBaselineCreation = FrameCounter - ObjectInfo->PrevBaselineCreationFrame;
	if ((MinimumNumberOfFramesBetweenBaselines >= 0) && (FramesSincePrevBaselineCreation >= static_cast<uint32>(MinimumNumberOfFramesBetweenBaselines)))
	{
		return true;
	}

	return false;
}

void FDeltaCompressionBaselineManager::InvalidateBaselinesDueToModifiedConditionals()
{
	IRIS_PROFILER_SCOPE(DeltaCompressionBaselineManager_InvalidateBaselinesDueToModifiedConditionals);

	const FNetBitArray& ValidConnections = Connections->GetValidConnections();
	const uint32 MaxConnectionId = ValidConnections.FindLastOne();
	// If there are no connections there are no baselines to invalidate.
	if (MaxConnectionId == ~0U)
	{
		return;
	}

	for (const FDeltaCompressionBaselineInvalidationTracker::FInvalidationInfo& InvalidationInfo : BaselineInvalidationTracker->GetBaselineInvalidationInfos())
	{
		ObjectInfoIndexType ObjectInfoIndex = ObjectIndexToObjectInfoIndex[InvalidationInfo.ObjectIndex];
		if (ObjectInfoIndex == InvalidObjectInfoIndex)
		{
			continue;
		}

		uint16 BaselineCount = BaselineCounts[ObjectInfoIndex];
		if (BaselineCount == 0)
		{
			continue;
		}

		int16 BaselineCountAdjustment = 0;
		FPerObjectInfo* ObjectInfo = GetPerObjectInfo(ObjectInfoIndex);
		if (InvalidationInfo.ConnId == BaselineInvalidationTracker->InvalidateBaselineForAllConnections)
		{
			for (uint32 ConnectionId : ValidConnections)
			{
				FObjectBaselineInfo& BaselineInfo = ObjectInfo->BaselinesForConnections[ConnectionId];
				for (const uint32 BaselineIndex : {0, 1})
				{
					FInternalBaseline& InternalBaseline = BaselineInfo.Baselines[BaselineIndex];
					if (InternalBaseline.IsValid())
					{
						--BaselineCountAdjustment;
						ReleaseInternalBaseline(InternalBaseline);
					}
				}
			}
		}
		else
		{
			if (!ValidConnections.GetBit(InvalidationInfo.ConnId))
			{
				continue;
			}

			FObjectBaselineInfo& BaselineInfo = ObjectInfo->BaselinesForConnections[InvalidationInfo.ConnId];
			for (const uint32 BaselineIndex : {0, 1})
			{
				FInternalBaseline& InternalBaseline = BaselineInfo.Baselines[BaselineIndex];
				if (InternalBaseline.IsValid())
				{
					--BaselineCountAdjustment;
					ReleaseInternalBaseline(InternalBaseline);
				}
			}
		}

		AdjustBaselineCount(ObjectInfo, ObjectInfoIndex, BaselineCountAdjustment);
	}
}

/* Expect DestroyBaseline to be called even for baselines that no longer exist. As we invalidate baselines behind the scenes
 * one is to expect that it no longer exists. As such not even the ObjectInfo may exist anymore. */
void FDeltaCompressionBaselineManager::DestroyBaseline(uint32 ConnId, uint32 ObjectIndex, uint32 BaselineIndex, EChangeMaskBehavior ChangeMaskBehavior)
{
	if (BaselineIndex == InvalidBaselineIndex)
	{
		return;
	}

	const ObjectInfoIndexType ObjectInfoIndex = ObjectIndexToObjectInfoIndex[ObjectIndex];
	if (ObjectInfoIndex == InvalidObjectInfoIndex)
	{
		return;
	}

	FPerObjectInfo* ObjectInfo = GetPerObjectInfo(ObjectInfoIndex);
	if (ObjectInfo == nullptr)
	{
		return;
	}

	UE_NET_DC_BASELINE_CHECK(BaselineIndex < MaxBaselineCount);

	FObjectBaselineInfo& BaselineInfo = ObjectInfo->BaselinesForConnections[ConnId];
	FInternalBaseline& InternalBaseline = BaselineInfo.Baselines[BaselineIndex];
	if (!InternalBaseline.IsValid())
	{
		return;
	}

	// Merge baseline changemask into connection changemask if so desired.
	if (ChangeMaskBehavior == EChangeMaskBehavior::Merge)
	{
		ChangeMaskStorageType* ConnectionChangeMask = GetChangeMaskPointerForConnection(ObjectInfo, ConnId);
		const ChangeMaskStorageType* BaselineChangeMask = InternalBaseline.ChangeMask;
		for (uint32 WordIt = 0, WordEndIt = ObjectInfo->ChangeMaskStride; WordIt != WordEndIt; ++WordIt)
		{
			ConnectionChangeMask[WordIt] |= BaselineChangeMask[WordIt];
		}
	}
	
	// Release the baseline since the connection will have to ask for a new one to be created.
	ReleaseInternalBaseline(InternalBaseline);

	constexpr int16 BaselineCountAdjustment = -1;
	AdjustBaselineCount(ObjectInfo, ObjectInfoIndex, BaselineCountAdjustment);
}

FString FDeltaCompressionBaselineManager::PrintDeltaCompressionStatus(uint32 ConnectionId, FInternalNetRefIndex ObjectIndex) const
{
	if (const FPerObjectInfo* PerObjectInfo = GetPerObjectInfoForObject(ObjectIndex))
	{
		//TODO: Print per-connection info ?
		//FObjectBaselineInfo& BaselineInfo = ObjectInfo->BaselinesForConnections[ConnId];
		return FString::Printf(TEXT("PrevBaselineCreationFrame: %u | ChangeMaskStride: %u"), PerObjectInfo->PrevBaselineCreationFrame, PerObjectInfo->ChangeMaskStride);
	}
	else
	{
		return FString::Printf(TEXT("No delta compression info for %u"), ObjectIndex);
	}
}

#if UE_SUPPORT_PARALLEL_IRIS
void FDeltaCompressionBaselineManager::AcquireBaselineManager(const bool bWriteLock)
{
	IRIS_PROFILER_SCOPE(FDeltaCompressionBaselineManager_AcquireBaselineManager);
	if (bWriteLock)
	{
		DeltaCompressionBaselineManagerMutex.WriteLock();
	}
	else
	{
		DeltaCompressionBaselineManagerMutex.ReadLock();
	}
}

void FDeltaCompressionBaselineManager::ReleaseBaselineManager(const bool bWriteLock)
{
	IRIS_PROFILER_SCOPE(FDeltaCompressionBaselineManager_ReleaseBaselineManager);
	if (bWriteLock)
	{
		DeltaCompressionBaselineManagerMutex.WriteUnlock();
	}
	else
	{
		DeltaCompressionBaselineManagerMutex.ReadUnlock();
	}
}
#endif // UE_SUPPORT_PARALLEL_IRIS

FScopedBaselineAccess::FScopedBaselineAccess(FDeltaCompressionBaselineManager* InBaselineManager, const bool bWriteLock)
{
#if UE_SUPPORT_PARALLEL_IRIS
	check(InBaselineManager);
	InBaselineManager->AcquireBaselineManager(bWriteLock);
	StoredBaselineManager = InBaselineManager;
	bStoredWriteLockType = bWriteLock;
#endif // UE_SUPPORT_PARALLEL_IRIS
}

FScopedBaselineAccess::~FScopedBaselineAccess()
{
#if UE_SUPPORT_PARALLEL_IRIS
	StoredBaselineManager->ReleaseBaselineManager(bStoredWriteLockType);
#endif // UE_SUPPORT_PARALLEL_IRIS
}

}

#undef UE_NET_DC_BASELINE_CHECK
