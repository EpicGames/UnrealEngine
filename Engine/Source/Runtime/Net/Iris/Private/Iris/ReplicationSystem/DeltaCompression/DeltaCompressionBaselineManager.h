// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/IrisConfig.h"
#include "Iris/ReplicationSystem/InternalNetRefIndexManager.h"
#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaseline.h"
#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineStorage.h"
#include "Misc/TransactionallySafeRWLock.h"

class UReplicationSystem;

namespace UE::Net
{
	enum class ENetConnectionTraits : uint32;
	enum class ENetObjectDeltaCompressionStatus : unsigned;
}

namespace UE::Net::Private
{
	struct FChangeMaskCache;
	class FDeltaCompressionBaselineInvalidationTracker;
	class FNetRefHandleManager;
	class FReplicationConnections;
	class FScopedBaselineAccess;
}

namespace UE::Net::Private
{

/** Is the delta compression feature enabled? Togglable via net.Iris.EnableDeltaCompression. */
bool IsDeltaCompressionEnabled();

struct FDeltaCompressionBaselineManagerInitParams
{
	const FDeltaCompressionBaselineInvalidationTracker* BaselineInvalidationTracker = nullptr;
	FReplicationConnections* Connections = nullptr;
	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	FReplicationStateStorage* ReplicationStateStorage = nullptr;
	UReplicationSystem* ReplicationSystem = nullptr;
	FInternalNetRefIndex MaxNetObjectCount = 0;
	FInternalNetRefIndex MaxInternalNetRefIndex = 0;
	uint32 MaxDeltaCompressedObjectCount = 0;
};

struct FDeltaCompressionBaselineManagerPreSendUpdateParams
{
	const FChangeMaskCache* ChangeMaskCache = nullptr;
};

struct FDeltaCompressionBaselineManagerPostSendUpdateParams
{
};

class FDeltaCompressionBaselineManager
{
public:
	enum : uint32 { MaxBaselineCount = 2 };
	enum : uint32 { InvalidBaselineIndex = MaxBaselineCount };
	enum : uint32 { BaselineIndexBitCount = 2 };

	FDeltaCompressionBaselineManager();
	~FDeltaCompressionBaselineManager();

	void Init(FDeltaCompressionBaselineManagerInitParams& InitParams);
	void Deinit();

	/** Called when the maximum InternalNetRefIndex increased and we need to realloc our lists */
	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

	void PreSendUpdate(FDeltaCompressionBaselineManagerPreSendUpdateParams& UpdateParams);
	void PostSendUpdate(FDeltaCompressionBaselineManagerPostSendUpdateParams& UpdateParams);

	void AddConnection(uint32 ConnectionId, ENetConnectionTraits NetConnectionTraits);
	void RemoveConnection(uint32 ConnectionId);

	void SetDeltaCompressionStatus(FInternalNetRefIndex Index, ENetObjectDeltaCompressionStatus Status);
	ENetObjectDeltaCompressionStatus GetDeltaCompressionStatus(FInternalNetRefIndex Index) const;

	uint32 GetMaxDeltaCompressedObjectCount() const;

	/**
	 * Creates a baseline if the policy allows it. May return an invalid baseline. The only guarantee is it will be
	 * valid for the current SendUpdate(). Both events outside of this manager's control and baseline policies, such
	 * as how many may exist in total and per object and how frequent state changes occur, may invalidate the baseline
	 * in the future. If unlucky a state changing conditional is enabled which forces the baseline to be invalidated
	 * as early as the next frame.
	 */
	FDeltaCompressionBaseline CreateBaseline(uint32 ConnId, uint32 ObjectIndex, uint32 BaselineIndex);

	/** Destroys a baseline if it's valid. */
	void DestroyBaseline(uint32 ConnId, uint32 ObjectIndex, uint32 BaselineIndex);
	/**
	 * Destroys a baseline if it's valid but merges the changemask into the connection specific one for this object
	 * so that a future call to GetBaseline() will include at least that changemask.
	 */
	void LostBaseline(uint32 ConnId, uint32 ObjectIndex, uint32 BaselineIndex);

	/**
	 * Returns a valid baseline if it exists, an invalid one if it no longer exists. The manager is free to invalidate
	 * baselines at its own discretion.
	 */
	FDeltaCompressionBaseline GetBaseline(uint32 ConnId, uint32 ObjectIndex, uint32 BaselineIndex) const;

	[[nodiscard]] FString PrintDeltaCompressionStatus(uint32 ConnectionId, FInternalNetRefIndex ObjectIndex) const;

private:
	using ObjectInfoIndexType = uint16;

	enum : unsigned
	{
		InvalidObjectInfoIndex = 0,
		InvalidBaselineStateInfoIndex = 0,

		ObjectInfoGrowCount = 128,
	};

	enum class EChangeMaskBehavior : unsigned
	{
		Discard,
		Merge,
	};

	class FInternalBaseline
	{
	public:
		bool IsValid() const { return BaselineStateInfoIndex != InvalidBaselineStateInfoIndex; }

		ChangeMaskStorageType* ChangeMask = nullptr;
		uint32 BaselineStateInfoIndex = InvalidBaselineStateInfoIndex;
	};

	class FObjectBaselineInfo
	{
	public:
		FInternalBaseline Baselines[2];
	};

	class FPerObjectInfo
	{
	private:
		// Use ConstructPerObjectInfo instead.
		FPerObjectInfo() = delete;
		// Use DestructPerObjectInfo instead.
		~FPerObjectInfo() = delete;

	public:
		// Single pointer for changemasks for all connections.
		ChangeMaskStorageType* ChangeMasksForConnections;

		// ObjectIndex
		uint32 ObjectIndex;

		// Local frame number of when this object last had a baseline created. 
		uint32 PrevBaselineCreationFrame;

		// How many ChangeMaskStorageTypes per change mask.
		uint16 ChangeMaskStride;

		// Baselines for all connections. Array has same size as MaxConnectionCount. This member needs to be last!
		FObjectBaselineInfo BaselinesForConnections[1];

		// No members after BaselinesForConnections!
	};

	struct FBaselineSharingContext
	{
		FNetBitArray ObjectInfoIndicesWithNewBaseline;
		TArray<DeltaCompressionBaselineStateInfoIndexType> ObjectInfoIndexToBaselineInfoIndex;
		uint32 CreatedBaselineCount = 0;
	};

#if UE_SUPPORT_PARALLEL_IRIS
	friend FScopedBaselineAccess;

	// For internal use by the FScopedBaselineAccess to lock DeltaCompressionBaselineManagerMutex
	void AcquireBaselineManager(const bool bWriteLock);
	// For internal use by the FScopedBaselineAccess to unlock DeltaCompressionBaselineManagerMutex
	void ReleaseBaselineManager(const bool bWriteLock);

	FTransactionallySafeRWLock DeltaCompressionBaselineManagerMutex;
#endif // UE_SUPPORT_PARALLEL_IRIS

private:

	void UpdateScope();
	void UpdateDirtyStateMasks(const FChangeMaskCache* ChangeMaskCache);

	void AddObjectToScope(uint32 ObjectIndex);
	void RemoveObjectFromScope(uint32 ObjectIndex);

	FPerObjectInfo* AllocPerObjectInfoForObject(uint32 ObjectIndex);
	void FreePerObjectInfoForObject(uint32 ObjectIndex);
	
	void ConstructPerObjectInfo(FPerObjectInfo*) const;
	void DestructPerObjectInfo(FPerObjectInfo*);

	ObjectInfoIndexType AllocPerObjectInfo();
	void FreePerObjectInfo(ObjectInfoIndexType Index);

	FPerObjectInfo* GetPerObjectInfo(ObjectInfoIndexType Index);
	const FPerObjectInfo* GetPerObjectInfo(ObjectInfoIndexType Index) const;

	FPerObjectInfo* GetPerObjectInfoForObject(uint32 ObjectIndex);
	const FPerObjectInfo* GetPerObjectInfoForObject(uint32 ObjectIndex) const;

	void FreeAllPerObjectInfos();

	void DestroyBaseline(uint32 ConnId, uint32 ObjectIndex, uint32 BaselineIndex, EChangeMaskBehavior ChangeMaskBehavior);

	void ReleaseInternalBaseline(FInternalBaseline& InternalBaseline);

	void AdjustBaselineCount(const FPerObjectInfo* ObjectInfo, ObjectInfoIndexType ObjectInfoIndex, int16 Adjustment);

	void ReleaseBaselinesForConnection(uint32 ConnId);

	void ConstructBaselineSharingContext(FBaselineSharingContext&);
	void DestructBaselineSharingContext(FBaselineSharingContext&);

	bool DoesObjectSupportDeltaCompression(uint32 ObjectIndex) const;

	ChangeMaskStorageType* GetChangeMaskPointerForBaseline(const FPerObjectInfo*, uint32 ConnId, uint32 BaselineIndex) const;
	ChangeMaskStorageType* GetChangeMaskPointerForConnection(const FPerObjectInfo*, uint32 ConnId) const;

	/**
	  * Whether a new baseline may be created for this object this frame based on baseline creation throttling policies.
	  * It's assumed that the object supports delta compression.
	  */
	bool IsAllowedToCreateBaselineForObject(uint32 ConnId, uint32 ObjectIndex, const FPerObjectInfo*, ObjectInfoIndexType InfoIndex) const;

	void InvalidateBaselinesDueToModifiedConditionals();

private:
	FDeltaCompressionBaselineStorage BaselineStorage;

	FNetBitArray DeltaCompressionEnabledObjects;
	FNetBitArray UsedPerObjectInfos;
	TArray<ObjectInfoIndexType> ObjectIndexToObjectInfoIndex;
	TArray<uint8, TAlignedHeapAllocator<alignof(FPerObjectInfo)>> ObjectInfoStorage;
	// This array will only be large enough to hold baseline counts for MaxDeltaCompressedObjectCount baselines.
	// Indexed using ObjectInfoIndex.
	TArray<uint16> BaselineCounts;

	FGlobalChangeMaskAllocator ChangeMaskAllocator;

	FReplicationConnections* Connections = nullptr;
	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	UReplicationSystem* ReplicationSystem = nullptr;
	const FDeltaCompressionBaselineInvalidationTracker* BaselineInvalidationTracker = nullptr;
	uint32 MaxConnectionCount = 0;
	uint32 BytesPerObjectInfo = 0;
	uint32 MaxDeltaCompressedObjectCount = 0;
	uint32 FrameCounter = 0;

	// Only used during send update to prevent creating multiple baselines for the same object.
	FBaselineSharingContext BaselineSharingContext;
};

/**
* A helper RAII class to allow thread-safe access to Baselines
* When making calls to the FDeltaCompressionBaselineManager in a parallel phase, first create FScopedBaselineAccess to
* ensure that the access is threadsafe during the lifetime of the FScopedBaselineAccess object.
*/
class FScopedBaselineAccess
{
public:
	
	explicit FScopedBaselineAccess(FDeltaCompressionBaselineManager* InBaselineManager, const bool bWriteLock);

	~FScopedBaselineAccess();

private:
	FDeltaCompressionBaselineManager* StoredBaselineManager = nullptr;
	bool bStoredWriteLockType = false;
};

inline uint32 FDeltaCompressionBaselineManager::GetMaxDeltaCompressedObjectCount() const
{
	return MaxDeltaCompressedObjectCount;
}

inline FDeltaCompressionBaselineManager::FPerObjectInfo* FDeltaCompressionBaselineManager::GetPerObjectInfo(ObjectInfoIndexType Index)
{
	const SIZE_T ObjectInfoStorageIndex = Index*BytesPerObjectInfo;

	checkSlow(ObjectInfoStorageIndex < (uint32)ObjectInfoStorage.Num());

	uint8* StoragePointer = ObjectInfoStorage.GetData() + ObjectInfoStorageIndex;
	return reinterpret_cast<FPerObjectInfo*>(StoragePointer);
}

inline const FDeltaCompressionBaselineManager::FPerObjectInfo* FDeltaCompressionBaselineManager::GetPerObjectInfo(ObjectInfoIndexType Index) const
{
	const SIZE_T ObjectInfoStorageIndex = Index*BytesPerObjectInfo;

	checkSlow(ObjectInfoStorageIndex < (uint32)ObjectInfoStorage.Num());

	const uint8* StoragePointer = ObjectInfoStorage.GetData() + ObjectInfoStorageIndex;
	return reinterpret_cast<const FPerObjectInfo*>(StoragePointer);
}

inline FDeltaCompressionBaselineManager::FPerObjectInfo* FDeltaCompressionBaselineManager::GetPerObjectInfoForObject(uint32 ObjectIndex)
{
	checkSlow(ObjectIndex != InvalidInternalNetRefIndex);
	checkSlow(ObjectIndex < (uint32)ObjectIndexToObjectInfoIndex.Num());

	const ObjectInfoIndexType InfoIndex = ObjectIndexToObjectInfoIndex[ObjectIndex];
	if (InfoIndex != InvalidObjectInfoIndex)
	{
		return GetPerObjectInfo(InfoIndex);
	}

	return nullptr;
}

inline const FDeltaCompressionBaselineManager::FPerObjectInfo* FDeltaCompressionBaselineManager::GetPerObjectInfoForObject(uint32 ObjectIndex) const
{
	checkSlow(ObjectIndex != InvalidInternalNetRefIndex);
	checkSlow(ObjectIndex < (uint32)ObjectIndexToObjectInfoIndex.Num());

	const ObjectInfoIndexType InfoIndex = ObjectIndexToObjectInfoIndex[ObjectIndex];
	if (InfoIndex != InvalidObjectInfoIndex)
	{
		return GetPerObjectInfo(InfoIndex);
	}

	return nullptr;
}

inline ChangeMaskStorageType* FDeltaCompressionBaselineManager::GetChangeMaskPointerForBaseline(const FPerObjectInfo* ObjectInfo, uint32 ConnId, uint32 BaselineIndex) const
{
	// Layout of storage per object is:
	// ChangeMaskForConnections[0 .. ConnectionCount]
	// ChangeMaskForBaselines[Conn0[0+1] .. ConnConnectionCount[0+1]]
	const uint32 ChangeMaskStride = ObjectInfo->ChangeMaskStride;
	const SIZE_T BaselineChangeMaskOffset = (MaxConnectionCount + 2U*ConnId + BaselineIndex)*ChangeMaskStride;
	ChangeMaskStorageType* ChangeMaskPointer = ObjectInfo->ChangeMasksForConnections + BaselineChangeMaskOffset;
	return ChangeMaskPointer; 
}

inline ChangeMaskStorageType* FDeltaCompressionBaselineManager::GetChangeMaskPointerForConnection(const FPerObjectInfo* ObjectInfo, uint32 ConnId) const
{
	const uint32 ChangeMaskStride = ObjectInfo->ChangeMaskStride;
	const SIZE_T ConnectionChangeMaskOffset = ConnId*ChangeMaskStride;
	ChangeMaskStorageType* ChangeMaskPointer = ObjectInfo->ChangeMasksForConnections + ConnectionChangeMaskOffset;
	return ChangeMaskPointer; 
}

}
