// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineStorage.h"
#include "Iris/IrisConfigInternal.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/ReplicationState/ReplicationStateStorage.h"
#include "HAL/LowLevelMemTracker.h"

#if UE_NET_VALIDATE_DC_BASELINES
#define UE_NET_DC_BASELINE_CHECK(...) check(__VA_ARGS__)
#else
#define UE_NET_DC_BASELINE_CHECK(...) 
#endif

namespace UE::Net::Private
{

FDeltaCompressionBaselineStorage::FDeltaCompressionBaselineStorage()
{
	static_assert(InvalidDeltaCompressionBaselineStateInfoIndex == 0, "This class assumes InvalidDeltaCompressionBaselineStateInfoIndex == 0");
}

FDeltaCompressionBaselineStorage::~FDeltaCompressionBaselineStorage()
{
	checkf(UsedBaselineStateInfos.IsAnyBitSet() == false, TEXT("Leak in FDeltaCompressionBaselineStorage"));
}

void FDeltaCompressionBaselineStorage::Init(FDeltaCompressionBaselineStorageInitParams& InitParams)
{
	UsedBaselineStateInfos.Init(InitParams.MaxBaselineCount + 1U);
	UsedBaselineStateInfos.SetBit(InvalidDeltaCompressionBaselineStateInfoIndex);
	ReplicationStateStorage = InitParams.ReplicationStateStorage;
}

void FDeltaCompressionBaselineStorage::Deinit()
{
	FreeAllBaselineStateInfos();
}

FDeltaCompressionBaselineStateInfo FDeltaCompressionBaselineStorage::CreateBaselineFromCurrentState(uint32 ObjectIndex)
{
	FDeltaCompressionBaselineStateInfo BaselineStateInfo;
	
	const uint32 StateInfoIndex = AllocBaselineStateInfo();
	if (StateInfoIndex == InvalidDeltaCompressionBaselineStateInfoIndex)
	{
		return BaselineStateInfo;
	}

	uint8* StateBuffer = ReplicationStateStorage->AllocBaseline(ObjectIndex, EReplicationStateType::CurrentSendState);
	if (StateBuffer == nullptr)
	{
		FreeBaselineStateInfo(StateInfoIndex);
		return BaselineStateInfo;
	}

	// Fill in internal baseline info
	{
		FInternalBaselineStateInfo* InternalBaselineStateInfo = GetBaselineStateInfo(StateInfoIndex);
		InternalBaselineStateInfo->StateBuffer = StateBuffer;
		InternalBaselineStateInfo->ObjectIndex = ObjectIndex;
	}

	// Fill in return value
	BaselineStateInfo.StateBuffer = StateBuffer;
	BaselineStateInfo.StateInfoIndex = StateInfoIndex;

	return BaselineStateInfo;
}

void FDeltaCompressionBaselineStorage::AddRefBaseline(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex)
{
	UE_NET_DC_BASELINE_CHECK(UsedBaselineStateInfos.GetBit(StateInfoIndex));

	FInternalBaselineStateInfo* InternalBaselineInfo = GetBaselineStateInfo(StateInfoIndex);
	++InternalBaselineInfo->RefCount;
}

void FDeltaCompressionBaselineStorage::ReleaseBaseline(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex)
{
	UE_NET_DC_BASELINE_CHECK(UsedBaselineStateInfos.GetBit(StateInfoIndex));

	FInternalBaselineStateInfo* InternalBaselineInfo = GetBaselineStateInfo(StateInfoIndex);
	UE_NET_DC_BASELINE_CHECK(InternalBaselineInfo->RefCount > 0);
	if (--InternalBaselineInfo->RefCount == 0)
	{
		FreeBaselineStateInfo(StateInfoIndex);
	}
}

FDeltaCompressionBaselineStateInfo FDeltaCompressionBaselineStorage::GetBaseline(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex) const
{
	UE_NET_DC_BASELINE_CHECK(UsedBaselineStateInfos.GetBit(StateInfoIndex));

	const FInternalBaselineStateInfo* InternalBaselineInfo = GetBaselineStateInfo(StateInfoIndex);

	FDeltaCompressionBaselineStateInfo BaselineStateInfo;
	BaselineStateInfo.StateBuffer = InternalBaselineInfo->StateBuffer;
	BaselineStateInfo.StateInfoIndex = StateInfoIndex;

	return BaselineStateInfo;
}

FDeltaCompressionBaselineStateInfo FDeltaCompressionBaselineStorage::ReserveBaselineForCurrentState(uint32 ObjectIndex)
{
	FDeltaCompressionBaselineStateInfo BaselineStateInfo;
	
	const uint32 StateInfoIndex = AllocBaselineStateInfo();
	if (StateInfoIndex == InvalidDeltaCompressionBaselineStateInfoIndex)
	{
		return BaselineStateInfo;
	}

	FReplicationStateStorage::FBaselineReservation Reservation = ReplicationStateStorage->ReserveBaseline(ObjectIndex, EReplicationStateType::CurrentSendState);
	if (!Reservation.IsValid())
	{
		FreeBaselineStateInfo(StateInfoIndex);
		return BaselineStateInfo;
	}

	// Fill in internal baseline info
	{
		FInternalBaselineStateInfo* InternalBaselineStateInfo = GetBaselineStateInfo(StateInfoIndex);
		InternalBaselineStateInfo->StateBuffer = Reservation.ReservedStorage;
		InternalBaselineStateInfo->ObjectIndex = ObjectIndex;
	}

	// Fill in return value
	BaselineStateInfo.StateBuffer = const_cast<uint8*>(Reservation.BaselineBaseStorage);
	BaselineStateInfo.StateInfoIndex = StateInfoIndex;

	return BaselineStateInfo;
}

void FDeltaCompressionBaselineStorage::OptionallyCommitAndDoReleaseBaseline(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex)
{
	UE_NET_DC_BASELINE_CHECK(UsedBaselineStateInfos.GetBit(StateInfoIndex));

	FInternalBaselineStateInfo* InternalBaselineInfo = GetBaselineStateInfo(StateInfoIndex);
	UE_NET_DC_BASELINE_CHECK(InternalBaselineInfo->RefCount > 0);
	if (--InternalBaselineInfo->RefCount == 0)
	{
		ReplicationStateStorage->CancelBaselineReservation(InternalBaselineInfo->ObjectIndex, InternalBaselineInfo->StateBuffer);
		InternalBaselineInfo->StateBuffer = nullptr;

		FreeBaselineStateInfo(StateInfoIndex);
	}
	else
	{
		ReplicationStateStorage->CommitBaselineReservation(InternalBaselineInfo->ObjectIndex, InternalBaselineInfo->StateBuffer, EReplicationStateType::CurrentSendState);
	}
}

FDeltaCompressionBaselineStateInfo FDeltaCompressionBaselineStorage::GetBaselineReservationForCurrentState(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex)
{
	UE_NET_DC_BASELINE_CHECK(UsedBaselineStateInfos.GetBit(StateInfoIndex));

	FInternalBaselineStateInfo* InternalBaselineInfo = GetBaselineStateInfo(StateInfoIndex);
	UE_NET_DC_BASELINE_CHECK(InternalBaselineInfo->RefCount > 0);

	FDeltaCompressionBaselineStateInfo BaselineStateInfo;
	BaselineStateInfo.StateBuffer = const_cast<uint8*>(ReplicationStateStorage->GetState(InternalBaselineInfo->ObjectIndex, EReplicationStateType::CurrentSendState));
	BaselineStateInfo.StateInfoIndex = StateInfoIndex;

	return BaselineStateInfo;
}

void FDeltaCompressionBaselineStorage::ConstructBaselineStateInfo(FInternalBaselineStateInfo* BaselineStateInfo) const
{
	*BaselineStateInfo = FInternalBaselineStateInfo();
}

void FDeltaCompressionBaselineStorage::DestructBaselineStateInfo(FInternalBaselineStateInfo* BaselineStateInfo)
{
	if (BaselineStateInfo->StateBuffer != nullptr)
	{
		ReplicationStateStorage->FreeBaseline(BaselineStateInfo->ObjectIndex, BaselineStateInfo->StateBuffer);
	}
}

DeltaCompressionBaselineStateInfoIndexType FDeltaCompressionBaselineStorage::AllocBaselineStateInfo()
{
	const uint32 StateInfoIndex = UsedBaselineStateInfos.FindFirstZero();
	if (StateInfoIndex != FNetBitArray::InvalidIndex)
	{
		if (StateInfoIndex >= uint32(BaselineStateInfos.Num()))
		{
			BaselineStateInfos.Add(BaselineStateInfoGrowCount);
			UE_NET_DC_BASELINE_CHECK(StateInfoIndex < uint32(BaselineStateInfos.Num()));
		}

		FInternalBaselineStateInfo& BaselineStateInfo = BaselineStateInfos[StateInfoIndex];
		ConstructBaselineStateInfo(&BaselineStateInfo);

		UsedBaselineStateInfos.SetBit(StateInfoIndex);

		return StateInfoIndex;
	}

	return InvalidDeltaCompressionBaselineStateInfoIndex;
}

void FDeltaCompressionBaselineStorage::FreeBaselineStateInfo(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex)
{
	UE_NET_DC_BASELINE_CHECK(UsedBaselineStateInfos.GetBit(StateInfoIndex));

	UsedBaselineStateInfos.ClearBit(StateInfoIndex);

	FInternalBaselineStateInfo* BaselineStateInfo = GetBaselineStateInfo(StateInfoIndex);
	DestructBaselineStateInfo(BaselineStateInfo);
}

void FDeltaCompressionBaselineStorage::FreeAllBaselineStateInfos()
{
	auto DestructStateInfo = [this](uint32 StateInfoIndex)
	{
		FInternalBaselineStateInfo* ObjectInfo = GetBaselineStateInfo(StateInfoIndex);
		DestructBaselineStateInfo(ObjectInfo);
	};

	UsedBaselineStateInfos.ClearBit(InvalidDeltaCompressionBaselineStateInfoIndex);
	UsedBaselineStateInfos.ForAllSetBits(DestructStateInfo);
	UsedBaselineStateInfos.ClearAllBits();
}

}

#undef UE_NET_DC_BASELINE_CHECK
