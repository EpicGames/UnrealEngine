// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ChunkedArray.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaseline.h"

namespace UE::Net
{
	class FReplicationStateStorage;
}

namespace UE::Net::Private
{

using DeltaCompressionBaselineStateInfoIndexType = uint32;

constexpr DeltaCompressionBaselineStateInfoIndexType InvalidDeltaCompressionBaselineStateInfoIndex = 0;

struct FDeltaCompressionBaselineStorageInitParams
{
	FReplicationStateStorage* ReplicationStateStorage = nullptr;
	uint32 MaxBaselineCount = 0;
};

class FDeltaCompressionBaselineStateInfo
{
public:
	bool IsValid() const { return StateBuffer != nullptr; }

	uint8* StateBuffer = nullptr;
	DeltaCompressionBaselineStateInfoIndexType StateInfoIndex = InvalidDeltaCompressionBaselineStateInfoIndex;
};


class FDeltaCompressionBaselineStorage
{
public:
	FDeltaCompressionBaselineStorage();
	~FDeltaCompressionBaselineStorage();

	void Init(FDeltaCompressionBaselineStorageInitParams& InitParams);
	void Deinit();

	FDeltaCompressionBaselineStateInfo CreateBaselineFromCurrentState(uint32 ObjectIndex);
	void AddRefBaseline(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex);
	void ReleaseBaseline(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex);
	/* GetBaseline requires the StateInfoIndex to point to a created baseline, not one that is just reserved. */
	FDeltaCompressionBaselineStateInfo GetBaseline(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex) const;

	/*
	 * Reserves a baseline for the current state. The returned info will contain a pointer to the current state.
	 * OptionallyCommitAndReleaseBaseline needs to be called later which will clone the current state if there are
	 * references after decrementing the ref count. AddRefBaseline and ReleaseBaseline may be called in between
	 * the first call to ReserveBaselineForCurrentState and a call to OptionallyCommitAndDoReleaseBaseline.
	 */
	FDeltaCompressionBaselineStateInfo ReserveBaselineForCurrentState(uint32 ObjectIndex);
	/* Decrements the ref count on the baseline and frees associated storage if it's zero and clones the current state if it's still referenced.  */
	void OptionallyCommitAndDoReleaseBaseline(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex);
	/* When you want to retrieve the info for a reserved baseline when GetBaseline() cannot be used. */
	FDeltaCompressionBaselineStateInfo GetBaselineReservationForCurrentState(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex);


private:
	enum : unsigned
	{
		BaselineStateInfoGrowCount = 256U,
	};

	struct FInternalBaselineStateInfo
	{
		uint8* StateBuffer = nullptr;
		uint32 ObjectIndex = 0;
		uint32 RefCount = 1;
	};

	void ConstructBaselineStateInfo(FInternalBaselineStateInfo* BaselineStateInfo) const;
	void DestructBaselineStateInfo(FInternalBaselineStateInfo* BaselineStateInfo);

	DeltaCompressionBaselineStateInfoIndexType AllocBaselineStateInfo();
	void FreeBaselineStateInfo(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex);
	void FreeAllBaselineStateInfos();

	FInternalBaselineStateInfo* GetBaselineStateInfo(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex);
	const FInternalBaselineStateInfo* GetBaselineStateInfo(DeltaCompressionBaselineStateInfoIndexType StateInfoIndex) const;

private:
	FNetBitArray UsedBaselineStateInfos;
	TChunkedArray<FInternalBaselineStateInfo, BaselineStateInfoGrowCount*sizeof(FInternalBaselineStateInfo)> BaselineStateInfos;
	FReplicationStateStorage* ReplicationStateStorage = nullptr;
};

inline FDeltaCompressionBaselineStorage::FInternalBaselineStateInfo* FDeltaCompressionBaselineStorage::GetBaselineStateInfo(uint32 StateInfoIndex)
{
	return &BaselineStateInfos[StateInfoIndex];
}

inline const FDeltaCompressionBaselineStorage::FInternalBaselineStateInfo* FDeltaCompressionBaselineStorage::GetBaselineStateInfo(uint32 StateInfoIndex) const
{
	return &BaselineStateInfos[StateInfoIndex];
}

}
