// Copyright Epic Games, Inc. All Rights Reserved.

#include "DequantizeAndApplyHelper.h"
#include "Containers/ArrayView.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemStats.h"
#include "Iris/Core/IrisProfiler.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Misc/MemStack.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "UObject/Package.h"

namespace UE::Net::Private
{

static bool bCVarForceFullDequantizeAndApply = false;
static FAutoConsoleVariableRef CVarForceFullDequantizeAndApply(
	TEXT("net.iris.ForceFullDequantizeAndApply"),
	bCVarForceFullDequantizeAndApply,
	TEXT("When enabled a full dequantize of dirty states will be used when applying received statedata regardless of traits set in the fragments."));

struct FDequantizeAndApplyHelper::FContext
{
	struct FStateData
	{
		FReplicationStateApplyContext::FStateBufferData StateBufferData;
		const FReplicationStateDescriptor* Descriptor;
		FReplicationFragment* Fragment;
		EReplicationFragmentTraits FragmentTraits;
		bool bHasUnresolvableReferences;
		bool bMightHaveUnresolvableInitReferences;
	};

	FStateData* CachedStateData;
	FReplicationStateOwnerCollector* OwnerCollector;
	const uint32* UnresolvedReferencesChangeMaskData;
	uint32 CachedStateCount;
	uint32 bNeedsLegacyCallbacks : 1;
	uint32 bIsInitialState : 1;
};

template<typename T>
static void CallLegacyFunctionForEachOwner(const FDequantizeAndApplyHelper::FContext& Context, T&& Functor)
{
	if (Context.OwnerCollector)
	{
		for (UObject* Object : MakeArrayView(Context.OwnerCollector->GetOwners(), Context.OwnerCollector->GetOwnerCount()))
		{
			LLM_SCOPE_BYNAME(TEXT("UObject/NetworkReceive"));
			LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Object->GetPackage(), ELLMTagSet::Assets);
			LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Object->GetClass(), ELLMTagSet::AssetClasses);
			UE_TRACE_METADATA_SCOPE_ASSET(Object, Object->GetClass());
			Functor(Object);
		}
	}
}

FDequantizeAndApplyHelper::FContext* FDequantizeAndApplyHelper::Initialize(FNetSerializationContext& NetSerializationContext, const FDequantizeAndApplyParameters& Parameters)
{
	IRIS_PROFILER_SCOPE(FDequantizeAndApplyHelper_Init);

	FMemStackBase& Allocator = *Parameters.Allocator;

	// Allocate context from temp allocator
	FContext* Context = new (Allocator, MEM_Zeroed) FContext;

	checkSlow(Context);

	const FReplicationInstanceProtocol* InstanceProtocol = Parameters.InstanceProtocol;
	const FReplicationProtocol* Protocol = Parameters.Protocol;
	const uint32* UnresolvedChangeMaskData = Parameters.UnresolvedReferencesChangeMaskData;
	
	// Allocate memory for our temporary cache
	FContext::FStateData* CachedStateData = new (Allocator, MEM_Zeroed) FContext::FStateData[Protocol->ReplicationStateCount];
	uint32 CachedStateCount = 0U;
	
	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	FReplicationFragment* const * Fragments = InstanceProtocol->Fragments;

	// We need to accumulate the offsets and alignment of each internal state
	const uint8* CurrentInternalStateBuffer = Parameters.SrcObjectStateBuffer;

	// We use this to extract the change mask for each state in the protocol
	FNetBitStreamReader ChangeMaskReader;
	ChangeMaskReader.InitBits(Parameters.ChangeMaskData, Protocol->ChangeMaskBitCount);
	uint32 CurrentChangeMaskBitOffset = 0;

	for (uint32 StateIt = 0, StateEndIt = Protocol->ReplicationStateCount; StateIt != StateEndIt; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);

		const uint32 ChangeMaskBitCount = CurrentDescriptor->ChangeMaskBitCount;

		// Unpack change mask
		FNetBitArrayView::StorageWordType* ChangeMaskStorage = new (Allocator) FNetBitArrayView::StorageWordType[FNetBitArrayView::CalculateRequiredWordCount(ChangeMaskBitCount)];
		FNetBitArrayView ChangeMask(ChangeMaskStorage, ChangeMaskBitCount, FNetBitArrayView::NoResetNoValidate);
		ChangeMask.ClearPaddingBits();
		ChangeMaskReader.ReadBitStream(ChangeMaskStorage, ChangeMaskBitCount);

		// Cache all ReplicationStates with dirty changes
		const bool bIsInitState = NetSerializationContext.IsInitState() && CurrentDescriptor->IsInitState();
		const bool bShouldDequantizeState = ChangeMask.IsAnyBitSet() || bIsInitState;
		if (bShouldDequantizeState)
		{
			FContext::FStateData& StateData = CachedStateData[CachedStateCount];
			StateData.Descriptor = CurrentDescriptor;
			StateData.Fragment = Fragments[StateIt];
			StateData.FragmentTraits = StateData.Fragment->GetTraits();

			// Dequantize state data
			if (!EnumHasAnyFlags(StateData.Fragment->GetTraits(), EReplicationFragmentTraits::HasPersistentTargetStateBuffer))
			{
				// allocate buffer for temporary state and construct the external state
				uint8* StateBuffer = (uint8*)Allocator.Alloc(CurrentDescriptor->ExternalSize, CurrentDescriptor->ExternalAlignment);
				CurrentDescriptor->ConstructReplicationState(StateBuffer, CurrentDescriptor);

				// Inject ChangeMask
				FNetBitArrayView DestChangeMask = Private::GetMemberChangeMask(StateBuffer, CurrentDescriptor);
				DestChangeMask.Copy(ChangeMask);
	
				// Dequantize state data, if the fragment supports partial dequantize we will only dequantize dirty members
				const bool bUseFullDequantizeAndApply = bIsInitState || bCVarForceFullDequantizeAndApply || !EnumHasAnyFlags(StateData.Fragment->GetTraits(), EReplicationFragmentTraits::SupportsPartialDequantizedState);
				if (bUseFullDequantizeAndApply)
				{
					FReplicationStateOperations::Dequantize(NetSerializationContext, StateBuffer, (uint8*)CurrentInternalStateBuffer, CurrentDescriptor);
				}
				else
				{
					NetSerializationContext.SetChangeMask(&ChangeMask);
					FReplicationStateOperations::DequantizeWithMask(NetSerializationContext, ChangeMask, 0U, StateBuffer, (uint8*)CurrentInternalStateBuffer, CurrentDescriptor);
				}

				StateData.StateBufferData.ExternalStateBuffer = StateBuffer;
			}
			else
			{
				StateData.StateBufferData.ChangeMaskData = ChangeMaskStorage;
				StateData.StateBufferData.RawStateBuffer = CurrentInternalStateBuffer;
			}

			// Check if we have or might have unresolvable references
			bool bHasUnresolvableReferences = false;
			bool bMightHaveUnresolvableInitReferences = false;
			if (CurrentDescriptor->HasObjectReference())
			{
				if (UnresolvedChangeMaskData)
				{
					const FNetBitArrayView UnresolvedChangeMask = MakeNetBitArrayView(UnresolvedChangeMaskData, Protocol->ChangeMaskBitCount);
					bHasUnresolvableReferences = UnresolvedChangeMask.IsAnyBitSet(CurrentChangeMaskBitOffset, CurrentDescriptor->ChangeMaskBitCount);
				}
				bMightHaveUnresolvableInitReferences = Parameters.bHasUnresolvedInitReferences && NetSerializationContext.IsInitState() && CurrentDescriptor->IsInitState();
			}

			StateData.bHasUnresolvableReferences = bHasUnresolvableReferences;
			StateData.bMightHaveUnresolvableInitReferences = bMightHaveUnresolvableInitReferences;

			++CachedStateCount;
		}

		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
		CurrentChangeMaskBitOffset += CurrentDescriptor->ChangeMaskBitCount;
	}

	Context->CachedStateData = CachedStateData;
	Context->CachedStateCount = CachedStateCount;
	Context->UnresolvedReferencesChangeMaskData = Parameters.UnresolvedReferencesChangeMaskData;
	Context->bNeedsLegacyCallbacks = EnumHasAnyFlags(InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::NeedsLegacyCallbacks);
	Context->bIsInitialState = NetSerializationContext.IsInitState();

	// Collect all states requiring legacy callbacks
	if (Context->bNeedsLegacyCallbacks)
	{
		// Extract owners for all fragments requiring legacy callbacks
		UObject** Owners = (UObject**)Allocator.Alloc(InstanceProtocol->FragmentCount * sizeof(UObject*), alignof(UObject*));
		Context->OwnerCollector = new (Allocator) FReplicationStateOwnerCollector(Owners, InstanceProtocol->FragmentCount);

		for (const FContext::FStateData& StateData : MakeArrayView(CachedStateData, CachedStateCount))
		{
			if (EnumHasAnyFlags(StateData.Fragment->GetTraits(), EReplicationFragmentTraits::NeedsLegacyCallbacks))
			{
				StateData.Fragment->CollectOwner(Context->OwnerCollector);
			}
		}
	}

	return Context;
}

void FDequantizeAndApplyHelper::Deinitialize(FDequantizeAndApplyHelper::FContext* Context)
{
	if (Context)
	{
		for (const FContext::FStateData& StateData : MakeArrayView(Context->CachedStateData, Context->CachedStateCount))
		{
			// Note: We do not use the fragment directly here as there are some code paths that destroys replicated instance & associated Fragments from PostNetReceive/RPC`s
			if (!EnumHasAnyFlags(StateData.FragmentTraits, EReplicationFragmentTraits::HasPersistentTargetStateBuffer))
			{
				StateData.Descriptor->DestructReplicationState(StateData.StateBufferData.ExternalStateBuffer, StateData.Descriptor);
			}
		}
	}
}

void FDequantizeAndApplyHelper::ApplyAndCallLegacyPreApplyFunction(FContext* Context, FNetSerializationContext& NetSerializationContext)
{
	IRIS_PROFILER_SCOPE(FDequantizeAndApplyHelper_Apply);

	checkSlow(Context);

	// Call PreNetReceive (if needed)
	CallLegacyFunctionForEachOwner(*Context, [](UObject* Object) { Object->PreNetReceive(); });

	// Apply the state
	for (const FContext::FStateData& StateData : MakeArrayView(Context->CachedStateData, Context->CachedStateCount))
	{
		FReplicationStateApplyContext SetStateContext;
		SetStateContext.NetSerializationContext = &NetSerializationContext;
		SetStateContext.Descriptor = StateData.Descriptor;
		SetStateContext.StateBufferData = StateData.StateBufferData;
		SetStateContext.bIsInit = Context->bIsInitialState;
		SetStateContext.bHasUnresolvableReferences = StateData.bHasUnresolvableReferences;
		SetStateContext.bMightHaveUnresolvableInitReferences = StateData.bMightHaveUnresolvableInitReferences;

		StateData.Fragment->ApplyReplicatedState(SetStateContext);
	}
}

void FDequantizeAndApplyHelper::ApplyAndCallLegacyFunctions(FContext* Context, FNetSerializationContext& NetSerializationContext)
{
	IRIS_PROFILER_SCOPE(FDequantizeAndApplyHelper_ApplyAndCallLegacyFunctions);

	ApplyAndCallLegacyPreApplyFunction(Context, NetSerializationContext);
	CallLegacyPostApplyFunctions(Context, NetSerializationContext);
}

void FDequantizeAndApplyHelper::CallLegacyPostApplyFunctions(FContext* Context, FNetSerializationContext& NetSerializationContext)
{
	IRIS_PROFILER_SCOPE(FDequantizeAndApplyHelper_CallLegacyPostApplyFunctions);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RepNotifies);

	checkSlow(Context);

	// Call PostNetReceive (if needed)
	CallLegacyFunctionForEachOwner(*Context, [](UObject* Object) { Object->PostNetReceive(); });

	// CallRepNotifies
	if (NetSerializationContext.GetInternalContext()->bAllowExecuteRepNotify)
	{
		if (Context->bNeedsLegacyCallbacks)
		{
			// We only call rep notifies for states that have received any data
			for (const FContext::FStateData& StateData : MakeArrayView(Context->CachedStateData, Context->CachedStateCount))
			{
				if (EnumHasAnyFlags(StateData.Fragment->GetTraits(), EReplicationFragmentTraits::HasRepNotifies))
				{
					FReplicationStateApplyContext SetStateContext;
					SetStateContext.NetSerializationContext = &NetSerializationContext;
					SetStateContext.Descriptor = StateData.Descriptor;
					SetStateContext.StateBufferData = StateData.StateBufferData;
					SetStateContext.bIsInit = Context->bIsInitialState;

					StateData.Fragment->CallRepNotifies(SetStateContext);
				}
			}
		}
	}

	// Call PostRepNotifies
	CallLegacyFunctionForEachOwner(*Context, [](UObject* Object) { Object->PostRepNotifies(); });
}

}
