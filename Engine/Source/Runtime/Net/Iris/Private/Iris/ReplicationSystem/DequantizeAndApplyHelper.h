// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Net/Core/NetBitArray.h"

class FMemStackBase;

namespace UE::Net
{
	class FNetSerializationContext;
	struct FReplicationInstanceProtocol;
	struct FReplicationProtocol;

	struct FDequantizeAndApplyParameters
	{
		FMemStackBase* Allocator = nullptr;
		const uint32* ChangeMaskData = nullptr;
		const uint32* UnresolvedReferencesChangeMaskData = nullptr;
		const uint8* SrcObjectStateBuffer = nullptr;
		const FReplicationInstanceProtocol* InstanceProtocol = nullptr;
		const FReplicationProtocol* Protocol = nullptr;
		bool bHasUnresolvedInitReferences = false;
	};
}

namespace UE::Net::Private
{	

struct FDequantizeAndApplyHelper
{
	struct FContext;

	static FContext* Initialize(FNetSerializationContext& NetSerializationContext, const FDequantizeAndApplyParameters& Parameters);
	static void ApplyAndCallLegacyFunctions(FContext* Context, FNetSerializationContext& NetSerializationContext);
	static void CallLegacyPostApplyFunctions(FContext* Context, FNetSerializationContext& NetSerializationContext);
	static void ApplyAndCallLegacyPreApplyFunction(FContext* Context, FNetSerializationContext& NetSerializationContext);
	static void Deinitialize(FContext* Context);
};

}
