// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirtyNetObjectTracker.h"
#include "Iris/IrisConstants.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"

#include "ProfilingDebugging/CsvProfiler.h"
#include "Traits/IntType.h"

DEFINE_LOG_CATEGORY(LogIrisDirtyTracker)

namespace UE::Net::Private
{

FDirtyNetObjectTracker::FDirtyNetObjectTracker()
: ReplicationSystemId(InvalidReplicationSystemId)
{
}

FDirtyNetObjectTracker::~FDirtyNetObjectTracker()
{
}

void FDirtyNetObjectTracker::Init(const FDirtyNetObjectTrackerInitParams& Params)
{
	check(Params.NetRefHandleManager != nullptr);

	NetRefHandleManager = Params.NetRefHandleManager;
	ReplicationSystemId = Params.ReplicationSystemId;
	
	NetObjectIdCount = Params.MaxInternalNetRefIndex;

	GlobalDirtyTrackerPollHandle = FGlobalDirtyNetObjectTracker::CreatePoller(FGlobalDirtyNetObjectTracker::FPreResetDelegate::CreateRaw(this, &FDirtyNetObjectTracker::ApplyGlobalDirtyObjectList));

	SetNetObjectListsSize(Params.MaxInternalNetRefIndex);

	NetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().AddRaw(this, &FDirtyNetObjectTracker::OnMaxInternalNetRefIndexIncreased);

	AllowExternalAccess();

	UE_LOGF(LogIrisDirtyTracker, Log, "FDirtyNetObjectTracker::Init[%u]: CurrentMaxSize: %u", ReplicationSystemId, NetObjectIdCount);
}

void FDirtyNetObjectTracker::Deinit()
{
	NetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().RemoveAll(this);
	GlobalDirtyTrackerPollHandle.Destroy();
	bShouldResetPolledGlobalDirtyTracker = false;
}

void FDirtyNetObjectTracker::SetNetObjectListsSize(FInternalNetRefIndex NewMaxInternalIndex)
{
	AccumulatedDirtyNetObjects.SetNumBits(NewMaxInternalIndex);
	ForceNetUpdateObjects.SetNumBits(NewMaxInternalIndex);
	DirtyNetObjects.SetNumBits(NewMaxInternalIndex);
}

void FDirtyNetObjectTracker::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	SetNetObjectListsSize(NewMaxInternalIndex);
	NetObjectIdCount = NewMaxInternalIndex;
}

bool FDirtyNetObjectTracker::MarkPushbasedPropertiesDirty(FInternalNetRefIndex ObjectIndex, uint16 OwnerIndex, const FNetBitArrayView& DirtyProperties)
{
	bool bShouldMarkObjectDirty = false;

	// For now, we do not support more than a single push based owner per protocol. JIRA: UE-278338
	if (!ensure(OwnerIndex == 0U))
	{
		return false;
	}

	const FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
	const FReplicationProtocol* Protocol = ReplicatedObjectData.Protocol;
	const FReplicationInstanceProtocol* InstanceProtocol = ReplicatedObjectData.InstanceProtocol;

	// Nothing to poll to or from.
	if (!InstanceProtocol || !NetRefHandleManager->GetReplicatedObjectStateBufferNoCheck(ObjectIndex))
	{
		return false;
	}
	
	if (Protocol->PushModelOwnerCount == 0U)
	{
		return false;
	}

	if (Protocol->PushModelOwnerCount > 1U)
	{
		UE_LOGF(LogIris, Error, "Failed to mark dirty due to too many pushmodel state owners ObjectIndex %d", ObjectIndex);					
		return false;
	}

	const FNetRefHandleManager* LocalNetRefHandleManager = NetRefHandleManager;
	DirtyProperties.ForAllSetBits([ObjectIndex, OwnerIndex, LocalNetRefHandleManager,  &Protocol, &InstanceProtocol, &bShouldMarkObjectDirty](uint32 RepIndex)
		{
			// Dynamic part
			const FReplicationProtocol::FRepIndexToFragmentIndexTable& FragmentIndexTable = Protocol->PushModelOwnerRepIndexToFragmentIndexTable[OwnerIndex];
			if (RepIndex >= FragmentIndexTable.NumEntries)
			{
				UE_LOGF(LogIris, Verbose, "Trying to mark invalid property dirty (Could be a disabled property). Invalid memberindex %ls : RepIndex [%d][%d]", *LocalNetRefHandleManager->PrintObjectFromIndex(ObjectIndex), OwnerIndex, RepIndex);
				return;
			}
			const FReplicationProtocol::FRepIndexToFragmentIndex FragmentIndex = FragmentIndexTable.RepIndexToFragmentIndex[RepIndex];

			if (FragmentIndex.FragmentIndex == FReplicationProtocol::FRepIndexToFragmentIndex::InvalidEntry)
			{
				return;
			}

			const FReplicationInstanceProtocol::FFragmentData& Fragment = InstanceProtocol->FragmentData[FragmentIndex.FragmentIndex];
			const FReplicationStateDescriptor* StateDescriptor = Protocol->ReplicationStateDescriptors[FragmentIndex.FragmentIndex];

			// Does this state contain this property?
			const FReplicationStateMemberRepIndexToMemberIndexDescriptor& RepIndexToMemberIndexDescriptor = StateDescriptor->MemberRepIndexToMemberIndexDescriptors[RepIndex];
			if (RepIndexToMemberIndexDescriptor.MemberIndex == FReplicationStateMemberRepIndexToMemberIndexDescriptor::InvalidEntry)
			{
				UE_LOGF(LogIris, Verbose, "Trying to mark not existing property dirty. Invalid memberindex %ls : RepIndex [%d][%d]", *LocalNetRefHandleManager->PrintObjectFromIndex(ObjectIndex), OwnerIndex, RepIndex);
				return;
			}

			// Skip custom conditionals
			{
				const EReplicationStateTraits Traits = StateDescriptor->Traits;
				if (EnumHasAnyFlags(Traits, EReplicationStateTraits::HasLifetimeConditionals))
				{
					const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskInfo = StateDescriptor->MemberChangeMaskDescriptors[RepIndexToMemberIndexDescriptor.MemberIndex];

					if (ChangeMaskInfo.BitCount > 0)
					{
						FNetBitArrayView MemberConditionalChangeMask = Private::GetMemberConditionalChangeMask(Fragment.ExternalSrcBuffer, StateDescriptor);
						if (!MemberConditionalChangeMask.GetBit(ChangeMaskInfo.BitOffset))
						{
							return;
						}
					}
				}
			}

			// Mark dirty
			FNetBitArrayView DirtyMembersForPolling = GetMemberPollMask(Fragment.ExternalSrcBuffer, StateDescriptor);
			DirtyMembersForPolling.SetBit(RepIndexToMemberIndexDescriptor.MemberIndex);
			bShouldMarkObjectDirty |= true;
		}
	);

	return bShouldMarkObjectDirty;
}

void FDirtyNetObjectTracker::ApplyGlobalDirtyObjectList()
{
	if (FGlobalDirtyNetObjectTracker::IsUsingPerPropertyDirtyTracking())
	{
		if (UReplicationSystem* ReplicationSystem = GetReplicationSystem(ReplicationSystemId))
		{
			FReplicationConditionals& Conditionals = ReplicationSystem->GetReplicationSystemInternal()->GetConditionals();

			const FGlobalDirtyNetObjectTracker::FDirtyHandleAndPropertyMap& GlobalDirtyNetObjects = FGlobalDirtyNetObjectTracker::GetDirtyNetObjectsAndProperties(GlobalDirtyTrackerPollHandle);
			for (const FGlobalDirtyNetObjectTracker::FDirtyHandleAndPropertyMap::ElementType& Element : GlobalDirtyNetObjects)
			{
				const FNetHandle NetHandle = Element.Key;

				const FInternalNetRefIndex NetObjectIndex = NetRefHandleManager->GetInternalIndexFromNetHandle(NetHandle);
				if (NetObjectIndex != InvalidInternalNetRefIndex)
				{
					// Update changemasks based on rep-indices marked dirty
					const FGlobalDirtyNetObjectTracker::FDirtyPropertyStorage& DirtyPropertyData = Element.Value;
					if (DirtyPropertyData.Num())
					{
						const uint32 BitCount = DirtyPropertyData.Num() * sizeof(FGlobalDirtyNetObjectTracker::FDirtyPropertyStorage::ElementType) * 8U;						
						constexpr uint32 OwnerIndex = 0U; // Always 0 as we currently onyly support a single push based fragment owner

						FNetBitArrayView DirtyRepIndices = MakeNetBitArrayView(DirtyPropertyData.GetData(), BitCount, FNetBitArrayView::NoResetNoValidate);
						if (const bool bShouldMarkObjectDirty = MarkPushbasedPropertiesDirty(NetObjectIndex, OwnerIndex, DirtyRepIndices))
						{
							// MarkObjectDirty
							DirtyNetObjects.SetBit(NetObjectIndex);							
						}
					}
					else
					{
						// If it was an explicit dirty call for the object, mark object dirty
						DirtyNetObjects.SetBit(NetObjectIndex);
					}
				}
			}
		}
	}
	else
	{
		const TSet<FNetHandle>& GlobalDirtyNetObjects = FGlobalDirtyNetObjectTracker::GetDirtyNetObjects(GlobalDirtyTrackerPollHandle);
		for (FNetHandle NetHandle : GlobalDirtyNetObjects)
		{
			const FInternalNetRefIndex NetObjectIndex = NetRefHandleManager->GetInternalIndexFromNetHandle(NetHandle);
			if (NetObjectIndex != InvalidInternalNetRefIndex)
			{
				DirtyNetObjects.SetBit(NetObjectIndex);
			}
		}
	}
}

void FDirtyNetObjectTracker::ApplyAndTryResetGlobalDirtyObjectList()
{
	ApplyGlobalDirtyObjectList();

	FGlobalDirtyNetObjectTracker::ResetDirtyNetObjectsIfSinglePoller(GlobalDirtyTrackerPollHandle);

	bShouldResetPolledGlobalDirtyTracker = true;
}

void FDirtyNetObjectTracker::UpdateDirtyNetObjects()
{
	if (!GlobalDirtyTrackerPollHandle.IsValid())
	{
		return;
	}

	IRIS_PROFILER_SCOPE(FDirtyNetObjectTracker_UpdateDirtyNetObjects)

	LockExternalAccess();

	ApplyAndTryResetGlobalDirtyObjectList();

	const uint32 NumWords = AccumulatedDirtyNetObjects.GetNumWords();

	// Note: We use the actual GlobalScopableInternalIndices BitArray as we allow new (sub)objects to be added
	// during the NetUpdate and we do not want them to be removed from the set of dirty objects.
	const FNetBitArrayView GlobalScopeList = NetRefHandleManager->GetGlobalScopableInternalIndices();
	const uint32* GlobalScopeListData = GlobalScopeList.GetDataChecked(NumWords);
	
	uint32* AccumulatedDirtyNetObjectsData = AccumulatedDirtyNetObjects.GetDataChecked(NumWords);
	uint32* DirtyNetObjectsData = DirtyNetObjects.GetDataChecked(NumWords);
		
	for (uint32 WordIndex = 0; WordIndex < NumWords; ++WordIndex)
	{
		// Due to objects having been marked as dirty and later removed we must make sure that all dirty objects are still in scope.
		uint32 DirtyObjectWord = DirtyNetObjectsData[WordIndex] & GlobalScopeListData[WordIndex];
		DirtyNetObjectsData[WordIndex] = DirtyObjectWord;

		// Add the latest dirty objects to the accumulated list and remove no-longer scoped objects that have never been copied.
		AccumulatedDirtyNetObjectsData[WordIndex] = (AccumulatedDirtyNetObjectsData[WordIndex] | DirtyNetObjectsData[WordIndex]) & GlobalScopeListData[WordIndex];
	}

	AllowExternalAccess();
}

void FDirtyNetObjectTracker::UpdateAndLockDirtyNetObjects()
{
	if (!GlobalDirtyTrackerPollHandle.IsValid())
	{
		return;
	}
	
	UpdateDirtyNetObjects();

	FGlobalDirtyNetObjectTracker::LockDirtyListUntilReset(GlobalDirtyTrackerPollHandle);
}

void FDirtyNetObjectTracker::UpdateAccumulatedDirtyList()
{
	IRIS_PROFILER_SCOPE(FDirtyNetObjectTracker_UpdateDirtyNetObjects)
	AccumulatedDirtyNetObjects.Combine(DirtyNetObjects, FNetBitArray::OrOp);
}

void FDirtyNetObjectTracker::MarkNetObjectDirty(FInternalNetRefIndex NetObjectIndex)
{
#if UE_NET_THREAD_SAFETY_CHECK
	checkf(bIsExternalAccessAllowed, TEXT("Cannot mark objects dirty while the bitarray is locked for modifications."));
#endif

	if (NetObjectIndex >= NetObjectIdCount || NetObjectIndex == InvalidInternalNetRefIndex)
	{
		UE_LOGF(LogIrisDirtyTracker, Warning, "FDirtyNetObjectTracker::MarkNetObjectDirty received invalid NetObjectIndex: %u | Max: %u", NetObjectIndex, NetObjectIdCount);
		return;
	}

#if UE_NET_IRIS_CSV_STATS
	PushModelDirtyObjectsCount += (DirtyNetObjects.IsBitSet(NetObjectIndex) ? 0 : 1);
#endif

	const uint32 BitOffset = NetObjectIndex;
	const StorageType BitMask = StorageType(1) << (BitOffset & (StorageTypeBitCount - 1));

	uint32* DirtyNetObjectsData = DirtyNetObjects.GetData();

	DirtyNetObjectsData[BitOffset/StorageTypeBitCount] |= BitMask;

	UE_LOGF(LogIrisDirtyTracker, Verbose, "FDirtyNetObjectTracker::MarkNetObjectDirty[%u]: %ls", ReplicationSystemId, *NetRefHandleManager->PrintObjectFromIndex(NetObjectIndex));
}

void FDirtyNetObjectTracker::ForceNetUpdate(FInternalNetRefIndex NetObjectIndex)
{
#if UE_NET_IRIS_CSV_STATS
	ForceNetUpdateObjectsCount += (ForceNetUpdateObjects.IsBitSet(NetObjectIndex)?0:1);
#endif

	ForceNetUpdateObjects.SetBit(NetObjectIndex);

	// Flag the object dirty so we update filters etc too
	MarkNetObjectDirty(NetObjectIndex);

	UE_LOGF(LogIrisDirtyTracker, Verbose, "FDirtyNetObjectTracker::ForceNetUpdateObjects[%u]: %ls", ReplicationSystemId, *NetRefHandleManager->PrintObjectFromIndex(NetObjectIndex));
}

void FDirtyNetObjectTracker::LockExternalAccess()
{
#if UE_NET_THREAD_SAFETY_CHECK
	bIsExternalAccessAllowed = false;
#endif
}

void FDirtyNetObjectTracker::AllowExternalAccess()
{
#if UE_NET_THREAD_SAFETY_CHECK
	bIsExternalAccessAllowed = true;
#endif
}

FNetBitArrayView FDirtyNetObjectTracker::GetDirtyNetObjectsThisFrame()
{
#if UE_NET_THREAD_SAFETY_CHECK
	checkf(!bIsExternalAccessAllowed, TEXT("Cannot access the DirtyNetObjects bitarray unless its locked for multithread access."));
#endif
	return MakeNetBitArrayView(DirtyNetObjects);
}

void FDirtyNetObjectTracker::ReconcilePolledList(const FNetBitArrayView& ObjectsPolled)
{
	LockExternalAccess();

	if (bShouldResetPolledGlobalDirtyTracker)
	{
		bShouldResetPolledGlobalDirtyTracker = false;
		FGlobalDirtyNetObjectTracker::ResetDirtyNetObjects(GlobalDirtyTrackerPollHandle);
	}

	// Clear ForceNetUpdate from every object that were polled.
	MakeNetBitArrayView(ForceNetUpdateObjects).Combine(ObjectsPolled, FNetBitArrayView::AndNotOp);

	// Clear dirty flags for objects that were polled
	MakeNetBitArrayView(AccumulatedDirtyNetObjects).Combine(ObjectsPolled, FNetBitArrayView::AndNotOp);

	// Clear the current frame dirty objects
	DirtyNetObjects.ClearAllBits();

	AllowExternalAccess();
}

#if UE_NET_IRIS_CSV_STATS
void FDirtyNetObjectTracker::ReportCSVStats()
{
	CSV_CUSTOM_STAT(Iris, PushModelDirtyObjects, PushModelDirtyObjectsCount, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(Iris, ForceNetUpdateObjects, ForceNetUpdateObjectsCount, ECsvCustomStatOp::Set);

	PushModelDirtyObjectsCount = 0;
	ForceNetUpdateObjectsCount = 0;
}
#endif

#pragma region GlobalFunctions

void MarkNetObjectStateDirty(uint32 ReplicationSystemId, FInternalNetRefIndex NetObjectIndex)
{
	if (UReplicationSystem* ReplicationSystem = GetReplicationSystem(ReplicationSystemId))
	{
		FDirtyNetObjectTracker& DirtyNetObjectTracker = ReplicationSystem->GetReplicationSystemInternal()->GetDirtyNetObjectTracker();
		DirtyNetObjectTracker.MarkNetObjectDirty(NetObjectIndex);
	}
}

void ForceNetUpdate(uint32 ReplicationSystemId, FInternalNetRefIndex NetObjectIndex)
{
	if (UReplicationSystem* ReplicationSystem = GetReplicationSystem(ReplicationSystemId))
	{
		FDirtyNetObjectTracker& DirtyNetObjectTracker = ReplicationSystem->GetReplicationSystemInternal()->GetDirtyNetObjectTracker();
		DirtyNetObjectTracker.ForceNetUpdate(NetObjectIndex);
	}
}

#pragma endregion

} // end namespace UE::Net::Private
