// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

#include "Net/Core/NetBitArray.h"
#include "Net/Core/DirtyNetObjectTracker/GlobalDirtyNetObjectTracker.h"

#include "Iris/IrisConfig.h"
#include "Iris/Core/IrisCsv.h"

namespace UE::Net
{
	typedef uint32 FInternalNetRefIndex;
}

namespace UE::Net::Private
{
	class FNetRefHandleManager;
	class FDirtyObjectsAccessor;
}

#ifndef UE_NET_DIRTYOBJECTTRACKER_LOG_COMPILE_VERBOSITY
// Don't compile verbose logs in Shipping builds	
#if UE_BUILD_SHIPPING
#	define UE_NET_DIRTYOBJECTTRACKER_LOG_COMPILE_VERBOSITY Log
#else
#	define UE_NET_DIRTYOBJECTTRACKER_LOG_COMPILE_VERBOSITY All
#endif
#endif

IRISCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogIrisDirtyTracker, Log, UE_NET_DIRTYOBJECTTRACKER_LOG_COMPILE_VERBOSITY);

namespace UE::Net::Private
{

IRISCORE_API void MarkNetObjectStateDirty(uint32 ReplicationSystemId, FInternalNetRefIndex NetObjectIndex);
IRISCORE_API void ForceNetUpdate(uint32 ReplicationSystemId, FInternalNetRefIndex NetObjectIndex);

struct FDirtyNetObjectTrackerInitParams
{
	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	uint32 ReplicationSystemId = 0;
	uint32 MaxInternalNetRefIndex = 0;
};

class FDirtyNetObjectTracker
{
public:
	FDirtyNetObjectTracker();
	~FDirtyNetObjectTracker();

	void Init(const FDirtyNetObjectTrackerInitParams& Params);
	void Deinit();

	/** Returns true if this dirty tracker can be used by the replication system */
	bool IsInit() const { return NetRefHandleManager != nullptr; }

	/** Update dirty objects with the set of globally marked dirty objects. */
	void UpdateDirtyNetObjects();

	/* Update dirty objects from the global list and then prevent future modifications to that list until it is reset. */
	void UpdateAndLockDirtyNetObjects();

	/** Add all the current frame dirty objects set into the accumulated list */
	void UpdateAccumulatedDirtyList();

	/** Set safety permissions so no one can write in the bit array via the public methods */
	void LockExternalAccess();

	/** Release safety permissions and allow to write in the bit array via the public methods */
	void AllowExternalAccess();

	/** Reset the global list and look at the final polled list and clear any flags for objects that got polled */
	void ReconcilePolledList(const FNetBitArrayView& ObjectsPolled);

#if UE_NET_IRIS_CSV_STATS
	void ReportCSVStats();
#endif

	/** Returns the list of objects that are dirty this frame or were dirty in previous frames but not cleaned up at that time. */
	const FNetBitArrayView GetAccumulatedDirtyNetObjects() const { return MakeNetBitArrayView(AccumulatedDirtyNetObjects); }

	/** Returns the list of objects who asked to force a replication this frame */
	FNetBitArrayView GetForceNetUpdateObjects() { return MakeNetBitArrayView(ForceNetUpdateObjects); }
	const FNetBitArrayView GetForceNetUpdateObjects() const { return MakeNetBitArrayView(ForceNetUpdateObjects); }

private:
	friend IRISCORE_API void MarkNetObjectStateDirty(uint32 ReplicationSystemId, FInternalNetRefIndex NetObjectIndex);
	friend IRISCORE_API void ForceNetUpdate(uint32 ReplicationSystemId, FInternalNetRefIndex NetObjectIndex);

	friend FDirtyObjectsAccessor;

	using StorageType = FNetBitArrayView::StorageWordType;
	static constexpr uint32 StorageTypeBitCount = FNetBitArrayView::WordBitCount;

	void SetNetObjectListsSize(FInternalNetRefIndex NewMaxInternalIndex);
	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

	void MarkNetObjectDirty(FInternalNetRefIndex NetObjectIndex);
	void ForceNetUpdate(FInternalNetRefIndex NetObjectIndex);
	void ApplyGlobalDirtyObjectList();

	/**
	 * Applies the dirty state from the global dirty object tracker.
	 * Then if this is the only poller of global dirty state, resets the global dirty state.
	 * If there are multiple pollers of global dirty state (multiple replication systems),
	 * the global state can't be reset until all pollers have gathered it. So we set the
	 * bShouldResetPolledGlobalDirtyTracker flag which will attempt another reset in ReconcilePolledList,
	 * which is called in PostSendUpdate after other pollers have had a chance to gather.
	 */
	void ApplyAndTryResetGlobalDirtyObjectList();

	/** Can only be accessed via FDirtyObjectsAccessor */
	FNetBitArrayView GetDirtyNetObjectsThisFrame();

	/** Propagate properties marked dirty for given object and OwnerIndex */
	bool MarkPushbasedPropertiesDirty(FInternalNetRefIndex ObjectIndex, uint16 OwnerIndex, const FNetBitArrayView& DirtyProperties);

private:

	// Dirty objects that persist across frames.
	FNetBitArray AccumulatedDirtyNetObjects;

    // Objects that want to force a replication this frame
	FNetBitArray ForceNetUpdateObjects;

	// List of objects set to be dirty this frame. Is always reset at the end of the net tick flush
	FNetBitArray DirtyNetObjects;

	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	
	FGlobalDirtyNetObjectTracker::FPollHandle GlobalDirtyTrackerPollHandle;

	uint32 ReplicationSystemId;

	uint32 NetObjectIdCount = 0;
	
	bool bShouldResetPolledGlobalDirtyTracker = false;

#if UE_NET_THREAD_SAFETY_CHECK
	bool bIsExternalAccessAllowed = false;
#endif

#if UE_NET_IRIS_CSV_STATS
	int32 PushModelDirtyObjectsCount = 0;
	int32 ForceNetUpdateObjectsCount = 0;
#endif
};

/**
 * Gives access to the list of dirty objects while detecting non-thread safe access to it.
 */
class FDirtyObjectsAccessor
{
public:
	FDirtyObjectsAccessor(FDirtyNetObjectTracker& InDirtyNetObjectTracker)
		: DirtyNetObjectTracker(InDirtyNetObjectTracker)
	{
		DirtyNetObjectTracker.LockExternalAccess();
	}

	~FDirtyObjectsAccessor()
	{
		DirtyNetObjectTracker.AllowExternalAccess();
	}

	FNetBitArrayView GetDirtyNetObjects()				{ return DirtyNetObjectTracker.GetDirtyNetObjectsThisFrame(); }
	const FNetBitArrayView GetDirtyNetObjects() const	{ return DirtyNetObjectTracker.GetDirtyNetObjectsThisFrame(); }

private:
	FDirtyNetObjectTracker& DirtyNetObjectTracker;
};

}
