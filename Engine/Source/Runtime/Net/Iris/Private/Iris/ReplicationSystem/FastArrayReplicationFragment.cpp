// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/FastArrayReplicationFragment.h"
#include "CoreTypes.h"
#include "Iris/Core/IrisLog.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationState/InternalPropertyReplicationState.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/InternalNetSerializerUtils.h"
#include "Iris/Core/IrisProfiler.h"

namespace UE::Net::Private
{

const FReplicationStateDescriptor* FFastArrayReplicationFragmentBase::GetFastArrayPropertyStructDescriptor() const
{
	const FReplicationStateMemberSerializerDescriptor& FastArrayMemberSerializerDescriptor = ReplicationStateDescriptor->MemberSerializerDescriptors[0];
	const FStructNetSerializerConfig* FastArrayStructSerializerConfig = static_cast<const FStructNetSerializerConfig*>(FastArrayMemberSerializerDescriptor.SerializerConfig);

	return FastArrayStructSerializerConfig->StateDescriptor;
}

FFastArrayReplicationFragmentBase::FFastArrayReplicationFragmentBase(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor, bool bValidateDescriptor)
: FReplicationFragment(InTraits)
, ReplicationStateDescriptor(InDescriptor)
, Owner(InOwner)
{
	// Verify that class descriptor is of expected type
	{
		check(EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::IsFastArrayReplicationState));
		check(!EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::IsNativeFastArrayReplicationState));
	}

	// We do not create temporary states
	Traits |= EReplicationFragmentTraits::HasPersistentTargetStateBuffer;

	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReplicate))
	{
		ReplicationState = MakeUnique<FPropertyReplicationState>(InDescriptor);
	}
	
	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReceive))
	{
		if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasRepNotifies))
		{
			if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::KeepPreviousState))
			{
				check(false);
			}

			Traits |= EReplicationFragmentTraits::HasRepNotifies;
		}

		// For now we always expect pre/post operations for legacy states, we might make this optional
		Traits |= EReplicationFragmentTraits::NeedsLegacyCallbacks;
	}
	
	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReplicate))
	{
		// For PropertyReplicationStates we need to poll properties from our owner in order to detect state changes.
		Traits |= EReplicationFragmentTraits::NeedsPoll;
	}

	// Propagate object reference.
	if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasObjectReference))
	{
		Traits |= EReplicationFragmentTraits::HasObjectReference;
	}

	Traits |= EReplicationFragmentTraits::HasPropertyReplicationState;
	Traits |= EReplicationFragmentTraits::SupportsPartialDequantizedState;

	// Look up the descriptor of the struct
	const FReplicationStateDescriptor* FastArrayStructDescriptor = GetFastArrayPropertyStructDescriptor();

	// And we are looking for the offset of ItemArray which is the relative offset from the FastArray struct
	const uint32 ItemArrayMemberIndex = FFastArrayReplicationFragmentHelper::GetFastArrayStructItemArrayMemberIndex(FastArrayStructDescriptor);
	WrappedArrayOffsetRelativeFastArraySerializerProperty = FastArrayStructDescriptor->MemberDescriptors[ItemArrayMemberIndex].ExternalMemberOffset;

	// Validate the expected member count
	ensureMsgf(!bValidateDescriptor || FastArrayStructDescriptor->MemberCount == 1U, TEXT("A FastArray using the default FastArrayReplicationFragment is expected to have a single replicated array property derived from FastArraySerializerItem"));

#if WITH_PUSH_MODEL
	if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness))
	{
		FFastArraySerializer* FastArraySerializer = reinterpret_cast<FFastArraySerializer*>(reinterpret_cast<uint8*>(InOwner) + InDescriptor->MemberProperties[0]->GetOffset_ForGC());
		FastArraySerializer->CachePushModelState(InOwner, InDescriptor->MemberProperties[0]->RepIndex);
		Traits |= EReplicationFragmentTraits::HasPushBasedDirtiness;
		if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasFullPushBasedDirtiness))
		{
			Traits |= EReplicationFragmentTraits::HasFullPushBasedDirtiness;
		}
	}
#endif
}

FFastArrayReplicationFragmentBase::~FFastArrayReplicationFragmentBase() = default;

const FReplicationStateDescriptor* FFastArrayReplicationFragmentBase::GetArrayElementDescriptor() const
{
	const FReplicationStateDescriptor* FastArrayDescriptor = ReplicationStateDescriptor.GetReference();
	const FReplicationStateDescriptor* FastArrayStructDescriptor = static_cast<const FStructNetSerializerConfig*>(FastArrayDescriptor->MemberSerializerDescriptors[0].SerializerConfig)->StateDescriptor;

	const uint32 ItemArrayMemberIndex = FFastArrayReplicationFragmentHelper::GetFastArrayStructItemArrayMemberIndex(FastArrayStructDescriptor);
	const FReplicationStateDescriptor* ElementStateDescriptor = static_cast<const FArrayPropertyNetSerializerConfig*>(FastArrayStructDescriptor->MemberSerializerDescriptors[ItemArrayMemberIndex].SerializerConfig)->StateDescriptor;

	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];
	
	return static_cast<const FStructNetSerializerConfig*>(ElementSerializerDescriptor.SerializerConfig)->StateDescriptor;
}

void FFastArrayReplicationFragmentBase::InternalCopyArrayElement(const FReplicationStateDescriptor* ArrayElementDescriptor, void* RESTRICT Dst, const void* RESTRICT Src)
{
	InternalCopyStructProperty(ArrayElementDescriptor, Dst, Src);
}

bool FFastArrayReplicationFragmentBase::InternalCompareArrayElement(const FReplicationStateDescriptor* ArrayElementDescriptor, void* RESTRICT Dst, const void* RESTRICT Src)
{
	return InternalCompareStructProperty(ArrayElementDescriptor, Dst, Src);
}

void FFastArrayReplicationFragmentBase::Register(FFragmentRegistrationContext& Context, EReplicationFragmentTraits InTraits)
{
	this->Traits |= InTraits;
	Context.RegisterReplicationFragment(this, ReplicationStateDescriptor.GetReference(), ReplicationState ? ReplicationState->GetStateBuffer() : nullptr);
}

void FFastArrayReplicationFragmentBase::InternalDequantizeFastArray(FNetSerializationContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	const uint32 ArrayMemberIndex = FFastArrayReplicationFragmentHelper::GetFastArrayStructItemArrayMemberIndex(Descriptor);
	check(ArrayMemberIndex < MemberCount);

	const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[ArrayMemberIndex];
	const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[ArrayMemberIndex];
	
	// Dequantize full array
	FNetDequantizeArgs Args;
	Args.Version = 0;
	Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
	Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstExternalBuffer + MemberDescriptor.ExternalMemberOffset);
	Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);	

	MemberSerializerDescriptor.Serializer->Dequantize(Context, Args);
}

void FFastArrayReplicationFragmentBase::InternalPartialDequantizeFastArray(FReplicationStateApplyContext& ApplyContext, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	const uint32 ArrayMemberIndex = FFastArrayReplicationFragmentHelper::GetFastArrayStructItemArrayMemberIndex(Descriptor);
	check(ArrayMemberIndex < MemberCount);

	const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[ArrayMemberIndex];
	const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[ArrayMemberIndex];
	const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = ApplyContext.Descriptor->MemberChangeMaskDescriptors[0];

	// Setup changemask
	FNetBitArrayView MemberChangeMask = MakeNetBitArrayView(ApplyContext.StateBufferData.ChangeMaskData, ApplyContext.Descriptor->ChangeMaskBitCount);
	ApplyContext.NetSerializationContext->SetChangeMask(&MemberChangeMask);

	FNetDequantizeArgs Args;
	Args.Version = 0;
	Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
	Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstExternalBuffer + MemberDescriptor.ExternalMemberOffset);
	Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);

	Args.ChangeMaskInfo.BitCount = MemberChangeMaskDescriptor.BitCount;
	Args.ChangeMaskInfo.BitOffset = MemberChangeMaskDescriptor.BitOffset;

	MemberSerializerDescriptor.Serializer->Dequantize(*ApplyContext.NetSerializationContext, Args);
}

void FFastArrayReplicationFragmentBase::InternalDequantizeExtraProperties(FNetSerializationContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const uint32 ArrayMemberIndex = FFastArrayReplicationFragmentHelper::GetFastArrayStructItemArrayMemberIndex(Descriptor);

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	check(ArrayMemberIndex < MemberCount);

	for (uint32 MemberIndex = 0; MemberIndex < MemberCount; ++MemberIndex)
	{
		if (MemberIndex == ArrayMemberIndex)
		{
			continue;
		}

		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIndex];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIndex];

		FNetDequantizeArgs Args;
		Args.Version = 0;
		Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstExternalBuffer + MemberDescriptor.ExternalMemberOffset);
		Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);	

		MemberSerializerDescriptor.Serializer->Dequantize(Context, Args);
	}
}

void FFastArrayReplicationFragmentBase::CollectOwner(FReplicationStateOwnerCollector* Owners) const
{
	Owners->AddOwner(Owner);
}

void FFastArrayReplicationFragmentBase::ToString(FStringBuilderBase& StringBuilder, const uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FProperty** MemberProperties = Descriptor->MemberProperties;
	const uint32 MemberCount = Descriptor->MemberCount;

	StringBuilder.Appendf(TEXT("FFastArrayReplicationState %s\n"), Descriptor->DebugName->Name);

	FString TempString;
	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FProperty* Property = MemberProperties[MemberIt];

		Property->ExportTextItem_Direct(TempString, StateBuffer + Descriptor->MemberDescriptors[MemberIt].ExternalMemberOffset, nullptr, nullptr, PPF_SimpleObjectText);
		StringBuilder.Appendf(TEXT("%u - %s : %s\n"), MemberIt, *Property->GetName(), ToCStr(TempString));
		TempString.Reset();
	}
}

uint32 FFastArrayReplicationFragmentHelper::GetFastArrayStructItemArrayMemberIndex(const FReplicationStateDescriptor* StructDescriptor)
{
	// We know that one of the members we are looking for is the item array and there will only be one of them.
	uint32 MemberIndex = 0U;
	for (;;)
	{
		if (IsArrayPropertyNetSerializer(StructDescriptor->MemberSerializerDescriptors[MemberIndex].Serializer))
		{
			return MemberIndex;
		}
		++MemberIndex;
	}
}

const FReplicationStateDescriptor* FNativeFastArrayReplicationFragmentBase::GetFastArrayPropertyStructDescriptor() const
{
	const FReplicationStateMemberSerializerDescriptor& FastArrayMemberSerializerDescriptor = ReplicationStateDescriptor->MemberSerializerDescriptors[0];
	const FStructNetSerializerConfig* FastArrayStructSerializerConfig = static_cast<const FStructNetSerializerConfig*>(FastArrayMemberSerializerDescriptor.SerializerConfig);

	return FastArrayStructSerializerConfig->StateDescriptor;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// NativeFastArrayReplicationFragment
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FNativeFastArrayReplicationFragmentBase::FNativeFastArrayReplicationFragmentBase(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor)
: FReplicationFragment(InTraits)
, ReplicationStateDescriptor(InDescriptor)
, Owner(InOwner)
{
	// Verify that class descriptor is of expected type
	{
		check(EnumHasAllFlags(InDescriptor->Traits, EReplicationStateTraits::IsFastArrayReplicationState | EReplicationStateTraits::IsNativeFastArrayReplicationState));
	}

	// We do not create temporary states
	Traits |= EReplicationFragmentTraits::HasPersistentTargetStateBuffer;
	
	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReceive))
	{
		if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasRepNotifies))
		{
			if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::KeepPreviousState))
			{
				check(false);
			}

			Traits |= EReplicationFragmentTraits::HasRepNotifies;
		}

		// For now we always expect pre/post operations for legacy states
		Traits |= EReplicationFragmentTraits::NeedsLegacyCallbacks;
	}
	
	// Propagate object reference.
	if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasObjectReference))
	{
		Traits |= EReplicationFragmentTraits::HasObjectReference;
	}

	// Look up the descriptor of the struct
	const FReplicationStateDescriptor* FastArrayStructDescriptor = GetFastArrayPropertyStructDescriptor();

	// And we are looking for the offset of ItemArray which is the relative offset from the FastArray struct
	const uint32 ItemArrayMemberIndex = FFastArrayReplicationFragmentHelper::GetFastArrayStructItemArrayMemberIndex(FastArrayStructDescriptor);
	WrappedArrayOffsetRelativeFastArraySerializerProperty = FastArrayStructDescriptor->MemberDescriptors[ItemArrayMemberIndex].ExternalMemberOffset;
}

const FReplicationStateDescriptor* FNativeFastArrayReplicationFragmentBase::GetArrayElementDescriptor() const
{
	const FReplicationStateDescriptor* FastArrayDescriptor = ReplicationStateDescriptor.GetReference();
	const FReplicationStateDescriptor* FastArrayStructDescriptor = static_cast<const FStructNetSerializerConfig*>(FastArrayDescriptor->MemberSerializerDescriptors[0].SerializerConfig)->StateDescriptor;

	const uint32 ItemArrayMemberIndex = FFastArrayReplicationFragmentHelper::GetFastArrayStructItemArrayMemberIndex(FastArrayStructDescriptor);
	const FReplicationStateDescriptor* ElementStateDescriptor = static_cast<const FArrayPropertyNetSerializerConfig*>(FastArrayStructDescriptor->MemberSerializerDescriptors[ItemArrayMemberIndex].SerializerConfig)->StateDescriptor;

	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];
	
	return static_cast<const FStructNetSerializerConfig*>(ElementSerializerDescriptor.SerializerConfig)->StateDescriptor;
}

void FNativeFastArrayReplicationFragmentBase::InternalDequantizeFastArray(FNetSerializationContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const uint32 MemberIndex = FFastArrayReplicationFragmentHelper::GetFastArrayStructItemArrayMemberIndex(Descriptor);

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	check(MemberIndex < MemberCount);

	const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIndex];
	const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIndex];

	FNetDequantizeArgs Args;
	Args.Version = 0;
	Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
	Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstExternalBuffer + MemberDescriptor.ExternalMemberOffset);
	Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);	

	MemberSerializerDescriptor.Serializer->Dequantize(Context, Args);
}

void FNativeFastArrayReplicationFragmentBase::ToString(FStringBuilderBase& StringBuilder, const uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FProperty** MemberProperties = Descriptor->MemberProperties;
	const uint32 MemberCount = Descriptor->MemberCount;

	StringBuilder.Appendf(TEXT("FNativeFastArrayReplicationState %s\n"), Descriptor->DebugName->Name);

	FString TempString;
	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FProperty* Property = MemberProperties[MemberIt];

		Property->ExportTextItem_Direct(TempString, StateBuffer + Descriptor->MemberDescriptors[MemberIt].ExternalMemberOffset, nullptr, nullptr, PPF_SimpleObjectText);
		StringBuilder.Appendf(TEXT("%u - %s : %s\n"), MemberIt, *Property->GetName(), ToCStr(TempString));
		TempString.Reset();
	}
}

void FNativeFastArrayReplicationFragmentBase::CollectOwner(FReplicationStateOwnerCollector* Owners) const
{
	Owners->AddOwner(Owner);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FFastArrayReplicationFragmentHelper
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
void FFastArrayReplicationFragmentHelper::InternalApplyArrayElement(const FReplicationStateDescriptor* ArrayElementDescriptor, void* RESTRICT Dst, const void* RESTRICT Src)
{
	InternalApplyStructProperty(ArrayElementDescriptor, Dst, Src);
}

void FFastArrayReplicationFragmentHelper::InternalCopyArrayElement(const FReplicationStateDescriptor* ArrayElementDescriptor, void* RESTRICT Dst, const void* RESTRICT Src)
{
	InternalCopyStructProperty(ArrayElementDescriptor, Dst, Src);
}

bool FFastArrayReplicationFragmentHelper::InternalCompareArrayElement(const FReplicationStateDescriptor* ArrayElementDescriptor, void* RESTRICT Dst, const void* RESTRICT Src)
{
	return InternalCompareStructProperty(ArrayElementDescriptor, Dst, Src);
}

}
