// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Filtering/AlwaysRelevantNetObjectFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AlwaysRelevantNetObjectFilter)

void UAlwaysRelevantNetObjectFilter::OnInit(const FNetObjectFilterInitParams& Params)
{
}

void UAlwaysRelevantNetObjectFilter::OnDeinit()
{
}

bool UAlwaysRelevantNetObjectFilter::AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectFilterAddObjectParams& Params)
{
	return true;
}

void UAlwaysRelevantNetObjectFilter::RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectFilteringInfo& Info)
{
}

void UAlwaysRelevantNetObjectFilter::UpdateObjects(FNetObjectFilterUpdateParams& Params)
{
}

void UAlwaysRelevantNetObjectFilter::Filter(FNetObjectFilteringParams& Params)
{
	// Allow everything
	Params.OutAllowedObjects.SetAllBits();
}

void UAlwaysRelevantNetObjectFilter::OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex)
{
}

