// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFiltering.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetObjectFilter)

namespace UE::Net
{
	const TCHAR* LexToString(ENetFilterStatus Status)
	{
		switch(Status)
		{
			case ENetFilterStatus::Disallow:
			{
				return TEXT("Disallow");
			}
		
			case ENetFilterStatus::Allow:
			{
				return TEXT("Allow");
			}

			default:
			{
				ensure(false);
				return TEXT("Missing");
			}
		}
	}
}

UNetObjectFilter::UNetObjectFilter()
{
}

void UNetObjectFilter::Init(const FNetObjectFilterInitParams& Params)
{
	FilteredObjects.Init(Params.CurrentMaxInternalIndex);

	{
		UE::Net::Private::FNetObjectFilteringInfoAccessor FilteringInfoAccessor;
		FilteringInfos = FilteringInfoAccessor.GetNetObjectFilteringInfos(Params.ReplicationSystem);
	}
	NetRefHandleManager = &Params.ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
	OnInit(Params);
}

void UNetObjectFilter::Deinit()
{
	OnDeinit();

	FilteringInfos = TArrayView<FNetObjectFilteringInfo>();
	NetRefHandleManager = nullptr;
}

void UNetObjectFilter::MaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex MaxInternalIndex, TArrayView<FNetObjectFilteringInfo> NewFilterInfoView)
{
	FilteredObjects.SetNumBits(MaxInternalIndex);
	
	//$IRIS TODO: Move the FilteringInfo somewhere else or pass it via function param only ?
	//      We shouldn't be holding views on arrays we don't own exactly for this reason...
	FilteringInfos = NewFilterInfoView;

	OnMaxInternalNetRefIndexIncreased(MaxInternalIndex);
}

void UNetObjectFilter::AddConnection(uint32 ConnectionId)
{
}

void UNetObjectFilter::RemoveConnection(uint32 ConnectionId)
{
}

void UNetObjectFilter::UpdateObjects(FNetObjectFilterUpdateParams&)
{
}

void UNetObjectFilter::PreFilter(FNetObjectPreFilteringParams&)
{
}

void UNetObjectFilter::Filter(FNetObjectFilteringParams&)
{
}

void UNetObjectFilter::PostFilter(FNetObjectPostFilteringParams&)
{
}

FNetObjectFilteringInfo* UNetObjectFilter::GetFilteringInfo(UE::Net::FInternalNetRefIndex ObjectIndex)
{
	// Only allow retrieving infos for objects handled by this instance.
	if (!IsObjectFiltered(ObjectIndex))
	{
		return nullptr;
	}

	return &FilteringInfos[ObjectIndex];
}

UE::Net::FInternalNetRefIndex UNetObjectFilter::GetObjectIndex(UE::Net::FNetRefHandle NetRefHandle) const
{
	return NetRefHandleManager->GetInternalIndex(NetRefHandle);
}
