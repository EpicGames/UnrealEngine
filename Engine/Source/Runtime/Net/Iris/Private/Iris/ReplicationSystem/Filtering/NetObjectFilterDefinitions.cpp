// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Filtering/NetObjectFilterDefinitions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetObjectFilterDefinitions)

TConstArrayView<FNetObjectFilterDefinition> UNetObjectFilterDefinitions::GetFilterDefinitions() const
{
	return MakeArrayView(NetObjectFilterDefinitions);
}
