// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Filtering/NetObjectGridFilter.h"
#include "Iris/ReplicationSystem/RepTag.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/WorldLocations.h"
#include "Iris/Core/IrisCsv.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetObjectGridFilter)

void UNetObjectGridFilter::OnInit(const FNetObjectFilterInitParams& Params)
{
	AddFilterTraits(ENetFilterTraits::Spatial);

	Config = TStrongObjectPtr<UNetObjectGridFilterConfig>(CastChecked<UNetObjectGridFilterConfig>(Params.Config));
	checkf(Config.IsValid(), TEXT("Need config to operate."));

	AssignedObjectInfoIndices.Init(Params.CurrentMaxInternalIndex);

	PerConnectionInfos.SetNum(Params.MaxConnectionCount + 1);

	MinWorldPos = GetDefault<UWorldLocationsConfig>()->MinPos;
	MaxWorldPos = GetDefault<UWorldLocationsConfig>()->MaxPos;

	MaxNetCullDistance = GetDefault<UWorldLocationsConfig>()->MaxNetCullDistance;
}

void UNetObjectGridFilter::OnDeinit()
{
	Config = nullptr;

	PerConnectionInfos.Empty();
	ObjectInfos.Empty();
	AssignedObjectInfoIndices.Empty();
	
	Cells.Empty();
}

void UNetObjectGridFilter::OnMaxInternalNetRefIndexIncreased(uint32 NewMaxInternalIndex)
{
	AssignedObjectInfoIndices.SetNumBits(NewMaxInternalIndex);
}

void UNetObjectGridFilter::AddConnection(uint32 ConnectionId)
{
	PerConnectionInfos[ConnectionId] = {};
}

void UNetObjectGridFilter::RemoveConnection(uint32 ConnectionId)
{
	PerConnectionInfos[ConnectionId] = {};
}

uint16 UNetObjectGridFilter::GetFrameCountBeforeCulling(FName ProfileName) const
{
	uint16 FrameCount = Config->DefaultFrameCountBeforeCulling;
	
	if (ProfileName.IsNone())
	{
		return FrameCount;
	}

	FNetObjectGridFilterProfile* FilterProfile = Config->FilterProfiles.FindByKey(ProfileName);
	ensureMsgf(FilterProfile, TEXT("UNetObjectGridFilterConfig does not hold any profile named %s"), *ProfileName.ToString());

	if (FilterProfile)
	{
		FrameCount = FilterProfile->FrameCountBeforeCulling;
	}

	return FrameCount;
}

bool UNetObjectGridFilter::AddObject(UE::Net::FInternalNetRefIndex ObjectNetIndex, FNetObjectFilterAddObjectParams& Params)
{
	// We only support objects that registered a world location
	const bool bHasLocation = BuildObjectInfo(ObjectNetIndex, Params);

	if (!bHasLocation)
	{
		return false;
	}

	FObjectLocationInfo& ObjectLocationInfo = static_cast<FObjectLocationInfo&>(Params.OutInfo);
	
	const uint32 InfoIndex = AllocObjectInfo();
	ObjectLocationInfo.SetInfoIndex(InfoIndex);

	FPerObjectInfo& PerObjectInfo = ObjectInfos[InfoIndex];

	PerObjectInfo.ObjectIndex = ObjectNetIndex;
	PerObjectInfo.FrameCountBeforeCulling = GetFrameCountBeforeCulling(Params.ProfileName);

	if (!AddCellInfoForObject(ObjectLocationInfo, Params.InstanceProtocol))
	{
		// Too big an object. We expect it to be costly to move it across cells.
		UE_LOGF(LogIris, Warning, "ReplicatedObject %ls cull distance %f is above the max %f. Object will become always relevant instead", *Params.NetRefIndexManager.PrintObjectFromIndex(ObjectNetIndex), PerObjectInfo.GetCullDistance(), MaxNetCullDistance);
		RemoveObject(ObjectNetIndex, ObjectLocationInfo);
		return false;
	}

	return true;
}

void UNetObjectGridFilter::RemoveObject(UE::Net::FInternalNetRefIndex ObjectNetIndex, const FNetObjectFilteringInfo& Info)
{
	// Remove the object from connection lists
	for (FPerConnectionInfo& ConnectionInfo : PerConnectionInfos)
	{
		ConnectionInfo.RecentObjectFrameCount.Remove(ObjectNetIndex);
	}

	const FObjectLocationInfo& ObjectLocationInfo = static_cast<const FObjectLocationInfo&>(Info);
	RemoveCellInfoForObject(ObjectLocationInfo);

	OnObjectRemoved(ObjectNetIndex);

	const uint32 InfoIndex = ObjectLocationInfo.GetInfoIndex();
	FreeObjectInfo(InfoIndex);
}

void UNetObjectGridFilter::PreFilter(FNetObjectPreFilteringParams&)
{
	++FrameIndex;
#if UE_NET_IRIS_CSV_STATS
	Stats.Reset();
#endif
}

void UNetObjectGridFilter::Filter(FNetObjectFilteringParams& Params)
{
	using namespace UE::Net;

	IRIS_PROFILER_SCOPE(UNetObjectGridFilter_Filter);

	// Update active cells for the connection
	FPerConnectionInfo& ConnectionInfo = PerConnectionInfos[Params.ConnectionId];
	TArray<FCellAndTimestamp, TInlineAllocator<32>> PrevCells = ConnectionInfo.RecentCells;
	ConnectionInfo.RecentCells.Reset();

	// Insert current cells for the views and remove any stale data from the previous cells.
	TArray<FCellAndTimestamp, TInlineAllocator<32>> NewCells;
	for (FReplicationView::FView& View : Params.View.Views)
	{
		FCellAndTimestamp CellAndTimestamp;
		CellAndTimestamp.Timestamp = FrameIndex;
		CalculateCellCoord(CellAndTimestamp.Cell, View.Pos);
		NewCells.AddUnique(CellAndTimestamp);

		for (const FCellAndTimestamp& PrevCell : PrevCells)
		{
			if ((PrevCell.Cell.X == CellAndTimestamp.Cell.X) & (PrevCell.Cell.Y == CellAndTimestamp.Cell.Y))
			{
				PrevCells.RemoveAtSwap(static_cast<int32>(&PrevCell - PrevCells.GetData()));
				break;
			}
		}
	}

	if (!Config->bUseExactCullDistance)
	{
		// Prune old cells and insert still relevant in the new list.
		const uint32 MaxFrameCount = Config->ViewPosRelevancyFrameCount;
		for (const FCellAndTimestamp& PrevCell : PrevCells)
		{
			if ((FrameIndex - PrevCell.Timestamp) > MaxFrameCount)
			{
				continue;
			}

			NewCells.Add(PrevCell);
		}
	}

	// Store new cells
	ConnectionInfo.RecentCells = NewCells;

	// Only allow objects in any of the relevant cells to replicate.
	FNetBitArrayView AllowedObjects = Params.OutAllowedObjects;
	AllowedObjects.ClearAllBits();

	/**
	 * The algorithm will simply iterate over all relevant cells and set the bits
	 * in the AllowedObjects bitarray for all the objects present in the cell. An object can span multiple cells,
	 * but We assume it's cheaper to modify the bitarray than to create a new set with the unique
	 * object indices and then iterate over that. In both cases one do need to iterate over all
	 * objects in all relevant cells anyway and setting a bit should be faster than inserting into a set.
	 */

	if (Config->bUseExactCullDistance)
	{
#if UE_NET_IRIS_CSV_STATS
		const uint64 StartTimeInCycles = FPlatformTime::Cycles64();
		uint32 CullTestedObjects = 0;
#endif
		for (const FCellAndTimestamp& CellAndTimestamp : NewCells)
		{
			if (FCellObjects* Objects = Cells.Find(CellAndTimestamp.Cell))
			{
				for (const UE::Net::FInternalNetRefIndex ObjectIndex : Objects->ObjectIndices)
				{
					const FObjectLocationInfo& ObjectLocationInfo = static_cast<const FObjectLocationInfo&>(Params.FilteringInfos[ObjectIndex]);
					const FPerObjectInfo& PerObjectInfo = ObjectInfos[ObjectLocationInfo.GetInfoIndex()];

					for (const FReplicationView::FView& View : Params.View.Views)
					{
						const double DistSq = PerObjectInfo.GetCullDistanceSq();
						const double ObjectToViewDistSq = FVector::DistSquared(PerObjectInfo.Position, View.Pos);

						if (ObjectToViewDistSq <= DistSq)
						{
							ConnectionInfo.RecentObjectFrameCount.Add(ObjectIndex, PerObjectInfo.FrameCountBeforeCulling);
							break; // Don't need to test all connections once one is within Distance
						}
					}
				}
#if UE_NET_IRIS_CSV_STATS
				Stats.CullTestedObjects += Objects->ObjectIndices.Num();
#endif
			}
		}

#if UE_NET_IRIS_CSV_STATS
		Stats.CullTestingTimeInCycles += (FPlatformTime::Cycles64() - StartTimeInCycles);
#endif

		// Set the AllowedObjects and decrease their frame count
		for (TMap<uint32, uint16>::TIterator It = ConnectionInfo.RecentObjectFrameCount.CreateIterator(); It; ++It)
		{
			if (It->Value > 0)
			{
				It->Value--;
				AllowedObjects.SetBit(It->Key);
			}
			else
			{
				It.RemoveCurrent();
			}
		}
	}
	else
	{
		for (const FCellAndTimestamp& CellAndTimestamp : NewCells)
		{
			if (FCellObjects* Objects = Cells.Find(CellAndTimestamp.Cell))
			{
				for (const uint32 ObjectIndex : Objects->ObjectIndices)
				{
					AllowedObjects.SetBit(ObjectIndex);
				}
			}
		}
	}
}

void UNetObjectGridFilter::PostFilter(FNetObjectPostFilteringParams&)
{
#if UE_NET_IRIS_CSV_STATS
	CSV_CUSTOM_STAT(Iris, CullTestingTimeInMS, FPlatformTime::ToMilliseconds64(Stats.CullTestingTimeInCycles), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(Iris, CullTestedObjectsCount, (int32)Stats.CullTestedObjects, ECsvCustomStatOp::Set);
#endif
}

uint32 UNetObjectGridFilter::AllocObjectInfo()
{
	uint32 Index = AssignedObjectInfoIndices.FindFirstZero();
	if (Index >= uint32(ObjectInfos.Num()))
	{
		constexpr int32 NumElementsPerChunk = ObjectInfosChunkSize/sizeof(FPerObjectInfo);
		ObjectInfos.Add(NumElementsPerChunk);
	}

	AssignedObjectInfoIndices.SetBit(Index);
	return Index;
}

void UNetObjectGridFilter::FreeObjectInfo(uint32 Index)
{
	AssignedObjectInfoIndices.ClearBit(Index);
	ObjectInfos[Index] = FPerObjectInfo();
}

bool UNetObjectGridFilter::AddCellInfoForObject(const FObjectLocationInfo& ObjectLocationInfo, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol)
{
	// Called for completely new objects
	FPerObjectInfo& PerObjectInfo = ObjectInfos[ObjectLocationInfo.GetInfoIndex()];
	
	// Set cull distance
	{
		PerObjectInfo.SetCullDistance(Config->DefaultCullDistance);
		UpdatePositionAndCullDistance(ObjectLocationInfo, PerObjectInfo, InstanceProtocol);
	}

	if (MaxNetCullDistance > 0.0f && PerObjectInfo.GetCullDistance() > MaxNetCullDistance)
	{
		return false;
	}

	FCellBox NewCellBox;

	// Set cellbox info
	{
		CalculateCellBox(PerObjectInfo, NewCellBox);
		PerObjectInfo.CellBox = NewCellBox;
	}

	const UE::Net::FInternalNetRefIndex ObjectIndex = PerObjectInfo.ObjectIndex;

	// Add object to new cells.
	{
		FCellCoord Coord;
		for (int32 X = NewCellBox.MinX, EndX = NewCellBox.MaxX + 1; X < EndX; ++X)
		{
			Coord.X = X;
			for (int32 Y = NewCellBox.MinY, EndY = NewCellBox.MaxY + 1; Y < EndY; ++Y)
			{
				Coord.Y = Y;
				FCellObjects& Cell = Cells.FindOrAdd(Coord);
				Cell.ObjectIndices.Add(ObjectIndex);
			}
		}
	}

	return true;
}

void UNetObjectGridFilter::RemoveCellInfoForObject(const FObjectLocationInfo& ObjectLocationInfo)
{
	FPerObjectInfo& PerObjectInfo = ObjectInfos[ObjectLocationInfo.GetInfoIndex()];

	const FCellBox CellBox = PerObjectInfo.CellBox;
	const UE::Net::FInternalNetRefIndex ObjectIndex = PerObjectInfo.ObjectIndex;

	// Remove object from cells.
	{
		FCellCoord Coord;
		for (int32 X = CellBox.MinX, EndX = CellBox.MaxX + 1; X < EndX; ++X)
		{
			Coord.X = X;
			for (int32 Y = CellBox.MinY, EndY = CellBox.MaxY + 1; Y < EndY; ++Y)
			{
				Coord.Y = Y;
				FCellObjects* Cell = Cells.Find(Coord);
				checkSlow(Cell != nullptr);
				if (Cell)
				{
					Cell->ObjectIndices.Remove(ObjectIndex);
				}
			}
		}
	}
}

void UNetObjectGridFilter::UpdateCellInfoForObject(const FObjectLocationInfo& ObjectLocationInfo, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol)
{
	FPerObjectInfo& PerObjectInfo = ObjectInfos[ObjectLocationInfo.GetInfoIndex()];

	UpdatePositionAndCullDistance(ObjectLocationInfo, PerObjectInfo, InstanceProtocol);

	const FCellBox PrevCellBox = PerObjectInfo.CellBox;
	FCellBox NewCellBox;
	CalculateCellBox(PerObjectInfo, NewCellBox);

	if (NewCellBox != PrevCellBox)
	{
		const UE::Net::FInternalNetRefIndex ObjectIndex = PerObjectInfo.ObjectIndex;
		PerObjectInfo.CellBox = NewCellBox;

		// Special case for disjoint cells
		if (AreCellsDisjoint(NewCellBox, PrevCellBox))
		{
			// Remove object from previous cells.
			{
				FCellCoord Coord;
				for (int32 X = PrevCellBox.MinX, EndX = PrevCellBox.MaxX + 1; X < EndX; ++X)
				{
					Coord.X = X;
					for (int32 Y = PrevCellBox.MinY, EndY = PrevCellBox.MaxY + 1; Y < EndY; ++Y)
					{
						Coord.Y = Y;
						FCellObjects* Cell = Cells.Find(Coord);
						checkSlow(Cell != nullptr);
						Cell->ObjectIndices.Remove(ObjectIndex);
					}
				}
			}

			// Add object to new cells.
			{
				FCellCoord Coord;
				for (int32 X = NewCellBox.MinX, EndX = NewCellBox.MaxX + 1; X < EndX; ++X)
				{
					Coord.X = X;
					for (int32 Y = NewCellBox.MinY, EndY = NewCellBox.MaxY + 1; Y < EndY; ++Y)
					{
						Coord.Y = Y;

						// Temporarily cache the function state for debugging purposes.
						DebugUpdateCellInfo.Coord = Coord;
						DebugUpdateCellInfo.NewCellBox = NewCellBox;
						DebugUpdateCellInfo.PrevCellBox = PrevCellBox;
						DebugUpdateCellInfo.CellsSize = Cells.Num();
						DebugUpdateCellInfo.ObjectPosition = PerObjectInfo.Position;
						DebugUpdateCellInfo.ObjectCullDistance = PerObjectInfo.GetCullDistance();

						FCellObjects& Cell = Cells.FindOrAdd(Coord);
						Cell.ObjectIndices.Add(ObjectIndex);
					}
				}
			}
		}
		// There's some overlap
		else
		{
			/**
			 * Naive algorithm to find out which cells to update.
			 * 
			 * 1. Calculate the cell overlap.
			 * 2. For each coord exclusive to the previous cell we will remove the object.
			 * 3. For each coord exclusive to the new cell we will add the object.
			 */
			FCellBox Overlap;
			Overlap.MinX = FPlatformMath::Max(NewCellBox.MinX, PrevCellBox.MinX);
			Overlap.MaxX = FPlatformMath::Min(NewCellBox.MaxX, PrevCellBox.MaxX);
			Overlap.MinY = FPlatformMath::Max(NewCellBox.MinY, PrevCellBox.MinY);
			Overlap.MaxY = FPlatformMath::Min(NewCellBox.MaxY, PrevCellBox.MaxY);
			
			// Remove object from previous cells.
			{
				FCellCoord Coord;
				for (int32 X = PrevCellBox.MinX, EndX = PrevCellBox.MaxX + 1; X < EndX; ++X)
				{
					Coord.X = X;
					for (int32 Y = PrevCellBox.MinY, EndY = PrevCellBox.MaxY + 1; Y < EndY; ++Y)
					{
						Coord.Y = Y;

						if (DoesCellContainCoord(Overlap, Coord))
						{
							continue;
						}

						FCellObjects* Cell = Cells.Find(Coord);
						checkSlow(Cell != nullptr);
						Cell->ObjectIndices.Remove(ObjectIndex);
					}
				}
			}

			// Add object to new cells.
			{
				FCellCoord Coord;
				for (int32 X = NewCellBox.MinX, EndX = NewCellBox.MaxX + 1; X < EndX; ++X)
				{
					Coord.X = X;
					for (int32 Y = NewCellBox.MinY, EndY = NewCellBox.MaxY + 1; Y < EndY; ++Y)
					{
						Coord.Y = Y;

						if (DoesCellContainCoord(Overlap, Coord))
						{
							continue;
						}

						// Temporarily cache the function state for debugging purposes.
						DebugUpdateCellInfo.Coord = Coord;
						DebugUpdateCellInfo.NewCellBox = NewCellBox;
						DebugUpdateCellInfo.PrevCellBox = PrevCellBox;
						DebugUpdateCellInfo.CellsSize = Cells.Num();
						DebugUpdateCellInfo.ObjectPosition = PerObjectInfo.Position;
						DebugUpdateCellInfo.ObjectCullDistance = PerObjectInfo.GetCullDistance();

						FCellObjects& Cell = Cells.FindOrAdd(Coord);
						Cell.ObjectIndices.Add(ObjectIndex);
					}
				}
			}
		}
	}
}

void UNetObjectGridFilter::UpdatePositionAndCullDistance(const UNetObjectGridFilter::FObjectLocationInfo& ObjectLocationInfo, UNetObjectGridFilter::FPerObjectInfo& PerObjectInfo, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol)
{
	UpdateObjectInfo(PerObjectInfo, ObjectLocationInfo, InstanceProtocol);
}

void UNetObjectGridFilter::CalculateCellBox(const UNetObjectGridFilter::FPerObjectInfo& PerObjectInfo, UNetObjectGridFilter::FCellBox& OutCellBox)
{
	// Clamp culldistance to sane value if MaxNetCullDistance is set.
	const double CullDistance = FMath::Min(FMath::Max(PerObjectInfo.GetCullDistance(), MaxNetCullDistance), PerObjectInfo.GetCullDistance());
	const FVector Position = PerObjectInfo.Position;

	ensureMsgf(!Position.ContainsNaN(), TEXT("Calculating grid cell box based off a position with a NaN for object: %s"), ToCStr(GetNetRefIndexManager().PrintObjectFromIndex(PerObjectInfo.ObjectIndex)));

	FVector MinPosition = Position - CullDistance;
	FVector MaxPosition = Position + CullDistance;

	// Clamp cells to the worldsize.
	MinPosition = MinPosition.ComponentMax(MinWorldPos);
	MaxPosition = MaxPosition.ComponentMin(MaxWorldPos);

	const int64 MinX = FPlatformMath::FloorToInt(MinPosition.X/Config->CellSizeX);
	const int64 MinY = FPlatformMath::FloorToInt(MinPosition.Y/Config->CellSizeY);
	const int64 MaxX = FPlatformMath::FloorToInt(MaxPosition.X/Config->CellSizeX);
	const int64 MaxY = FPlatformMath::FloorToInt(MaxPosition.Y/Config->CellSizeY);

	FCellBox CellBox;
	CellBox.MinX = static_cast<int32>(MinX);
	CellBox.MinY = static_cast<int32>(MinY);
	CellBox.MaxX = static_cast<int32>(MaxX);
	CellBox.MaxY = static_cast<int32>(MaxY);

	// Current large world max of 8796093022208.0 requires a cell size of at least around 4500 to not overflow an int32. 
	checkSlow(MinX == CellBox.MinX && MinY == CellBox.MinY && MaxX == CellBox.MaxX && MaxY == CellBox.MaxY);

	OutCellBox = CellBox;
}

void UNetObjectGridFilter::CalculateCellCoord(UNetObjectGridFilter::FCellCoord& OutCoord, const FVector& Pos)
{
	FCellCoord Coord;
	Coord.X = static_cast<int32>(FPlatformMath::FloorToInt(Pos.X/Config->CellSizeX));
	Coord.Y = static_cast<int32>(FPlatformMath::FloorToInt(Pos.Y/Config->CellSizeY));

	OutCoord = Coord;
}

bool UNetObjectGridFilter::AreCellsDisjoint(const UNetObjectGridFilter::FCellBox& A, const UNetObjectGridFilter::FCellBox& B)
{
	return (A.MinX > B.MaxX) | (A.MaxX < B.MinX) | (A.MinY > B.MaxY) | (A.MaxY < B.MinY);
}

bool UNetObjectGridFilter::DoesCellContainCoord(const UNetObjectGridFilter::FCellBox& Cell, const UNetObjectGridFilter::FCellCoord& Coord)
{
	return (Coord.X >= Cell.MinX) & (Coord.X <= Cell.MaxX) & (Coord.Y >= Cell.MinY) & (Coord.Y <= Cell.MaxY);
}

FString UNetObjectGridFilter::PrintDebugInfoForObject(const FDebugInfoParams& Params, uint32 ObjectIndex) const
{
	using namespace UE::Net;

	const FObjectLocationInfo& ObjectLocationInfo = static_cast<const FObjectLocationInfo&>(Params.FilteringInfos[ObjectIndex]);

	const FPerObjectInfo& PerObjectInfo = ObjectInfos[ObjectLocationInfo.GetInfoIndex()];

	const FPerConnectionInfo* const PerConnectionInfo = PerConnectionInfos.IsValidIndex(Params.ConnectionId) ? &PerConnectionInfos[Params.ConnectionId] : nullptr;
	const uint16* CulledFrameCountPtr = PerConnectionInfo ? PerConnectionInfo->RecentObjectFrameCount.Find(ObjectIndex) : nullptr;
	const uint16 CulledFrameCount = CulledFrameCountPtr ? *CulledFrameCountPtr : 0;

	double Dist2d = DOUBLE_BIG_NUMBER;
	double DistZ = DOUBLE_BIG_NUMBER;
	
	for (const FReplicationView::FView& View : Params.View.Views)
	{
		Dist2d = FMath::Min(FVector::Dist2D(PerObjectInfo.Position, View.Pos), Dist2d);
		DistZ = FMath::Min(FMath::Abs(PerObjectInfo.Position.Z - View.Pos.Z), DistZ);
	}
	
	return FString::Printf(TEXT("[GridFilter] Dist2D: %.2f, CullDistance: %.2f, CulledFrameCount: %u, DistZ: %.2f, Pos: %s"),
		Dist2d, PerObjectInfo.GetCullDistance(), CulledFrameCount, DistZ, *PerObjectInfo.Position.ToCompactString()
	);
}


//*************************************************************************************************
// UNetObjectGridWorldLocFilter
//*************************************************************************************************

void UNetObjectGridWorldLocFilter::OnInit(const FNetObjectFilterInitParams& Params)
{
	Super::OnInit(Params);

	WorldLocations = &Params.ReplicationSystem->GetWorldLocations();
}

void UNetObjectGridWorldLocFilter::OnDeinit()
{
	WorldLocations = nullptr;

	Super::OnDeinit();
}

void UNetObjectGridWorldLocFilter::PreFilter(FNetObjectPreFilteringParams& Params)
{
	Super::PreFilter(Params);

	// Update logic performed here in order to not rely on any object being dirtied.
	auto UpdateCells = [this, &Params](uint32 ObjectIndex)
		{
			const FObjectLocationInfo& ObjectLocationInfo = static_cast<const FObjectLocationInfo&>(Params.FilteringInfos[ObjectIndex]);
			this->UpdateCellInfoForObject(ObjectLocationInfo, static_cast<const UE::Net::FReplicationInstanceProtocol*>(nullptr));
		};

	// Update cell info for all objects that have moved.
	UE::Net::FNetBitArrayView ObjectsWithDirtyWorldLocations = WorldLocations->GetObjectsWithDirtyInfo();
	UE::Net::FNetBitArrayView::ForAllSetBits(GetFilteredObjects(), ObjectsWithDirtyWorldLocations, UE::Net::FNetBitArrayBase::AndOp, UpdateCells);
}

void UNetObjectGridWorldLocFilter::UpdateObjectInfo(UNetObjectGridFilter::FPerObjectInfo& PerObjectInfo, const UNetObjectGridFilter::FObjectLocationInfo& ObjectLocationInfo, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol)
{
	const TOptional<UE::Net::FWorldLocations::FWorldInfo> Info = WorldLocations->GetWorldInfo(PerObjectInfo.ObjectIndex);
	
	if (Info)
	{
		PerObjectInfo.Position = Info->WorldLocation;
		PerObjectInfo.SetCullDistance(Info->CullDistance);
	}
}

bool UNetObjectGridWorldLocFilter::BuildObjectInfo(uint32 ObjectIndex, FNetObjectFilterAddObjectParams& Params)
{
	checkf(WorldLocations->HasInfoForObject(ObjectIndex), TEXT("UNetObjectGridWorldLocFilter only supports objects with valid world location data."));

	return true;
}

