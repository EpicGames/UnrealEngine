// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetObjectGroups.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/IConsoleManager.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Containers/ArrayView.h"

DEFINE_LOG_CATEGORY_STATIC(LogIrisGroup, Log, All)

namespace UE::Net::Private
{

	namespace NetObjectGroupsInternal
	{
		static FNetObjectGroupHandle::FGroupIndexType NextEpoch = 1U;
	}

static int32 CVarEnsureIfNumGroupMembershipsExceedsNum = 128;
static FAutoConsoleVariableRef CVarEnsureIfNumGroupMembershipsExceeds(TEXT("net.Iris.EnsureIfNumGroupMembershipsExceeds"), CVarEnsureIfNumGroupMembershipsExceedsNum, TEXT("If set to a positive number we will warn and ensure if an object is added to a high number of groups."), ECVF_Default );

FNetObjectGroups::FNetObjectGroups()
: CurrentEpoch((++NetObjectGroupsInternal::NextEpoch) & FNetObjectGroupHandle::EpochMask)
{
	if (CurrentEpoch == 0U)
	{
		UE_LOGF(LogIris, Warning, "FNetObjectGroups::Epoch wraparound detected.");
	}
}

FNetObjectGroups::~FNetObjectGroups()
{
}

bool FNetObjectGroups::IsMemberOf(const FNetObjectGroupMembership& Target, FNetObjectGroupHandle Group)
{
	return Target.ContainsMembership(Group);
}

bool FNetObjectGroups::AddGroupMembership(FNetObjectGroupMembership& Target, FNetObjectGroupHandle Group)
{
	if (!Target.ContainsMembership(Group))
	{
		Target.AddMembership(Group);

		return true;
	}

	return false;
}

void FNetObjectGroups::ResetGroupMembership(FNetObjectGroupMembership& Target)
{
	Target.ResetMemberships();
}

void FNetObjectGroups::Init(const FNetObjectGroupInitParams& Params)
{
	NetRefHandleManager = Params.NetRefHandleManager;

	ensureMsgf(Params.MaxGroupCount <= FNetObjectGroupHandle::MaxGroupIndexCount, TEXT("MaxGroupCount cannot exceed %u"), FNetObjectGroupHandle::MaxGroupIndexCount);
	MaxGroupCount = FMath::Clamp<uint32>(Params.MaxGroupCount, 0U, FNetObjectGroupHandle::MaxGroupIndexCount);

	// Reserve first as invalid group
	Groups.Add(FNetObjectGroup());

	GroupMemberships.SetNum(Params.MaxInternalNetRefIndex);
	GroupFilteredOutObjects.Init(Params.MaxInternalNetRefIndex);
}

void FNetObjectGroups::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	GroupMemberships.SetNum(NewMaxInternalIndex);
	GroupFilteredOutObjects.SetNumBits(NewMaxInternalIndex);
}

FNetObjectGroupHandle FNetObjectGroups::CreateGroup(FName InGroupName)
{
	const bool bCanCreateGroup = (uint32)Groups.Num() < MaxGroupCount;

	if (!bCanCreateGroup)
	{
		UE_LOGF(LogIrisGroup, Warning, "Maximum allowed groups allocated: %u. Group: %ls cannot be created", MaxGroupCount, *InGroupName.ToString());
		ensureMsgf(false, TEXT("FNetObjectGroups::CreateGroup: Maximum allowed groups allocated."));
		return FNetObjectGroupHandle();
	}

	// Auto-generate a name if none was passed	
	if (InGroupName == NAME_None)
	{
		InGroupName = FName(TEXT("NetObjectGroup"), AutogeneratedGroupNameId++);
	}

	// Validate that the given name is unique
	{
		const FNetObjectGroupHandle ExistingGroup = FindGroupHandle(InGroupName);
		if (ExistingGroup.IsValid())
		{
			UE_LOGF(LogIrisGroup, Warning, "Tried to create a group named: %ls. But this name was already registered to groupHandle: %u", *InGroupName.ToString(), ExistingGroup.GetGroupIndex());
			ensureMsgf(false, TEXT("FNetObjectGroups::CreateGroup: Group name %s is already registered"), *InGroupName.ToString());
			return FNetObjectGroupHandle();
		}
		
	}

	const uint32 NewGroupId = NextGroupUniqueId++;
	
	const uint32 Index = (uint32)Groups.Emplace(FNetObjectGroup{.GroupName = InGroupName, .GroupId = NewGroupId});

	UE_LOGF(LogIrisGroup, Verbose, "Created GroupHandle named: %ls Index: %u Id: %u", *InGroupName.ToString(), Index, NewGroupId);

	FNetObjectGroupHandle GroupHandle;
	GroupHandle.Index = static_cast<FNetObjectGroupHandle::FGroupIndexType>(Index);
	GroupHandle.Epoch = CurrentEpoch;
	GroupHandle.UniqueId = NewGroupId;

	ensureMsgf(NextGroupUniqueId != 0, TEXT("We created more than MAX_uint32 unique NetGroups. An Id overflow occured and it may cause conflicts with existing handles."));

	return GroupHandle;
}

void FNetObjectGroups::DestroyGroup(FNetObjectGroupHandle GroupHandle)
{
	if (IsValidGroup(GroupHandle))
	{
		FNetObjectGroup& Group = Groups[GroupHandle.GetGroupIndex()];

		UE_LOGF(LogIrisGroup, Verbose, "Destroyed GroupHandle named: %ls Index: %u Id: %u", *Group.GroupName.ToString(), GroupHandle.Index, Group.GroupId);

		ClearGroup(GroupHandle);		
		Groups.RemoveAt(GroupHandle.GetGroupIndex());
	}
}

void FNetObjectGroups::ClearGroup(FNetObjectGroupHandle GroupHandle)
{
	if (IsValidGroup(GroupHandle))
	{
		FNetObjectGroup& Group = Groups[GroupHandle.GetGroupIndex()];

		for (FInternalNetRefIndex InternalIndex : Group.Members)
		{
			checkSlow(IsMemberOf(GroupMemberships[InternalIndex], GroupHandle));
			GroupMemberships[InternalIndex].RemoveMembership(GroupHandle);
		}

		Group.Members.Empty();
	}
}

const FNetObjectGroup* FNetObjectGroups::GetGroup(FNetObjectGroupHandle GroupHandle) const
{
	return IsValidGroup(GroupHandle) ? &Groups[GroupHandle.GetGroupIndex()] : nullptr;
}

FNetObjectGroup* FNetObjectGroups::GetGroup(FNetObjectGroupHandle GroupHandle)
{
	return IsValidGroup(GroupHandle) ? &Groups[GroupHandle.GetGroupIndex()] : nullptr;
}

const FNetObjectGroup* FNetObjectGroups::GetGroupFromIndex(FNetObjectGroupHandle::FGroupIndexType GroupIndex) const
{
	return (GroupIndex != FNetObjectGroupHandle::InvalidNetObjectGroupIndex && Groups.IsValidIndex(GroupIndex)) ? &Groups[GroupIndex] : nullptr;
}

FNetObjectGroup* FNetObjectGroups::GetGroupFromIndex(FNetObjectGroupHandle::FGroupIndexType GroupIndex)
{
	return (GroupIndex != FNetObjectGroupHandle::InvalidNetObjectGroupIndex && Groups.IsValidIndex(GroupIndex)) ? &Groups[GroupIndex] : nullptr;
}

FNetObjectGroupHandle FNetObjectGroups::GetHandleFromGroup(const FNetObjectGroup* InGroup) const
{
	check(InGroup);
	return FNetObjectGroupHandle(GetIndexFromGroup(InGroup), CurrentEpoch, InGroup->GroupId);
}

bool FNetObjectGroups::Contains(FNetObjectGroupHandle GroupHandle, FInternalNetRefIndex InternalIndex) const
{
	// If the group does not exist we cannot be in it..
	const FNetObjectGroup* Group = GetGroup(GroupHandle);
	if (!Group)
	{
		return false;
	}

	checkSlow(InternalIndex == 0U || (Group->Members.Contains(InternalIndex) == IsMemberOf(GroupMemberships[InternalIndex], GroupHandle)));

	return InternalIndex ? IsMemberOf(GroupMemberships[InternalIndex], GroupHandle) : false;
}

void FNetObjectGroups::AddToGroup(FNetObjectGroupHandle GroupHandle, FInternalNetRefIndex InternalIndex)
{
	if (InternalIndex == InvalidInternalNetRefIndex) 
	{
		return;
	}
	
	FNetObjectGroup* Group = GetGroup(GroupHandle);
	if (!Group)
	{
		UE_LOGF(LogIrisGroup, Warning, "FNetObjectGroups::AddToGroup received invalid group (Index: %u Id: %u). Cannot add %ls to it.", GroupHandle.Index, GroupHandle.UniqueId, *NetRefHandleManager->PrintObjectFromIndex(InternalIndex));
		return;
	}

	UE_LOGF(LogIrisGroup, Verbose, "FNetObjectGroups::AddToGroup Adding %ls to Group %ls (Index: %u Id: %u)",
		*NetRefHandleManager->PrintObjectFromIndex(InternalIndex), *GetGroupName(GroupHandle).ToString(), GroupHandle.Index, GroupHandle.UniqueId);

	FNetObjectGroupMembership& Membership = GroupMemberships[InternalIndex];
	if (AddGroupMembership(Membership, GroupHandle))
	{
		Group->Members.AddUnique(InternalIndex);

		if (IsFilterGroup(*Group))
		{
			GroupFilteredOutObjects.SetBit(InternalIndex);
		}

		if (CVarEnsureIfNumGroupMembershipsExceedsNum > 0 && Membership.NumMemberships() > CVarEnsureIfNumGroupMembershipsExceedsNum)
		{
			UE_LOGF(LogIrisGroup, Error, "FNetObjectGroups::AddGroupMembership Unexpected high num groupmemberships for group %ls (Index: %u Id: %u) NetObject %ls is member of %d groups.",
				*GetGroupName(GroupHandle).ToString(), GroupHandle.Index, GroupHandle.UniqueId, *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), Membership.NumMemberships());
			ensure(false);
		}
	}
}

void FNetObjectGroups::RemoveFromGroup(FNetObjectGroupHandle GroupHandle, FInternalNetRefIndex InternalIndex)
{
	if (InternalIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	FNetObjectGroup* Group = GetGroup(GroupHandle);
	if (!Group)
	{
		UE_LOGF(LogIrisGroup, Warning, "FNetObjectGroups::RemoveFromGroup received invalid group (Index: %u Id: %u). Cannot remove %ls from it.", GroupHandle.Index, GroupHandle.UniqueId, *NetRefHandleManager->PrintObjectFromIndex(InternalIndex));
		return;
	}

	UE_LOGF(LogIrisGroup, Verbose, "FNetObjectGroups::RemoveFromGroup Removing %ls from Group %ls (Index: %u Id: %u)",
		*NetRefHandleManager->PrintObjectFromIndex(InternalIndex), *GetGroupName(GroupHandle).ToString(), GroupHandle.Index, GroupHandle.UniqueId);

	FNetObjectGroupMembership& GroupMembership = GroupMemberships[InternalIndex];
	checkSlow(IsMemberOf(GroupMembership, GroupHandle));

	GroupMembership.RemoveMembership(GroupHandle);
	Group->Members.RemoveSingle(InternalIndex);

	// Check to see if the object is still part of a filter group
	if (!IsInAnyFilterGroup(GroupMembership))
	{
		GroupFilteredOutObjects.ClearBit(InternalIndex);
	}
}

void FNetObjectGroups::AddExclusionFilterTrait(FNetObjectGroupHandle GroupHandle)
{
	if (FNetObjectGroup* Group = GetGroup(GroupHandle))
	{
		if (!IsFilterGroup(*Group))
		{
			UE_LOGF(LogIrisGroup, Verbose, "FNetObjectGroups::AddExclusionFilterTrait to Group %ls (GroupIndex: %u)", *GetGroupName(GroupHandle).ToString(), GroupHandle.GetGroupIndex());

			Group->Traits |= ENetObjectGroupTraits::IsExclusionFiltering;

			// Flag all current members of this group that they are now filterable
			for (FInternalNetRefIndex MemberIndex : Group->Members)
			{
				GroupFilteredOutObjects.SetBit(MemberIndex);
			}
		}
	}
}

void FNetObjectGroups::RemoveExclusionFilterTrait(FNetObjectGroupHandle GroupHandle)
{
	FNetObjectGroup* Group = GetGroup(GroupHandle);
	if (Group == nullptr)
	{
		return;
	}

	if (!IsExclusionFilterGroup(*Group))
	{
		return;
	}

	UE_LOGF(LogIrisGroup, Verbose, "FNetObjectGroups::RemoveExclusionFilterTrait to Group %ls (GroupIndex: %u)", *GetGroupName(GroupHandle).ToString(), GroupHandle.GetGroupIndex());

	Group->Traits &= ~(ENetObjectGroupTraits::IsExclusionFiltering);

	for (FInternalNetRefIndex MemberIndex : Group->Members)
	{
		// Check to see if the object is still part of a filter group
		if (!IsInAnyFilterGroup(GroupMemberships[MemberIndex]))
		{
			GroupFilteredOutObjects.ClearBit(MemberIndex);
		}
	}
}

void FNetObjectGroups::AddInclusionFilterTrait(FNetObjectGroupHandle GroupHandle)
{
	if (FNetObjectGroup* Group = GetGroup(GroupHandle))
	{
		// Can't be both inclusion and exclusion so let's do nothing if the group has any sort of filter trait.
		if (!IsFilterGroup(*Group))
		{
			Group->Traits |= ENetObjectGroupTraits::IsInclusionFiltering;
		}
	}
}

void FNetObjectGroups::RemoveInclusionFilterTrait(FNetObjectGroupHandle GroupHandle)
{
	FNetObjectGroup* Group = GetGroup(GroupHandle);
	if (Group == nullptr)
	{
		return;
	}

	// Simply remove the trait.
	Group->Traits &= ~(ENetObjectGroupTraits::IsInclusionFiltering);
}

bool FNetObjectGroups::IsFilterGroup(FNetObjectGroupHandle GroupHandle) const
{
	if (const FNetObjectGroup* Group = GetGroup(GroupHandle))
	{
		return EnumHasAnyFlags(Group->Traits, ENetObjectGroupTraits::IsExclusionFiltering | ENetObjectGroupTraits::IsInclusionFiltering);
	}

	return false;
}

bool FNetObjectGroups::IsInAnyFilterGroup(const FNetObjectGroupMembership& GroupMembership) const
{
	for (FNetObjectGroupHandle::FGroupIndexType AssignedGroupIndex : GroupMembership.GetGroupIndexes())
	{
		if (const FNetObjectGroup* Group = GetGroupFromIndex(AssignedGroupIndex))
		{
			if (IsFilterGroup(*Group))
			{
				return true;
			}
		}
	}

	return false;
}

const TArrayView<const FNetObjectGroupHandle::FGroupIndexType> FNetObjectGroups::GetGroupIndexesOfNetObject(FInternalNetRefIndex InternalIndex) const
{
	if (ensure(GroupMemberships.IsValidIndex(InternalIndex)))
	{
		return GroupMemberships[InternalIndex].GetGroupIndexes();
	}

	return TArrayView<const FNetObjectGroupHandle::FGroupIndexType>();
}

void FNetObjectGroups::GetGroupHandlesOfNetObject(FInternalNetRefIndex InternalIndex, TArray<FNetObjectGroupHandle>& OutHandles) const
{
	if (ensure(GroupMemberships.IsValidIndex(InternalIndex)))
	{
		const TArrayView<const FNetObjectGroupHandle::FGroupIndexType> GroupIndexes = GroupMemberships[InternalIndex].GetGroupIndexes();
		OutHandles.Reserve(GroupIndexes.Num());

		for (FNetObjectGroupHandle::FGroupIndexType AssignedGroupIndex : GroupIndexes)
		{
			OutHandles.Add(GetHandleFromIndex(AssignedGroupIndex));
		}
	}
}

} // end namespace UE::Net::Private
