// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "UObject/NameTypes.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationSystem/NetObjectGroupHandle.h"

namespace UE::Net
{
	typedef uint32 FInternalNetRefIndex;

	namespace Private
	{
		class FNetRefHandleManager;
	}
}

namespace UE::Net::Private
{

enum class ENetObjectGroupTraits : uint32
{
	None                 = 0x0000,
	IsExclusionFiltering = 0x0001,
	IsInclusionFiltering = 0x0002,
};
ENUM_CLASS_FLAGS(ENetObjectGroupTraits);

struct FNetObjectGroup
{
	// Group members can only be replicated objects that have internal indices
	TArray<FInternalNetRefIndex> Members;
	FName GroupName;
	uint32 GroupId = 0U;
	ENetObjectGroupTraits Traits = ENetObjectGroupTraits::None;
};

struct FNetObjectGroupInitParams
{
	FNetRefHandleManager* NetRefHandleManager = nullptr;
	uint32 MaxInternalNetRefIndex = 0U;
	uint32 MaxGroupCount = 0U;
};

class FNetObjectGroups
{
	UE_NONCOPYABLE(FNetObjectGroups)

public:
	FNetObjectGroups();
	~FNetObjectGroups();

	void Init(const FNetObjectGroupInitParams& Params);

	FNetObjectGroupHandle CreateGroup(FName GroupName);
	void DestroyGroup(FNetObjectGroupHandle GroupHandle);

	void ClearGroup(FNetObjectGroupHandle GroupHandle);

	FNetObjectGroupHandle FindGroupHandle(FName GroupName) const;
	
	const FNetObjectGroup* GetGroup(FNetObjectGroupHandle GroupHandle) const;
	FNetObjectGroup* GetGroup(FNetObjectGroupHandle GroupHandle);
	
	const FNetObjectGroup* GetGroupFromIndex(FNetObjectGroupHandle::FGroupIndexType GroupIndex) const;
	FNetObjectGroup* GetGroupFromIndex(FNetObjectGroupHandle::FGroupIndexType GroupIndex);

	FNetObjectGroupHandle GetHandleFromGroup(const FNetObjectGroup* InGroup) const;
	FNetObjectGroupHandle GetHandleFromIndex(FNetObjectGroupHandle::FGroupIndexType GroupIndex) const;

	inline FName GetGroupName(FNetObjectGroupHandle GroupHandle) const;
	inline FString GetGroupNameString(FNetObjectGroupHandle GroupHandle) const;

	bool IsValidGroup(FNetObjectGroupHandle GroupHandle) const;

	bool Contains(FNetObjectGroupHandle GroupHandle, FInternalNetRefIndex InternalIndex) const;
	void AddToGroup(FNetObjectGroupHandle GroupHandle, FInternalNetRefIndex InternalIndex);
	void RemoveFromGroup(FNetObjectGroupHandle GroupHandle, FInternalNetRefIndex InternalIndex);

	/** Called when a group is to be used as an exclusion filter group */
	void AddExclusionFilterTrait(FNetObjectGroupHandle GroupHandle);

	/** Called when a group is no longer used as an exclusion filter group */
	void RemoveExclusionFilterTrait(FNetObjectGroupHandle GroupHandle);

	/** Called when a group is to be used as an inclusion filter group */
	void AddInclusionFilterTrait(FNetObjectGroupHandle GroupHandle);

	/** Called when a group is no longer used as an inclusion filter group */
	void RemoveInclusionFilterTrait(FNetObjectGroupHandle GroupHandle);

	/** Does the group have a filter trait, either exclusion or inclusion */
	bool IsFilterGroup(FNetObjectGroupHandle GroupHandle) const;

	/** Does the group have the exclusion filter trait */
	bool IsExclusionFilterGroup(FNetObjectGroupHandle GroupHandle) const;

	/** Does the group have the inclusion filter trait */
	bool IsInclusionFilterGroup(FNetObjectGroupHandle GroupHandle) const;

	/** Get a reference to the indexes of all groups that the NetObject is a member of */
	const TArrayView<const FNetObjectGroupHandle::FGroupIndexType> GetGroupIndexesOfNetObject(FInternalNetRefIndex InternalIndex) const;

	/** Get a list of all group handles the NetObject is a member of */
	void GetGroupHandlesOfNetObject(FInternalNetRefIndex InternalIndex, TArray<FNetObjectGroupHandle>& OutHandles) const;

	/** Returns a list of all objects currently part of a group with the filter trait */
	const FNetBitArrayView GetGroupFilteredOutObjects() const { return MakeNetBitArrayView(GroupFilteredOutObjects); }

	/** Called when the maximum InternalNetRefIndex increased and we need to realloc our lists */
	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

private:
	struct FNetObjectGroupMembership
	{
	private:
		enum { NumInlinedGroupHandles = 2 };
		/** The indexes of the groups the netobject is a member of. */
		TArray<FNetObjectGroupHandle::FGroupIndexType, TInlineAllocator<NumInlinedGroupHandles>> GroupIndexes;

	public:

		bool ContainsMembership(FNetObjectGroupHandle InGroupHandle) const	{ return GroupIndexes.Contains(InGroupHandle.GetGroupIndex()); }
		void AddMembership(FNetObjectGroupHandle InGroupHandle)		{ GroupIndexes.Add(InGroupHandle.GetGroupIndex()); }
		void RemoveMembership(FNetObjectGroupHandle InGroupHandle)	{ GroupIndexes.RemoveSingleSwap(InGroupHandle.GetGroupIndex()); }
		void ResetMemberships()										{ GroupIndexes.Reset(); }
		int32 NumMemberships() const								{ return GroupIndexes.Num(); }

		const TArrayView<const FNetObjectGroupHandle::FGroupIndexType> GetGroupIndexes() const { return MakeArrayView(GroupIndexes.GetData(), GroupIndexes.Num()); }
	};

	static bool AddGroupMembership(FNetObjectGroupMembership& Target, FNetObjectGroupHandle Group);
	static void ResetGroupMembership(FNetObjectGroupMembership& Target);
	static bool IsMemberOf(const FNetObjectGroupMembership& Target, FNetObjectGroupHandle Group);

	bool IsFilterGroup(const FNetObjectGroup& Group) const;
	bool IsExclusionFilterGroup(const FNetObjectGroup& Group) const;
	bool IsInclusionFilterGroup(const FNetObjectGroup& Group) const;

	bool IsInAnyFilterGroup(const FNetObjectGroupMembership& GroupMembership) const;

	const FNetObjectGroupHandle::FGroupIndexType GetIndexFromGroup(const FNetObjectGroup* InGroup) const;

private:

	FNetRefHandleManager* NetRefHandleManager = nullptr;

	// Group usage pattern should not be high frequency so memory layout should not be a major concern
	TSparseArray<FNetObjectGroup> Groups;

	// Track what groups each internal handle is a member of, we can tighten this up a bit if needed
	TArray<FNetObjectGroupMembership> GroupMemberships;
	
	// Maximum number of groups that can be exist at once
	uint32 MaxGroupCount = 0U;

	// Index to use for groups with auto-generated names
	int32 AutogeneratedGroupNameId = 0;

	// List of objects that are members of a group with a filter trait
	FNetBitArray GroupFilteredOutObjects;

	// Identifies the ReplicationSystem the group handles were created by
	FNetObjectGroupHandle::FGroupIndexType CurrentEpoch = 0U;

	// Unique Id assigned to each group handle
	uint32 NextGroupUniqueId = 1U;
};

inline bool FNetObjectGroups::IsValidGroup(FNetObjectGroupHandle GroupHandle) const
{
	const bool bGroupIndexExists = GroupHandle.IsValid() && GroupHandle.Epoch == CurrentEpoch && Groups.IsValidIndex(GroupHandle.Index);
	return bGroupIndexExists && Groups[GroupHandle.Index].GroupId == GroupHandle.UniqueId;
}

inline bool FNetObjectGroups::IsFilterGroup(const FNetObjectGroup& Group) const
{
	return EnumHasAnyFlags(Group.Traits, ENetObjectGroupTraits::IsExclusionFiltering | ENetObjectGroupTraits::IsInclusionFiltering);
}

inline bool FNetObjectGroups::IsExclusionFilterGroup(const FNetObjectGroup& Group) const
{
	return EnumHasAnyFlags(Group.Traits, ENetObjectGroupTraits::IsExclusionFiltering);
}

inline bool FNetObjectGroups::IsInclusionFilterGroup(const FNetObjectGroup& Group) const
{
	return EnumHasAnyFlags(Group.Traits, ENetObjectGroupTraits::IsInclusionFiltering);
}

inline FNetObjectGroupHandle FNetObjectGroups::FindGroupHandle(FName InGroupName) const
{
	for (const FNetObjectGroup& Group : Groups)
	{
		if (Group.GroupName == InGroupName)
		{
			const int32 Index = GetIndexFromGroup(&Group);
			return FNetObjectGroupHandle((FNetObjectGroupHandle::FGroupIndexType)Index, CurrentEpoch, Group.GroupId);
		}
	}

	return FNetObjectGroupHandle();
}

inline FNetObjectGroupHandle FNetObjectGroups::GetHandleFromIndex(FNetObjectGroupHandle::FGroupIndexType GroupIndex) const
{
	const FNetObjectGroup* Group = GetGroupFromIndex(GroupIndex);
	return Group ? FNetObjectGroupHandle(GroupIndex, CurrentEpoch, Group->GroupId) : FNetObjectGroupHandle();
}

inline const FNetObjectGroupHandle::FGroupIndexType FNetObjectGroups::GetIndexFromGroup(const FNetObjectGroup* InGroup) const
{
	check(InGroup);
	return (FNetObjectGroupHandle::FGroupIndexType)Groups.PointerToIndex(InGroup);
}

inline FName FNetObjectGroups::GetGroupName(FNetObjectGroupHandle GroupHandle) const
{
	if (const FNetObjectGroup* Group = GetGroup(GroupHandle))
	{
		return Group->GroupName;
	}

	return FName();
}

inline FString FNetObjectGroups::GetGroupNameString(FNetObjectGroupHandle GroupHandle) const
{
	return GetGroupName(GroupHandle).ToString();
}

} // end namespace UE::Net::Private
