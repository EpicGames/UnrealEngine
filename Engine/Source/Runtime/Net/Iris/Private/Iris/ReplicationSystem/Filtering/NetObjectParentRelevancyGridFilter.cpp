// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Filtering/NetObjectParentRelevancyGridFilter.h"

#include "Algo/Sort.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"

#include "Net/Core/NetBitArray.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetObjectParentRelevancyGridFilter)

void UNetObjectParentRelevancyGridFilter::OnInit(const FNetObjectFilterInitParams& Params)
{
	Super::OnInit(Params);

	if (const UNetObjectParentRelevancyGridFilterConfig* FilterConfig = Cast<UNetObjectParentRelevancyGridFilterConfig>(Params.Config))
	{
		MaxParentChainDepth = (int32)FilterConfig->MaxParentChainDepth;
	}

	ScratchList.Init(Params.CurrentMaxInternalIndex);
}

void UNetObjectParentRelevancyGridFilter::OnDeinit()
{
	ParentChains.Empty();
	ScratchList.Empty();
	Super::OnDeinit();
}

void UNetObjectParentRelevancyGridFilter::OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex)
{
	Super::OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
	ScratchList.SetNumBits(NewMaxInternalIndex);
}

bool UNetObjectParentRelevancyGridFilter::AddObject(UE::Net::FInternalNetRefIndex ObjectNetIndex, FNetObjectFilterAddObjectParams& Params)
{
	if (!Super::AddObject(ObjectNetIndex, Params))
	{
		return false;
	}

	// Reuse the base class info index for our own ParentChain storage index
	const FObjectLocationInfo& ObjectLocationInfo = static_cast<const FObjectLocationInfo&>(Params.OutInfo);
	const uint32 InfoIndex = ObjectLocationInfo.GetInfoIndex();

	// Allocate new chunks if required
	if (InfoIndex >= uint32(ParentChains.Num()))
	{
		constexpr int32 NumElementsPerChunk = ParentChainsChunkSize / sizeof(FParentChain);
		ParentChains.Add(NumElementsPerChunk);
	}

	check(ParentChains.IsValidIndex(InfoIndex));

	// TODO: If Iris ever allows setting creation dependencies at replication start, then we'd need to build the chain here.

	return true;
}

void UNetObjectParentRelevancyGridFilter::RemoveObject(UE::Net::FInternalNetRefIndex ObjectNetIndex, const FNetObjectFilteringInfo& Info)
{
	// Read the info index before the Super clears it
	const FObjectLocationInfo& ObjectLocationInfo = static_cast<const FObjectLocationInfo&>(Info);
	const uint32 InfoIndex = ObjectLocationInfo.GetInfoIndex();

	if (ensure(ParentChains.IsValidIndex((int32)InfoIndex)))
	{
		FParentChain& ParentChain = ParentChains[InfoIndex];
		ParentChain.ParentNetIndexes.Empty();
		ParentChain.NumImmediateParents = 0;
	}

	Super::RemoveObject(ObjectNetIndex, Info);
}

void UNetObjectParentRelevancyGridFilter::PreFilter(FNetObjectPreFilteringParams& Params)
{
	using namespace UE::Net;

	Super::PreFilter(Params);

	IRIS_PROFILER_SCOPE(UNetObjectParentRelevancyGridFilter_PreFilter);

	const FNetBitArrayView DirtyCreationDeps = Params.NetRefIndexManager.GetObjectsWithDirtyCreationDependencies();
	const FNetBitArrayView ObjectsInFilter = GetFilteredObjects();

	ScratchList.ClearAllBits();

	FNetBitArrayView RebuiltObjects = MakeNetBitArrayView(ScratchList);

	auto RebuildParentChain = [this, &Params, &RebuiltObjects, &ObjectsInFilter](FInternalNetRefIndex ObjectToRebuild)
	{
		const FObjectLocationInfo& ObjectLocationInfo = static_cast<const FObjectLocationInfo&>(Params.FilteringInfos[ObjectToRebuild]);
		const uint32 InfoIndex = ObjectLocationInfo.GetInfoIndex();
		checkSlow(ParentChains.IsValidIndex(InfoIndex));

		FParentChain& ParentChain = ParentChains[InfoIndex];
		BuildParentChain(ObjectToRebuild, ParentChain, Params.NetRefIndexManager, ObjectsInFilter);

		RebuiltObjects.SetBit(ObjectToRebuild);
	};

	// Rebuild the parent list of every object with dirty creation dependencies
	FNetBitArrayView::ForAllSetBits(DirtyCreationDeps, ObjectsInFilter, FNetBitArrayView::AndOp, [&](FInternalNetRefIndex ObjectIndex)
	{
		if (!RebuiltObjects.IsBitSet(ObjectIndex))
		{
			RebuildParentChain(ObjectIndex);
	
			// Ask all it's children to rebuild their chains too
			for (const FInternalNetRefIndex ChildIndex : Params.NetRefIndexManager.GetCreationDependents(ObjectIndex))
			{
				// The children must not have been rebuilt yet and be managed by this filter
				if (!RebuiltObjects.IsBitSet(ChildIndex) && ObjectsInFilter.IsBitSet(ChildIndex))
				{
					RebuildParentChain(ChildIndex);
				}
			}	
		}
	});
}

void UNetObjectParentRelevancyGridFilter::BuildParentChain(UE::Net::FInternalNetRefIndex ChildNetIndex, FParentChain& OutParentChain, const UE::Net::FInternalNetRefIndexManager& NetRefIndexManager, const UE::Net::FNetBitArrayView ObjectsInFilter) const
{
	using namespace UE::Net;

	OutParentChain.ParentNetIndexes.Reset();
	OutParentChain.NumImmediateParents = 0;
	OutParentChain.bIsLinearChain = false;

	TArray<FInternalNetRefIndex>& OutParentList = OutParentChain.ParentNetIndexes;

	TArray<FInternalNetRefIndex, TInlineAllocator<16>> ParentsToVisit;

	// Immediate parents are stored at the beginning of the list so that Filter() can test them first and possibly skip the entire parent chain
	for (const FInternalNetRefIndex ParentNetIndex : NetRefIndexManager.GetCreationDependencies(ChildNetIndex))
	{
		// Look at the creation dependencies of this parent 
		ParentsToVisit.Emplace(ParentNetIndex);

		// We can only promote this parent if its part of this filter.
		if (ObjectsInFilter.IsBitSet(ParentNetIndex))
		{
			OutParentList.Emplace(ParentNetIndex);
		}
	}

	const int32 NumImmediateParents = OutParentList.Num();
	ensureMsgf(NumImmediateParents <= UINT8_MAX, TEXT("Object %s has too many creation dependencies: (%d). Max allowed is: (255)"), *NetRefIndexManager.PrintObjectFromIndex(ChildNetIndex), NumImmediateParents);
	OutParentChain.NumImmediateParents = (uint8)(FMath::Min<int32>(NumImmediateParents, UINT8_MAX));

	// A chain is linear if every object only has a single creation dependency
	bool bIsLinearChain = (NumImmediateParents == 1);

	int32 LastIndex = 0;
	while (LastIndex < ParentsToVisit.Num())
	{
		// Stop if the parent chain gets too big
		if (MaxParentChainDepth > 0 && OutParentList.Num() >= MaxParentChainDepth)
		{
			ensureMsgf(false, TEXT("UNetObjectParentRelevancyGridFilter replicated object: %s has a parent dependency chain that exceeds MaxParentChainDepth=%u"), *NetRefIndexManager.PrintObjectFromIndex(ChildNetIndex), MaxParentChainDepth);
			OutParentList.SetNum(MaxParentChainDepth);
			return;
		}

		const int32 CurrentIndexMax = ParentsToVisit.Num();

		// Add the dependencies of the parents added in the last iteration
		for (int32 Index = LastIndex; Index < CurrentIndexMax; ++Index)
		{
			TConstArrayView<const FInternalNetRefIndex> CreationDependencies = NetRefIndexManager.GetCreationDependencies(ParentsToVisit[Index]);

			int32 NumAddedFromThisNode = 0;
			for (const FInternalNetRefIndex ParentNetIndex : CreationDependencies)
			{
				ParentsToVisit.Emplace(ParentNetIndex);

				// We can only promote this parent if its part of this filter.
				if (ObjectsInFilter.IsBitSet(ParentNetIndex))
				{
					OutParentList.Emplace(ParentNetIndex);

					bIsLinearChain &= CreationDependencies.Num() <= 1;
				}
			}
		}

		// Now look at the parents added on this iteration
		LastIndex = CurrentIndexMax;
	}

	OutParentChain.bIsLinearChain = bIsLinearChain;
}

void UNetObjectParentRelevancyGridFilter::Filter(FNetObjectFilteringParams& Params)
{
	using namespace UE::Net;

	IRIS_PROFILER_SCOPE(UNetObjectParentRelevancyGridFilter_Filter);

	// Run the base grid filter first. Then we will look at the filtered in objects to also force their parents to be visible
	Super::Filter(Params);

	ScratchList.ClearAllBits();
	FNetBitArrayView VisitedParents = MakeNetBitArrayView(ScratchList);

	// Note: We assume the base filter didn't set bits to objects not managed by this filter. If that ever changes we would need to AND with the list of objects owned by this filter.
	FNetBitArrayView FilteredInObjects = Params.OutAllowedObjects;

	FNetBitArrayView::ForAllSetBits(FilteredInObjects, Params.GroupFilteredOutObjects, FNetBitArrayView::AndNotOp, [&](FInternalNetRefIndex ObjectIndex)
	{
		const FObjectLocationInfo& ObjectLocationInfo = static_cast<const FObjectLocationInfo&>(Params.FilteringInfos[ObjectIndex]);
		const uint32 InfoIndex = ObjectLocationInfo.GetInfoIndex();
		checkSlow(ParentChains.IsValidIndex(InfoIndex));

		const FParentChain& ParentChain = ParentChains[InfoIndex];

		bool bAllImmediatesVisited = true;

		for (int32 i=0; i < ParentChain.ParentNetIndexes.Num(); ++i)
		{
			const FInternalNetRefIndex ParentNetIndex = ParentChain.ParentNetIndexes[i];
			const bool bWasVisitedAlready = VisitedParents.IsBitSet(ParentNetIndex);

			// When every parent has a single parent, we can stop the moment we hit a visited one
			if (ParentChain.bIsLinearChain && bWasVisitedAlready)
			{
				break;
			}

			// When we have multiple immediate parents, we can stop if all immediates were visited
			if (i < ParentChain.NumImmediateParents)
			{
				bAllImmediatesVisited &= bWasVisitedAlready;
				
				// If this is the last immediate, look if we can stop iterating
				if (bAllImmediatesVisited && (ParentChain.NumImmediateParents - i == 1))
				{
					break;
				}
			}

			// Flag this parent as visited
			VisitedParents.SetBit(ParentNetIndex);

			// This parent is filtered out by other APIs so we can't actually make it visible. Stop looking at the parent chain immediately otherwise we could make the higher order parents relevant for nothing
			if (Params.GroupFilteredOutObjects.IsBitSet(ParentNetIndex))
			{
				break;
			}
				
			// Force this parent to be relevant
			FilteredInObjects.SetBit(ParentNetIndex);
		}
	});
}

FString UNetObjectParentRelevancyGridFilter::PrintDebugInfoForObject(const FDebugInfoParams& Params, UE::Net::FInternalNetRefIndex ObjectNetIndex) const
{
	FString Result = Super::PrintDebugInfoForObject(Params, ObjectNetIndex);

	const FObjectLocationInfo& ObjectLocationInfo = static_cast<const FObjectLocationInfo&>(Params.FilteringInfos[ObjectNetIndex]);
	const uint32 InfoIndex = ObjectLocationInfo.GetInfoIndex();
	if (InfoIndex < uint32(ParentChains.Num()))
	{
		const FParentChain& Slot = ParentChains[InfoIndex];
		Result += FString::Printf(TEXT(", ParentChainDepth: %d (Immediate: %u)"), Slot.ParentNetIndexes.Num(), Slot.NumImmediateParents);
	}

	return Result;
}
