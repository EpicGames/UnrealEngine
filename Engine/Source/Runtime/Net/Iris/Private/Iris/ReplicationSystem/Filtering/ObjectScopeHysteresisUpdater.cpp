// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Filtering/ObjectScopeHysteresisUpdater.h"
#include "Iris/Core/IrisProfiler.h"
#include "Containers/ArrayView.h"

namespace UE::Net::Private
{

void FObjectScopeHysteresisUpdater::Init(uint32 MaxObjectCount)
{
	ObjectsToUpdate.Init(MaxObjectCount);
}

void FObjectScopeHysteresisUpdater::Deinit()
{
	FrameCounters.Empty();
	LocalIndexToNetRefIndex.Empty();
	NetRefIndexToLocalIndex.Empty();
	UsedLocalIndices.Empty();
	ObjectsToUpdate.Empty();
}

void FObjectScopeHysteresisUpdater::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	ObjectsToUpdate.SetNumBits(NewMaxInternalIndex);
}

void FObjectScopeHysteresisUpdater::SetHysteresisFrameCount(FInternalNetRefIndex NetRefIndex, uint16 HysteresisFrameCount)
{
	const FLocalIndex LocalIndex = GetOrCreateLocalIndex(NetRefIndex);
	FrameCounters[LocalIndex] = HysteresisFrameCount;
}

void FObjectScopeHysteresisUpdater::RemoveHysteresis(FInternalNetRefIndex NetRefIndex)
{
	if (FLocalIndex* LocalIndex = NetRefIndexToLocalIndex.Find(NetRefIndex))
	{
		FreeLocalIndex(*LocalIndex);
	}
}

/** Removes all objects in the bitarray vfrom hysteresis. */
void FObjectScopeHysteresisUpdater::RemoveHysteresis(const FNetBitArrayView& ObjectsToRemove)
{
	IRIS_PROFILER_SCOPE(FObjectScopeHysteresisUpdater_RemoveHysteresis);
	if (NetRefIndexToLocalIndex.IsEmpty())
	{
		return;
	}

	FNetBitArrayView::ForAllSetBits(MakeNetBitArrayView(ObjectsToUpdate), ObjectsToRemove, FNetBitArrayView::AndOp, [this](uint32 NetRefIndex)
		{
			this->RemoveHysteresis(NetRefIndex);
		}
	);
}

void FObjectScopeHysteresisUpdater::RemoveHysteresis(TArrayView<const uint32> ObjectsToRemove)
{
	IRIS_PROFILER_SCOPE(FObjectScopeHysteresisUpdater_RemoveHysteresis);
	if (NetRefIndexToLocalIndex.IsEmpty())
	{
		return;
	}

	for (const uint32 ObjectIndex : ObjectsToRemove)
	{
		this->RemoveHysteresis(ObjectIndex);
	}
}

void FObjectScopeHysteresisUpdater::Update(uint8 FramesSinceLastUpdate, TArray<FInternalNetRefIndex>& OutObjectsToFilterOut)
{
	IRIS_PROFILER_SCOPE(FObjectScopeHysteresisUpdater_Update);

	ensure(FramesSinceLastUpdate > 0 && FramesSinceLastUpdate <= 128);

	typedef FNetBitArrayView::StorageWordType WordType;
	constexpr uint32 WordBitCount = FNetBitArrayView::WordBitCount;

	const uint16 FilterOutCompareValue = (65536U - FramesSinceLastUpdate) & 65535U;
	uint16* CountersData = FrameCounters.GetData();

	TArray<FInternalNetRefIndex, TInlineAllocator<32>> ObjectsToRemoveFromUpdate;

	const WordType* LocalIndicesData = UsedLocalIndices.GetData();
	for (FLocalIndex ObjectIt = 0, ObjectEndIt = UsedLocalIndices.GetNumBits(), IndexOffset = 0; ObjectIt < ObjectEndIt; ObjectIt += WordBitCount, ++LocalIndicesData, IndexOffset += 32U)
	{
		// Skip ranges with no objects to update
		WordType LocalIndicesWord = *LocalIndicesData;
		if (!LocalIndicesWord)
		{
			continue;
		}

		WordType LocalIndicesMask = LocalIndicesWord;
		for (WordType LocalIndexOffset = 0; LocalIndexOffset < WordBitCount; LocalIndexOffset += 4U, LocalIndicesMask >>= 4U)
		{
			if (!(LocalIndicesMask & 15U))
			{
				continue;
			}

			uint16 Counters[4];
			Counters[0] = CountersData[IndexOffset + LocalIndexOffset + 0];
			Counters[1] = CountersData[IndexOffset + LocalIndexOffset + 1];
			Counters[2] = CountersData[IndexOffset + LocalIndexOffset + 2];
			Counters[3] = CountersData[IndexOffset + LocalIndexOffset + 3];

			Counters[0] -= FramesSinceLastUpdate;
			Counters[1] -= FramesSinceLastUpdate;
			Counters[2] -= FramesSinceLastUpdate;
			Counters[3] -= FramesSinceLastUpdate;

			// We can update the counters regardless of whether the objects were kept in scope or not. We're not expecting calls to Set/Remove while updating.
			CountersData[IndexOffset + LocalIndexOffset + 0] = Counters[0];
			CountersData[IndexOffset + LocalIndexOffset + 1] = Counters[1];
			CountersData[IndexOffset + LocalIndexOffset + 2] = Counters[2];
			CountersData[IndexOffset + LocalIndexOffset + 3] = Counters[3];

			// For counter values < FramesSinceLastUpdate the object should remain in scope
			for (uint32 Offset : {0, 1, 2, 3})
			{
				if (LocalIndicesMask & (1U << Offset))
				{
					if (Counters[Offset] >= FilterOutCompareValue)
					{
						// Object should now stay filtered out. Remove from updates.
						ObjectsToRemoveFromUpdate.Add(IndexOffset + LocalIndexOffset + Offset);
					}
				}
			}
		}

		if (!ObjectsToRemoveFromUpdate.IsEmpty())
		{
			OutObjectsToFilterOut.Reserve(OutObjectsToFilterOut.Num() + ObjectsToRemoveFromUpdate.Num());
			for (FLocalIndex LocalIndex : ObjectsToRemoveFromUpdate)
			{
				OutObjectsToFilterOut.Add(LocalIndexToNetRefIndex[LocalIndex]);
				FreeLocalIndex(LocalIndex);
			}

			ObjectsToRemoveFromUpdate.Reset();
		}
	}
}

FObjectScopeHysteresisUpdater::FLocalIndex FObjectScopeHysteresisUpdater::GetOrCreateLocalIndex(FInternalNetRefIndex NetRefIndex)
{
	if (FLocalIndex* LocalIndex = NetRefIndexToLocalIndex.Find(NetRefIndex))
	{
		return *LocalIndex;
	}

	uint32 LocalIndex = UsedLocalIndices.FindFirstZero();
	if (LocalIndex == FNetBitArray::InvalidIndex)
	{
		LocalIndex = UsedLocalIndices.GetNumBits();
		UsedLocalIndices.AddBits(LocalIndexGrowCount);
		LocalIndexToNetRefIndex.AddZeroed(LocalIndexGrowCount);
		FrameCounters.AddZeroed(LocalIndexGrowCount);
	}

	UsedLocalIndices.SetBit(LocalIndex);
	LocalIndexToNetRefIndex[LocalIndex] = NetRefIndex;
	NetRefIndexToLocalIndex.Add(NetRefIndex, LocalIndex);
	ObjectsToUpdate.SetBit(NetRefIndex);

	return LocalIndex;
}

void FObjectScopeHysteresisUpdater::FreeLocalIndex(FLocalIndex LocalIndex)
{
	UsedLocalIndices.ClearBit(LocalIndex);
	const FInternalNetRefIndex NetRefIndex = LocalIndexToNetRefIndex[LocalIndex];
	NetRefIndexToLocalIndex.Remove(NetRefIndex);
	ObjectsToUpdate.ClearBit(NetRefIndex);
	// Intentionally not updating LocalIndexToNetRefIndex since it isn't accessed for unset local indices.
}

}
