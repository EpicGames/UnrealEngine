// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Net/Core/NetBitArray.h"

namespace UE::Net
{
	typedef uint32 FInternalNetRefIndex;
}

namespace UE::Net::Private
{

class FObjectScopeHysteresisUpdater
{
public:
	void Init(uint32 MaxObjectCount);
	void Deinit();

	/** Called when the maximum InternalNetRefIndex increased and we need to realloc our lists */
	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

	/** Sets an hysteresis frame count such that an object will be kept in scope until such many frames has passed. */
	void SetHysteresisFrameCount(FInternalNetRefIndex NetRefIndex, uint16 HysteresisFrameCount);

	/** Remove an object from hysteresis update. Needed when an object goes out of scope. An object is automatically removed from hysteresis updates when a previously set frame count has expired. */
	void RemoveHysteresis(FInternalNetRefIndex NetRefIndex);

	/** Removes all objects in the bitarray from hysteresis. */
	void RemoveHysteresis(const FNetBitArrayView& ObjectsToRemove);

	/** Removes all objects in the array from hysteresis. */
	void RemoveHysteresis(TArrayView<const uint32> ObjectsToRemove);

	/** Adjusts the connection relevant objects based on hysteresis frame counts. */
	void Update(uint8 FramesSinceLastUpdate, TArray<FInternalNetRefIndex>& OutObjectsToFilterOut);

	/** Whether any objects are updated for hysteresis. If not there's no point in calling Update(). */
	bool HasObjectsToUpdate() const;

	/** Returns the bitarray of objects affected by hysteresis */
	FNetBitArrayView GetUpdatedObjects() const;

	/** Returns true if the object is currently updated. */
	bool IsObjectUpdated(FInternalNetRefIndex NetRefIndex) const;

private:
	enum : unsigned
	{
		LocalIndexGrowCount = 256U,
	};

	typedef uint32 FLocalIndex;

	FLocalIndex GetOrCreateLocalIndex(FInternalNetRefIndex NetRefIndex);
	void FreeLocalIndex(FLocalIndex LocalIndex);

	// Per LocalIndex how many frames left to update before filtering out the object.
	TArray<uint16> FrameCounters;
	// Lookup table for local index to InternalNetRefIndex.
	TArray<FInternalNetRefIndex> LocalIndexToNetRefIndex;
	// Lookup map for InternalNetRefIndex to LocalIndex to be able to figure out whether an object already is assigned a LocalIndex or not.
	TMap<FInternalNetRefIndex, FLocalIndex> NetRefIndexToLocalIndex;
	// A set bit indicates that the LocalIndex is used. Only stores MaxLocalIndex bits. The bitarray grows as needed.
	FNetBitArray UsedLocalIndices;
	// A set bit indicates that the InternalNetRefIndex is being updated. Can always hold the MaxObjectCount passed to Init().
	FNetBitArray ObjectsToUpdate;
};

inline bool FObjectScopeHysteresisUpdater::HasObjectsToUpdate() const
{
	return !NetRefIndexToLocalIndex.IsEmpty();
}

inline FNetBitArrayView FObjectScopeHysteresisUpdater::GetUpdatedObjects() const
{
	return MakeNetBitArrayView(ObjectsToUpdate);
}

inline bool FObjectScopeHysteresisUpdater::IsObjectUpdated(FInternalNetRefIndex ObjectIndex) const
{
	return ObjectsToUpdate.GetBit(ObjectIndex);
}

}
