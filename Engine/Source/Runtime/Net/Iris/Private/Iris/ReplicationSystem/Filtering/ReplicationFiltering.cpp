// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationFiltering.h"
#include "Algo/AnyOf.h"
#include "HAL/PlatformMath.h"
#include "HAL/IConsoleManager.h"
#include "UObject/Package.h"

#include "Iris/IrisConfigInternal.h"
#include "Iris/IrisConstants.h"
#include "Iris/Core/BitTwiddling.h"
#include "Iris/Core/IrisCsv.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisLogUtils.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/ReplicationSystem/InternalNetRefIndexManager.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationConnections.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemDelegates.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectGroups.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilterDefinitions.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFilteringConfig.h"

#include <inttypes.h>
#include <limits>

namespace UE::Net::Private
{

bool bCVarRepFilterCullNonRelevant = true;
static FAutoConsoleVariableRef CVarRepFilterCullNonRelevant(TEXT("Net.Iris.CullNonRelevant"), bCVarRepFilterCullNonRelevant, TEXT("When enabled will cull replicated actors that are not relevant to any client."), ECVF_Default);

bool bCVarRepFilterValidateNoSubObjectInScopeWithFilteredOutRootObject = false;
static FAutoConsoleVariableRef CVarRepFilterValidateNoSubObjectInScopeWithFilteredOutRootObject(TEXT("Net.Iris.Filtering.ValidateNobSubObjectInScopeWithFilteredOutRootObject"), bCVarRepFilterValidateNoSubObjectInScopeWithFilteredOutRootObject, TEXT("Validate there are no subobjects in scope with a filtered out root object."), ECVF_Default);

bool bCVarRepFilterUpdateCreationDependenciesOnRelevancyChange = false;
static FAutoConsoleVariableRef CVarRepFilterUpdateCreationDependenciesOnRelevancyChange(TEXT("Net.Iris.Filtering.UpdateCreationDependenciesOnRelevancyChange"), bCVarRepFilterUpdateCreationDependenciesOnRelevancyChange, TEXT("When enabled, only update filtering of objects in creation dependency trees when they change relevancy."), ECVF_Default);

FName GetStaticFilterName(FNetObjectFilterHandle Filter)
{
	switch (Filter)
	{
		case InvalidNetObjectFilterHandle:
		{
			static const FName NoFilterName = "NoFilter";
			return NoFilterName;
		} 

		case ToOwnerFilterHandle:
		{
			static const FName ToOwnerFilterName = "ToOwnerFilter";
			return ToOwnerFilterName;
		} 

		case ConnectionFilterHandle:
		{
			static const FName ConnectionFilterName = "ConnectionFilter";
			return ConnectionFilterName;
		} 

		default:
		{
			ensureMsgf(false, TEXT("ReplicationFiltering GetStaticFilterName() received undefined static filter handle %d."), Filter);
		} break;
	}

	return NAME_None;
}

class FNetObjectFilterHandleUtil
{
public:
	static bool IsInvalidHandle(FNetObjectFilterHandle Handle);
	static bool IsDynamicFilter(FNetObjectFilterHandle Handle);
	static bool IsStaticFilter(FNetObjectFilterHandle Handle);

	static FNetObjectFilterHandle MakeDynamicFilterHandle(uint32 FilterIndex);
	static uint32 GetDynamicFilterIndex(FNetObjectFilterHandle Handle);

private:
	// Most significant bit in the filter handle acts as a dynamic/static filter classifier.
	static constexpr FNetObjectFilterHandle DynamicNetObjectFilterHandleFlag = 1U << (sizeof(FNetObjectFilterHandle)*8U - 1U);
};

//*************************************************************************************************
// FUpdateDirtyObjectsBatchHelper
//*************************************************************************************************

class FReplicationFiltering::FUpdateDirtyObjectsBatchHelper
{
public:
	enum Constants : uint32
	{
		MaxObjectCountPerBatch = 512U,
	};

	struct FPerFilterInfo
	{
		UE::Net::FInternalNetRefIndex* ObjectIndices = nullptr;
		uint32 ObjectCount = 0;
	};

	FUpdateDirtyObjectsBatchHelper(const FNetRefHandleManager* InNetRefHandleManager, const TArray<FFilterInfo>& DynamicFilters)
		: NetRefHandleManager(InNetRefHandleManager)
	{
		const int32 NumFilters = DynamicFilters.Num();

		PerFilterInfos.SetNum(NumFilters, EAllowShrinking::No);
		ObjectIndicesStorage.SetNumUninitialized(NumFilters * MaxObjectCountPerBatch);
		
		uint32 BufferIndex = 0;
		for (FPerFilterInfo& PerFilterInfo : PerFilterInfos)
		{
			PerFilterInfo.ObjectIndices = ObjectIndicesStorage.GetData() + BufferIndex * MaxObjectCountPerBatch;
			++BufferIndex;
		}
	}

	void PrepareBatch(const UE::Net::FInternalNetRefIndex* ObjectIndices, uint32 ObjectCount, const TArray<uint8>& FilterIndices)
	{
		ResetBatch();

		for (const UE::Net::FInternalNetRefIndex ObjectIndex : MakeArrayView(ObjectIndices, ObjectCount))
		{
			const uint8 FilterIndex = FilterIndices[ObjectIndex];
			if (FilterIndex == InvalidDynamicFilterIndex)
			{
				continue;
			}

			FPerFilterInfo& PerFilterInfo = PerFilterInfos[FilterIndex];

			// If the info has a buffer assigned then it's an active filter
			if (PerFilterInfo.ObjectIndices)
			{
				PerFilterInfo.ObjectIndices[PerFilterInfo.ObjectCount] = ObjectIndex;
				++PerFilterInfo.ObjectCount;
			}
		}
	}

	TArray<FPerFilterInfo, TInlineAllocator<16>> PerFilterInfos;

private:
	void ResetBatch()
	{
		for (FPerFilterInfo& PerFilterInfo : PerFilterInfos)
		{
			PerFilterInfo.ObjectCount = 0U;
		}
	}

	TArray<uint32> ObjectIndicesStorage;
	const FNetRefHandleManager* NetRefHandleManager = nullptr;
};

// Helper class instance to avoid logging redundant dependent objects more than once per object class.
#if !UE_BUILD_SHIPPING
static FIrisLogOnceTracker ReplicationFiltering_MootDependentObjectTracker;
#endif

//*************************************************************************************************
// FReplicationFiltering
//*************************************************************************************************

FReplicationFiltering::FReplicationFiltering()
: bHasNewConnection(0)
, bHasRemovedConnection(0)
, bHasDirtyOwnerFilter(0)
, bHasDirtyOwner(0)
, bHasDynamicFilters(0)
, bHasDirtyExclusionFilterGroup(0)
, bHasDirtyInclusionFilterGroup(0)
, bHasDynamicFiltersWithUpdateTrait(0)
{
	StaticChecks();
}

void FReplicationFiltering::StaticChecks()
{
	static_assert(std::numeric_limits<PerObjectInfoIndexType>::max() % (UsedPerObjectInfoStorageGrowSize*32U) == UsedPerObjectInfoStorageGrowSize*32U - 1, "Bit array grow code expects not to be able to return an out of bound index.");
	static_assert(sizeof(FNetBitArrayBase::StorageWordType) == sizeof(uint32), "Expected FNetBitArrayBase::StorageWordType to be four bytes in size."); 
}

void FReplicationFiltering::Init(FReplicationFilteringInitParams& Params)
{
	check(Params.Connections != nullptr);
	check(Params.Connections->GetMaxConnectionCount() <= std::numeric_limits<decltype(ObjectIndexToOwningConnection)::ElementType>::max());

	Config = TStrongObjectPtr(GetDefault<UReplicationFilteringConfig>());

	ReplicationSystem = Params.ReplicationSystem;

	Connections = Params.Connections;
	NetRefHandleManager = Params.NetRefHandleManager;
	Groups = Params.Groups;

	MaxInternalNetRefIndex = Params.MaxInternalNetRefIndex;

	// Connection specifics
	ConnectionInfos.SetNum(Params.Connections->GetMaxConnectionCount() + 1U);
	ValidConnections.Init(ConnectionInfos.Num());
	NewConnections.Init(ConnectionInfos.Num());

	// Initialize all InternalNetRefIndex lists
	SetNetObjectListsSize(MaxInternalNetRefIndex);

	// Group filtering
	{
		const uint32 InMaxGroupCount = Params.MaxGroupCount;
		check(InMaxGroupCount <= std::numeric_limits<FNetObjectGroupHandle::FGroupIndexType>::max());
		MaxGroupCount = InMaxGroupCount;
		GroupInfos.SetNumZeroed(InMaxGroupCount);
		ExclusionFilterGroups.Init(InMaxGroupCount);
		InclusionFilterGroups.Init(InMaxGroupCount);
		DirtyExclusionFilterGroups.Init(InMaxGroupCount);
		DirtyInclusionFilterGroups.Init(InMaxGroupCount);
		// SubObjectFilter groups
		SubObjectFilterGroups.Init(InMaxGroupCount);
		DirtySubObjectFilterGroups.Init(InMaxGroupCount);
	}

	PerObjectInfoStorageCountForConnections = Align(FPlatformMath::Max(uint32(ConnectionInfos.Num()), 1U), 32U)/32U;
	PerObjectInfoStorageCountPerItem = sizeof(FPerObjectInfo) + PerObjectInfoStorageCountForConnections - 1U;

	InitFilters();
	InitObjectScopeHysteresis();
}

void FReplicationFiltering::Deinit()
{
	for (FFilterInfo& FilterInfo : DynamicFilterInfos)
	{
		if (FilterInfo.Filter)
		{
			FilterInfo.Filter->Deinit();
		}
	}

	// Clear most buffers by setting size to 0
	SetNetObjectListsSize(0);
}

void FReplicationFiltering::SetNetObjectListsSize(FInternalNetRefIndex MaxInternalIndex)
{
	WordCountForObjectBitArrays = Align(MaxInternalIndex, sizeof(FNetBitArrayBase::StorageWordType) * 8U) / (sizeof(FNetBitArrayBase::StorageWordType) * 8U);

	// Increase NetBitArrays
	{
		ObjectsWithDirtyOwnerFilter.SetNumBits(MaxInternalIndex);
		ObjectsWithDirtyOwner.SetNumBits(MaxInternalIndex);
		ObjectsWithOwnerFilter.SetNumBits(MaxInternalIndex);
		ObjectsWithPerObjectInfo.SetNumBits(MaxInternalIndex);

		ScopeDeltaPrevObjects.SetNumBits(MaxInternalIndex);
		ScopeDeltaNewlyInObjects.SetNumBits(MaxInternalIndex);
		ScopeDeltaNewlyOutObjects.SetNumBits(MaxInternalIndex);

		DynamicFilterEnabledObjects.SetNumBits(MaxInternalIndex);
		
		ObjectsRequiringDynamicFilterUpdate.SetNumBits(MaxInternalIndex);
	}

	// Increase TArrays whose index maps to a NetBitArray
	{
		ObjectIndexToPerObjectInfoIndex.SetNumZeroed(MaxInternalIndex);
		ObjectIndexToOwningConnection.SetNumZeroed(MaxInternalIndex);
		ObjectScopeHysteresisFrameCounts.SetNumZeroed(MaxInternalIndex);
		NetObjectFilteringInfos.SetNumZeroed(MaxInternalIndex);
	}

	// ObjectIndexToDynamicFilterIndex is initialized to a non-zero value.
	{
		const int32 PrevMaxSize = ObjectIndexToDynamicFilterIndex.Num();
		ObjectIndexToDynamicFilterIndex.SetNumUninitialized(MaxInternalIndex);

		// Initialize the newly allocated buffer portion
		if (MaxInternalIndex > 0)
		{
			checkf(MaxInternalIndex > (uint32)PrevMaxSize, TEXT("Not expected for the array to get smaller."));
			uint8* NewBufferToInit = ObjectIndexToDynamicFilterIndex.GetData();
			NewBufferToInit += PrevMaxSize;
			FMemory::Memset(NewBufferToInit, InvalidDynamicFilterIndex, (MaxInternalIndex - PrevMaxSize) * sizeof(decltype(ObjectIndexToDynamicFilterIndex)::ElementType));
		}
	}

	// Always allocated and maintained regardless of whether the feature is enabled or not.
	HysteresisState.ObjectsToClear.SetNumBits(MaxInternalNetRefIndex);
	HysteresisState.ObjectsExemptFromHysteresis.SetNumBits(MaxInternalNetRefIndex);
}

void FReplicationFiltering::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	MaxInternalNetRefIndex = NewMaxInternalIndex;

	SetNetObjectListsSize(NewMaxInternalIndex);

	// Resize the per-connection data
	{
		IRIS_PROFILER_SCOPE(FReplicationFiltering_ResizeAllPerConnectionLists);

		auto ResizePerConnectionInfo = [this, NewMaxInternalIndex](uint32 ConnectionId)
		{
			FPerConnectionInfo& ConnectionInfo = this->ConnectionInfos[ConnectionId];
			this->SetPerConnectionListsSize(ConnectionInfo, NewMaxInternalIndex);
		};

		ValidConnections.ForAllSetBits(ResizePerConnectionInfo);
	}

	// Propagate the increase to the DynamicFilters
	{
		TArrayView<FNetObjectFilteringInfo> NewFilterInfoView = GetNetObjectFilteringInfos();
		for (FFilterInfo& FilterInfo : DynamicFilterInfos)
		{
			if (FilterInfo.Filter)
			{
				FilterInfo.Filter->MaxInternalNetRefIndexIncreased(NewMaxInternalIndex, NewFilterInfoView);
			}
		}
	}
}

void FReplicationFiltering::OnInternalNetRefIndicesFreed(const TConstArrayView<FInternalNetRefIndex>& FreedIndices)
{
	// Clear owner info just as the index is freed. We want to keep the owner info even when an object goes out of scope so that state flushing works as expected.
	for (const FInternalNetRefIndex ObjectIndex : FreedIndices)
	{
		ObjectIndexToOwningConnection[ObjectIndex] = UE::Net::InvalidConnectionId;
	}
}

void FReplicationFiltering::Filter()
{
#if UE_NET_IRIS_CSV_STATS
	CSV_SCOPED_TIMING_STAT(Iris, Filter_PrePoll);
#endif

	++FrameIndex;

	ResetRemovedConnections();

	InitNewConnections();

	UpdateObjectsInScope();

	UpdateGroupExclusionFiltering();

	UpdateGroupInclusionFiltering();

	UpdateOwnerFiltering();

	UpdateSubObjectFilters();

	PreUpdateObjectScopeHysteresis();

	if (HasDynamicFilters())
	{
		UpdateDynamicFilters();
	}
	else
	{
		// Dynamic filters are responsible for updating ObjectsInScope. 
		// Do it here if no filters were executed.
		ValidConnections.ForAllSetBits([this](uint32 ConnectionId)
		{
			FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
			ConnectionInfo.ObjectsInScope.Copy(ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering);
		});
	}

	FilterNonRelevantObjects();

	// New connections are exempt from object scope hysteresis so we keep the info around until we're done with all filtering.
	NewConnections.ClearAllBits();
}

void FReplicationFiltering::FilterNonRelevantObjects()
{
	IRIS_PROFILER_SCOPE(FReplicationFiltering_FilterNonRelevantObjects);

	if (bCVarRepFilterValidateNoSubObjectInScopeWithFilteredOutRootObject)
	{
		ValidConnections.ForAllSetBits([this](uint32 ConnectionIndex)
			{
				ensureMsgf(!HasSubObjectInScopeWithFilteredOutRootObject(ConnectionIndex), TEXT("Connection %u has orphaned subobjects."), ConnectionIndex);
			});
	}

	if (!bCVarRepFilterCullNonRelevant)
	{
		// Make every object in the global scope part of the relevant list
		NetRefHandleManager->GetRelevantObjectsInternalIndices().Copy(NetRefHandleManager->GetCurrentFrameScopableInternalIndices());
		return;
	}

	// Start by filling the relevant object list with those considered AlwaysRelevant.
	FNetBitArrayView GlobalRelevantObjects = NetRefHandleManager->GetRelevantObjectsInternalIndices();
	BuildAlwaysRelevantList(GlobalRelevantObjects, NetRefHandleManager->GetCurrentFrameScopableInternalIndices());

	//$IRIS TODO: Should sample if it's faster for the AlwaysRelevant list to be kept cached instead of recalculating it here every frame.

	// Build the list of currently relevant objects. e.g. always relevant objects + filterable objects relevant to at least one connection.	
	auto MergeConnectionScopes = [this, &GlobalRelevantObjects](uint32 ConnectionId)
	{
		const FNetBitArrayView ConnectionFilteredScope = MakeNetBitArrayView(ConnectionInfos[ConnectionId].ObjectsInScope);
		GlobalRelevantObjects.Combine(ConnectionFilteredScope, FNetBitArray::OrOp);
	};
	
	ValidConnections.ForAllSetBits(MergeConnectionScopes);

	//$IRIS TODO: Need to ensure newly relevant objects are immediately polled similar to calling ForceNetUpdate.
}

void FReplicationFiltering::BuildAlwaysRelevantList(FNetBitArrayView OutAlwaysRelevantList, const FNetBitArrayView ScopeList) const
{
	const uint32 MaxWords = OutAlwaysRelevantList.GetNumWords();

	// The list of all replicated objects
	const uint32* const ScopeListData = ScopeList.GetDataChecked(MaxWords);

	// The different list of filtered objects
	const uint32* const WithOwnerData = ObjectsWithOwnerFilter.GetDataChecked(MaxWords);
	const uint32* const DynamicFilteredData = DynamicFilterEnabledObjects.GetDataChecked(MaxWords);

	const FNetBitArrayView GroupFilteredOutView = Groups->GetGroupFilteredOutObjects();
	const uint32* const GroupFilteredOutData = GroupFilteredOutView.GetDataChecked(MaxWords);

	uint32* OutAlwaysRelevantListData = OutAlwaysRelevantList.GetDataChecked(MaxWords);

	for (uint32 WordIndex = 0; WordIndex < MaxWords; ++WordIndex)
	{
		// Build the list of always relevant objects, e.g. objects that have no filters
		OutAlwaysRelevantListData[WordIndex] = ScopeListData[WordIndex] & ~(WithOwnerData[WordIndex] | DynamicFilteredData[WordIndex] | GroupFilteredOutData[WordIndex]);
	}
}

/** Dynamic filters allows users to filter out objects based on arbitrary criteria. */
void FReplicationFiltering::UpdateDynamicFilters()
{
	NotifyFiltersOfDirtyObjects();

	PreUpdateDynamicFiltering();
	UpdateDynamicFiltering();
	PostUpdateDynamicFiltering();
}

void FReplicationFiltering::SetOwningConnection(FInternalNetRefIndex ObjectIndex, uint32 ConnectionId)
{
	// We allow valid connections as well as 0, which would prevent the object from being replicated to anyone.
	if ((ConnectionId != 0U) && !Connections->IsValidConnection(ConnectionId))
	{
		UE_LOGF(LogIrisFiltering, Warning, "Trying to set unknown owning connection on object %u. Connection: %u", ObjectIndex, ConnectionId);
		return;
	}

	const uint16 OldConnectionId = ObjectIndexToOwningConnection[ObjectIndex];
	ObjectIndexToOwningConnection[ObjectIndex] = static_cast<uint16>(ConnectionId);
	if (ConnectionId != OldConnectionId)
	{
		bHasDirtyOwner = 1;
		ObjectsWithDirtyOwner.SetBit(ObjectIndex);
		if (HasOwnerFilter(ObjectIndex))
		{
			bHasDirtyOwnerFilter = 1;
			ObjectsWithDirtyOwnerFilter.SetBit(ObjectIndex);
		}
	}
}

bool FReplicationFiltering::SetFilter(FInternalNetRefIndex ObjectIndex, FNetObjectFilterHandle Filter, FName FilterConfigProfile)
{
	if (Filter == ConnectionFilterHandle)
	{
		ensureMsgf(false, TEXT("Use SetConnectionFilter to enable connection filtering of objects. Cause of ensure must be fixed!"));
		return false;
	}

	UE_LOGF(LogIrisFiltering, Verbose, "Setting filter %ls to %ls (profile %ls)", *GetFilterName(Filter).ToString(), *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), *FilterConfigProfile.ToString());

	const bool bWantsToUseDynamicFilter = FNetObjectFilterHandleUtil::IsDynamicFilter(Filter);
	const uint8 OldDynamicFilterIndex = ObjectIndexToDynamicFilterIndex[ObjectIndex];
	const uint32 NewDynamicFilterIndex = bWantsToUseDynamicFilter ? FNetObjectFilterHandleUtil::GetDynamicFilterIndex(Filter) : InvalidDynamicFilterIndex;
	const bool bWasUsingDynamicFilter = OldDynamicFilterIndex != InvalidDynamicFilterIndex;

	// Validate the filter
	if (bWantsToUseDynamicFilter && !IsValidDynamicFilter(NewDynamicFilterIndex))
	{
		ensureMsgf(false, TEXT("Invalid dynamic filter 0x%08X. Filter is not being changed. Cause of ensure must be fixed!"), NewDynamicFilterIndex);
		return false;
	}
	else if (!bWantsToUseDynamicFilter && (Filter != InvalidNetObjectFilterHandle) && (Filter != ToOwnerFilterHandle))
	{
		ensureMsgf(false, TEXT("Invalid static filter 0x%08X. Filter is not being changed. Cause of ensure must be fixed!"), Filter);
		return false;
	}

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
	const TNetChunkedArray<uint8*>& ReplicatedObjectsStateBuffers = NetRefHandleManager->GetReplicatedObjectStateBuffers();
	// Let subobjects be filtered like their owners.
	if (bWantsToUseDynamicFilter && (ObjectData.SubObjectRootIndex != InvalidInternalNetRefIndex))
	{
		UE_LOGF(LogIrisFiltering, Warning, "Cannot set dynamic filters on subobjects. Filter change for %ls is ignored", *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex));
		return false;
	}

	auto ClearAndDirtyStaticFilter = [this](uint32 ObjIndex)
	{
		this->bHasDirtyOwnerFilter = 1;

		this->ObjectsWithOwnerFilter.ClearBit(ObjIndex);
		this->ObjectsWithDirtyOwnerFilter.SetBit(ObjIndex);
		this->FreePerObjectInfoForObject(ObjIndex);
	};

	const FInternalNetRefIndexManager NetRefIndexManager(*NetRefHandleManager);

	auto TrySetDynamicFilter = [this, &ObjectData, &ReplicatedObjectsStateBuffers, FilterConfigProfile, NetRefIndexManager](uint32 ObjIndex, uint32 FilterIndex)
	{
		FNetObjectFilteringInfo& NetObjectFilteringInfo = this->NetObjectFilteringInfos[ObjIndex];
		NetObjectFilteringInfo = {};
		
		FNetObjectFilterAddObjectParams AddParams
		{ 
			.OutInfo = NetObjectFilteringInfo,
			.ProfileName = FilterConfigProfile,
			.InstanceProtocol = ObjectData.InstanceProtocol,
			.Protocol = ObjectData.Protocol,
			.NetRefIndexManager = NetRefIndexManager,
			.StateBuffer = ReplicatedObjectsStateBuffers[ObjIndex] 
		};

		FFilterInfo& FilterInfo = this->DynamicFilterInfos[FilterIndex];
		if (FilterInfo.Filter->AddObject(ObjIndex, AddParams))
		{
			++FilterInfo.ObjectCount;
			FilterInfo.Filter->GetFilteredObjects().SetBit(ObjIndex);
			this->ObjectIndexToDynamicFilterIndex[ObjIndex] = static_cast<uint8>(FilterIndex);
			this->DynamicFilterEnabledObjects.SetBit(ObjIndex);
			this->ObjectScopeHysteresisFrameCounts[ObjIndex] = this->GetObjectScopeHysteresisFrameCount(FilterConfigProfile);
			return true;
		}
		
		return false;
	};

	// Clear dynamic filter info if needed
	if (bWasUsingDynamicFilter)
	{
		RemoveFromDynamicFilter(ObjectIndex, OldDynamicFilterIndex);
	}
	// Clear static filter info if needed
	else
	{
		ClearAndDirtyStaticFilter(ObjectIndex);
	}

	// Set dynamic filter
	if (bWantsToUseDynamicFilter)
	{
		const bool bSuccess = TrySetDynamicFilter(ObjectIndex, NewDynamicFilterIndex);
		if (bSuccess)
		{
			return true;
		}
		else
		{
			UE_LOGF(LogIrisFiltering, Verbose, "Filter '%ls' does not support object %ls.", ToCStr(DynamicFilterInfos[NewDynamicFilterIndex].Filter->GetFName().GetPlainNameString()), *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex));
			return false;
		}
	}
	// Set static filter
	else
	{
		if (Filter == InvalidNetObjectFilterHandle)
		{
			return true;
		}
		else if (Filter == ToOwnerFilterHandle)
		{
			ObjectsWithOwnerFilter.SetBit(ObjectIndex);
			return true;
		}

		// The validity of the filter has already been verified. How did we end up here?
		check(false);
	}

	// Unknown filter
	return false;
}

bool FReplicationFiltering::IsUsingSpatialFilter(FInternalNetRefIndex ObjectIndex) const
{
	const uint8 DynamicFilterIndex = ObjectIndexToDynamicFilterIndex[ObjectIndex];
	if (!IsValidDynamicFilter(DynamicFilterIndex))
	{
		return false;
	}

	const FFilterInfo& FilterInfo = DynamicFilterInfos[DynamicFilterIndex];
	const bool bIsUsingSpatialFilter = EnumHasAnyFlags(FilterInfo.Filter->GetFilterTraits(), ENetFilterTraits::Spatial);
	return bIsUsingSpatialFilter;
}

FNetObjectFilterHandle FReplicationFiltering::GetFilterHandle(const FName FilterName) const
{
	for (const FFilterInfo& Info : DynamicFilterInfos)
	{
		if (Info.Name == FilterName)
		{
			return FNetObjectFilterHandleUtil::MakeDynamicFilterHandle(static_cast<uint32>(&Info - DynamicFilterInfos.GetData()));
		}
	}

	return InvalidNetObjectFilterHandle;
}

UNetObjectFilter* FReplicationFiltering::GetFilter(const FName FilterName) const
{
	for (const FFilterInfo& Info : DynamicFilterInfos)
	{
		if (Info.Name == FilterName)
		{
			return Info.Filter.Get();
		}
	}

	return nullptr;
}

FName FReplicationFiltering::GetFilterName(FNetObjectFilterHandle Filter) const
{
	if (FNetObjectFilterHandleUtil::IsDynamicFilter(Filter))
	{
		const uint32 DynamicFilterIndex = FNetObjectFilterHandleUtil::GetDynamicFilterIndex(Filter);
		return DynamicFilterInfos[DynamicFilterIndex].Name;
	}

	return GetStaticFilterName(Filter);
}

// Connection handling
void FReplicationFiltering::AddConnection(uint32 ConnectionId, ENetConnectionTraits NetConnectionTraits)
{
	bHasNewConnection = 1;
	ValidConnections.SetBit(ConnectionId);
	NewConnections.SetBit(ConnectionId);

	UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::AddConnection ConnectionId: %u", ConnectionId);

	for (FFilterInfo& Info : DynamicFilterInfos)
	{
		if (Info.Filter)
		{
			Info.Filter->AddConnection(ConnectionId);
		}
	}

	// We defer needed processing to InitNewConnections().
}

void FReplicationFiltering::RemoveConnection(uint32 ConnectionId)
{
	bHasRemovedConnection = 1U;
	ValidConnections.ClearBit(ConnectionId);
	// If for whatever reason this connection is removed before we've done any new connection processing.
	NewConnections.ClearBit(ConnectionId);

	UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::RemoveConnection ConnectionId: %u", ConnectionId);
	
	// Reset connection info
	FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
	ConnectionInfo.Deinit();

	for (FFilterInfo& Info : DynamicFilterInfos)
	{
		if (Info.Filter)
		{
			Info.Filter->RemoveConnection(ConnectionId);
		}
	}

	// Reset SubObject filter for removed connection
	FConnectionHandle ConnectionHandle(ConnectionId);
	SubObjectFilterGroups.ForAllSetBits([this, ConnectionId, ConnectionHandle](uint32 GroupIndex)
	{
		if (!FNetObjectGroupHandle::IsReservedNetObjectGroupIndex(static_cast<FNetObjectGroupHandle::FGroupIndexType>(GroupIndex)))
		{
			// Clear the connection specific info in the group filter info and disallow replication.
			if (FPerSubObjectFilterGroupInfo* GroupInfo = GetPerSubObjectFilterGroupInfo(static_cast<FNetObjectGroupHandle::FGroupIndexType>(GroupIndex)))
			{
				GroupInfo->ConnectionFilterStatus.RemoveConnection(ConnectionHandle);
				SetConnectionFilterStatus(*GetPerObjectInfo(GroupInfo->ConnectionStateIndex), ConnectionId, ENetFilterStatus::Disallow);
				DirtySubObjectFilterGroups.SetBit(GroupIndex);
			}
		}
	});
}

void FReplicationFiltering::InitNewConnections()
{
	if (!bHasNewConnection)
	{
		return;
	}

	IRIS_PROFILER_SCOPE(FReplicationFiltering_InitNewConnections);

	bHasNewConnection = 0;
	
	auto InitNewConnection = [this](uint32 ConnectionId)
	{
		// Copy default scope
		const FNetBitArrayView ScopableInternalIndices = NetRefHandleManager->GetCurrentFrameScopableInternalIndices();
		FPerConnectionInfo& ConnectionInfo = this->ConnectionInfos[ConnectionId];

		const FInternalNetRefIndex CurrentMaxInternalindex = NetRefHandleManager->GetCurrentMaxInternalNetRefIndex();

		this->SetPerConnectionListsSize(ConnectionInfo, CurrentMaxInternalindex);

		ConnectionInfo.ConnectionFilteredObjects.Copy(ScopableInternalIndices);
		ConnectionInfo.ConnectionFilteredObjects.ClearBit(InvalidInternalNetRefIndex);

		// Update group exclusion filtering
		{
			auto InitExclusionGroupFilterForConnection = [this, &ConnectionInfo, ConnectionId](uint32 InGroupIndex)
			{
				const FNetObjectGroupHandle::FGroupIndexType GroupIndex = static_cast<FNetObjectGroupHandle::FGroupIndexType>(InGroupIndex);
				const FPerObjectInfo* ConnectionState = GetPerObjectInfo(this->GroupInfos[GroupIndex].ConnectionStateIndex);

				// Setup exclusion filter for connection
				if (GetConnectionFilterStatus(*ConnectionState, ConnectionId) == ENetFilterStatus::Disallow)
				{
					// Apply filter
					const FNetObjectGroup* Group = this->Groups->GetGroupFromIndex(GroupIndex);
					FNetBitArray& GroupExcludedObjects = ConnectionInfo.GroupExcludedObjects;
					for (const FInternalNetRefIndex ObjectIndex : Group->Members)
					{
						GroupExcludedObjects.SetBit(ObjectIndex);

						// Filter subobjects
						for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
						{
							GroupExcludedObjects.SetBit(SubObjectIndex);
						}
					}
				}
			};

			ExclusionFilterGroups.ForAllSetBits(InitExclusionGroupFilterForConnection);
		}

		// Update group inclusion filtering
		{
			auto InitInclusionGroupFilterForConnection = [this, &ConnectionInfo, ConnectionId](uint32 InGroupIndex)
			{
				const FNetObjectGroupHandle::FGroupIndexType GroupIndex = static_cast<FNetObjectGroupHandle::FGroupIndexType>(InGroupIndex);
				const FPerObjectInfo* ConnectionState = GetPerObjectInfo(this->GroupInfos[GroupIndex].ConnectionStateIndex);

				// Setup inclusion filter for connection
				if (GetConnectionFilterStatus(*ConnectionState, ConnectionId) == ENetFilterStatus::Allow)
				{
					const FNetBitArray& ObjectsInScope = ConnectionInfo.ConnectionFilteredObjects;
					const FNetBitArrayView SubObjectInternalIndices = NetRefHandleManager->GetSubObjectInternalIndicesView();

					// Apply filter
					const FNetObjectGroup* Group = this->Groups->GetGroupFromIndex(GroupIndex);
					FNetBitArray& GroupIncludedObjects = ConnectionInfo.GroupIncludedObjects;
					for (const FInternalNetRefIndex ObjectIndex : Group->Members)
					{
						// SubObjects follow root object.
						if (UNLIKELY(SubObjectInternalIndices.GetBit(ObjectIndex)))
						{
							continue;
						}

						GroupIncludedObjects.SetBit(ObjectIndex);

						// Filter subobjects
						for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
						{
							GroupIncludedObjects.SetBit(SubObjectIndex);
						}
					}
				}
			};

			InclusionFilterGroups.ForAllSetBits(InitInclusionGroupFilterForConnection);
		}

		// Update connection scope with owner filtering
		{
			auto MaskObjectToOwner = [this, ConnectionId, &ConnectionInfo](uint32 ObjectIndex)
			{
				const bool bIsOwner = (ConnectionId == ObjectIndexToOwningConnection[ObjectIndex]);
				ConnectionInfo.ConnectionFilteredObjects.SetBitValue(ObjectIndex, bIsOwner);
				for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
				{
					ConnectionInfo.ConnectionFilteredObjects.SetBitValue(SubObjectIndex, bIsOwner);
				}
			};

			ObjectsWithOwnerFilter.ForAllSetBits(MaskObjectToOwner);
		}

		// Update connection scope with connection filtering
		{
			auto MaskObjectToConnection = [this, ConnectionId, &ConnectionInfo](uint32 ObjectIndex)
			{
				const PerObjectInfoIndexType ObjectInfoIndex = ObjectIndexToPerObjectInfoIndex[ObjectIndex];
				const FPerObjectInfo* ObjectInfo = GetPerObjectInfo(ObjectInfoIndex);
				const ENetFilterStatus ReplicationStatus = GetConnectionFilterStatus(*ObjectInfo, ConnectionId);
				const bool bIsAllowedToReplicate = (ReplicationStatus == ENetFilterStatus::Allow);
				ConnectionInfo.ConnectionFilteredObjects.SetBitValue(ObjectIndex, bIsAllowedToReplicate);
				for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
				{
					ConnectionInfo.ConnectionFilteredObjects.SetBitValue(SubObjectIndex, bIsAllowedToReplicate);
				}
			};

			ObjectsWithPerObjectInfo.ForAllSetBits(MaskObjectToConnection);
		}

		// Combine connection and group filtering
		ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.Copy(ConnectionInfo.ConnectionFilteredObjects);
		ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.Combine(ConnectionInfo.GroupExcludedObjects, FNetBitArray::AndNotOp);
	};

	NewConnections.ForAllSetBits(InitNewConnection);
}

void FReplicationFiltering::ResetRemovedConnections()
{
	if (!bHasRemovedConnection)
	{
		return;
	}

	IRIS_PROFILER_SCOPE(FReplicationFiltering_ResetRemovedConnections);

	bHasRemovedConnection = 0;

	// Reset group filter status
	// We might want to introduce a way to specify default state for group filters
	// Currently we just clear the filter
	auto ResetGroupFilterStatus = [this](uint32 GroupIndex)
	{
		FPerObjectInfo* ConnectionStateInfo = GetPerObjectInfo(this->GroupInfos[GroupIndex].ConnectionStateIndex);

		// Special case we want to mask with the valid connections
		const uint32 NumWords = this->ValidConnections.GetNumWords();

		// We do not need to do anything more than to restore the state to the default as we reset the effects of the filter
		for (uint32 WordIt = 0; WordIt < NumWords; ++WordIt)
		{
			ConnectionStateInfo->ConnectionIds[WordIt] &= ValidConnections.GetWord(WordIt);
		}
	};

	FNetBitArray::ForAllSetBits(ExclusionFilterGroups, InclusionFilterGroups, FNetBitArrayBase::OrOp, ResetGroupFilterStatus);
}

void FReplicationFiltering::SetPerConnectionListsSize(FPerConnectionInfo& ConnectionInfo, FInternalNetRefIndex NewMaxInternalIndex)
{
	ConnectionInfo.ConnectionFilteredObjects.SetNumBits(NewMaxInternalIndex);

	// Do not filter out anything by default.
	ConnectionInfo.GroupExcludedObjects.SetNumBits(NewMaxInternalIndex);

	// Do not override dynamic filtering by default.
	ConnectionInfo.GroupIncludedObjects.SetNumBits(NewMaxInternalIndex);

	// The combined result of scoped objects, connection filtering and group exclusion filtering.
	ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.SetNumBits(NewMaxInternalIndex);

	// Creation dependents that have been filtered out because their parents are filtered.
	ConnectionInfo.FilteredOutCreationDependents.SetNumBits(NewMaxInternalIndex);

	// The final result of all filtering.
	ConnectionInfo.ObjectsInScope.SetNumBits(NewMaxInternalIndex);

	if (HasDynamicFilters())
	{
		ConnectionInfo.DynamicFilteredOutObjects.SetNumBits(NewMaxInternalIndex);
		ConnectionInfo.InProgressDynamicFilteredOutObjects.SetNumBits(NewMaxInternalIndex);
		ConnectionInfo.DynamicFilteredOutObjectsHysteresisAdjusted.SetNumBits(NewMaxInternalIndex);
		ConnectionInfo.HysteresisUpdater.OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
	}
}

void FReplicationFiltering::UpdateObjectsInScope()
{
	IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateObjectsInScope);

	const FNetBitArrayView ObjectsInScope = NetRefHandleManager->GetCurrentFrameScopableInternalIndices();
	const FNetBitArrayView PrevObjectsInScope = NetRefHandleManager->GetPrevFrameScopableInternalIndices();

	/**
	 * It's possible for an object to be created, have some filtering applied and then be removed later the same frame.
	 * We can detect it using our various dirty bit arrays and force deletion of filtering data associated with the object.
	 */
	FNetBitArray FakePrevObjectsInScope(ObjectsInScope.GetNumBits());

	TArray<FNetBitArrayBase::StorageWordType> ModifiedWords;
	ModifiedWords.SetNumUninitialized(WordCountForObjectBitArrays);

	uint32 ModifiedWordIndex = 0;
	uint32* ObjectsWithDirtyOwnerFilterStorage = ObjectsWithDirtyOwnerFilter.GetDataChecked(WordCountForObjectBitArrays);
	uint32* ObjectsWithDirtyOwnerStorage = ObjectsWithDirtyOwner.GetDataChecked(WordCountForObjectBitArrays);
	const uint32* ObjectsInScopeStorage = ObjectsInScope.GetDataChecked(WordCountForObjectBitArrays);
	uint32* FakePrevObjectsInScopeStorage = FakePrevObjectsInScope.GetDataChecked(WordCountForObjectBitArrays);
	{
		uint32* ModifiedWordsStorage = ModifiedWords.GetData();
		const uint32* PrevObjectsInScopeStorage = PrevObjectsInScope.GetDataChecked(WordCountForObjectBitArrays);

		for (uint32 WordIt = 0, WordEndIt = WordCountForObjectBitArrays; WordIt != WordEndIt; ++WordIt)
		{
			const uint32 ObjectsInScopeWord = ObjectsInScopeStorage[WordIt];
			const uint32 PrevObjectsInScopeWord = PrevObjectsInScopeStorage[WordIt];
			const uint32 ObjectsWithDirtyOwnerFilterWord = ObjectsWithDirtyOwnerFilterStorage[WordIt];
			const uint32 ObjectsWithDirtyOwnerWord = ObjectsWithDirtyOwnerStorage[WordIt];
			const uint32 SameFrameRemovedWord = ~(ObjectsInScopeWord | PrevObjectsInScopeWord) & (ObjectsWithDirtyOwnerFilterWord | ObjectsWithDirtyOwnerWord);

			// Pretend that same frame removed objects existed in the previous frame.
			FakePrevObjectsInScopeStorage[WordIt] = PrevObjectsInScopeWord | SameFrameRemovedWord;

			// Store modified word index to later avoid iterating over unchanged words.
			const bool bWordDiffers = ((ObjectsInScopeWord ^ PrevObjectsInScopeWord) | SameFrameRemovedWord) != 0U;
			ModifiedWordsStorage[ModifiedWordIndex] = WordIt;
			ModifiedWordIndex += bWordDiffers;
		}
	}

	// If the scope didn't change there's nothing for us to do.
	if (ModifiedWordIndex == 0)
	{
		return;
	}

	// Clear info for deleted objects and dirty filter information for added objects.
	{
		uint32* ObjectsWithOwnerFilterStorage = ObjectsWithOwnerFilter.GetDataChecked(WordCountForObjectBitArrays);
		uint32* ObjectsExemptFromHysteresisStorage = HysteresisState.ObjectsExemptFromHysteresis.GetDataChecked(WordCountForObjectBitArrays);
		const FNetBitArrayView SubObjectInternalIndices = NetRefHandleManager->GetSubObjectInternalIndicesView();
		uint32 PrevParentIndex = InvalidInternalNetRefIndex;
		for (uint32 WordIt = 0, WordEndIt = ModifiedWordIndex; WordIt != WordEndIt; ++WordIt)
		{
			const uint32 WordIndex = ModifiedWords[WordIt];
			const uint32 PrevExistingObjects = FakePrevObjectsInScopeStorage[WordIndex];
			const uint32 ExistingObjects = ObjectsInScopeStorage[WordIndex];

			// Deleted objects can't be dirty and can't have filtering.
			ObjectsWithDirtyOwnerFilterStorage[WordIndex] &= ExistingObjects;
			ObjectsWithOwnerFilterStorage[WordIndex] &= ExistingObjects;
			ObjectsWithDirtyOwnerStorage[WordIndex] &= ExistingObjects;

			// Clear dynamic filters and owner info from deleted objects.
			const uint32 BitOffset = WordIndex*32U;
			uint32 DeletedObjects = (PrevExistingObjects & ~ExistingObjects);
			for ( ; DeletedObjects; )
			{
				const uint32 LeastSignificantBit = GetLeastSignificantBit(DeletedObjects);
				DeletedObjects ^= LeastSignificantBit;

				const uint32 ObjectIndex = BitOffset + FPlatformMath::CountTrailingZeros(LeastSignificantBit);

				// Free per object info if present
				FreePerObjectInfoForObject(ObjectIndex);

				// Remove dynamic filtering
				const uint32 DynamicFilterIndex = ObjectIndexToDynamicFilterIndex[ObjectIndex];
				if (DynamicFilterIndex != InvalidDynamicFilterIndex)
				{
					RemoveFromDynamicFilter(ObjectIndex, DynamicFilterIndex);
				}
			}

			const uint32 AddedObjects = ExistingObjects & ~PrevExistingObjects;

			// Prevent hysteresis from kicking in on just added objects.
			ObjectsExemptFromHysteresisStorage[WordIndex] |= AddedObjects;

			// Make sure subobjects that are added after the parent gets properly updated.
			// Dirtying the parent will cause the subobjects to be updated too.
			if (uint32 AddedSubObjects = AddedObjects & SubObjectInternalIndices.GetWord(WordIndex))
			{
				for ( ; AddedSubObjects; )
				{
					const uint32 LeastSignificantBit = GetLeastSignificantBit(AddedSubObjects);
					AddedSubObjects ^= LeastSignificantBit;

					const uint32 ObjectIndex = BitOffset + FPlatformMath::CountTrailingZeros(LeastSignificantBit);
					const uint32 ParentIndex = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex).SubObjectRootIndex;
					
					if (ParentIndex == PrevParentIndex || ParentIndex == InvalidInternalNetRefIndex)
					{
						continue;
					}

					// We need to make sure the subobject gets the same filter status as the parent.
					ObjectsRequiringDynamicFilterUpdate.SetBit(ParentIndex);

					PrevParentIndex = ParentIndex;

					// If parent is a member of a group filter we need to refresh group filtering to include subobject
					const TArrayView<const FNetObjectGroupHandle::FGroupIndexType> GroupIndexes = Groups->GetGroupIndexesOfNetObject(ParentIndex);
					for (const FNetObjectGroupHandle::FGroupIndexType GroupIndex : GroupIndexes)
					{
						if (ExclusionFilterGroups.GetBit(GroupIndex))
						{
							DirtyExclusionFilterGroups.SetBit(GroupIndex);
							bHasDirtyExclusionFilterGroup = 1;
						}
						else if (InclusionFilterGroups.GetBit(GroupIndex))
						{
							DirtyInclusionFilterGroups.SetBit(GroupIndex);
							bHasDirtyInclusionFilterGroup = 1;
						}
					}

					// If the parent is new as well we don't have to do anything, but if it already existed
					// we may need to do some work if it has owner or filtering info.
					if (!PrevObjectsInScope.GetBit(ParentIndex))
					{
						continue;
					}

					if (ObjectIndexToOwningConnection[ParentIndex])
					{
						bHasDirtyOwner = 1;
						ObjectsWithDirtyOwner.SetBit(ParentIndex);
					}

					if (HasOwnerFilter(ParentIndex))
					{
						bHasDirtyOwnerFilter = 1;
						// Updating the parent will update all subobjects.
						ObjectsWithDirtyOwnerFilter.SetBit(ParentIndex);
					}
				}
			}
		}
	}

	// Update the scope for all valid connections.
	for (uint32 ConnectionId : ValidConnections)
	{
		FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];

		// Or in brand new objects and mask off deleted objects by anding with now existing objects.
		uint32* FilteredObjectsStorage = ConnectionInfo.ConnectionFilteredObjects.GetDataChecked(WordCountForObjectBitArrays);
		uint32* GroupExcludedObjectsStorage = ConnectionInfo.GroupExcludedObjects.GetDataChecked(WordCountForObjectBitArrays);
		uint32* GroupIncludedObjectsStorage = ConnectionInfo.GroupIncludedObjects.GetDataChecked(WordCountForObjectBitArrays);
		uint32* ObjectsInScopeBeforeDynamicFiltering = ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetDataChecked(WordCountForObjectBitArrays);
		uint32* DynamicFilteredOutObjects = HasDynamicFilters() ? ConnectionInfo.DynamicFilteredOutObjects.GetDataChecked(WordCountForObjectBitArrays) : nullptr;
		uint32* DynamicFilteredOutObjectsHysteresisAdjusted = HasDynamicFilters() ? ConnectionInfo.DynamicFilteredOutObjectsHysteresisAdjusted.GetDataChecked(WordCountForObjectBitArrays) : nullptr;

		for (uint32 WordIt = 0, WordEndIt = ModifiedWordIndex; WordIt != WordEndIt; ++WordIt)
		{
			const uint32 WordIndex = ModifiedWords[WordIt];

			const uint32 PrevExistingObjects = FakePrevObjectsInScopeStorage[WordIndex];
			const uint32 ExistingObjects = ObjectsInScopeStorage[WordIndex];
			const uint32 NewObjects = ExistingObjects & ~PrevExistingObjects;

			const uint32 FilteredObjectsWord = (FilteredObjectsStorage[WordIndex] | NewObjects) & ExistingObjects;
			FilteredObjectsStorage[WordIndex] = FilteredObjectsWord;

			const uint32 GroupExcludedObjectsWord = GroupExcludedObjectsStorage[WordIndex] & ExistingObjects;
			GroupExcludedObjectsStorage[WordIndex] = GroupExcludedObjectsWord;

			const uint32 GroupIncludedObjectsWord = GroupIncludedObjectsStorage[WordIndex] & ExistingObjects;
			GroupIncludedObjectsStorage[WordIndex] = GroupIncludedObjectsWord;

			// Note that we only filter out objects from exclusion groups here. Inclusion groups only overrides dynamic filtering.
			ObjectsInScopeBeforeDynamicFiltering[WordIndex] = FilteredObjectsWord & ~GroupExcludedObjectsWord;

			// Make sure to reset dynamic filtering for new objects. These can have been subobjects to other dynamically filtered objects.
			if (HasDynamicFilters())
			{
				DynamicFilteredOutObjects[WordIndex] &= ~NewObjects;
				DynamicFilteredOutObjectsHysteresisAdjusted[WordIndex] &= ~NewObjects;
			}
		}
	}
}

uint32 FReplicationFiltering::GetOwningConnectionIfDirty(uint32 ObjectIndex) const
{
	// If this is a subobject we must check if our parent is updated or not
	if (NetRefHandleManager->GetSubObjectInternalIndices().GetBit(ObjectIndex))
	{
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
		const uint32 ParentIndex = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex).SubObjectRootIndex;
		return ObjectIndexToOwningConnection[ParentIndex];
	}
	
	return ObjectIndexToOwningConnection[ObjectIndex];
}

void FReplicationFiltering::UpdateOwnerFiltering()
{
	if (!(bHasDirtyOwner || bHasDirtyOwnerFilter))
	{
		return;
	}

	// Update owners
	if (bHasDirtyOwner)
	{
		IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateDirtyOwnerValue);

		auto UpdateOwners = [this](uint32 ObjectIndex)
		{
			const uint32 OwningConnectionId = ObjectIndexToOwningConnection[ObjectIndex];

			for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
			{
				const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(SubObjectIndex);
				ObjectIndexToOwningConnection[SubObjectIndex] = static_cast<uint16>(OwningConnectionId);
			}
		};

		ObjectsWithDirtyOwner.ForAllSetBits(UpdateOwners);
	}

	const FNetBitArrayView CurrentFrameObjectsInScope = NetRefHandleManager->GetCurrentFrameScopableInternalIndices();

	// Update filtering
	if (bHasDirtyOwnerFilter)
	{
		IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateDirtyOwnerFilter);

		auto UpdateConnectionScope = [this, &CurrentFrameObjectsInScope](uint32 ConnectionId)
		{
			FPerConnectionInfo& ConnectionInfo = this->ConnectionInfos[ConnectionId];
			FNetBitArrayView ConnectionScope = MakeNetBitArrayView(ConnectionInfo.ConnectionFilteredObjects);
			FNetBitArrayView GroupExcludedObjects = MakeNetBitArrayView(ConnectionInfo.GroupExcludedObjects);
			FNetBitArrayView ObjectsInScopeBeforeDynamicFiltering = MakeNetBitArrayView(ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering);

			// Update filter info
			{
				auto MaskObject = [this, &ConnectionScope, &GroupExcludedObjects, &ObjectsInScopeBeforeDynamicFiltering, ConnectionId, &CurrentFrameObjectsInScope](uint32 ObjectIndex)
				{
					bool bObjectIsInScope = true;
					if (HasOwnerFilter(ObjectIndex))
					{
						const uint32 OwningConnection = ObjectIndexToOwningConnection[ObjectIndex];
						const bool bIsOwner = (ConnectionId == OwningConnection);
						bObjectIsInScope = bIsOwner;
					}

					// Update scope for parent object.
					{
						const bool bIsGroupEnabled = !GroupExcludedObjects.GetBit(ObjectIndex);
						ConnectionScope.SetBitValue(ObjectIndex, bObjectIsInScope);
						ObjectsInScopeBeforeDynamicFiltering.SetBitValue(ObjectIndex, bObjectIsInScope & bIsGroupEnabled);
					}

					// Subobjects follow suit.
					for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
					{
						const bool bEnableObject = bObjectIsInScope && CurrentFrameObjectsInScope.GetBit(SubObjectIndex);
						const bool bIsGroupEnabled = !GroupExcludedObjects.GetBit(SubObjectIndex);

						ConnectionScope.SetBitValue(SubObjectIndex, bEnableObject);
						ObjectsInScopeBeforeDynamicFiltering.SetBitValue(SubObjectIndex, bEnableObject & bIsGroupEnabled);
					}
				};

				ObjectsWithDirtyOwnerFilter.ForAllSetBits(MaskObject);
			}
		};

		ValidConnections.ForAllSetBits(UpdateConnectionScope);
	}
	
	// Clear out dirtiness
	bHasDirtyOwnerFilter = 0;
	bHasDirtyOwner = 0;
	ObjectsWithDirtyOwnerFilter.ClearAllBits();
	ObjectsWithDirtyOwner.ClearAllBits();
}

void FReplicationFiltering::UpdateGroupExclusionFiltering()
{
	if (!bHasDirtyExclusionFilterGroup)
	{
		return;
	}

	// Adding objects to an active group filter is deferred in order to avoid triggering constant filter updates.
	auto UpdateGroupFilter = [this](uint32 GroupIndex)
	{
		IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateGroupExclusionFiltering);

		const FPerObjectInfo* ConnectionStateInfo = GetPerObjectInfo(this->GroupInfos[GroupIndex].ConnectionStateIndex);
		const FNetObjectGroup* Group = this->Groups->GetGroupFromIndex(FNetObjectGroupHandle::FGroupIndexType(GroupIndex));

		const FNetBitArrayView CurrentFrameScopableObjects = NetRefHandleManager->GetCurrentFrameScopableInternalIndices();

		auto UpdateGroupFilterForConnection = [this, ConnectionStateInfo, Group, CurrentFrameScopableObjects](uint32 ConnectionId)
		{
			if (this->GetConnectionFilterStatus(*ConnectionStateInfo, ConnectionId) == ENetFilterStatus::Disallow)
			{
				FNetBitArray& GroupExcludedObjects = ConnectionInfos[ConnectionId].GroupExcludedObjects;
				FNetBitArray& ObjectsInScopeBeforeDynamicFiltering = ConnectionInfos[ConnectionId].ObjectsInScopeBeforeDynamicFiltering;

				for (const FInternalNetRefIndex ObjectIndex : Group->Members)
				{
					GroupExcludedObjects.SetBit(ObjectIndex);
					ObjectsInScopeBeforeDynamicFiltering.ClearBit(ObjectIndex);

					// Filter subobjects
					for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
					{
						const bool bIsScopable = CurrentFrameScopableObjects.GetBit(SubObjectIndex);
						GroupExcludedObjects.SetBitValue(SubObjectIndex, bIsScopable);
						ObjectsInScopeBeforeDynamicFiltering.ClearBit(SubObjectIndex);
					}
				}
			}

			ensureMsgf(!bCVarRepFilterValidateNoSubObjectInScopeWithFilteredOutRootObject || !HasSubObjectInScopeWithFilteredOutRootObject(MakeNetBitArrayView(ConnectionInfos[ConnectionId].GroupExcludedObjects)), TEXT("UpdateGroupExclusionFiltering GroupExcludedObjects"));
		};

		this->ValidConnections.ForAllSetBits(UpdateGroupFilterForConnection);
	};

	DirtyExclusionFilterGroups.ForAllSetBits(UpdateGroupFilter);

	// Clear out dirtiness
	bHasDirtyExclusionFilterGroup = 0;
	DirtyExclusionFilterGroups.ClearAllBits();
}

void FReplicationFiltering::UpdateGroupInclusionFiltering()
{
	if (!bHasDirtyInclusionFilterGroup)
	{
		return;
	}

	auto UpdateGroupFilter = [this](uint32 GroupIndex)
	{
		IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateGroupInclusionFiltering);

		const FPerObjectInfo* ConnectionStateInfo = GetPerObjectInfo(this->GroupInfos[GroupIndex].ConnectionStateIndex);
		const FNetObjectGroup* Group = this->Groups->GetGroupFromIndex(FNetObjectGroupHandle::FGroupIndexType(GroupIndex));

		const FNetBitArrayView CurrentFrameScopableObjects = NetRefHandleManager->GetCurrentFrameScopableInternalIndices();
		const FNetBitArrayView SubObjectInternalIndices = NetRefHandleManager->GetSubObjectInternalIndicesView();

		auto UpdateGroupFilterForConnection = [this, ConnectionStateInfo, Group, &CurrentFrameScopableObjects, &SubObjectInternalIndices](uint32 ConnectionId)
		{
			if (this->GetConnectionFilterStatus(*ConnectionStateInfo, ConnectionId) == ENetFilterStatus::Allow)
			{
				FNetBitArray& GroupIncludedObjects = ConnectionInfos[ConnectionId].GroupIncludedObjects;

				for (const FInternalNetRefIndex ObjectIndex : Group->Members)
				{
					// SubObjects follow root object.
					if (UNLIKELY(SubObjectInternalIndices.GetBit(ObjectIndex)))
					{
						continue;
					}

					GroupIncludedObjects.SetBit(ObjectIndex);

					// Filter subobjects
					for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
					{
						GroupIncludedObjects.SetBit(SubObjectIndex);
					}
				}
			}
		};

		this->ValidConnections.ForAllSetBits(UpdateGroupFilterForConnection);
	};

	DirtyInclusionFilterGroups.ForAllSetBits(UpdateGroupFilter);

	// Clear out dirtiness
	bHasDirtyInclusionFilterGroup = 0;
	DirtyInclusionFilterGroups.ClearAllBits();
}

void FReplicationFiltering::PreUpdateDynamicFiltering()
{
	IRIS_PROFILER_SCOPE(FReplicationFiltering_PreUpdateDynamicFiltering);

	const FInternalNetRefIndexManager NetRefIndexManager(*NetRefHandleManager);

	// Give filters a chance to prepare for filtering. It's only called if any object has the filter set.
	{
		for (FFilterInfo& Info : DynamicFilterInfos)
		{
			if (!Info.Filter || Info.ObjectCount == 0U)
			{
				continue;
			}

			FNetObjectPreFilteringParams PreFilteringParams
			{
				.ValidConnections = MakeNetBitArrayView(ValidConnections),
				.FilteringInfos = MakeArrayView(NetObjectFilteringInfos),
				.NetRefIndexManager = NetRefIndexManager,
			};
			Info.Filter->PreFilter(PreFilteringParams);
		}
	}
}

void FReplicationFiltering::FilterOutCreationDependents(uint32 ParentIndex, FPerConnectionInfo& ConnectionInfo, const FNetBitArrayView ObjectsWithCreationDependents) const
{
	TArray<FInternalNetRefIndex, TInlineAllocator<64>> ObjectsToProcess;
	ObjectsToProcess.Add(ParentIndex);
	while (ObjectsToProcess.Num() > 0)
	{
		const FInternalNetRefIndex CurrentIndex = ObjectsToProcess.Pop(EAllowShrinking::No);

		TConstArrayView<const FInternalNetRefIndex> Children = NetRefHandleManager->GetCreationDependents(CurrentIndex);
		for (const FInternalNetRefIndex ChildIndex : Children)
		{
			if (ConnectionInfo.FilteredOutCreationDependents.GetBit(ChildIndex))
			{
				// This section of the dependency graph has already been filtered out, so we can skip processing it.
				continue;
			}

			// Filter out this child and its subobjects.
			ConnectionInfo.FilteredOutCreationDependents.SetBit(ChildIndex);
			for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ChildIndex))
			{
				ConnectionInfo.FilteredOutCreationDependents.SetBit(SubObjectIndex);
			}

			if (ObjectsWithCreationDependents.GetBit(ChildIndex))
			{
				// This child has dependents, so process them as well.
				ObjectsToProcess.Push(ChildIndex);
			}
		}
	}
}

void FReplicationFiltering::FilterInCreationDependents(uint32 ParentIndex, FPerConnectionInfo& ConnectionInfo, const FNetBitArrayView ObjectsWithCreationDependents) const
{
	TArray<FInternalNetRefIndex, TInlineAllocator<64>> ObjectsToProcess;
	ObjectsToProcess.Add(ParentIndex);
	while (ObjectsToProcess.Num() > 0)
	{
		const FInternalNetRefIndex CurrentParent = ObjectsToProcess.Pop(EAllowShrinking::No);

		TConstArrayView<const FInternalNetRefIndex> Children = NetRefHandleManager->GetCreationDependents(CurrentParent);
		for (const FInternalNetRefIndex ChildIndex : Children)
		{
			if (!ConnectionInfo.FilteredOutCreationDependents.GetBit(ChildIndex))
			{
				continue;
			}

			bool bFilteredOutByParent = false;
			TConstArrayView<const FInternalNetRefIndex> Dependencies = NetRefHandleManager->GetCreationDependencies(ChildIndex);
			for (const FInternalNetRefIndex DependencyIndex : Dependencies)
			{
				if (ConnectionInfo.FilteredOutCreationDependents.GetBit(DependencyIndex) || !ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetBit(DependencyIndex) || ConnectionInfo.DynamicFilteredOutObjectsHysteresisAdjusted.GetBit(DependencyIndex))
				{
					// This object has a parent that has been filtered out, so we can't filter it in.
					//
					// Note: In the case that an object has N>1 parents that would be filtered in by one call to FilterInCreationDependents,
					//       that object will be processed N times and only be filtered in the Nth time.
					// Example: W starts filtered out. X, Y, and Z would be relevant, but creation dependency filters them out.
					//          W is filtered in. Because of DFS, we process in the order W->Y->Z->X->Z. Z is only filtered in the 2nd time it is processed.
					//   W
					//  / \
					// X   Y
					//  \ /
					//   Z
					bFilteredOutByParent = true;
					break;
				}
			}

			if (!bFilteredOutByParent)
			{
				// Filter in this child and its subobjects.
				ConnectionInfo.FilteredOutCreationDependents.ClearBit(ChildIndex);
				for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ChildIndex))
				{
					ConnectionInfo.FilteredOutCreationDependents.ClearBit(SubObjectIndex);
				}

				if (ObjectsWithCreationDependents.GetBit(ChildIndex))
				{
					// This child has dependents, so process them as well.
					ObjectsToProcess.Add(ChildIndex);
				}
			}
		}
	}
}

void FReplicationFiltering::RefreshCreationDependencyFilterStatus(uint32 ObjectIndex, FPerConnectionInfo& ConnectionInfo, const FNetBitArrayView ObjectsWithCreationDependents) const
{
	if (NetRefHandleManager->IsSubObject(ObjectIndex))
	{
		// Subobjects should follow the state of their root
		// NOTE: We flag removed subobjects, but by the time this function is called, they are not subobjects. They will have their bit cleared in the !bHasFilteredParent block below.
		const uint32 RootIndex = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex).SubObjectRootIndex;
		const bool bIsRootFiltered = ConnectionInfo.FilteredOutCreationDependents.GetBit(RootIndex);
		ConnectionInfo.FilteredOutCreationDependents.SetBitValue(ObjectIndex, bIsRootFiltered);
		return;
	}

	bool bHasFilteredParent = false;
	TConstArrayView<const FInternalNetRefIndex> Parents = NetRefHandleManager->GetCreationDependencies(ObjectIndex);
	for (const FInternalNetRefIndex ParentIndex : Parents)
	{
		if (ConnectionInfo.FilteredOutCreationDependents.GetBit(ParentIndex) || !ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetBit(ParentIndex) || ConnectionInfo.DynamicFilteredOutObjectsHysteresisAdjusted.GetBit(ParentIndex))
		{
			bHasFilteredParent = true;
			break;
		}
	}

	const bool bWasFilteredCreationDependent = ConnectionInfo.FilteredOutCreationDependents.GetBit(ObjectIndex);
	if (bHasFilteredParent)
	{
		// This object has a filtered dependency, so it and its subobjects should be filtered by creation dependency.
		ConnectionInfo.FilteredOutCreationDependents.SetBit(ObjectIndex);
		for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
		{
			ConnectionInfo.FilteredOutCreationDependents.SetBit(SubObjectIndex);
		}

		// Only filter out dependents if this object wasn't already filtered out by creation dependents.
		if (ObjectsWithCreationDependents.GetBit(ObjectIndex) && !bWasFilteredCreationDependent)
		{
			FilterOutCreationDependents(ObjectIndex, ConnectionInfo, ObjectsWithCreationDependents);
		}
	}
	else
	{
		// All of this object's dependencies aren't filtered, so it and its subobjects shouldn't be filtered by creation dependency.
		ConnectionInfo.FilteredOutCreationDependents.ClearBit(ObjectIndex);
		for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
		{
			ConnectionInfo.FilteredOutCreationDependents.ClearBit(SubObjectIndex);
		}

		// Only filter in dependents if this object was filtered by creation dependency and its not currently filtered otherwise.
		if (ObjectsWithCreationDependents.GetBit(ObjectIndex) && bWasFilteredCreationDependent && ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetBit(ObjectIndex) && !ConnectionInfo.DynamicFilteredOutObjectsHysteresisAdjusted.GetBit(ObjectIndex))
		{
			FilterInCreationDependents(ObjectIndex, ConnectionInfo, ObjectsWithCreationDependents);
		}
	}
}

void FReplicationFiltering::UpdateCreationDependentParent(uint32 ChildIndex, const FNetBitArrayView ObjectsWithCreationDependencies, FNetBitArrayView OutConnectionObjectsInScope, FNetBitArrayView ProcessedObjects, bool bRecursive) const
{
	const bool bStartedRelevant = OutConnectionObjectsInScope.IsBitSet(ChildIndex);
	if (!bStartedRelevant)
	{
		// No need to check it's parents, the object must be relevant by itself first.
		return;
	}

	const bool bAlreadyProcessed = ProcessedObjects.IsBitSet(ChildIndex);
	if (bAlreadyProcessed)
	{
		return;
	}

	TConstArrayView<const FInternalNetRefIndex> Parents = NetRefHandleManager->GetCreationDependencies(ChildIndex);

	for (FInternalNetRefIndex ParentIndex : Parents)
	{
		const bool bHasCreationDependency = ObjectsWithCreationDependencies.IsBitSet(ParentIndex);
					
		// If the parent is a dependent AND relevant check his status first
		if (bHasCreationDependency && OutConnectionObjectsInScope.IsBitSet(ParentIndex))
		{
			constexpr bool bIsRecursiveCall = true;
			UpdateCreationDependentParent(ParentIndex, ObjectsWithCreationDependencies, OutConnectionObjectsInScope, ProcessedObjects, bIsRecursiveCall);
		}

		const bool bIsParentRelevant = OutConnectionObjectsInScope.IsBitSet(ParentIndex);
		OutConnectionObjectsInScope.AndBitValue(ChildIndex, bIsParentRelevant);
	}

	// Check if we are still relevant, if not clear our subobjects
	const bool bIsStillRelevant = OutConnectionObjectsInScope.IsBitSet(ChildIndex);
	if (!bRecursive && !bIsStillRelevant)
	{
		for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ChildIndex))
		{
			OutConnectionObjectsInScope.ClearBit(SubObjectIndex);
		}
	}

	// Mark the object as processed so we don't process it again if it's another object's parent.
	ProcessedObjects.SetBit(ChildIndex);
};

/**
 * Per connection we will combine the effects of connection/owner and group filtering before passing
 * the objects with a specific dynamic filter set. The filter can then decide which objects should
 * be filtered out or not. The result is ANDed with the objects passed so that it's trivial to
 * implement on or off filters and as a safety mechanism to prevent filters from changing the status
 * of objects it should not be concerned about.
 */

void FReplicationFiltering::UpdateDynamicFiltering()
{
	IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateDynamicFiltering);

	constexpr uint32 TotalObjectCountOnStack = 1024;
	constexpr uint32 FilteredOutDependentObjectCountOnStack = 192;
	constexpr uint32 FilteredInObjectCountOnStack = 192;
	constexpr uint32 FilteredOutObjectCountOnStack = TotalObjectCountOnStack - FilteredInObjectCountOnStack - FilteredOutDependentObjectCountOnStack;

	TArray<uint32, TInlineAllocator<FilteredOutObjectCountOnStack>> FilteredOutObjects;
	TArray<uint32, TInlineAllocator<FilteredInObjectCountOnStack>> FilteredInObjects;
	TArray<uint32, TInlineAllocator<FilteredOutDependentObjectCountOnStack>> FilteredOutDependentObjects;
	TArray<uint32> FilteredOutByHysteresisObjects;
	FilteredOutByHysteresisObjects.Reserve(256);

	const FInternalNetRefIndexManager NetRefIndexManager(*NetRefHandleManager);

	// For circular dependency detection/avoidance when retrieving dependent object filter status.
	TSet<FInternalNetRefIndex> VisitedDependentObjects;
	VisitedDependentObjects.Reserve(32);

	uint32* AllowedObjectsData = static_cast<uint32*>(FMemory_Alloca(WordCountForObjectBitArrays * sizeof(uint32)));
	FNetBitArrayView AllowedObjects(AllowedObjectsData, MaxInternalNetRefIndex, FNetBitArrayView::NoResetNoValidate);

	// Subobjects will never be added to a dynamic filter, but objects can become dependent at any time.
	// We need to make sure they are not filtered out.
	const uint32* SubObjectsData = NetRefHandleManager->GetSubObjectInternalIndices().GetDataChecked(WordCountForObjectBitArrays);
	const uint32* ObjectsRequiringDynamicFilterUpdateData = ObjectsRequiringDynamicFilterUpdate.GetDataChecked(WordCountForObjectBitArrays);

	// List of rootobjects that have a dependency trait with another parent root object
	const FNetBitArrayView DependentObjectsView = NetRefHandleManager->GetDependentObjectInternalIndices();
	const uint32* DependentObjectsData = DependentObjectsView.GetDataChecked(WordCountForObjectBitArrays);

	// List of objects that had a change in their creation dependency
	const FNetBitArrayView ObjectsWithDirtyCreationDependencies = NetRefHandleManager->GetObjectsWithDirtyCreationDependencies();

	// List of rootobjects that have childs depending on them
	const FNetBitArrayView ObjectsWithCreationDependents = NetRefHandleManager->GetObjectsWithCreationDependents();
	const uint32* ObjectsWithCreationDependentsData = ObjectsWithCreationDependents.GetDataChecked(WordCountForObjectBitArrays);

	// List of child rootobjects that depend on a parent for instantiation
	const FNetBitArrayView ObjectsWithCreationDependencies = NetRefHandleManager->GetObjectsWithCreationDependencies();
	const uint32* ObjectsWithCreationDependenciesData = ObjectsWithCreationDependencies.GetDataChecked(WordCountForObjectBitArrays);

	uint32* ConnectionIds = static_cast<uint32*>(FMemory_Alloca(ValidConnections.GetNumBits() * sizeof(uint32)));
	uint32 ConnectionCount = 0;
	ValidConnections.ForAllSetBits([ConnectionIds, &ConnectionCount](uint32 Bit) { ConnectionIds[ConnectionCount++] = Bit; });

	for (const uint32 ConnId : MakeArrayView(ConnectionIds, ConnectionCount))
	{
		FPerConnectionInfo& ConnectionInfo = this->ConnectionInfos[ConnId];

		const bool bTrackScopeDeltas = ReplicationSystem->GetDelegates().ConnectionObjectScopeChangedDelegate.IsBound();
		if (bTrackScopeDeltas)
		{
			FMemory::Memcpy(
				ScopeDeltaPrevObjects.GetDataChecked(WordCountForObjectBitArrays),
				ConnectionInfo.ObjectsInScope.GetDataChecked(WordCountForObjectBitArrays),
				WordCountForObjectBitArrays * sizeof(uint32));
		}

		ConnectionInfo.InProgressDynamicFilteredOutObjects.ClearAllBits();
		uint32* InProgressDynamicFilteredOutObjectsData = HasDynamicFilters() ? ConnectionInfo.InProgressDynamicFilteredOutObjects.GetDataChecked(WordCountForObjectBitArrays) : nullptr;

		/*
		 * Apply dynamic filters.
		 *
		 * The algorithm will loop over all connections, call each filter and update
		 * the DynamicFilteredOutObjects bit array. Once all filters have been applied
		 * the result is compared with that of the previous frame.
		 * Special handling is required for objects who are marked as requiring update,
		 * as they may have updated subobject information, as well as dependent objects.
		 */
		{
			for (FFilterInfo& Info : DynamicFilterInfos)
			{
				if (!Info.Filter || Info.ObjectCount == 0U)
				{
					continue;
				}

				FNetObjectFilteringParams FilteringParams
				{
					.OutAllowedObjects = AllowedObjects,
					.FilteringInfos = MakeArrayView(NetObjectFilteringInfos),
					.StateBuffers = nullptr,
					.ConnectionId = ConnId,
					.View = Connections->GetReplicationView(ConnId),
					.GroupFilteredOutObjects = ReplicationSystem->GetReplicationSystemInternal()->GetFiltering().GetGroupFilteredOutObjects(ConnId),
					.NetRefIndexManager = NetRefIndexManager,
				};

				// Execute the filter here
				Info.Filter->Filter(FilteringParams);

				const FNetBitArrayView FilteredObjectsView = Info.Filter->GetFilteredObjects();
				const uint32* FilteredObjectsData = FilteredObjectsView.GetData();
				for (SIZE_T WordIt = 0, WordEndIt = WordCountForObjectBitArrays; WordIt != WordEndIt; ++WordIt)
				{
					const uint32 FilteredObjects = FilteredObjectsData[WordIt];
					const uint32 FilterAllowedObjects = AllowedObjectsData[WordIt];
					// Keep status of objects not affected by this filter. Inverse the effect of the filter and add which objects are filtered out.
					InProgressDynamicFilteredOutObjectsData[WordIt] = (InProgressDynamicFilteredOutObjectsData[WordIt] & ~FilteredObjects) | (~FilterAllowedObjects & FilteredObjects);
				}
			}
		}

		/*
		 * Now that all dynamic filters have been called we can do some post-processing
		 * and try only to do the more expensive operations, such as subobject management,
		 * for objects that have changed filter status since the previous frame.
		 */
		FNetBitArrayView DynamicFilteredOutObjects = MakeNetBitArrayView(ConnectionInfo.DynamicFilteredOutObjects);
		FNetBitArrayView DynamicFilteredOutObjectsHysteresisAdjusted = MakeNetBitArrayView(ConnectionInfo.DynamicFilteredOutObjectsHysteresisAdjusted);
		{
			IRIS_PROFILER_SCOPE(FReplicationFiltering_OnFilterStatusChanged);
			FilteredOutObjects.Reset();
			FilteredOutDependentObjects.Reset();
			FilteredInObjects.Reset();

			const uint32* DynamicFilterEnabledObjectsData = DynamicFilterEnabledObjects.GetData();
			uint32* DynamicFilteredOutObjectsData = ConnectionInfo.DynamicFilteredOutObjects.GetData();
			const uint32* GroupIncludedObjectsData = ConnectionInfo.GroupIncludedObjects.GetData();
			for (uint32 WordIt = 0, WordEndIt = WordCountForObjectBitArrays; WordIt != WordEndIt; ++WordIt)
			{
				const uint32 SubObjects = SubObjectsData[WordIt];
				// Set of dependents objects that don't have the parent filter trait
				const uint32 DependentObjects = (DependentObjectsData[WordIt] & ~ObjectsWithCreationDependenciesData[WordIt]);
				const uint32 ObjectsRequiringUpdate = ObjectsRequiringDynamicFilterUpdateData[WordIt];
				const uint32 PrevFilteredOutObjects = DynamicFilteredOutObjectsData[WordIt];
				// Mask off group included objects in this loops so they're accounted for when updating hysteresis. A no longer group included object should also be subject to hysteresis.
				const uint32 CurrentFilteredOutObjects = InProgressDynamicFilteredOutObjectsData[WordIt] & ~GroupIncludedObjectsData[WordIt];
				const uint32 FilterEnabledObjects = DynamicFilterEnabledObjectsData[WordIt];

				const uint32 ModifiedScopeObjects = (PrevFilteredOutObjects ^ CurrentFilteredOutObjects);
				// ObjectsRequiringUpdate may contain objects that had a dynamic filter set last frame. We need to update
				// the filtered out objects. We process dependencies every frame to deal with cases where they're filtered out and they should no
				// longer be allowed to replicate due to the object with the dependency being filtered out to.
				const uint32 ObjectsToProcess = ((ModifiedScopeObjects | DependentObjects) & FilterEnabledObjects) | ObjectsRequiringUpdate;

				if (!ObjectsToProcess)
				{
					continue;
				}

				// Update dynamic filter
				DynamicFilteredOutObjectsData[WordIt] = CurrentFilteredOutObjects;

				const uint32 BitOffset = WordIt * 32U;

				// Calculate which objects need to be updated due to being filtered out.
				// We want to process as few as possible since modifying subobject status is expensive.
				for (uint32 DisabledObjects = CurrentFilteredOutObjects & ObjectsToProcess; DisabledObjects; )
				{
					// Extract one set bit from DisabledObjects. The code below will extract the least significant bit set and then clear it.
					const uint32 LeastSignificantBit = GetLeastSignificantBit(DisabledObjects);
					DisabledObjects ^= LeastSignificantBit;

					// Store disabled objects for later processing to minimize performance impact of hysteresis
					const uint32 ObjectIndex = BitOffset + FPlatformMath::CountTrailingZeros(LeastSignificantBit);

					if (DependentObjects & LeastSignificantBit)
					{
						// Store dependent objects for later processing.
						FilteredOutDependentObjects.Add(ObjectIndex);
					}
					else
					{
						// Store objects for later processing
						FilteredOutObjects.Add(ObjectIndex);
					}
				}

				for (uint32 EnabledObjects = ~CurrentFilteredOutObjects & ObjectsToProcess; EnabledObjects; )
				{
					const uint32 LeastSignificantBit = GetLeastSignificantBit(EnabledObjects);
					EnabledObjects ^= LeastSignificantBit;

					const uint32 ObjectIndex = BitOffset + FPlatformMath::CountTrailingZeros(LeastSignificantBit);
					FilteredInObjects.Add(ObjectIndex);
				}
			}
		}

		// Hysteresis path need to add explicitly filtered out objects to hysteresis. Objects not supported by hysteresis as well as objects' whose timeout has passed needs to be filtered out immediately.
		const uint32 ConnIdMod = ConnId % HysteresisState.ConnectionIdStride;
		if (HysteresisState.Mode == EHysteresisProcessingMode::Enabled)
		{
			IRIS_CSV_PROFILER_SCOPE(Iris, FReplicationFiltering_OnFilterStatusChanged_Hysteresis);

			// With connection throttling we need to adjust the hysteresis with the difference between the update rate and the number of frames til next update.
			// If we're updating this frame we need to add HysteresisState.ConnectionIdStride to compensate for an extra update of that amount, if we're updating the next frame we need to add HysteresisState.ConnectionIdStride - 1 and so on.
			const uint32 AdjustHysteresisForUpdateThrottling = HysteresisState.ConnectionIdStride - ((ConnIdMod + HysteresisState.ConnectionIdStride - HysteresisState.ConnectionStartId) % HysteresisState.ConnectionIdStride);

			// Remove filtered in objects from hysteresis immediately. 
			ConnectionInfo.HysteresisUpdater.RemoveHysteresis(MakeArrayView(FilteredInObjects));

			// Make sure filtered in objects aren't filtered out.
			for (const uint32 ObjectIndex : FilteredInObjects)
			{
				DynamicFilteredOutObjectsHysteresisAdjusted.ClearBit(ObjectIndex);
				for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
				{
					DynamicFilteredOutObjectsHysteresisAdjusted.ClearBit(SubObjectIndex);
				}
			}

			if (const bool bConnIsExemptFromHysteresis = NewConnections.GetBit(ConnId))
			{
				// If it's a new connection we should not apply hysteresis to any object- filter out objects not in scope immediately.
				for (const uint32 ObjectIndex : FilteredOutObjects)
				{
					DynamicFilteredOutObjectsHysteresisAdjusted.SetBit(ObjectIndex);
					for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
					{
						DynamicFilteredOutObjectsHysteresisAdjusted.SetBit(SubObjectIndex);
					}
				}
			}
			else
			{
				// Add filtered out objects to hysteresis if eligible, meaning they support it and are still dynamically filtered. Dependent objects are not processed here.
				for (const uint32 ObjectIndex : FilteredOutObjects)
				{
					const uint32 HysteresisFrameCount = ObjectScopeHysteresisFrameCounts[ObjectIndex];
					const bool bAlreadyFilteredOut = DynamicFilteredOutObjectsHysteresisAdjusted.GetBit(ObjectIndex);
					// If the object is already filtered out we cannot add to hysteresis. It may be that a subobject was added and the root object was marked for processing, in which case we should immediately filter out the subobject.
					if (!bAlreadyFilteredOut && HysteresisFrameCount && DynamicFilterEnabledObjects.GetBit(ObjectIndex) && !HysteresisState.ObjectsExemptFromHysteresis.GetBit(ObjectIndex))
					{
						// We need to adjust the hysteresis frame count to account for when it will be updated. The -1 stems from the fact that updating won't happen until next frame at the earliest.
						const uint16 TotalHysteresisFrameCount = static_cast<uint16>(HysteresisFrameCount - 1 + AdjustHysteresisForUpdateThrottling);
						ConnectionInfo.HysteresisUpdater.SetHysteresisFrameCount(ObjectIndex, TotalHysteresisFrameCount);
					}
					else
					{
						// Filter out
						DynamicFilteredOutObjectsHysteresisAdjusted.SetBit(ObjectIndex);
						for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
						{
							DynamicFilteredOutObjectsHysteresisAdjusted.SetBit(SubObjectIndex);
						}
					}
				}
			}

			// Update hysteresis
			if (ConnIdMod == HysteresisState.ConnectionStartId)
			{
				FilteredOutByHysteresisObjects.Reset();
				ConnectionInfo.HysteresisUpdater.Update(static_cast<uint8>(HysteresisState.ConnectionIdStride), FilteredOutByHysteresisObjects);

				// Immediately filter out objects whose hysteresis timed out.
				for (const uint32 ObjectIndex : FilteredOutByHysteresisObjects)
				{
					DynamicFilteredOutObjectsHysteresisAdjusted.SetBit(ObjectIndex);
					for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
					{
						DynamicFilteredOutObjectsHysteresisAdjusted.SetBit(SubObjectIndex);
					}
				}
			}

		}
		else
		{
			IRIS_CSV_PROFILER_SCOPE(Iris, FReplicationFiltering_OnFilterStatusChanged_NonHysteresis);

			// Make sure filtered in objects aren't filtered out.
			for (const uint32 ObjectIndex : FilteredInObjects)
			{
				DynamicFilteredOutObjectsHysteresisAdjusted.ClearBit(ObjectIndex);
				for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
				{
					DynamicFilteredOutObjectsHysteresisAdjusted.ClearBit(SubObjectIndex);
				}
			}

			// Filter out objects except dependent ones.
			for (const uint32 ObjectIndex : FilteredOutObjects)
			{
				DynamicFilteredOutObjectsHysteresisAdjusted.SetBit(ObjectIndex);
				for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
				{
					DynamicFilteredOutObjectsHysteresisAdjusted.SetBit(SubObjectIndex);
				}
			}
		}

		ensure(!bCVarRepFilterValidateNoSubObjectInScopeWithFilteredOutRootObject || !HasSubObjectInScopeWithFilteredOutRootObject(DynamicFilteredOutObjectsHysteresisAdjusted));

		// New optimized creation dependency path
		if (bCVarRepFilterUpdateCreationDependenciesOnRelevancyChange)
		{
			IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateCreationDependencies);
			// Reevaluate filter status for any objects explicitly dirtied.
			ObjectsWithDirtyCreationDependencies.ForAllSetBits([this, &ConnectionInfo, &ObjectsWithCreationDependents](uint32 ObjectIndex)
			{
				this->RefreshCreationDependencyFilterStatus(ObjectIndex, ConnectionInfo, ObjectsWithCreationDependents);
			});

			// Update filter status for any object with creation dependents that became relevant/filtered this frame.
			const uint32* PrevObjectsInScopeData = ConnectionInfo.ObjectsInScope.GetDataChecked(WordCountForObjectBitArrays);
			const uint32* ObjectsInScopeBeforeDynamicFilteringData = ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetDataChecked(WordCountForObjectBitArrays);
			const uint32* DynamicFilteredOutObjectsHysteresisAdjustedData = DynamicFilteredOutObjectsHysteresisAdjusted.GetDataChecked(WordCountForObjectBitArrays);
			const uint32* FilteredOutCreationDependentsData = ConnectionInfo.FilteredOutCreationDependents.GetDataChecked(WordCountForObjectBitArrays);
			for (uint32 WordIt = 0, WordEndIt = WordCountForObjectBitArrays; WordIt != WordEndIt; ++WordIt)
			{
				const uint32 BitOffset = WordIt * 32U;

				const uint32 PrevInScope = PrevObjectsInScopeData[WordIt];
				const uint32 InScope = ObjectsInScopeBeforeDynamicFilteringData[WordIt] & ~DynamicFilteredOutObjectsHysteresisAdjustedData[WordIt];
				const uint32 WithCreationDependents = ObjectsWithCreationDependentsData[WordIt];
				const uint32 FilteredOutCreationDependents = FilteredOutCreationDependentsData[WordIt];
				
				uint32 NewlyFilteredOut = PrevInScope & ~InScope & WithCreationDependents & ~FilteredOutCreationDependents;
				while (NewlyFilteredOut)
				{
					const uint32 LeastSignificantBit = GetLeastSignificantBit(NewlyFilteredOut);
					NewlyFilteredOut ^= LeastSignificantBit;

					const uint32 ObjectIndex = BitOffset + FPlatformMath::CountTrailingZeros(LeastSignificantBit);
					FilterOutCreationDependents(ObjectIndex, ConnectionInfo, ObjectsWithCreationDependents);
				}

				uint32 NewlyFilteredIn = ~PrevInScope & InScope & WithCreationDependents & ~FilteredOutCreationDependents;
				while (NewlyFilteredIn)
				{
					const uint32 LeastSignificantBit = GetLeastSignificantBit(NewlyFilteredIn);
					NewlyFilteredIn ^= LeastSignificantBit;

					const uint32 ObjectIndex = BitOffset + FPlatformMath::CountTrailingZeros(LeastSignificantBit);
					FilterInCreationDependents(ObjectIndex, ConnectionInfo, ObjectsWithCreationDependents);
				}
			}
		}

		// Update the entire scope for the connection.
		{
			IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateConnectionScope);

			uint32* ObjectsInScopeData = ConnectionInfo.ObjectsInScope.GetDataChecked(WordCountForObjectBitArrays);
			const uint32* ObjectsInScopeBeforeDynamicFilteringData = ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetDataChecked(WordCountForObjectBitArrays);
			const uint32* DynamicFilteredOutObjectsHysteresisAdjustedData = DynamicFilteredOutObjectsHysteresisAdjusted.GetDataChecked(WordCountForObjectBitArrays);
			const uint32* FilteredOutCreationDependentsData = ConnectionInfo.FilteredOutCreationDependents.GetDataChecked(WordCountForObjectBitArrays);
			for (SIZE_T WordIt = 0, WordEndIt = WordCountForObjectBitArrays; WordIt != WordEndIt; ++WordIt)
			{
				const uint32 ObjectsInScopeBeforeWord = ObjectsInScopeBeforeDynamicFilteringData[WordIt];
				const uint32 DynamicFilteredOutWord = DynamicFilteredOutObjectsHysteresisAdjustedData[WordIt];
				const uint32 FilteredOutCreationDependentsWord = FilteredOutCreationDependentsData[WordIt];
				ObjectsInScopeData[WordIt] = ObjectsInScopeBeforeWord & ~DynamicFilteredOutWord & ~FilteredOutCreationDependentsWord;
			}

			// Unconditionally filter out filtered out dependent objects from ObjectsInScope as GetDependentObjectFilterStatus, called later, requires it. Subobjects are dealt with later.
			for (const uint32 DependentObjectIndex : FilteredOutDependentObjects)
			{
				ConnectionInfo.ObjectsInScope.ClearBit(DependentObjectIndex);
			}
		}

		// Update dependent objects that can only be relevant if their parent also is
		// Old Non-optimized creation dependency path
		if (!bCVarRepFilterUpdateCreationDependenciesOnRelevancyChange)
		{
			IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateCreationDependencies);
			if (ObjectsWithCreationDependencies.IsAnyBitSet())
			{
				FNetBitArrayView ConnectionObjectsInScope = MakeNetBitArrayView(ConnectionInfo.ObjectsInScope);
				FNetBitArray ProcessedObjects(ConnectionInfo.ObjectsInScope.GetNumBits());
				FNetBitArrayView ProcessedObjectsView = MakeNetBitArrayView(ProcessedObjects);

				FNetBitArrayView::ForAllSetBits(ObjectsWithCreationDependencies, ConnectionObjectsInScope, FNetBitArrayView::AndOp, [&](uint32 ChildIndex)
				{
					constexpr bool bIsNotRecursive = false;
					this->UpdateCreationDependentParent(ChildIndex, ObjectsWithCreationDependencies, ConnectionObjectsInScope, ProcessedObjectsView, bIsNotRecursive);
				});
			}
		}

		// The scope for the connection is now fully updated, apart from disabled dependent objects.
		// If any object that has a dependency that isn't filtered out we must enable the dependent object.
		if (HysteresisState.Mode == EHysteresisProcessingMode::Enabled)
		{
			IRIS_CSV_PROFILER_SCOPE(Iris, FReplicationFiltering_DependentObjects_Hysteresis);

			const bool bConnIsExemptFromHysteresis = NewConnections.GetBit(ConnId);

			// We didn't filter out any dependent objects yet so we enable/disable as needed.
			// Different algorithm for figuring out update throttling adjustment since we already updated this frame.
			const uint32 AdjustHysteresisForUpdateThrottling = (HysteresisState.ConnectionStartId + HysteresisState.ConnectionIdStride - ConnIdMod) % HysteresisState.ConnectionIdStride;
			for (const uint32 DependentObjectIndex : FilteredOutDependentObjects)
			{
				VisitedDependentObjects.Reset();
				const bool bAllowReplication = GetDependentObjectFilterStatus(ConnId, DependentObjectIndex, VisitedDependentObjects) == ENetFilterStatus::Allow;
				if (bAllowReplication)
				{
					ConnectionInfo.HysteresisUpdater.RemoveHysteresis(DependentObjectIndex);
					// We use the status of the hysteresis adjusted dynamic filtering to figure out whether hysteresis should be enabled or not. ObjectsInScope have already been updated and can't be used for this purpose.
					DynamicFilteredOutObjectsHysteresisAdjusted.ClearBit(DependentObjectIndex);
					
					ConnectionInfo.ObjectsInScope.SetBit(DependentObjectIndex);
					for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(DependentObjectIndex))
					{
						DynamicFilteredOutObjectsHysteresisAdjusted.ClearBit(SubObjectIndex);
						const bool bIsInScope = ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetBit(SubObjectIndex);
						ConnectionInfo.ObjectsInScope.SetBitValue(SubObjectIndex, bIsInScope);
					}
				}
				else
				{
					// Start/continue hysteresis or time to prevent replication?
					bool bIsFilteredOut = bConnIsExemptFromHysteresis || DynamicFilteredOutObjectsHysteresisAdjusted.GetBit(DependentObjectIndex) || !ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetBit(DependentObjectIndex);
					if (!bIsFilteredOut)
					{
						// If we're updated by hysteresis at this point we should not start again, but if other conditions allow it we should start!
						if (!ConnectionInfo.HysteresisUpdater.IsObjectUpdated(DependentObjectIndex))
						{
							const uint32 HysteresisFrameCount = ObjectScopeHysteresisFrameCounts[DependentObjectIndex];
							const bool bIsExemptFromHysteresis = HysteresisState.ObjectsExemptFromHysteresis.GetBit(DependentObjectIndex);
							if (HysteresisFrameCount && !bIsExemptFromHysteresis)
							{
								const uint16 TotalHysteresisFrameCount = static_cast<uint16>(HysteresisFrameCount - 1 + AdjustHysteresisForUpdateThrottling);
								ConnectionInfo.HysteresisUpdater.SetHysteresisFrameCount(DependentObjectIndex, TotalHysteresisFrameCount);
							}
							else
							{
								// Object is exempt from or doesn't support hysteresis so we need to filter it out immediately.
								bIsFilteredOut = true;
							}
						}
						// Else let hysteresis updating continue. Once fully processed the DynamicFilteredOutObjectsHysteresisAdjusted bitarray will be adjusted accordingly and filter out the object.

						// Explicitly enable dependent object and its subobjects.
						ConnectionInfo.ObjectsInScope.SetBitValue(DependentObjectIndex, !bIsFilteredOut);
						DynamicFilteredOutObjectsHysteresisAdjusted.SetBitValue(DependentObjectIndex, bIsFilteredOut);
						for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(DependentObjectIndex))
						{
							DynamicFilteredOutObjectsHysteresisAdjusted.SetBitValue(SubObjectIndex, bIsFilteredOut);
							const bool bIsInScope = ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetBit(SubObjectIndex);
							ConnectionInfo.ObjectsInScope.SetBitValue(SubObjectIndex, !bIsFilteredOut && bIsInScope);
						}
					}
					else
					{
						// Explicitly filter out dependent object and its subobjects
						DynamicFilteredOutObjectsHysteresisAdjusted.SetBit(DependentObjectIndex);
						ConnectionInfo.ObjectsInScope.ClearBit(DependentObjectIndex);
						for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(DependentObjectIndex))
						{
							DynamicFilteredOutObjectsHysteresisAdjusted.SetBit(SubObjectIndex);
							ConnectionInfo.ObjectsInScope.ClearBit(SubObjectIndex);
						}
					}
				}
			}
		}
		else
		{
			IRIS_CSV_PROFILER_SCOPE(Iris, FReplicationFiltering_DependentObjects_Standard);
			// We didn't filter out any dependent objects yet so we enable/disable as needed.
			for (const uint32 DependentObjectIndex : FilteredOutDependentObjects)
			{
				VisitedDependentObjects.Reset();
				const bool bAllowReplication = GetDependentObjectFilterStatus(ConnId, DependentObjectIndex, VisitedDependentObjects) == ENetFilterStatus::Allow;
				ConnectionInfo.ObjectsInScope.SetBitValue(DependentObjectIndex, bAllowReplication);
				for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(DependentObjectIndex))
				{
					const bool bIsInScope = ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetBit(SubObjectIndex);
					ConnectionInfo.ObjectsInScope.SetBitValue(SubObjectIndex, bIsInScope && bAllowReplication);
				}
			}
		}

		if (bTrackScopeDeltas)
		{
			const uint32* PrevData = ScopeDeltaPrevObjects.GetDataChecked(WordCountForObjectBitArrays);
			const uint32* CurrData = ConnectionInfo.ObjectsInScope.GetDataChecked(WordCountForObjectBitArrays);
			uint32* InData = ScopeDeltaNewlyInObjects.GetDataChecked(WordCountForObjectBitArrays);
			uint32* OutData = ScopeDeltaNewlyOutObjects.GetDataChecked(WordCountForObjectBitArrays);
			uint32 AnyDiff = 0;
			for (uint32 WordIt = 0; WordIt < WordCountForObjectBitArrays; ++WordIt)
			{
				const uint32 Diff = PrevData[WordIt] ^ CurrData[WordIt];
				InData[WordIt] = Diff & CurrData[WordIt];
				OutData[WordIt] = Diff & PrevData[WordIt];
				AnyDiff |= Diff;
			}

			if (AnyDiff != 0)
			{
				ReplicationSystem->GetDelegates().ConnectionObjectScopeChangedDelegate.Broadcast(
					FConnectionHandle(ConnId),
					MakeNetBitArrayView(ScopeDeltaNewlyInObjects),
					MakeNetBitArrayView(ScopeDeltaNewlyOutObjects),
					[this](FInternalNetRefIndex Index) -> UObject*
					{
						return NetRefHandleManager->GetReplicatedObjectInstance(Index);
					});
			}
		}
	}
}

void FReplicationFiltering::PostUpdateDynamicFiltering()
{
	IRIS_PROFILER_SCOPE(FReplicationFiltering_PostUpdateDynamicFiltering);

	ObjectsRequiringDynamicFilterUpdate.ClearAllBits();

	// We always process all objects with dirty creation dependencies on every valid connection.
	{
		FNetBitArrayView ObjectsWithDirtyCreationDependencies = NetRefHandleManager->GetObjectsWithDirtyCreationDependencies();
		ObjectsWithDirtyCreationDependencies.ClearAllBits();
	}

	// Tell filters to clean up after filtering. It's only called if any object has the filter set.
	{
		FNetObjectPostFilteringParams PostFilteringParams;
		for (FFilterInfo& Info : DynamicFilterInfos)
		{
			if (!Info.Filter || Info.ObjectCount == 0U)
			{
				continue;
			}
			
			Info.Filter->PostFilter(PostFilteringParams);
		}
	}

	PostUpdateObjectScopeHysteresis();
}

void FReplicationFiltering::NotifyFiltersOfDirtyObjects()
{
	if (!bHasDynamicFiltersWithUpdateTrait)
	{
		// No filters use UpdateObjects so we can skip early.
		return;
	}

	IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateFilterWithDirtyObjects);

	FDirtyObjectsAccessor DirtyObjectsAccessor(ReplicationSystem->GetReplicationSystemInternal()->GetDirtyNetObjectTracker());
	const FNetBitArrayView DirtyObjectsThisFrame = DirtyObjectsAccessor.GetDirtyNetObjects();

	FUpdateDirtyObjectsBatchHelper BatchHelper(NetRefHandleManager, DynamicFilterInfos);

	constexpr SIZE_T MaxBatchObjectCount = FUpdateDirtyObjectsBatchHelper::Constants::MaxObjectCountPerBatch;
	UE::Net::FInternalNetRefIndex ObjectIndices[MaxBatchObjectCount];

	// Find up to 512 dirty objects to update every iteration.
	// Their InternalIndex will get added to ObjectIndices and passed to the filter.
	const uint32 BitCount = ~0U;
	for (uint32 ObjectCount, StartIndex = 0; (ObjectCount = DirtyObjectsThisFrame.GetSetBitIndices(StartIndex, BitCount, ObjectIndices, MaxBatchObjectCount)) > 0; )
	{
		BatchNotifyFiltersOfDirtyObjects(BatchHelper, ObjectIndices, ObjectCount);

		StartIndex = ObjectIndices[ObjectCount - 1] + 1U;
		if ((StartIndex == DirtyObjectsThisFrame.GetNumBits()) | (ObjectCount < MaxBatchObjectCount))
		{
			break;
		}
	}
}

void FReplicationFiltering::BatchNotifyFiltersOfDirtyObjects(FUpdateDirtyObjectsBatchHelper& BatchHelper, const UE::Net::FInternalNetRefIndex* DirtyObjectIndices, uint32 ObjectCount)
{
	BatchHelper.PrepareBatch(DirtyObjectIndices, ObjectCount, ObjectIndexToDynamicFilterIndex);

	FNetObjectFilterUpdateParams UpdateParameters;
	UpdateParameters.FilteringInfos = MakeArrayView(NetObjectFilteringInfos);

	for (const FUpdateDirtyObjectsBatchHelper::FPerFilterInfo& PerFilterInfo : BatchHelper.PerFilterInfos)
	{
		const int32 FilterIndex = static_cast<int32>(&PerFilterInfo - BatchHelper.PerFilterInfos.GetData());

		if (PerFilterInfo.ObjectCount == 0 || !IsValidDynamicFilter(FilterIndex))
		{
			continue;
		}

		UpdateParameters.ObjectIndices = PerFilterInfo.ObjectIndices;
		UpdateParameters.ObjectCount = PerFilterInfo.ObjectCount;
		UNetObjectFilter* Filter = DynamicFilterInfos[FilterIndex].Filter.Get();
		Filter->UpdateObjects(UpdateParameters);
	}
}

bool FReplicationFiltering::HasOwnerFilter(uint32 ObjectIndex) const
{
	const bool bHasOwnerFilter = ObjectsWithOwnerFilter.GetBit(ObjectIndex);
	return bHasOwnerFilter;
}

FReplicationFiltering::PerObjectInfoIndexType FReplicationFiltering::AllocPerObjectInfo()
{
	const uint32 UsedPerObjectInfoBitsPerWord = sizeof(decltype(UsedPerObjectInfoStorage)::ElementType)*8;

	FNetBitArrayView UsedPerObjectInfos = MakeNetBitArrayView(UsedPerObjectInfoStorage.GetData(), UsedPerObjectInfoStorage.Num()*UsedPerObjectInfoBitsPerWord);
	uint32 FreeIndex = UsedPerObjectInfos.FindFirstZero();

	// Grow used indices bit array if needed
	if (FreeIndex == FNetBitArrayView::InvalidIndex)
	{
		checkf(UsedPerObjectInfos.GetNumBits() < std::numeric_limits<PerObjectInfoIndexType>::max(), TEXT("Filtering per object info storage exhausted. Contact the UE Network team."));
		FreeIndex = UsedPerObjectInfos.GetNumBits();
		UsedPerObjectInfoStorage.AddZeroed(UsedPerObjectInfoStorageGrowSize/UsedPerObjectInfoBitsPerWord);
		FNetBitArrayView NewUsedPerObjectInfos = MakeNetBitArrayView(UsedPerObjectInfoStorage.GetData(), UsedPerObjectInfoStorage.Num()*UsedPerObjectInfoBitsPerWord);
		NewUsedPerObjectInfos.SetBit(FreeIndex);
		// Mark index 0 as used so we can use it as an invalid index
		if (FreeIndex == 0U)
		{
			FreeIndex = 1U;
			NewUsedPerObjectInfos.SetBit(1);
		}
		PerObjectInfoStorage.AddUninitialized(PerObjectInfoStorageCountPerItem*UsedPerObjectInfoStorageGrowSize);
	}
	else
	{
		UsedPerObjectInfos.SetBit(FreeIndex);
	}
	
	return static_cast<PerObjectInfoIndexType>(FreeIndex);
}

void FReplicationFiltering::FreePerObjectInfo(PerObjectInfoIndexType Index)
{
	const uint32 UsedPerObjectInfoBitsPerWord = sizeof(decltype(UsedPerObjectInfoStorage)::ElementType)*8U;

	FNetBitArrayView UsedPerObjectInfos = MakeNetBitArrayView(UsedPerObjectInfoStorage.GetData(), UsedPerObjectInfoStorage.Num()*UsedPerObjectInfoBitsPerWord);
	UsedPerObjectInfos.ClearBit(Index);
}

FReplicationFiltering::FPerObjectInfo* FReplicationFiltering::AllocPerObjectInfoForObject(uint32 ObjectIndex)
{
	ObjectsWithPerObjectInfo.SetBit(ObjectIndex);
	const PerObjectInfoIndexType ObjectInfoIndex = AllocPerObjectInfo();
	ObjectIndexToPerObjectInfoIndex[ObjectIndex] = ObjectInfoIndex;

	FPerObjectInfo* ObjectInfo = GetPerObjectInfo(ObjectInfoIndex);
	SetPerObjectInfoFilterStatus(*ObjectInfo, ENetFilterStatus::Allow);

	return ObjectInfo;
}

void FReplicationFiltering::FreePerObjectInfoForObject(uint32 ObjectIndex)
{
	PerObjectInfoIndexType& ObjectInfoIndex = ObjectIndexToPerObjectInfoIndex[ObjectIndex];
	if (ObjectInfoIndex == 0)
	{
		return;
	}

	ObjectsWithPerObjectInfo.ClearBit(ObjectIndex);

	FreePerObjectInfo(ObjectInfoIndex);
	ObjectInfoIndex = 0;
}

FReplicationFiltering::FPerObjectInfo* FReplicationFiltering::GetPerObjectInfo(PerObjectInfoIndexType Index)
{
	const uint32 ObjectInfoIndex = Index*PerObjectInfoStorageCountPerItem;

	checkSlow(ObjectInfoIndex < (uint32)PerObjectInfoStorage.Num());

	uint32* StoragePointer = PerObjectInfoStorage.GetData() + ObjectInfoIndex;
	return reinterpret_cast<FPerObjectInfo*>(StoragePointer);
}

const FReplicationFiltering::FPerObjectInfo* FReplicationFiltering::GetPerObjectInfo(PerObjectInfoIndexType Index) const
{
	const uint32 ObjectInfoIndex = Index*PerObjectInfoStorageCountPerItem;

	checkSlow(ObjectInfoIndex < (uint32)PerObjectInfoStorage.Num());

	const uint32* StoragePointer = PerObjectInfoStorage.GetData() + ObjectInfoIndex;
	return reinterpret_cast<const FPerObjectInfo*>(StoragePointer);
}

void FReplicationFiltering::AddSubObjectFilter(FNetObjectGroupHandle GroupHandle)
{
	const bool bIsValidGroup = ensureMsgf(Groups->IsValidGroup(GroupHandle), TEXT("AddSubObjectFilter received invalid group Index: %u Id: %u"), GroupHandle.GetGroupIndex(), GroupHandle.GetUniqueId());
	if (!bIsValidGroup)
	{
		return;
	}

	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();

	const bool bIsFiltering = Groups->IsFilterGroup(GroupHandle) || SubObjectFilterGroups.GetBit(GroupIndex);
	ensureMsgf(!bIsFiltering, TEXT("NetObjectGroup Name: %s Index: %u Id: %u was asked to start subobject filtering but it was already used for filtering."), *Groups->GetGroupNameString(GroupHandle), GroupIndex, GroupHandle.GetUniqueId());
	if (bIsFiltering)
	{
		return;
	}

	SubObjectFilterGroups.SetBit(GroupIndex);

	CreatePerSubObjectGroupFilterInfo(GroupIndex);

	UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::AddSubObjectFilter Group: %ls Index: %u FilterStatus: DisallowReplication", *Groups->GetGroupNameString(GroupHandle), GroupIndex);
}

void FReplicationFiltering::RemoveSubObjectFilter(FNetObjectGroupHandle GroupHandle)
{
	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();
	if (GroupHandle.IsValid() && SubObjectFilterGroups.GetBit(GroupIndex))
	{
		// Mark group as no longer a SubObjectFilter group
		SubObjectFilterGroups.ClearBit(GroupIndex);

		DestroyPerSubObjectGroupFilterInfo(GroupIndex);

		UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::RemoveSubObjectFilter Group: %ls", *Groups->GetGroupNameString(GroupHandle));
	}
}

void FReplicationFiltering::UpdateSubObjectFilters()
{
	IRIS_PROFILER_SCOPE(FReplicationFiltering_UpdateSubObjectFilters);

	// We want to remove all groups that have no members and no enabled connections
	auto UpdateSubObjectFilterGroup = [this](uint32 BitIndex)
	{
		FNetObjectGroupHandle::FGroupIndexType GroupIndex = static_cast<FNetObjectGroupHandle::FGroupIndexType>(BitIndex);
		if (const FNetObjectGroup* Group = Groups->GetGroupFromIndex(GroupIndex);
			Group && Group->Members.IsEmpty())
		{
			const FPerSubObjectFilterGroupInfo* GroupInfo = GetPerSubObjectFilterGroupInfo(GroupIndex);
			if (!IsAnyConnectionFilterStatusAllowed(*GetPerObjectInfo(GroupInfo->ConnectionStateIndex)))
			{
				UE_LOGF(LogIrisFiltering, Verbose, "UpdateSubObjectFilters is destroying group %ls since its empty", *Group->GroupName.ToString());
				// Note that the below call will cause functions to be called in this class, like RemoveSubObjectFilter.
				ReplicationSystem->DestroyGroup(Groups->GetHandleFromGroup(Group));
			}
		}
	};

	FNetBitArray::ForAllSetBits(DirtySubObjectFilterGroups, SubObjectFilterGroups, FNetBitArray::AndOp, UpdateSubObjectFilterGroup);
	DirtySubObjectFilterGroups.ClearAllBits();
}

void FReplicationFiltering::SetSubObjectFilterStatus(FNetObjectGroupHandle GroupHandle, FConnectionHandle ConnectionHandle, ENetFilterStatus ReplicationStatus)
{
	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();
	if (GroupHandle.IsReservedNetObjectGroup())
	{
		UE_LOGF(LogIrisFiltering, Warning, "FReplicationFiltering::SetSubObjectFilterStatus - Trying to set filter for reserved Group: %ls which is not allowed.", *Groups->GetGroupNameString(GroupHandle));
		return;
	}

	const uint32 ParentConnectionId = ConnectionHandle.GetParentConnectionId();
	if (ensure(ValidConnections.GetBit(ParentConnectionId) && SubObjectFilterGroups.GetBit(GroupIndex)))
	{
		UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::SetSubObjectFilterStatus Group: %ls, ConnectionHandle: %u:%u, FilterStatus: %u", *Groups->GetGroupNameString(GroupHandle), ConnectionHandle.GetParentConnectionId(), ConnectionHandle.GetChildConnectionId(), ReplicationStatus == ENetFilterStatus::Allow ? 1U : 0U);
		FPerSubObjectFilterGroupInfo* GroupInfo = GetPerSubObjectFilterGroupInfo(GroupIndex);
		GroupInfo->ConnectionFilterStatus.SetFilterStatus(ConnectionHandle, ReplicationStatus);
		ENetFilterStatus ConnectionReplicationStatus = GroupInfo->ConnectionFilterStatus.GetFilterStatus(ParentConnectionId);
		FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfo->ConnectionStateIndex);
		SetConnectionFilterStatus(*ConnectionState, ParentConnectionId, ConnectionReplicationStatus);
		if (!IsAnyConnectionFilterStatusAllowed(*ConnectionState))
		{
			DirtySubObjectFilterGroups.SetBit(GroupIndex);
		}
	}
}

bool FReplicationFiltering::GetSubObjectFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ParentConnectionId, ENetFilterStatus& OutReplicationStatus) const
{
	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();
	if (!(ValidConnections.GetBit(ParentConnectionId) && SubObjectFilterGroups.GetBit(GroupIndex)))
	{
		return false;
	}

	if (const FPerSubObjectFilterGroupInfo* GroupInfo = GetPerSubObjectFilterGroupInfo(GroupIndex);
		ensure(GroupInfo))
	{
		const FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfo->ConnectionStateIndex);
		OutReplicationStatus = GetConnectionFilterStatus(*ConnectionState, ParentConnectionId);
		return true;
	}

	return false;
}

bool FReplicationFiltering::AddExclusionFilterGroup(FNetObjectGroupHandle GroupHandle)
{
	const bool bIsValidGroup = ensureMsgf(Groups->IsValidGroup(GroupHandle), TEXT("AddExclusionFilterGroup received an invalid group: Index: %u Id: %u"), GroupHandle.GetGroupIndex(), GroupHandle.GetUniqueId());
	if (!bIsValidGroup)
	{
		return false;
	}

	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();

	const bool bIsFiltering = Groups->IsFilterGroup(GroupHandle) || SubObjectFilterGroups.GetBit(GroupIndex);
	ensureMsgf(!bIsFiltering, TEXT("NetObjectGroup Name: %s Index: %u Id: %u was asked to start exclusion filtering but it was already used for filtering."), *Groups->GetGroupNameString(GroupHandle), GroupIndex, GroupHandle.GetUniqueId());
	if (bIsFiltering)
	{
		return false;
	}

	Groups->AddExclusionFilterTrait(GroupHandle);

	ExclusionFilterGroups.SetBit(GroupIndex);
	DirtyExclusionFilterGroups.SetBit(GroupIndex);
	bHasDirtyExclusionFilterGroup = 1;

	// By default we filter out the group members for all connections
	GroupInfos[GroupIndex].ConnectionStateIndex = AllocPerObjectInfo();
	SetPerObjectInfoFilterStatus(*GetPerObjectInfo(GroupInfos[GroupIndex].ConnectionStateIndex), ENetFilterStatus::Disallow);

	UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::AddExclusionGroupFilter on %ls, FilterStatus: DisallowReplication", *Groups->GetGroupNameString(GroupHandle));
	return true;
}

bool FReplicationFiltering::AddInclusionFilterGroup(FNetObjectGroupHandle GroupHandle)
{
	const bool bIsValidGroup = ensureMsgf(Groups->IsValidGroup(GroupHandle), TEXT("AddInclusionFilterGroup received an invalid group: Index: %u Id: %u"), GroupHandle.GetGroupIndex(), GroupHandle.GetUniqueId());
	if (!bIsValidGroup)
	{
		return false;
	}

	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();

	const bool bIsFiltering = Groups->IsFilterGroup(GroupHandle) || SubObjectFilterGroups.GetBit(GroupIndex);
	ensureMsgf(!bIsFiltering, TEXT("NetObjectGroup Name: %s Index: %u Id: %u was asked to start exclusion filtering but it was already used for filtering."), *Groups->GetGroupNameString(GroupHandle), GroupIndex, GroupHandle.GetUniqueId());
	if (bIsFiltering)
	{
		return false;
	}

	Groups->AddInclusionFilterTrait(GroupHandle);

	InclusionFilterGroups.SetBit(GroupIndex);
	DirtyInclusionFilterGroups.SetBit(GroupIndex);
	bHasDirtyInclusionFilterGroup = 1;

	// By default we do not override dynamic filtering.
	GroupInfos[GroupIndex].ConnectionStateIndex = AllocPerObjectInfo();
	SetPerObjectInfoFilterStatus(*GetPerObjectInfo(GroupInfos[GroupIndex].ConnectionStateIndex), ENetFilterStatus::Disallow);

	UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::AddInclusionFilterGroup on %ls, FilterStatus: DoNotOverride", *Groups->GetGroupNameString(GroupHandle));
	return true;
}

void FReplicationFiltering::RemoveGroupFilter(FNetObjectGroupHandle GroupHandle)
{
	// Remove the filter trait from the group
	if (!Groups->IsValidGroup(GroupHandle))
	{
		return;
	}

	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();
	if (ExclusionFilterGroups.GetBit(GroupIndex))
	{
		DirtyExclusionFilterGroups.ClearBit(GroupIndex);

		ValidConnections.ForAllSetBits([GroupHandle, this](uint32 ConnectionIndex) 
		{
			this->SetGroupFilterStatus(GroupHandle, ConnectionIndex, ENetFilterStatus::Allow); 
		});

		// Mark group as no longer an exclusion filter
		ExclusionFilterGroups.ClearBit(GroupIndex);
		const PerObjectInfoIndexType ConnectionStateIndex = GroupInfos[GroupIndex].ConnectionStateIndex;
		GroupInfos[GroupIndex].ConnectionStateIndex = 0U;
		FreePerObjectInfo(ConnectionStateIndex);

		Groups->RemoveExclusionFilterTrait(GroupHandle);

		UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::RemoveGroupFilter ExclusionFilter Group: %ls", *Groups->GetGroupNameString(GroupHandle));
	}
	else if (InclusionFilterGroups.GetBit(GroupIndex))
	{
		DirtyInclusionFilterGroups.ClearBit(GroupIndex);

		// Clear filter effects
		SetGroupFilterStatus(GroupHandle, ENetFilterStatus::Disallow);

		// Mark group as no longer an inclusion filter
		InclusionFilterGroups.ClearBit(GroupIndex);
		const PerObjectInfoIndexType ConnectionStateIndex = GroupInfos[GroupIndex].ConnectionStateIndex;
		GroupInfos[GroupIndex].ConnectionStateIndex = 0U;
		FreePerObjectInfo(ConnectionStateIndex);

		Groups->RemoveInclusionFilterTrait(GroupHandle);

		UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::RemoveGroupFilter InclusionFilter Group: %ls", *Groups->GetGroupNameString(GroupHandle));
	}
}

void FReplicationFiltering::SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, ENetFilterStatus ReplicationStatus)
{
	IRIS_PROFILER_SCOPE(SetGroupFilterStatus)

	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();
	if (GroupHandle.IsReservedNetObjectGroup())
	{
		UE_LOGF(LogIrisFiltering, Warning, "FReplicationFiltering::SetGroupFilterStatus - Trying to set filter for reserved Group: %ls which is not allowed.", *Groups->GetGroupNameString(GroupHandle));
		return;
	}

	if (ExclusionFilterGroups.GetBit(GroupIndex))
	{
		// It is intentional that we iterate over all connections, not only the valid ones as we want the group filter status to be initialized correctly for new connections as well.
		for (uint32 ConnectionId = 1U; ConnectionId < ValidConnections.GetNumBits(); ++ConnectionId)
		{
			InternalSetExclusionGroupFilterStatus(GroupHandle, ConnectionId, ReplicationStatus);
		}

		return;
	}

	if (InclusionFilterGroups.GetBit(GroupIndex))
	{
		InternalSetInclusionGroupFilterStatus(GroupHandle, ReplicationStatus);
		return;
	}
	
	UE_LOGF(LogIrisFiltering, Warning, "FReplicationFiltering::SetGroupFilterStatus - Trying to set filter for invalid Group: %ls, Make sure group is added to filtering", *Groups->GetGroupNameString(GroupHandle));
}

void FReplicationFiltering::SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, const FNetBitArrayView& ConnectionsBitArray, ENetFilterStatus ReplicationStatus)
{
	IRIS_PROFILER_SCOPE(SetGroupFilterStatus)
		
	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();
	if (GroupHandle.IsReservedNetObjectGroup())
	{
		UE_LOGF(LogIrisFiltering, Warning, "FReplicationFiltering::SetGroupFilterStatus - Trying to set filter on reserved Group: %ls which is not allowed.", *Groups->GetGroupNameString(GroupHandle));
		return;
	}

	if (ConnectionsBitArray.GetNumBits() > ValidConnections.GetNumBits())
	{
		UE_LOGF(LogIrisFiltering, Warning, "FReplicationFiltering::SetGroupFilterStatus - Trying to set filter on Group: %ls, with invalid Connections parameters.", *Groups->GetGroupNameString(GroupHandle));
		return;
	}

	if (ExclusionFilterGroups.GetBit(GroupIndex))
	{
		// It is intentional that we iterate over all connections, not only the valid ones as we want the group filter status to be initialized correctly for new connections as well.
		for (uint32 ConnectionId = 1U, EndConnectionId = ValidConnections.GetNumBits(); ConnectionId < EndConnectionId; ++ConnectionId)
		{
			ENetFilterStatus ReplicationStatusToSet = ReplicationStatus;
			if (ConnectionId >= ConnectionsBitArray.GetNumBits() || !ConnectionsBitArray.GetBit(ConnectionId))
			{
				ReplicationStatusToSet = ReplicationStatus == ENetFilterStatus::Allow ? ENetFilterStatus::Disallow : ENetFilterStatus::Allow;
			}
			InternalSetExclusionGroupFilterStatus(GroupHandle, ConnectionId, ReplicationStatusToSet);
		}

		return;
	}
	
	if (InclusionFilterGroups.GetBit(GroupIndex))
	{
		// It is intentional that we iterate over all connections, not only the valid ones as we want the group filter status to be initialized correctly for new connections as well.
		for (uint32 ConnectionId = 1U, EndConnectionId = ValidConnections.GetNumBits(); ConnectionId < EndConnectionId; ++ConnectionId)
		{
			ENetFilterStatus ReplicationStatusToSet = ReplicationStatus;
			if (ConnectionId >= ConnectionsBitArray.GetNumBits() || !ConnectionsBitArray.GetBit(ConnectionId))
			{
				ReplicationStatusToSet = ReplicationStatus == ENetFilterStatus::Allow ? ENetFilterStatus::Disallow : ENetFilterStatus::Allow;
			}
			InternalSetInclusionGroupFilterStatus(GroupHandle, ConnectionId, ReplicationStatusToSet);
		}
		
		return;
	}

	UE_LOGF(LogIrisFiltering, Warning, "FReplicationFiltering::SetGroupFilterStatus - Trying to set filter for invalid Group: %ls, Make sure group is added to filtering", *Groups->GetGroupNameString(GroupHandle));
}

void FReplicationFiltering::SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId, ENetFilterStatus ReplicationStatus)
{
	IRIS_PROFILER_SCOPE(SetGroupFilterStatus)

	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();
	if (GroupHandle.IsReservedNetObjectGroup())
	{
		UE_LOGF(LogIrisFiltering, Warning, "FReplicationFiltering::SetGroupFilterStatus - Trying to set filter on reserved Group: %ls which is not allowed.", *Groups->GetGroupNameString(GroupHandle));
		return;
	}

	if (ExclusionFilterGroups.GetBit(GroupIndex))
	{
		InternalSetExclusionGroupFilterStatus(GroupHandle, ConnectionId, ReplicationStatus);
		return;
	}
	
	if (InclusionFilterGroups.GetBit(GroupIndex))
	{
		InternalSetInclusionGroupFilterStatus(GroupHandle, ConnectionId, ReplicationStatus);
		return;
	}

	UE_LOGF(LogIrisFiltering, Warning, "FReplicationFiltering::SetGroupFilterStatus - Trying to set invalid filter status: %u on Group: %ls for ConnectionId: %u", EnumToUnderlyingType(ReplicationStatus), *Groups->GetGroupNameString(GroupHandle), ConnectionId);
}

bool FReplicationFiltering::IsExcludedByAnyGroup(uint32 ObjectInternalIndex, uint32 ConnectionId) const
{
	const TArrayView<const FNetObjectGroupHandle::FGroupIndexType> GroupIndexes = Groups->GetGroupIndexesOfNetObject(ObjectInternalIndex);
	for (const FNetObjectGroupHandle::FGroupIndexType GroupIndex : GroupIndexes)
	{
		if (ExclusionFilterGroups.GetBit(GroupIndex))
		{
			const FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupIndex].ConnectionStateIndex);
			if (GetConnectionFilterStatus(*ConnectionState, ConnectionId) == ENetFilterStatus::Disallow)
			{
				return true;
			}
		}
	}

	return false;
}

bool FReplicationFiltering::IsIncludedByAnyGroup(uint32 ObjectInternalIndex, uint32 ConnectionId) const
{
	const TArrayView<const FNetObjectGroupHandle::FGroupIndexType> GroupIndexes = Groups->GetGroupIndexesOfNetObject(ObjectInternalIndex);
	for (const FNetObjectGroupHandle::FGroupIndexType GroupIndex : GroupIndexes)
	{
		if (InclusionFilterGroups.GetBit(GroupIndex))
		{
			const FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupIndex].ConnectionStateIndex);
			if (GetConnectionFilterStatus(*ConnectionState, ConnectionId) == ENetFilterStatus::Allow)
			{
				return true;
			}
		}
	}

	return false;
}

bool FReplicationFiltering::ClearGroupExclusionFilterEffectsForObject(uint32 ObjectIndex, uint32 ConnectionId)
{
	FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
	FNetBitArray& GroupExcludedObjects = ConnectionInfo.GroupExcludedObjects;
	FNetBitArray& ConnectionFilteredObjects = ConnectionInfo.ConnectionFilteredObjects;
	FNetBitArray& ObjectsInScopeBeforeDynamicFiltering = ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering;

	auto ClearGroupFilterForObject = [&GroupExcludedObjects, &ConnectionFilteredObjects, &ObjectsInScopeBeforeDynamicFiltering](uint32 InternalObjectIndex)
	{
		GroupExcludedObjects.ClearBit(InternalObjectIndex);
		ObjectsInScopeBeforeDynamicFiltering.SetBitValue(InternalObjectIndex, ConnectionFilteredObjects.GetBit(InternalObjectIndex));
	};

	if (!IsExcludedByAnyGroup(ObjectIndex, ConnectionId))
	{
		ClearGroupFilterForObject(ObjectIndex);
	
		// Filter subobjects
		for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
		{
			if (!IsExcludedByAnyGroup(SubObjectIndex, ConnectionId))
			{
				ClearGroupFilterForObject(SubObjectIndex);
			}
		}
		
		return true;
	}

	return false;
}

bool FReplicationFiltering::ClearGroupInclusionFilterEffectsForObject(uint32 ObjectIndex, uint32 ConnectionId)
{
	FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];

	// Ignore subobjects. Inclusion groups overrides dynamic filters which only operate on root objects. SubObjects need to be filtered like the root.
	if (NetRefHandleManager->GetSubObjectInternalIndices().GetBit(ObjectIndex))
	{
		return false;
	}

	if (!IsIncludedByAnyGroup(ObjectIndex, ConnectionId))
	{
		// Dynamically filtered objects are subject to hysteresis.
		if (HysteresisState.Mode == EHysteresisProcessingMode::Enabled && DynamicFilterEnabledObjects.GetBit(ObjectIndex))
		{
			// Force processing of dynamically filtered object. If the object is still in scope and dynamically filtered out it should cause hysteresis to kick in.
			ObjectsRequiringDynamicFilterUpdate.SetBit(ObjectIndex);
		}

		FNetBitArray& GroupIncludedObjects = ConnectionInfo.GroupIncludedObjects;
		
		GroupIncludedObjects.ClearBit(ObjectIndex);

		for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
		{
			GroupIncludedObjects.ClearBit(SubObjectIndex);
		}
		
		return true;
	}

	return false;
}

void FReplicationFiltering::InternalSetExclusionGroupFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId, ENetFilterStatus ReplicationStatus)
{
	if (ReplicationStatus == ENetFilterStatus::Disallow)
	{
		// Filter out objects in group
		FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupHandle.GetGroupIndex()].ConnectionStateIndex);
		if (GetConnectionFilterStatus(*ConnectionState, ConnectionId) == ENetFilterStatus::Allow)
		{
			IRIS_PROFILER_SCOPE(FReplicationFiltering_InternalSetExclusionGroupFilterStatus_Disallow);

			// Mark filter as active for the connection
			SetConnectionFilterStatus(*ConnectionState, ConnectionId, ENetFilterStatus::Disallow);

			UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::SetGroupFilterStatus ExclusionGroup: %ls, ConnectionId: %u, FilterStatus: DisallowReplication", *Groups->GetGroupNameString(GroupHandle), ConnectionId);

			if (ValidConnections.GetBit(ConnectionId))
			{
				// New Connections will be initialized in the next filter update
				if (!NewConnections.GetBit(ConnectionId))
				{
					const FNetObjectGroup* Group = Groups->GetGroup(GroupHandle);
			
					// Update group filter mask for the connection
					FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
					FNetBitArray& GroupExcludedObjects = ConnectionInfo.GroupExcludedObjects;
					FNetBitArray& ObjectsInScopeBeforeDynamicFiltering = ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering;
					const FNetBitArrayView GlobalScopableObjects = NetRefHandleManager->GetGlobalScopableInternalIndices();
					for (const FInternalNetRefIndex ObjectIndex : Group->Members)
					{
						GroupExcludedObjects.SetBit(ObjectIndex);

						//$IRIS TODO: ObjectsInScopeBeforeDynamicFiltering should not be accessed outside PreSendUpdate since its reset there. Is this needed ?
						ObjectsInScopeBeforeDynamicFiltering.ClearBit(ObjectIndex);
						// Filter subobjects
						for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
						{
							GroupExcludedObjects.SetBitValue(SubObjectIndex, GlobalScopableObjects.GetBit(SubObjectIndex));
							ObjectsInScopeBeforeDynamicFiltering.ClearBit(SubObjectIndex);
						}
					}
				}
			}
		}
	}
	else
	{
		// Clear filter effects
		FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupHandle.GetGroupIndex()].ConnectionStateIndex);
		if (GetConnectionFilterStatus(*ConnectionState, ConnectionId) == ENetFilterStatus::Disallow)
		{
			IRIS_PROFILER_SCOPE(FReplicationFiltering_InternalSetExclusionGroupFilterStatus_Allow);

			// Mark filter as no longer being active for the connection
			SetConnectionFilterStatus(*ConnectionState, ConnectionId, ENetFilterStatus::Allow);

			UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::SetGroupFilterStatus ExclusionGroup: %ls, ConnectionId: %u, FilterStatus: AllowReplication", *Groups->GetGroupNameString(GroupHandle), ConnectionId);

			if (ValidConnections.GetBit(ConnectionId))
			{
				// New Connections will be initialized the next filter update
				if (!NewConnections.GetBit(ConnectionId))
				{
					const FNetObjectGroup* Group = Groups->GetGroup(GroupHandle);

					// Update group filter mask for the connection
					for (const FInternalNetRefIndex ObjectIndex : Group->Members)
					{
						ClearGroupExclusionFilterEffectsForObject(ObjectIndex, ConnectionId);
					}
				}
			}
		}
	}
}

// Set same status for all connections
void FReplicationFiltering::InternalSetInclusionGroupFilterStatus(FNetObjectGroupHandle GroupHandle, ENetFilterStatus ReplicationStatus)
{
	FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupHandle.GetGroupIndex()].ConnectionStateIndex);

	// If status is intact we can avoid expensive processing.
	if (ReplicationStatus == ENetFilterStatus::Disallow && !IsAnyConnectionFilterStatusAllowed(*ConnectionState))
	{
		return;
	}

	auto UpdateFilterStatusForConnection = [this, GroupHandle, ReplicationStatus](uint32 ConnectionId)
	{
		InternalSetInclusionGroupFilterStatus(GroupHandle, ConnectionId, ReplicationStatus);
	};

	// Update filter status for all initialized connections
	FNetBitArray::ForAllSetBits(ValidConnections, NewConnections, FNetBitArrayBase::AndNotOp, UpdateFilterStatusForConnection);

	// Make sure status is set for all connections.
	SetPerObjectInfoFilterStatus(*ConnectionState, ReplicationStatus);
}

// Set status for a single connection
void FReplicationFiltering::InternalSetInclusionGroupFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId, ENetFilterStatus ReplicationStatus)
{
	FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupHandle.GetGroupIndex()].ConnectionStateIndex);
	// If status remains there's nothing to do.
	if (GetConnectionFilterStatus(*ConnectionState, ConnectionId) == ReplicationStatus)
	{
		return;
	}

	SetConnectionFilterStatus(*ConnectionState, ConnectionId, ReplicationStatus);

	// Ignore processing of not yet initialized connections
	if (!ValidConnections.GetBit(ConnectionId) || NewConnections.GetBit(ConnectionId))
	{
		return;
	}

	const FNetObjectGroup* Group = Groups->GetGroup(GroupHandle);
	if (ReplicationStatus == ENetFilterStatus::Disallow)
	{
		UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::SetGroupFilterStatus InclusionGroup: %ls, ConnectionId: %u, FilterStatus: DisallowReplication", *Groups->GetGroupNameString(GroupHandle), ConnectionId);
		for (const FInternalNetRefIndex ObjectIndex : Group->Members)
		{
			ClearGroupInclusionFilterEffectsForObject(ObjectIndex, ConnectionId);
		}
	}
	else
	{
		UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::SetGroupFilterStatus InclusionGroup: %ls, ConnectionId: %u, FilterStatus: AllowReplication", *Groups->GetGroupNameString(GroupHandle), ConnectionId);

		FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
		FNetBitArray& GroupIncludedObjects = ConnectionInfo.GroupIncludedObjects;
		const FNetBitArrayView GlobalScopableObjects = NetRefHandleManager->GetGlobalScopableInternalIndices();
		const FNetBitArrayView SubObjectInternalIndices = NetRefHandleManager->GetSubObjectInternalIndicesView();
		for (const FInternalNetRefIndex ObjectIndex : Group->Members)
		{
			// SubObjects follow root object.
			if (SubObjectInternalIndices.GetBit(ObjectIndex))
			{
				continue;
			}

			GroupIncludedObjects.SetBitValue(ObjectIndex, GlobalScopableObjects.GetBit(ObjectIndex));
			for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectIndex))
			{
				GroupIncludedObjects.SetBitValue(SubObjectIndex, GlobalScopableObjects.GetBit(SubObjectIndex));
			}
		}
	}
}

bool FReplicationFiltering::GetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId, ENetFilterStatus& OutReplicationStatus) const
{
	if (!(ValidConnections.GetBit(ConnectionId) && ExclusionFilterGroups.GetBit(GroupHandle.GetGroupIndex())))
	{
		return false;
	}

	const FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupHandle.GetGroupIndex()].ConnectionStateIndex);
	OutReplicationStatus = GetConnectionFilterStatus(*ConnectionState, ConnectionId);

	return true;
}

void FReplicationFiltering::NotifyObjectAddedToGroup(FNetObjectGroupHandle GroupHandle, FInternalNetRefIndex ObjectIndex)
{
	// As we want to avoid the fundamentally bad case of doing tons of single object adds to a group and updating group filter for every call we defer the update until next call to filter
	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();
	if (SubObjectFilterGroups.GetBit(GroupIndex))
	{
		UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::NotifyObjectAddedToGroup Added %ls to SubObjectFilter group: %ls", *(NetRefHandleManager->PrintObjectFromIndex(ObjectIndex)), *Groups->GetGroupNameString(GroupHandle));
	}
	else if (ExclusionFilterGroups.GetBit(GroupIndex))
	{
		const FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupHandle.GetGroupIndex()].ConnectionStateIndex);

		// Avoid expensive filter updates if possible.
		if (IsAnyConnectionFilterStatusDisallowed(*ConnectionState))
		{
			DirtyExclusionFilterGroups.SetBit(GroupIndex);
			bHasDirtyExclusionFilterGroup = 1;
		}

		UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::NotifyObjectAddedToGroup Added %ls to ExclusionFilter group: %ls", *(NetRefHandleManager->PrintObjectFromIndex(ObjectIndex)), *Groups->GetGroupNameString(GroupHandle));
	}
	else if (InclusionFilterGroups.GetBit(GroupIndex))
	{
		const FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupHandle.GetGroupIndex()].ConnectionStateIndex);

		// Avoid expensive filter updates if possible.
		if (IsAnyConnectionFilterStatusAllowed(*ConnectionState))
		{
			DirtyInclusionFilterGroups.SetBit(GroupIndex);
			bHasDirtyInclusionFilterGroup = 1;
		}

		UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::NotifyObjectAddedToGroup Added %ls to InclusionFilter group: %ls", *(NetRefHandleManager->PrintObjectFromIndex(ObjectIndex)), *Groups->GetGroupNameString(GroupHandle));
	}
}

void FReplicationFiltering::NotifyObjectRemovedFromGroup(FNetObjectGroupHandle GroupHandle, FInternalNetRefIndex ObjectIndex)
{
	const FNetObjectGroupHandle::FGroupIndexType GroupIndex = GroupHandle.GetGroupIndex();

	UE_LOGF(LogIrisFiltering, Verbose, "ReplicationFiltering::NotifyObjectRemovedFromGroup Removing %ls from Group: %ls", ToCStr(NetRefHandleManager->PrintObjectFromIndex(ObjectIndex)), *Groups->GetGroupNameString(GroupHandle));

	if (SubObjectFilterGroups.GetBit(GroupIndex))
	{
		DirtySubObjectFilterGroups.SetBit(GroupIndex);
	}
	else if (ExclusionFilterGroups.GetBit(GroupIndex))
	{
		const FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupIndex].ConnectionStateIndex);

		auto UpdateGroupEffects = [this, ObjectIndex, GroupHandle, ConnectionState](uint32 ConnectionId)
		{
			if (GetConnectionFilterStatus(*ConnectionState, ConnectionId) == ENetFilterStatus::Disallow)
			{
				FNetBitArray& GroupExcludedObjects = this->ConnectionInfos[ConnectionId].GroupExcludedObjects;
				ClearGroupExclusionFilterEffectsForObject(ObjectIndex, ConnectionId);
			}
		};

		// We need to update filter status for all initialized connections
		FNetBitArray::ForAllSetBits(ValidConnections, NewConnections, FNetBitArrayBase::AndNotOp, UpdateGroupEffects);
	}
	else if (InclusionFilterGroups.GetBit(GroupIndex))
	{
		const FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupIndex].ConnectionStateIndex);

		auto UpdateGroupEffects = [this, ObjectIndex, GroupHandle, ConnectionState](uint32 ConnectionId)
		{
			if (GetConnectionFilterStatus(*ConnectionState, ConnectionId) == ENetFilterStatus::Allow)
			{
				FNetBitArray& GroupIncludedObjects = this->ConnectionInfos[ConnectionId].GroupIncludedObjects;
				ClearGroupInclusionFilterEffectsForObject(ObjectIndex, ConnectionId);
			}
		};

		// We need to update filter status for all initialized connections
		FNetBitArray::ForAllSetBits(ValidConnections, NewConnections, FNetBitArrayBase::AndNotOp, UpdateGroupEffects);
	}
}

void FReplicationFiltering::NotifyAddedDependentObject(FInternalNetRefIndex ObjectIndex)
{
#if !UE_BUILD_SHIPPING
	if (UE_LOG_ACTIVE(LogIrisFiltering, Display))
	{
		if (!this->DynamicFilterEnabledObjects.GetBit(ObjectIndex))
		{
			const UObject* Object = NetRefHandleManager->GetReplicatedObjectInstance(ObjectIndex);
			UE_CLOGF(Object && ReplicationFiltering_MootDependentObjectTracker.ShouldLog(Object->GetClass()->GetFName()), LogIrisFiltering, Display, "FReplicationFiltering::NotifyAddedDependentObject: Object doesn't have a dynamic filter set so having a dependency on it won't change when it's replicated. %ls", ToCStr(NetRefHandleManager->PrintObjectFromIndex(ObjectIndex)));
		}
	}
#endif
	ObjectsRequiringDynamicFilterUpdate.SetBit(ObjectIndex);
}

void FReplicationFiltering::NotifyRemovedDependentObject(FInternalNetRefIndex ObjectIndex)
{
	ObjectsRequiringDynamicFilterUpdate.SetBit(ObjectIndex);
}

// Determine whether a dynamically filtered out dependent object should be in scope or not. Note that this function is expected to only be called for dependent objects NOT in scope. The Visited set is expected to be empty when calling this function.
ENetFilterStatus FReplicationFiltering::GetDependentObjectFilterStatus(uint32 ConnectionId, FInternalNetRefIndex DependentObjectIndex, TSet<FInternalNetRefIndex>& Visited)
{
	const FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];

	TArray<FInternalNetRefIndex, TInlineAllocator<64>> DependentObjects;	
	bool bIsAlreadyVisited = false;

	// Iteratively visit objects and their parents until we find one in scope.
	DependentObjects.Add(DependentObjectIndex);
	do
	{
		const FInternalNetRefIndex ObjectIndex = DependentObjects.Pop(EAllowShrinking::No);

		// If we have a cyclic dependency chain or already visited the object we continue to the next parent as we already know this one nor any of its parents are in scope.
		Visited.Add(ObjectIndex, &bIsAlreadyVisited);
		if (bIsAlreadyVisited)
		{
			constexpr uint32 ArbitraryMaxLogCount = 3U;
			UE_CLOG((ObjectIndex == DependentObjectIndex) && (LoggedCyclicDependencyCount++ < ArbitraryMaxLogCount), LogIrisFiltering, Error, TEXT("Found circular reference while retrieving dependent object filter status for %s"), ToCStr(NetRefHandleManager->PrintObjectFromIndex(DependentObjectIndex)));
			ensureMsgf(ObjectIndex != DependentObjectIndex, TEXT("Found circular reference while retrieving dependent object filter status for %s"), ToCStr(NetRefHandleManager->PrintObjectFromIndex(DependentObjectIndex)));
			continue;
		}

		// If the parent is filtered out prior to dynamic filtering we should continue looking for other parents which may allow replication.
		if (!ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering.GetBit(ObjectIndex))
		{
			// There may be other parents in scope so we should continue processing those.
			continue;
		}

		// If the parent is in scope so should the dependent object be.
		if (ConnectionInfo.ObjectsInScope.GetBit(ObjectIndex))
		{
			return ENetFilterStatus::Allow;
		}

		// If it's filtered out dynamically and not a dependent object this parent can't allow replication. In that case it won't have any parents anyway so it's safe to call GetDependentObjectParents.
		DependentObjects.Append(NetRefHandleManager->GetDependentObjectParents(ObjectIndex));
	}
	while (!DependentObjects.IsEmpty());
	
	// If no parent is in scope this object shouldn't be either. 
	return ENetFilterStatus::Disallow;
}

ENetFilterStatus FReplicationFiltering::GetConnectionFilterStatus(const FPerObjectInfo& ObjectInfo, uint32 ConnectionId) const
{
	return (ObjectInfo.ConnectionIds[ConnectionId >> 5U] & (1U << (ConnectionId & 31U))) != 0U ? ENetFilterStatus::Allow : ENetFilterStatus::Disallow;
}

bool FReplicationFiltering::IsAnyConnectionFilterStatusAllowed(const FPerObjectInfo& ObjectInfo) const
{
	static_assert(uint32(ENetFilterStatus::Disallow) == 0U && uint32(ENetFilterStatus::Allow) == 1U, "Inappropriate assumptions regarding ENetFilterStatus values");
	constexpr uint32 FirstValidConnectionIndex = 1U;
	return FNetBitArrayView(const_cast<FNetBitArrayView::StorageWordType*>(ObjectInfo.ConnectionIds), ValidConnections.GetNumBits(), FNetBitArrayBase::NoResetNoValidate).FindFirstOne(FirstValidConnectionIndex) != FNetBitArrayBase::InvalidIndex;
}

bool FReplicationFiltering::IsAnyConnectionFilterStatusDisallowed(const FPerObjectInfo& ObjectInfo) const
{
	static_assert(uint32(ENetFilterStatus::Disallow) == 0U && uint32(ENetFilterStatus::Allow) == 1U, "Inappropriate assumptions regarding ENetFilterStatus values");
	constexpr uint32 FirstValidConnectionIndex = 1U;
	return FNetBitArrayView(const_cast<FNetBitArrayView::StorageWordType*>(ObjectInfo.ConnectionIds), ValidConnections.GetNumBits(), FNetBitArrayBase::NoResetNoValidate).FindFirstZero(FirstValidConnectionIndex) != FNetBitArrayBase::InvalidIndex;
}

void FReplicationFiltering::SetConnectionFilterStatus(FPerObjectInfo& ObjectInfo, uint32 ConnectionId, ENetFilterStatus ReplicationStatus)
{
	const uint32 WordMask = 1 << (ConnectionId & 31U);
	const uint32 ValueMask = ReplicationStatus == ENetFilterStatus::Allow ? WordMask : 0U;

	ObjectInfo.ConnectionIds[ConnectionId >> 5U] = (ObjectInfo.ConnectionIds[ConnectionId >> 5U] & ~WordMask) | ValueMask;
}

void FReplicationFiltering::SetPerObjectInfoFilterStatus(FPerObjectInfo& ObjectInfo, ENetFilterStatus ReplicationStatus)
{
	const uint32 InitialValue = ReplicationStatus == ENetFilterStatus::Allow ? ~0U : 0U;
	for (uint32 WordIt = 0, WordEndIt = PerObjectInfoStorageCountPerItem; WordIt != WordEndIt; ++WordIt)
	{
		ObjectInfo.ConnectionIds[WordIt] = InitialValue;
	}
}

FReplicationFiltering::FPerSubObjectFilterGroupInfo& FReplicationFiltering::CreatePerSubObjectGroupFilterInfo(FNetObjectGroupHandle::FGroupIndexType GroupIndex)
{
	FPerSubObjectFilterGroupInfo& GroupInfo = SubObjectFilterGroupInfos.Add(GroupIndex);
	// Ensure group info didn't already exist
	ensure(GroupInfo.ConnectionStateIndex == 0);
	GroupInfo.ConnectionStateIndex = AllocPerObjectInfo();
	SetPerObjectInfoFilterStatus(*GetPerObjectInfo(GroupInfo.ConnectionStateIndex), ENetFilterStatus::Disallow);

	return GroupInfo;
}

void FReplicationFiltering::DestroyPerSubObjectGroupFilterInfo(FNetObjectGroupHandle::FGroupIndexType GroupIndex)
{
	if (FPerSubObjectFilterGroupInfo* GroupInfo = SubObjectFilterGroupInfos.Find(GroupIndex))
	{
		FreePerObjectInfo(GroupInfo->ConnectionStateIndex);
		SubObjectFilterGroupInfos.Remove(GroupIndex);
	}
}

FReplicationFiltering::FPerSubObjectFilterGroupInfo* FReplicationFiltering::GetPerSubObjectFilterGroupInfo(FNetObjectGroupHandle::FGroupIndexType GroupIndex)
{
	return SubObjectFilterGroupInfos.Find(GroupIndex);
}

const FReplicationFiltering::FPerSubObjectFilterGroupInfo* FReplicationFiltering::GetPerSubObjectFilterGroupInfo(FNetObjectGroupHandle::FGroupIndexType GroupIndex) const
{
	return SubObjectFilterGroupInfos.Find(GroupIndex);
}

void FReplicationFiltering::InitFilters()
{
	const UNetObjectFilterDefinitions* FilterDefinitions = GetDefault<UNetObjectFilterDefinitions>();

	// We store a uint8 per object to filter.
	check(FilterDefinitions->GetFilterDefinitions().Num() <= 256);

	for (const FNetObjectFilterDefinition& FilterDefinition : FilterDefinitions->GetFilterDefinitions())
	{
		CreateFilter(FilterDefinition);
	}
}

TWeakObjectPtr<UNetObjectFilter> FReplicationFiltering::CreateFilter(const FNetObjectFilterDefinition& FilterDefinition)
{
	if (DynamicFilterInfos.Num() >= InvalidDynamicFilterIndex)
	{
		UE_LOG(
			LogIrisFilterConfig,
			Error,
			TEXT("Cannot create filter %s: maximum number of filters reached (%" PRId32 ")"),
			*FilterDefinition.FilterName.ToString(),
			DynamicFilterInfos.Num());
		return nullptr;
	}

	// Filter names must be unique.
	if (Algo::AnyOf(DynamicFilterInfos, [&Name = FilterDefinition.FilterName](const FFilterInfo& Info) { return Info.Name == Name; }))
	{
		UE_LOGF(
			LogIrisFilterConfig,
			Warning,
			"Attempt to create filter %ls when one with that name already exists",
			*FilterDefinition.FilterName.ToString());
		return nullptr;
	}

	constexpr UObject* ClassOuter = nullptr;
	UClass* NetObjectFilterClass = Cast<UClass>(StaticFindObject(UClass::StaticClass(), ClassOuter, ToCStr(FilterDefinition.ClassName.ToString()), EFindObjectFlags::ExactClass));
	if (NetObjectFilterClass == nullptr || !NetObjectFilterClass->IsChildOf(UNetObjectFilter::StaticClass()))
	{
		ensureMsgf(false, TEXT("NetObjectFilter class is not a NetObjectFilter or could not be found: %s"), ToCStr(FilterDefinition.ClassName.ToString()));
		return nullptr;
	}

	UClass* NetObjectFilterConfigClass = nullptr;
	if (!FilterDefinition.ConfigClassName.IsNone())
	{
		NetObjectFilterConfigClass = Cast<UClass>(StaticFindObject(UClass::StaticClass(), ClassOuter, ToCStr(FilterDefinition.ConfigClassName.ToString()), EFindObjectFlags::ExactClass));
		if (NetObjectFilterConfigClass == nullptr || !NetObjectFilterConfigClass->IsChildOf(UNetObjectFilterConfig::StaticClass()))
		{
			ensureMsgf(false, TEXT("NetObjectFilterConfig class is not a NetObjectFilterConfig or could not be found: %s"), ToCStr(FilterDefinition.ConfigClassName.ToString()));
			return nullptr;
		}
	}

	FFilterInfo& Info = DynamicFilterInfos.Emplace_GetRef();
	Info.Filter = TStrongObjectPtr<UNetObjectFilter>(NewObject<UNetObjectFilter>(GetTransientPackage(), NetObjectFilterClass, MakeUniqueObjectName(nullptr, NetObjectFilterClass, FilterDefinition.FilterName)));
	check(Info.Filter.IsValid());
	Info.Name = FilterDefinition.FilterName;
	Info.ObjectCount = 0;

	Info.Filter->Init(GetFilterInitParams(NetObjectFilterConfigClass));

	bHasDynamicFilters = true;
	bHasDynamicFiltersWithUpdateTrait = bHasDynamicFiltersWithUpdateTrait || Info.Filter->HasFilterTrait(ENetFilterTraits::NeedsUpdate);

	ValidConnections.ForAllSetBits([&Filter = *Info.Filter](const uint32 ConnectionId) { Filter.AddConnection(ConnectionId); });

	return Info.Filter.Get();
}

void FReplicationFiltering::DestroyFilter(const FName Name)
{
	for (FFilterInfo& FilterInfo : DynamicFilterInfos)
	{
		if (FilterInfo.Name != Name || !FilterInfo.Filter)
		{
			continue;
		}

		UE_LOGF(LogIrisFiltering, Verbose, "Destroying filter \"%ls\"", *FilterInfo.Name.ToString());

		UNetObjectFilter& Filter = *FilterInfo.Filter;

		// Unassign any objects assigned to the filter.
		// Construct FNetBitArray from FNetBitArrayView (this conversion is not provided by FNetBitArray at time of writing).
		auto CloneBitArray = [](const FNetBitArrayView& View) -> FNetBitArray
			{
				FNetBitArray BitArray;
				BitArray.InitAndCopy(View);
				return BitArray;
			};

		// Local copy to avoid modifying the bitset while iterating it
		if (const FNetBitArray AssignedObjects = CloneBitArray(Filter.GetFilteredObjects());
			AssignedObjects.IsAnyBitSet())
		{
			UE_LOG(
				LogIrisFiltering,
				Log,
				TEXT("Orphaning %" PRIu32 " objects assigned to filter %s at filter destruction"),
				AssignedObjects.CountSetBits(),
				*Filter.GetName());

			AssignedObjects.ForAllSetBits(
				[this](uint32 ObjectIndex)
				{
					SetFilter(FInternalNetRefIndex(ObjectIndex), InvalidNetObjectFilterHandle, NAME_None);
				});
		}

		Filter.Deinit();

		// Destroy the filter but keep its index reserved to avoid renumbering subsequent filter handles and prevent reuse of stale handles with
		// an unintended filter.
		FilterInfo = FFilterInfo{
			.Filter = nullptr,
			.Name = NAME_None,
			.ObjectCount = 0,
		};

		break; // Filter names are unique; only one may match.
	}
}

void FReplicationFiltering::InitObjectScopeHysteresis()
{
	HysteresisState.Mode = Config->IsObjectScopeHysteresisEnabled() ? EHysteresisProcessingMode::Enabled : EHysteresisProcessingMode::Disabled;
}

bool FReplicationFiltering::IsValidDynamicFilter(uint32 FilterIndex) const
{
	return FilterIndex != InvalidDynamicFilterIndex
		&& FilterIndex < static_cast<uint32>(DynamicFilterInfos.Num())
		&& DynamicFilterInfos[FilterIndex].Filter != nullptr;
}

void FReplicationFiltering::RemoveFromDynamicFilter(UE::Net::FInternalNetRefIndex ObjectIndex, uint32 FilterIndex)
{
	UE_LOGF(LogIrisFiltering, Verbose, "RemoveFromDynamicFilter removing %ls from Dynamic Filter %ls", 
		*NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), *GetFilterName(FNetObjectFilterHandleUtil::MakeDynamicFilterHandle(ObjectIndexToDynamicFilterIndex[ObjectIndex])).ToString());

	ObjectIndexToDynamicFilterIndex[ObjectIndex] = InvalidDynamicFilterIndex;
	FNetObjectFilteringInfo& NetObjectFilteringInfo = NetObjectFilteringInfos[ObjectIndex];
	if (IsValidDynamicFilter(FilterIndex))
	{
		FFilterInfo& FilterInfo = DynamicFilterInfos[FilterIndex];
		--FilterInfo.ObjectCount;
		FilterInfo.Filter->GetFilteredObjects().ClearBit(ObjectIndex);
		FilterInfo.Filter->RemoveObject(ObjectIndex, NetObjectFilteringInfo);
	}

	DynamicFilterEnabledObjects.ClearBit(ObjectIndex);
	ObjectsRequiringDynamicFilterUpdate.SetBit(ObjectIndex);

	// Remove from hysteresis
	HysteresisState.ClearFromHysteresis(ObjectIndex);
}


TArrayView<FNetObjectFilteringInfo> FReplicationFiltering::GetNetObjectFilteringInfos()
{
	return MakeArrayView(NetObjectFilteringInfos);
}

FString FReplicationFiltering::PrintFilterObjectInfo(FInternalNetRefIndex ObjectIndex, uint32 ConnectionId) const
{
	// $IRIS TODO: Support printing filter info for subobjects

	TStringBuilder<512> PrintBuilder;

	if (ValidConnections.IsBitSet(ConnectionId))
	{
		if (HasOwnerFilter(ObjectIndex))
		{
			const uint32 OwningConnection = ObjectIndexToOwningConnection[ObjectIndex];
			const bool bIsOwner = (ConnectionId == OwningConnection);
			
			PrintBuilder.Appendf(TEXT("\r\t OwnerFilter to connectionId:%u | %s"), ConnectionId, bIsOwner?TEXT("relevant"):TEXT("not relevant"));
		}

		const FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];

		const TArrayView<const FNetObjectGroupHandle::FGroupIndexType> GroupIndexes = Groups->GetGroupIndexesOfNetObject(ObjectIndex);

		// List of objects excluded to this connection via a group
		const FNetBitArray& GroupExcludedObjects = ConnectionInfo.GroupExcludedObjects;

		// Is Excluded via group
		if (GroupExcludedObjects.IsBitSet(ObjectIndex))
		{
			PrintBuilder.Append(TEXT("\r\t ExclusionGroups:"));
			for (const FNetObjectGroupHandle::FGroupIndexType GroupIndex : GroupIndexes)
			{
				if (ExclusionFilterGroups.GetBit(GroupIndex))
				{
					const FNetObjectGroup* Group = Groups->GetGroupFromIndex(GroupIndex);
					check(Group);
					const FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupIndex].ConnectionStateIndex);
					check(ConnectionState);
					
					ENetFilterStatus FilterStatus = GetConnectionFilterStatus(*ConnectionState, ConnectionId);
					if (FilterStatus == ENetFilterStatus::Disallow)
					{
						PrintBuilder.Appendf(TEXT(" | %s status: %s"), ToCStr(Group->GroupName.ToString()), LexToString(FilterStatus));
					}
				}
			}
		}

		// List of objects included to this connection via a group
		const FNetBitArray& GroupIncludedObjects = ConnectionInfo.GroupIncludedObjects;

		// Is Included via group
		if (GroupIncludedObjects.IsBitSet(ObjectIndex))
		{
			PrintBuilder.Append(TEXT("\r\t InclusionGroups:"));
			for (const FNetObjectGroupHandle::FGroupIndexType GroupIndex : GroupIndexes)
			{
				if (InclusionFilterGroups.GetBit(GroupIndex))
				{
					const FNetObjectGroup* Group = Groups->GetGroupFromIndex(GroupIndex);
					check(Group);
					const FPerObjectInfo* ConnectionState = GetPerObjectInfo(GroupInfos[GroupIndex].ConnectionStateIndex);
					check(ConnectionState);
					
					ENetFilterStatus FilterStatus = GetConnectionFilterStatus(*ConnectionState, ConnectionId);
					if (FilterStatus == ENetFilterStatus::Allow)
					{
						PrintBuilder.Appendf(TEXT(" | %s status: %s"), ToCStr(Group->GroupName.ToString()), LexToString(FilterStatus));
					}
				}
			}
		}
	}

	const uint8 DynamicFilterIndex = ObjectIndexToDynamicFilterIndex[ObjectIndex];
	if (!IsValidDynamicFilter(DynamicFilterIndex))
	{
		return FString::Printf(TEXT("[NoDynamicFilter] %s"), PrintBuilder.ToString());
	}

	const FFilterInfo& FilterInfo = DynamicFilterInfos[DynamicFilterIndex];

	if (!FilterInfo.Filter->GetFilteredObjects().IsBitSet(ObjectIndex))
	{
		ensureMsgf(false, TEXT("Problem with Filter configs for %s.  DynamicIndex %u Filter %s but not in FilteredObjects list"), *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), DynamicFilterIndex, *FilterInfo.Name.ToString());
		return FString::Printf(TEXT("[WrongFilter Setting!] %s"), PrintBuilder.ToString());
	}

	UNetObjectFilter::FDebugInfoParams DebugParams;
	DebugParams.FilterName = FilterInfo.Name;
	DebugParams.FilteringInfos = MakeArrayView(NetObjectFilteringInfos);
	DebugParams.ConnectionId = ConnectionId;
	DebugParams.View = Connections->GetReplicationView(ConnectionId);

	return FString::Printf(TEXT("[DynamicFilter]: %s (%s)\r\t %s%s"), 
						   *FilterInfo.Name.ToString(), *FilterInfo.Filter->GetClass()->GetName(),
						   *FilterInfo.Filter->PrintDebugInfoForObject(DebugParams, ObjectIndex),
						   PrintBuilder.ToString());
}

void FReplicationFiltering::BuildObjectsInFilterList(FNetBitArrayView OutObjectsInFilter, FName FilterName) const
{
	for (const FFilterInfo& FilterInfo : DynamicFilterInfos)
	{
		if (FilterInfo.Name == FilterName)
		{
			if (FilterInfo.Filter)
			{
				OutObjectsInFilter.Copy(FilterInfo.Filter->GetFilteredObjects());
			}
			return;
		}
	}
}

void FReplicationFiltering::PreUpdateObjectScopeHysteresis()
{
	if (HysteresisState.Mode == EHysteresisProcessingMode::Enabled)
	{
		HysteresisState.ConnectionStartId = (FrameIndex % Config->GetHysteresisUpdateConnectionThrottling());
		HysteresisState.ConnectionIdStride = Config->GetHysteresisUpdateConnectionThrottling();
	}

	ClearObjectsFromHysteresis();
}

void FReplicationFiltering::PostUpdateObjectScopeHysteresis()
{
	HysteresisState.ObjectsToClearCount = 0;
	HysteresisState.ObjectsToClear.ClearAllBits();
	HysteresisState.ObjectsExemptFromHysteresis.ClearAllBits();
}

void FReplicationFiltering::ClearObjectsFromHysteresis()
{
	if (!HysteresisState.ObjectsToClearCount)
	{
		return;
	}

	IRIS_PROFILER_SCOPE(FObjectScopeHysteresisUpdater_ClearObjectsFromHysteresis);

	// $IRIS TODO: Use more optimal path for low counts of objects to clear, for example passing an ArrayView instead.
	ValidConnections.ForAllSetBits([this](uint32 ConnectionId)
		{
			FPerConnectionInfo& ConnectionInfo = this->ConnectionInfos[ConnectionId];
			ConnectionInfo.HysteresisUpdater.RemoveHysteresis(MakeNetBitArrayView(this->HysteresisState.ObjectsToClear));
		});
}

uint8 FReplicationFiltering::GetObjectScopeHysteresisFrameCount(FName ProfileName) const
{
	if (const FObjectScopeHysteresisProfile* Profile = Config->GetHysteresisProfiles().FindByKey(ProfileName))
	{
		return Profile->HysteresisFrameCount;
	}

	return Config->GetDefaultHysteresisFrameCount();
}

bool FReplicationFiltering::HasSubObjectInScopeWithFilteredOutRootObject(FNetBitArrayView Objects) const
{
	bool bReturnValue = false;
	Objects.ForAllSetBits([this, &Objects, &bReturnValue](uint32 ObjectIndex)
		{
			const FNetRefHandleManager::FReplicatedObjectData& ReplicatedObjectData = this->NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
			if (ReplicatedObjectData.SubObjectRootIndex != InvalidInternalNetRefIndex)
			{
				if (!Objects.GetBit(ReplicatedObjectData.SubObjectRootIndex))
				{
					bReturnValue = true;
					ensureMsgf(Objects.GetBit(ReplicatedObjectData.SubObjectRootIndex), TEXT("Root index %u is not in scope for subobject %u"), ReplicatedObjectData.SubObjectRootIndex, ObjectIndex);
				}
			}
			if (ReplicatedObjectData.SubObjectParentIndex != InvalidInternalNetRefIndex)
			{
				if (!Objects.GetBit(ReplicatedObjectData.SubObjectParentIndex))
				{
					bReturnValue = true;
					ensureMsgf(Objects.GetBit(ReplicatedObjectData.SubObjectParentIndex), TEXT("Parent index %u is not in scope for subobject %u"), ReplicatedObjectData.SubObjectParentIndex, ObjectIndex);
				}
			}
		});

	return bReturnValue;
}

bool FReplicationFiltering::HasSubObjectInScopeWithFilteredOutRootObject(uint32 ConnectionId) const
{
	const FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
	if (!ensureMsgf(!HasSubObjectInScopeWithFilteredOutRootObject(MakeNetBitArrayView(ConnectionInfo.ObjectsInScope)), TEXT("ObjectsInScope")))
	{
		ensureMsgf(!HasSubObjectInScopeWithFilteredOutRootObject(MakeNetBitArrayView(ConnectionInfo.GroupIncludedObjects)), TEXT("IncludedObjects"));
		ensureMsgf(!HasSubObjectInScopeWithFilteredOutRootObject(MakeNetBitArrayView(ConnectionInfo.GroupExcludedObjects)), TEXT("ExcludedObjects"));
		ensureMsgf(!HasSubObjectInScopeWithFilteredOutRootObject(MakeNetBitArrayView(ConnectionInfo.ObjectsInScopeBeforeDynamicFiltering)), TEXT("BeforeDynamic"));
		ensureMsgf(!HasSubObjectInScopeWithFilteredOutRootObject(MakeNetBitArrayView(ConnectionInfo.ConnectionFilteredObjects)), TEXT("ConnectionFiltered"));
		ensureMsgf(!HasSubObjectInScopeWithFilteredOutRootObject(MakeNetBitArrayView(ConnectionInfo.DynamicFilteredOutObjectsHysteresisAdjusted)), TEXT("DynamicFilteredOutObjectsHysteresisAdjusted"));
		return true;
	}

	return false;
}

FNetObjectFilterInitParams FReplicationFiltering::GetFilterInitParams(UClass* ConfigClass) const
{
	return FNetObjectFilterInitParams{
		.ReplicationSystem = ReplicationSystem,
		.Config = ConfigClass ? NewObject<UNetObjectFilterConfig>(GetTransientPackage(), ConfigClass) : nullptr,
		.AbsoluteMaxNetObjectCount = NetRefHandleManager->GetMaxActiveObjectCount(),
		.CurrentMaxInternalIndex = MaxInternalNetRefIndex,
		.MaxConnectionCount = Connections->GetMaxConnectionCount(),
	};
}

void FReplicationFiltering::FPerConnectionInfo::Deinit()
{
	ConnectionFilteredObjects.Empty();
	GroupExcludedObjects.Empty();
	ObjectsInScopeBeforeDynamicFiltering.Empty();
	GroupIncludedObjects.Empty();
	FilteredOutCreationDependents.Empty();
	ObjectsInScope.Empty();
	DynamicFilteredOutObjects.Empty();
	InProgressDynamicFilteredOutObjects.Empty();
	DynamicFilteredOutObjectsHysteresisAdjusted.Empty();
	HysteresisUpdater.Deinit();
}

//*************************************************************************************************
// FObjectScopeHysteresisState
//*************************************************************************************************
void FReplicationFiltering::FObjectScopeHysteresisState::ClearFromHysteresis(FInternalNetRefIndex NetRefIndex)
{
	ObjectsToClear.SetBit(NetRefIndex);
	++ObjectsToClearCount;
}

//*************************************************************************************************
// FNetObjectFilteringInfoAccessor
//*************************************************************************************************
TArrayView<FNetObjectFilteringInfo> FNetObjectFilteringInfoAccessor::GetNetObjectFilteringInfos(UReplicationSystem* ReplicationSystem) const
{
	if (ReplicationSystem)
	{
		return ReplicationSystem->GetReplicationSystemInternal()->GetFiltering().GetNetObjectFilteringInfos();
	}

	return TArrayView<FNetObjectFilteringInfo>();
}

//*************************************************************************************************
// FNetObjectFilterHandleUtil
//*************************************************************************************************
inline bool FNetObjectFilterHandleUtil::IsInvalidHandle(FNetObjectFilterHandle Handle)
{
	return Handle == InvalidNetObjectFilterHandle;
}

inline bool FNetObjectFilterHandleUtil::IsDynamicFilter(FNetObjectFilterHandle Handle)
{
	return (Handle & DynamicNetObjectFilterHandleFlag) != 0;
}

inline bool FNetObjectFilterHandleUtil::IsStaticFilter(FNetObjectFilterHandle Handle)
{
	return (Handle & DynamicNetObjectFilterHandleFlag) == 0;
}

inline FNetObjectFilterHandle FNetObjectFilterHandleUtil::MakeDynamicFilterHandle(uint32 FilterIndex)
{
	const FNetObjectFilterHandle Handle = DynamicNetObjectFilterHandleFlag | FNetObjectFilterHandle(FilterIndex);
	return Handle;
}

inline uint32 FNetObjectFilterHandleUtil::GetDynamicFilterIndex(FNetObjectFilterHandle Handle)
{
	return (Handle & DynamicNetObjectFilterHandleFlag) ? (Handle & ~DynamicNetObjectFilterHandleFlag) : ~0U;
}

} // end namespace UE::Net::Private
