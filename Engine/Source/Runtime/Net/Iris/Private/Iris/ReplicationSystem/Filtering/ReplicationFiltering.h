// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationSystem/Filtering/SharedConnectionFilterStatus.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/Filtering/ObjectScopeHysteresisUpdater.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFilteringConfig.h"
#include "Iris/ReplicationSystem/NetObjectGroupHandle.h"
#include "Net/Core/Connection/ConnectionHandle.h"
#include "Containers/Array.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UReplicationFilteringConfig;
class UReplicationSystem;
struct FNetObjectFilterDefinition;

namespace UE::Net
{
	enum class ENetConnectionTraits : uint32;
	typedef uint32 FNetObjectFilterHandle;
	typedef uint32 FInternalNetRefIndex;

	namespace Private
	{
		class FNetRefHandleManager;
		class FNetObjectGroups;
		class FReplicationConnections;
	}
}

namespace UE::Net::Private
{

class FNetObjectFilteringInfoAccessor
{
private:
	/** Returns all the NetObjectFilteringInfos for the filtering system. */
	TArrayView<FNetObjectFilteringInfo> GetNetObjectFilteringInfos(UReplicationSystem* ReplicationSystem) const;

private:
	// Friends
	friend UNetObjectFilter;
};


struct FReplicationFilteringInitParams
{
	TObjectPtr<UReplicationSystem> ReplicationSystem = nullptr;
	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	FNetObjectGroups* Groups = nullptr;
	FReplicationConnections* Connections = nullptr;
	FInternalNetRefIndex MaxInternalNetRefIndex = 0;
	uint32 MaxGroupCount = 0;
};

class FReplicationFiltering
{
public:
	FReplicationFiltering();

	void Init(FReplicationFilteringInitParams& Params);
	void Deinit();

	/** Called when the maximum InternalNetRefIndex increased and we need to realloc our lists */
	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

	/** Called when when one or more NetRefInternalIndices have been freed and can be re-assigned to new objects. */
	void OnInternalNetRefIndicesFreed(const TConstArrayView<FInternalNetRefIndex>& FreedIndices);

	/**
	 * Executes group, owner and connection filtering then any dynamic filters.
	 * At the end any object that is not relevant to at least one connection will be removed from the scoped object list.
	 * Exception to this rule are always relevant (e.g. non-filtered) objects.
	 */
	void Filter();

	/**  Returns the list of objects relevant to a given connection. This represents the global scope list minus the objects that were filtered out for the given connection. */
	const FNetBitArrayView GetRelevantObjectsInScope(uint32 ConnectionId) const
	{
		return MakeNetBitArrayView(ConnectionInfos[ConnectionId].ObjectsInScope);
	}

	const FNetBitArrayView GetGroupFilteredOutObjects(uint32 ConnectionId) const
	{
		return MakeNetBitArrayView(ConnectionInfos[ConnectionId].GroupExcludedObjects);
	}

	// Who owns what
	void SetOwningConnection(FInternalNetRefIndex ObjectIndex, uint32 ConnectionId);
	uint32 GetOwningConnection(FInternalNetRefIndex ObjectIndex) const { return !bHasDirtyOwner ? ObjectIndexToOwningConnection[ObjectIndex] : GetOwningConnectionIfDirty(ObjectIndex); }

	/**
	 * Setup an object to be filtered by the passed filter handle.
	 * @param ObjectIndex The object that wants to be filtered
	 * @param Filter The handle to the filter to add the object into
	 * @param FilterConfigProfile Optional profile name that can be used to specialize the filter parameters
	 */
	bool SetFilter(FInternalNetRefIndex ObjectIndex, FNetObjectFilterHandle Filter, FName FilterConfigProfile);

	/** 
	 * Returns true if the object is part of a filter with the ENetFilterTraits::Spatial trait. 
	 * Indicates that the object is location filtered.
	 */
	bool IsUsingSpatialFilter(FInternalNetRefIndex ObjectIndex) const;

	FNetObjectFilterHandle GetFilterHandle(const FName FilterName) const;
	UNetObjectFilter* GetFilter(const FName FilterName) const;

	/** Returns the name of the Filter represented by the handle. */
	FName GetFilterName(FNetObjectFilterHandle Filter) const;

	/**
	 * Create a filter at runtime.
	 * @see UReplicationSystem::CreateFilter().
	 */
	TWeakObjectPtr<UNetObjectFilter> CreateFilter(const FNetObjectFilterDefinition& Definition);

	/**
	* Destroy a filter.
	* @see UReplicationSystem::DestroyFilter().
	*/
	void DestroyFilter(const FName FilterName);

	/** Fill the passed in list with root objects considered always relevant */
	void BuildAlwaysRelevantList(FNetBitArrayView OutAlwaysRelevantList, const FNetBitArrayView ScopeList) const;

	/** Fill the passed in list with objects affected by a given filter*/
	void BuildObjectsInFilterList(FNetBitArrayView OutObjectsInFilter, FName FilterName) const;

	// Connection handling
	void AddConnection(uint32 ConnectionId, ENetConnectionTraits NetConnectionTraits);
	void RemoveConnection(uint32 ConnectionId);

	// Group based filtering
	bool AddExclusionFilterGroup(FNetObjectGroupHandle GroupHandle);
	bool AddInclusionFilterGroup(FNetObjectGroupHandle GroupHandle);

	void RemoveGroupFilter(FNetObjectGroupHandle GroupHandle);
	bool IsExclusionFilterGroup(FNetObjectGroupHandle GroupHandle) const { return GroupHandle.IsValid() && ExclusionFilterGroups.GetBit(GroupHandle.GetGroupIndex()); }
	bool IsInclusionFilterGroup(FNetObjectGroupHandle GroupHandle) const { return GroupHandle.IsValid() && InclusionFilterGroups.GetBit(GroupHandle.GetGroupIndex()); }

	void SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, ENetFilterStatus ReplicationStatus);
	void SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, const FNetBitArrayView& ConnectionsBitArray, ENetFilterStatus);
	void SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId, ENetFilterStatus ReplicationStatus);
	bool GetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId, ENetFilterStatus& OutReplicationStatus) const;

	void NotifyObjectAddedToGroup(FNetObjectGroupHandle GroupHandle, FInternalNetRefIndex ObjectIndex);
	void NotifyObjectRemovedFromGroup(FNetObjectGroupHandle GroupHandle, FInternalNetRefIndex ObjectIndex);

	void NotifyAddedDependentObject(FInternalNetRefIndex ObjectIndex);
	void NotifyRemovedDependentObject(FInternalNetRefIndex ObjectIndex);
	
	// SubObjectFilter status
	void AddSubObjectFilter(FNetObjectGroupHandle GroupHandle);
	void RemoveSubObjectFilter(FNetObjectGroupHandle GroupHandle);
	bool IsSubObjectFilterGroup(FNetObjectGroupHandle GroupHandle) const { return GroupHandle.IsValid() && SubObjectFilterGroups.GetBit(GroupHandle.GetGroupIndex()); }

	void SetSubObjectFilterStatus(FNetObjectGroupHandle GroupHandle, FConnectionHandle ConnectionHandle, ENetFilterStatus ReplicationStatus);
	bool GetSubObjectFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ParentConnectionId, ENetFilterStatus& OutReplicationStatus) const;

	/** Print the filter information we have regarding the passed ObjectIndex and his relation to the passed Connection */
	FString PrintFilterObjectInfo(FInternalNetRefIndex ObjectIndex, uint32 ConnectionId) const;

private:
	struct FPerConnectionInfo
	{
		void Deinit();

		// Objects filtered depending on owning connection or user set connection filtering
		FNetBitArray ConnectionFilteredObjects;
		// Objects filtered out due to one or more exclusion groups it belongs to is filtered out
		FNetBitArray GroupExcludedObjects;
		// Connection and group exclusion filtering is assumed to happen seldom. Avoid recalculating from scratch every frame.
		FNetBitArray ObjectsInScopeBeforeDynamicFiltering;
		// Group inclusion filtering is assumed to happen seldom. Applied after dynamic filtering, but before dependent objects.
		FNetBitArray GroupIncludedObjects;
		// Creation dependents that have been filtered because a dependency has been filtered.
		FNetBitArray FilteredOutCreationDependents;
		// Objects in scope after all kinds of filtering, including dynamic filtering, has been applied
		FNetBitArray ObjectsInScope;
		// Which objects are filtered out after dynamic filters have been processed.
		FNetBitArray DynamicFilteredOutObjects;
		// List of objects currently filtered out after processing dynamic filter passes. This could be temporary allocations in UpdateDynamicFiltering() but does require one bitarray per connection.
		FNetBitArray InProgressDynamicFilteredOutObjects;
		// Which objects are filtered out after dynamic filters, inclusion groups and hysteresis have been processed.
		FNetBitArray DynamicFilteredOutObjectsHysteresisAdjusted;
		// Updater of hysteresis for objects being dynamically filtered out
		FObjectScopeHysteresisUpdater HysteresisUpdater;
	};

	struct FPerObjectInfo
	{
		// Note: Array is likely larger than one element.
		uint32 ConnectionIds[1];
	};

	static constexpr uint32 UsedPerObjectInfoStorageGrowSize = 32; // 256 bytes, 1024 indices
	typedef uint32 PerObjectInfoIndexType;

	struct FPerGroupInfo
	{
		PerObjectInfoIndexType ConnectionStateIndex;
	};

	class FPerSubObjectFilterGroupInfo
	{
	public:
		// Tracks parent and child connection filter status. Necessary for splitscreen support.
		FSharedConnectionFilterStatusCollection ConnectionFilterStatus;
		// Index to storage for parent connection bit array. 
		PerObjectInfoIndexType ConnectionStateIndex = 0;
	};

	struct FFilterInfo
	{
		bool IsValid() const { return Filter != nullptr; };
		explicit operator bool() const { return IsValid(); }

		TStrongObjectPtr<UNetObjectFilter> Filter;
		FName Name;
		uint32 ObjectCount = 0;
	};

private:
	class FUpdateDirtyObjectsBatchHelper;
	friend FNetObjectFilteringInfoAccessor;
	friend FPerSubObjectFilterGroupInfo;
	
	static void StaticChecks();

	void InitFilters();
	void InitObjectScopeHysteresis();

	void SetNetObjectListsSize(FInternalNetRefIndex MaxInternalIndex);
	void SetPerConnectionListsSize(FPerConnectionInfo& ConnectionInfo, FInternalNetRefIndex NewMaxInternalIndex);

	void InitNewConnections();
	void ResetRemovedConnections();
	void UpdateObjectsInScope();
	void UpdateOwnerFiltering();
	void UpdateGroupExclusionFiltering();
	void UpdateGroupInclusionFiltering();
	void UpdateSubObjectFilters();

	void UpdateDynamicFilters();
	void PreUpdateDynamicFiltering();
	void UpdateDynamicFiltering();
	void PostUpdateDynamicFiltering();

	void PreUpdateObjectScopeHysteresis();
	void PostUpdateObjectScopeHysteresis();
	void ClearObjectsFromHysteresis();

	/** Build the list of always relevant objects + objects that are currently relevant to at least one connection. */
	void FilterNonRelevantObjects();

	bool HasDynamicFilters() const;

	// Mark creation dependents as filtered out. This may only be called from within the creation dependency update.
	void FilterOutCreationDependents(uint32 ParentIndex, FPerConnectionInfo& ConnectionInfo, const FNetBitArrayView ObjectsWithCreationDependents) const;
	// Mark creation dependents as filtered in if none of their ancestors are filtered out. This may only be called from within the creation dependency update.
	void FilterInCreationDependents(uint32 ParentIndex, FPerConnectionInfo& ConnectionInfo, const FNetBitArrayView ObjectsWithCreationDependents) const;
	// Reevaluate the filter status of an object based on creation dependencies. This may only be called from within the creation dependency update.
	void RefreshCreationDependencyFilterStatus(uint32 ObjectIndex, FPerConnectionInfo& ConnectionInfo, const FNetBitArrayView ObjectsWithCreationDependents) const;

	void UpdateCreationDependentParent(uint32 ChildIndex, const FNetBitArrayView ObjectsWithCreationDependencies, FNetBitArrayView OutConnectionObjectsInScope, FNetBitArrayView ProcessedObjects, bool bIsRecursive) const;

	// Helper to update and reset group exclusion filter effects if objects are removed from a filter or after a filter status change, returns true if the group filter was changed
	bool ClearGroupExclusionFilterEffectsForObject(uint32 ObjectIndex, uint32 ConnectionId);
	// Helper to update and reset group inclusion filter effects if objects are removed from a filter or after a filter status change, returns true if the group filter was changed
	bool ClearGroupInclusionFilterEffectsForObject(uint32 ObjectIndex, uint32 ConnectionId);
	bool HasOwnerFilter(uint32 ObjectIndex) const;

	PerObjectInfoIndexType AllocPerObjectInfo();
	void FreePerObjectInfo(PerObjectInfoIndexType Index);

	FPerObjectInfo* AllocPerObjectInfoForObject(uint32 ObjectIndex);
	void FreePerObjectInfoForObject(uint32 ObjectIndex);
	
	FPerObjectInfo* GetPerObjectInfo(PerObjectInfoIndexType Index);
	const FPerObjectInfo* GetPerObjectInfo(PerObjectInfoIndexType Index) const;

	void SetPerObjectInfoFilterStatus(FPerObjectInfo& ObjectInfo, ENetFilterStatus ReplicationStatus);

	// SubObjectGroup filtering support
	FPerSubObjectFilterGroupInfo& CreatePerSubObjectGroupFilterInfo(FNetObjectGroupHandle::FGroupIndexType GroupIndex);
	void DestroyPerSubObjectGroupFilterInfo(FNetObjectGroupHandle::FGroupIndexType GroupIndex);
	FPerSubObjectFilterGroupInfo* GetPerSubObjectFilterGroupInfo(FNetObjectGroupHandle::FGroupIndexType GroupIndex);
	const FPerSubObjectFilterGroupInfo* GetPerSubObjectFilterGroupInfo(FNetObjectGroupHandle::FGroupIndexType GroupIndex) const;

	ENetFilterStatus GetDependentObjectFilterStatus(uint32 ConnectionId, FInternalNetRefIndex ObjectIndex, TSet<FInternalNetRefIndex>& Visited);
	ENetFilterStatus GetConnectionFilterStatus(const FPerObjectInfo& ObjectInfo, uint32 ConnectionId) const;
	bool IsAnyConnectionFilterStatusAllowed(const FPerObjectInfo& ObjectInfo) const;
	bool IsAnyConnectionFilterStatusDisallowed(const FPerObjectInfo& ObjectInfo) const;
	void SetConnectionFilterStatus(FPerObjectInfo& ObjectInfo, uint32 ConnectionId, ENetFilterStatus ReplicationStatus);
	// Returns true if object is filtered out by any exclusion group.
	bool IsExcludedByAnyGroup(uint32 ObjectInternalIndex, uint32 ConnectionId) const;
	// Returns true if object is allowed to replicate by any inclusion group.
	bool IsIncludedByAnyGroup(uint32 ObjectInternalIndex, uint32 ConnectionId) const;
	void InternalSetExclusionGroupFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId, ENetFilterStatus ReplicationStatus);
	void InternalSetInclusionGroupFilterStatus(FNetObjectGroupHandle GroupHandle, ENetFilterStatus ReplicationStatus);
	void InternalSetInclusionGroupFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId, ENetFilterStatus ReplicationStatus);

	uint32 GetOwningConnectionIfDirty(uint32 ObjectIndex) const;

	bool IsValidDynamicFilter(uint32 FilterIndex) const;
	void RemoveFromDynamicFilter(UE::Net::FInternalNetRefIndex ObjectIndex, uint32 FilterIndex);

	void NotifyFiltersOfDirtyObjects();
	void BatchNotifyFiltersOfDirtyObjects(FUpdateDirtyObjectsBatchHelper& BatchHelper, const UE::Net::FInternalNetRefIndex* ObjectIndices, uint32 ObjectCount);

	/** Returns all the filtering infos. */
	TArrayView<FNetObjectFilteringInfo> GetNetObjectFilteringInfos();

	uint8 GetObjectScopeHysteresisFrameCount(FName Profile) const;

	bool HasSubObjectInScopeWithFilteredOutRootObject(FNetBitArrayView Objects) const;
	bool HasSubObjectInScopeWithFilteredOutRootObject(uint32 connectionId) const;

	// Get InitParams for a filter.
	// @param ConfigClass Filter-specific config class. May be nullptr.
	// @return InitParams suitable for passing to UNetObjectFilter::Init().
	FNetObjectFilterInitParams GetFilterInitParams(UClass* ConfigClass) const;

	// Create & initialize an instance of the no-op filter.
	TStrongObjectPtr<UNetObjectFilter> MakeNoOpNetObjectFilter() const;

private:
	enum EHysteresisProcessingMode : uint32
	{
		Disabled,
		Enabled,
	};

	// Scope hysteresis state. Hysteresis is applied to objects going out of scope for objects that so desire.
	struct FObjectScopeHysteresisState
	{
	public:
		void ClearFromHysteresis(FInternalNetRefIndex NetRefIndex);

		// Processing mode
		EHysteresisProcessingMode Mode = EHysteresisProcessingMode::Disabled;
		// Which connection ID to start with for updating.
		uint32 ConnectionStartId = 0;
		// Stride for connection update throttling.
		uint32 ConnectionIdStride = 1;

		// Approximate number of objects that should be cleared from hysteresis.
		uint32 ObjectsToClearCount = 0;

		// Objects to clear from hysteresis due to being destroyed or removed from dynamic filtering.
		FNetBitArray ObjectsToClear;

		// Objects that should not be added to hysteresis this frame. Example use case is newly added objects that become filtered out on the first frame.
		FNetBitArray ObjectsExemptFromHysteresis;
	};

	// Used for ObjectIndexToDynamicFilterIndex lookup
	static constexpr uint8 InvalidDynamicFilterIndex = 255U;

	// Config
	TStrongObjectPtr<const UReplicationFilteringConfig> Config;

	// General
	TObjectPtr<UReplicationSystem> ReplicationSystem = nullptr;
	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	uint32 FrameIndex = 0;

	// Groups
	FNetObjectGroups* Groups = nullptr;

	// Connection specifics
	FReplicationConnections* Connections = nullptr;
	TArray<FPerConnectionInfo> ConnectionInfos;
	FNetBitArray ValidConnections;
	FNetBitArray NewConnections;

	// Object specifics
	FInternalNetRefIndex MaxInternalNetRefIndex = 0;
	uint32 WordCountForObjectBitArrays = 0;

	// Scratch buffers for scope delta tracking (shared across connections, reused each iteration)
	FNetBitArray ScopeDeltaPrevObjects;
	FNetBitArray ScopeDeltaNewlyInObjects;
	FNetBitArray ScopeDeltaNewlyOutObjects;

	// Filter specifics
	FNetBitArray ObjectsWithDirtyOwnerFilter;
	FNetBitArray ObjectsWithDirtyOwner;

	FNetBitArray ObjectsWithOwnerFilter;

	//$TODO: This should be a on-demand data allocation. Very few objects have an owner and the memory usage would be minimal
	TArray<uint16> ObjectIndexToOwningConnection;

	// For non-owner filtered objects
	// The storage for PerObjectInfo.
	TArray<uint32> PerObjectInfoStorage;
	// Storage for bit array indicating used/free status of PerObjectInfos
	TArray<uint32> UsedPerObjectInfoStorage;

	FNetBitArray ObjectsWithPerObjectInfo;

	// Groups
	TArray<FPerGroupInfo> GroupInfos;
	uint32 MaxGroupCount = 0;

	// SubObject filter groups
	TMap<uint32, FPerSubObjectFilterGroupInfo> SubObjectFilterGroupInfos;

	// Hysteresis frame counts for dynamically filtered objects
	TArray<uint8> ObjectScopeHysteresisFrameCounts;

	/** NetObjectGroups used for filtering out objects. */
	FNetBitArray ExclusionFilterGroups;

	/** NetObjectGroups used to allow allowing replication of dynamically filtered out objects. */
	FNetBitArray InclusionFilterGroups;

	/** Exclusion filtering groups with newly added members and that need to filter out objects for at least one connection. */
	FNetBitArray DirtyExclusionFilterGroups;

	/** Inclusion filtering groups with newly added members and that need to include objects for at least one connection. */
	FNetBitArray DirtyInclusionFilterGroups;

	// Group indices which are subobject filter groups
	FNetBitArray SubObjectFilterGroups;
	// Group indices which are subobject filter groups and in need of updating
	FNetBitArray DirtySubObjectFilterGroups;
	// Object indices with a connection filter
	FNetBitArray AllConnectionFilteredObjects;

	TArray<PerObjectInfoIndexType> ObjectIndexToPerObjectInfoIndex;
	uint32 PerObjectInfoStorageCountForConnections = 0;
	// How many elements from UsedPerObjectInfoStorage is needed to hold a FPerObjectInfo
	uint32 PerObjectInfoStorageCountPerItem = 0;

	// Dynamic filters 
	TArray<FNetObjectFilteringInfo> NetObjectFilteringInfos;
	TArray<uint8> ObjectIndexToDynamicFilterIndex;
	// Dynamic Filter infos. May contain invalid/null filters after dynamic filters are destroyed. Invalid/null filters must be ignored.
	TArray<FFilterInfo> DynamicFilterInfos;

	FNetBitArray DynamicFilterEnabledObjects;
	FNetBitArray ObjectsRequiringDynamicFilterUpdate;

	// Object scope hystereris
	FObjectScopeHysteresisState HysteresisState;

	uint32 bHasNewConnection : 1;
	uint32 bHasRemovedConnection : 1;
	uint32 bHasDirtyOwnerFilter: 1;
	uint32 bHasDirtyOwner : 1;
	uint32 bHasDynamicFilters : 1;
	uint32 bHasDirtyExclusionFilterGroup : 1;
	uint32 bHasDirtyInclusionFilterGroup : 1;
	// Is true if any initialized DynamicFilter has the NeedsUpdate trait
	uint32 bHasDynamicFiltersWithUpdateTrait : 1;

	uint32 LoggedCyclicDependencyCount = 0U;
};

inline bool FReplicationFiltering::HasDynamicFilters() const
{
	return bHasDynamicFilters;
}

}
