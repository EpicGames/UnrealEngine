// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Filtering/SharedConnectionFilterStatus.h"
#include "Iris/Core/IrisLog.h"

namespace UE::Net
{

// FSharedConnectionFilterStatus implementation
bool FSharedConnectionFilterStatus::SetFilterStatus(FConnectionHandle ConnectionHandle, ENetFilterStatus FilterStatus)
{
	if (!ConnectionHandle.IsValid())
	{
		UE_LOGF(LogIrisFiltering, Error, "%ls", TEXT("Trying to set filter status for an invalid FConnectionHandle on an FConnectionHandleFilterGroup"));
		return false;
	}

	if (ParentConnectionId != InvalidConnectionId && ConnectionHandle.GetParentConnectionId() != ParentConnectionId)
	{
		UE_LOGF(LogIrisFiltering, Error, "FConnectionHandleFilterGroup ignoring SetFilterStatus call due to connection ID mismatch. Expected %u got %u.", ParentConnectionId, ConnectionHandle.GetParentConnectionId());
		return false;
	}

	ParentConnectionId = ConnectionHandle.GetParentConnectionId();
	if (FilterStatus == ENetFilterStatus::Allow)
	{
		AllowConnections.Add(ConnectionHandle.GetChildConnectionId());
	}
	else
	{
		AllowConnections.Remove(ConnectionHandle.GetChildConnectionId());
	}

	return true;
}

void FSharedConnectionFilterStatus::RemoveConnection(FConnectionHandle ConnectionHandle)
{
	if (ConnectionHandle.GetParentConnectionId() == ParentConnectionId)
	{
		if (ConnectionHandle.IsParentConnection())
		{
			AllowConnections.Reset();
			// Allow this instance to be repurposed for a different parent connection.
			ParentConnectionId = InvalidConnectionId;
		}
		else
		{
			AllowConnections.Remove(ConnectionHandle.GetChildConnectionId());
		}
	}
}

// FSharedConnectionFilterStatusCollection implementation
void FSharedConnectionFilterStatusCollection::SetFilterStatus(FConnectionHandle ConnectionHandle, ENetFilterStatus FilterStatus)
{
	if (!ConnectionHandle.IsValid())
	{
		UE_LOGF(LogIrisFiltering, Error, "%ls", TEXT("Trying to set filter status for an invalid FConnectionHandle on an FConnectionHandleFilterGroupCollection"));
		return;
	}

	const uint32 ParentConnId = ConnectionHandle.GetParentConnectionId();
	if (FilterStatus == ENetFilterStatus::Allow)
	{
		FSharedConnectionFilterStatus& SharedFilterStatus = FindOrAddSharedConnectionFilterStatus(ParentConnId);
		SharedFilterStatus.SetFilterStatus(ConnectionHandle, FilterStatus);
	}
	else
	{
		if (FSharedConnectionFilterStatus* SharedFilterStatus = FindSharedConnectionFilterStatus(ParentConnId))
		{
			SharedFilterStatus->SetFilterStatus(ConnectionHandle, FilterStatus);
			// If after setting the filter status the group disallows replication then we can remove it
			if (SharedFilterStatus->GetFilterStatus() == ENetFilterStatus::Disallow)
			{
				ParentToFilterStatus.Remove(ParentConnId);
			}
		}
	}
}

void FSharedConnectionFilterStatusCollection::RemoveConnection(FConnectionHandle ConnectionHandle)
{
	if (ConnectionHandle.IsParentConnection())
	{
		ParentToFilterStatus.Remove(ConnectionHandle.GetParentConnectionId());
	}
	else if (ConnectionHandle.IsChildConnection())
	{
		if (FSharedConnectionFilterStatus* SharedFilterStatus = FindSharedConnectionFilterStatus(ConnectionHandle.GetParentConnectionId()))
		{
			SharedFilterStatus->RemoveConnection(ConnectionHandle);
		}
	}
}

ENetFilterStatus FSharedConnectionFilterStatusCollection::GetFilterStatus(uint32 ParentConnectionId) const
{
	if (const FSharedConnectionFilterStatus* SharedFilterStatus = FindSharedConnectionFilterStatus(ParentConnectionId))
	{
		return SharedFilterStatus->GetFilterStatus();
	}

	return ENetFilterStatus::Disallow;
}

FSharedConnectionFilterStatus* FSharedConnectionFilterStatusCollection::FindSharedConnectionFilterStatus(uint32 ParentConnectionId)
{
	return ParentToFilterStatus.Find(ParentConnectionId);
}

const FSharedConnectionFilterStatus* FSharedConnectionFilterStatusCollection::FindSharedConnectionFilterStatus(uint32 ParentConnectionId) const
{
	return ParentToFilterStatus.Find(ParentConnectionId);
}

FSharedConnectionFilterStatus& FSharedConnectionFilterStatusCollection::FindOrAddSharedConnectionFilterStatus(uint32 ParentConnectionId)
{
	FSharedConnectionFilterStatus& SharedFilterStatus = ParentToFilterStatus.FindOrAdd(ParentConnectionId);
	// Set disallow status to establish the ParentConnectionId of the group. It's not strictly necessary but it could help detect issues with the collection implementation.
	SharedFilterStatus.SetFilterStatus(FConnectionHandle(ParentConnectionId), ENetFilterStatus::Disallow);
	return SharedFilterStatus;
}

}
