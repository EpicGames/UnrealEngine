// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/InternalNetRefIndexManager.h"

#include "Iris/ReplicationSystem/NetRefHandleManager.h"

namespace UE::Net
{

FInternalNetRefIndexManager::FInternalNetRefIndexManager(const UE::Net::Private::FNetRefHandleManager& InNetRefHandleManager)
	: NetRefHandleManager(InNetRefHandleManager)
{
	// nothing
}

TConstArrayView<FInternalNetRefIndex> FInternalNetRefIndexManager::GetCreationDependencies(FInternalNetRefIndex ObjectIndex) const
{
	return NetRefHandleManager.GetCreationDependencies(ObjectIndex);
}

TConstArrayView<FInternalNetRefIndex> FInternalNetRefIndexManager::GetCreationDependents(FInternalNetRefIndex ParentIndex) const
{
	return NetRefHandleManager.GetCreationDependents(ParentIndex);
}

const FNetBitArrayView FInternalNetRefIndexManager::GetObjectsWithCreationDependencies() const
{
	return NetRefHandleManager.GetObjectsWithCreationDependencies();
}

const FNetBitArrayView FInternalNetRefIndexManager::GetObjectsWithDirtyCreationDependencies() const
{
	return NetRefHandleManager.GetObjectsWithDirtyCreationDependencies();
}

FString FInternalNetRefIndexManager::PrintObjectFromIndex(FInternalNetRefIndex ObjectIndex) const
{
	return NetRefHandleManager.PrintObjectFromIndex(ObjectIndex);
}

} // end namespace UE::Net