// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/IrisCreationFlowLog.h"

#include "Containers/Map.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/ReplicationSystem/IrisCreationFlowLogConfig.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"

#if UE_NET_ENABLE_IRISCREATIONFLOWLOG

DEFINE_LOG_CATEGORY(LogIrisCreationFlow);

namespace UE::Net::Private::CreationFlowLog
{

struct FCreationFlowLogRule
{
	bool bMatched = false;
	bool bTraceOwningConnectionOnly = false;
};

static FCreationFlowLogRule ResolveRule(const UClass* Class)
{
	IRIS_PROFILER_SCOPE(CreationFlowLog_ResolveRule);

	if (Class == nullptr)
	{
		return FCreationFlowLogRule{};
	}

	const UIrisCreationFlowLogConfig* ConfigObj = UIrisCreationFlowLogConfig::GetConfig();
	if (ConfigObj == nullptr)
	{
		return FCreationFlowLogRule{};
	}

	const TConstArrayView<FCreationFlowLogClassConfig> Filters = ConfigObj->GetClassFilters();
	for (const UClass* CurrentClass = Class; CurrentClass != nullptr; CurrentClass = CurrentClass->GetSuperClass())
	{
		const FTopLevelAssetPath ClassPath = CurrentClass->GetClassPathName();
		for (const FCreationFlowLogClassConfig& Entry : Filters)
		{
			if (Entry.ClassPath == ClassPath)
			{
				if (CurrentClass == Class || Entry.bIncludeSubclasses)
				{
					return FCreationFlowLogRule{ .bMatched = true, .bTraceOwningConnectionOnly = Entry.bTraceOwningConnectionOnly };
				}
				break;
			}
		}
	}

	return FCreationFlowLogRule{};
}

// TObjectKey survives UClass destruction, so stale entries are unreachable rather than dangling.
// Entries are cleared in PruneStaleEntries on PostGarbageCollect.
static TMap<TObjectKey<UClass>, FCreationFlowLogRule> GCreationFlowLogRuleCache;
static FDelegateHandle GPostGarbageCollectHandle;

static FCreationFlowLogRule FindOrCacheRule(const UClass* Class)
{
	IRIS_PROFILER_SCOPE(CreationFlowLog_FindOrCacheRule);
	check(Class != nullptr);
	const TObjectKey<UClass> Key(Class);
	if (const FCreationFlowLogRule* Cached = GCreationFlowLogRuleCache.Find(Key))
	{
		return *Cached;
	}
	const FCreationFlowLogRule Rule = ResolveRule(Class);
	GCreationFlowLogRuleCache.Add(Key, Rule);
	return Rule;
}

static void PruneStaleEntries()
{
	IRIS_PROFILER_SCOPE(CreationFlowLog_PruneStaleEntries);

	for (auto It = GCreationFlowLogRuleCache.CreateIterator(); It; ++It)
	{
		if (It.Key().ResolveObjectPtr() == nullptr)
		{
			It.RemoveCurrent();
		}
	}
}

} // namespace UE::Net::Private::CreationFlowLog

namespace UE::Net::CreationFlowLog
{

bool IsTracedClass(const UClass* Class)
{
	return Private::CreationFlowLog::FindOrCacheRule(Class).bMatched;
}

bool ShouldEmitForConnection(const UObject* Instance, uint32 OwningConnectionId, uint32 TargetConnectionId)
{
	if (Instance == nullptr)
	{
		return false;
	}

	const Private::CreationFlowLog::FCreationFlowLogRule Rule = Private::CreationFlowLog::FindOrCacheRule(Instance->GetClass());
	if (!Rule.bMatched)
	{
		return false;
	}

	if (Rule.bTraceOwningConnectionOnly)
	{
		return OwningConnectionId == TargetConnectionId;
	}

	return true;
}

void ClearCache()
{
	check(IsInGameThread());
	Private::CreationFlowLog::GCreationFlowLogRuleCache.Reset();
}

void Init()
{
	if (!Private::CreationFlowLog::GPostGarbageCollectHandle.IsValid())
	{
		Private::CreationFlowLog::GPostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddStatic(&Private::CreationFlowLog::PruneStaleEntries);
	}
}

void Shutdown()
{
	if (Private::CreationFlowLog::GPostGarbageCollectHandle.IsValid())
	{
		FCoreUObjectDelegates::GetPostGarbageCollect().Remove(Private::CreationFlowLog::GPostGarbageCollectHandle);
		Private::CreationFlowLog::GPostGarbageCollectHandle.Reset();
	}
	Private::CreationFlowLog::GCreationFlowLogRuleCache.Reset();
}

} // namespace UE::Net::CreationFlowLog

#endif // UE_NET_ENABLE_IRISCREATIONFLOWLOG
