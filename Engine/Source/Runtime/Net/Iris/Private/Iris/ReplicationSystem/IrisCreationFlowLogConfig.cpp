// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/IrisCreationFlowLogConfig.h"

#include "Iris/ReplicationSystem/IrisCreationFlowLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IrisCreationFlowLogConfig)

const UIrisCreationFlowLogConfig* UIrisCreationFlowLogConfig::GetConfig()
{
	return GetDefault<UIrisCreationFlowLogConfig>();
}

TConstArrayView<FCreationFlowLogClassConfig> UIrisCreationFlowLogConfig::GetClassFilters() const
{
	return MakeArrayView(ClassFilters);
}

void UIrisCreationFlowLogConfig::PostReloadConfig(FProperty* PropertyToLoad)
{
	Super::PostReloadConfig(PropertyToLoad);
#if UE_NET_ENABLE_IRISCREATIONFLOWLOG
	UE::Net::CreationFlowLog::ClearCache();
#endif // UE_NET_ENABLE_IRISCREATIONFLOWLOG
}
