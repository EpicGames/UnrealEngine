// Copyright Epic Games, Inc. All Rights Reserved.

#include "LegacyPushModel.h"
#include "Iris/IrisConfigInternal.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Net/Core/DirtyNetObjectTracker/GlobalDirtyNetObjectTracker.h"
#include "Containers/ArrayView.h"

#if WITH_PUSH_MODEL
#include "HAL/IConsoleManager.h"
#include "UObject/Object.h"

namespace UE::Net::Private
{

bool bIsIrisPushModelForceEnabled = false;
int IrisPushModelMode = 2;
static FAutoConsoleVariableRef CVarIrisPushModelMode(
		TEXT("net.Iris.PushModelMode"),
		IrisPushModelMode,
		TEXT("0 = disabled but runtime togglable, 1 = enabled and not togglable, 2 = enabled but runtime togglable. Requires Net.IsPushModelEnabled is true and WITH_PUSH_MODEL > 0 to use push based dirtiness in the backwards compatibility mode."
		));

void FNetHandleLegacyPushModelHelper::InitPushModel()
{
	bIsIrisPushModelForceEnabled = IrisPushModelMode == 1;
	if (bIsIrisPushModelForceEnabled && ensureMsgf(IS_PUSH_MODEL_ENABLED(), TEXT("Trying to force enable Iris push model support when push model is disabled, falling back to optional path. Set Net.IsPushModelEnabled true")))
	{
		// Push model is assumed to be enabled at all times. The cvar will not be checked.
		UE_NET_SET_IRIS_MARK_PROPERTY_DIRTY_DELEGATE(UEPushModelPrivate::FIrisMarkPropertyDirty::CreateStatic(&FNetHandleLegacyPushModelHelper::MarkPropertyOwnerDirty));
		UE_NET_SET_IRIS_MARK_PROPERTIES_DIRTY_DELEGATE(UEPushModelPrivate::FIrisMarkPropertiesDirty::CreateStatic(&FNetHandleLegacyPushModelHelper::MarkPropertiesOwnerDirty));
	}
	else
	{
		// Initialize our delegate so that push model can be toggled at runtime.
		UE_NET_SET_IRIS_MARK_PROPERTY_DIRTY_DELEGATE(UEPushModelPrivate::FIrisMarkPropertyDirty::CreateStatic(&FNetHandleLegacyPushModelHelper::OptionallyMarkPropertyOwnerDirty));
		UE_NET_SET_IRIS_MARK_PROPERTIES_DIRTY_DELEGATE(UEPushModelPrivate::FIrisMarkPropertiesDirty::CreateStatic(&FNetHandleLegacyPushModelHelper::OptionallyMarkPropertiesOwnerDirty));
	}
}

void FNetHandleLegacyPushModelHelper::ShutdownPushModel()
{
	UE_NET_SET_IRIS_MARK_PROPERTY_DIRTY_DELEGATE({});
	UE_NET_SET_IRIS_MARK_PROPERTIES_DIRTY_DELEGATE({});
}

void FNetHandleLegacyPushModelHelper::SetNetPushID(UObject* Object, FNetPushObjectHandle Handle)
{
	FObjectNetPushIdHelper::SetNetPushIdDynamic(Object, UEPushModelPrivate::FNetPushObjectId(Handle.GetPushObjectId()).GetValue());
}

void FNetHandleLegacyPushModelHelper::ClearNetPushID(UObject* Object)
{
	FObjectNetPushIdHelper::SetNetPushIdDynamic(Object, UEPushModelPrivate::FNetPushObjectId().GetValue());
}

void FNetHandleLegacyPushModelHelper::MarkPropertyOwnerDirty(const UObject* Object, UEPushModelPrivate::FNetIrisPushObjectId PushId, const int32 RepIndex)
{
	const FNetPushObjectHandle Handle(PushId);
	MarkNetObjectStateDirty(Handle.GetNetHandle(), RepIndex, RepIndex);
}

void FNetHandleLegacyPushModelHelper::MarkPropertiesOwnerDirty(const UObject* Object, UEPushModelPrivate::FNetIrisPushObjectId PushId, const int32 StartRepIndex, const int32 EndRepIndex)
{
	const FNetPushObjectHandle Handle(PushId);
	MarkNetObjectStateDirty(Handle.GetNetHandle(), StartRepIndex, EndRepIndex);
}

void FNetHandleLegacyPushModelHelper::OptionallyMarkPropertyOwnerDirty(const UObject* Object, UEPushModelPrivate::FNetIrisPushObjectId PushId, const int32 RepIndex)
{
	if (IsIrisPushModelEnabled(true))
	{
		const FNetPushObjectHandle Handle(PushId);
		MarkNetObjectStateDirty(Handle.GetNetHandle(), RepIndex, RepIndex);
	}
}

void FNetHandleLegacyPushModelHelper::OptionallyMarkPropertiesOwnerDirty(const UObject* Object, UEPushModelPrivate::FNetIrisPushObjectId PushId, const int32 StartRepIndex, const int32 EndRepIndex)
{
	if (IsIrisPushModelEnabled(true))
	{
		const FNetPushObjectHandle Handle(PushId);
		MarkNetObjectStateDirty(Handle.GetNetHandle(), StartRepIndex, EndRepIndex);
	}
}	

}

#endif
