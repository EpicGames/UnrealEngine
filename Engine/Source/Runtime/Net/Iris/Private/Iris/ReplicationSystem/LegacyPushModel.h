// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_PUSH_MODEL

#include "CoreTypes.h"
#include "Net/Core/NetHandle/NetHandle.h"
#include "Net/Core/PushModel/PushModel.h"

namespace UE::Net
{
	struct FReplicationProtocol;
	struct FReplicationInstanceProtocol;
}

namespace UE::Net::Private
{

extern IRISCORE_API bool bIsIrisPushModelForceEnabled;
extern IRISCORE_API int IrisPushModelMode;
inline bool IsIrisPushModelEnabled(bool bIsPushModelEnabled = IS_PUSH_MODEL_ENABLED()) { return (bIsIrisPushModelForceEnabled | (IrisPushModelMode > 0)) & bIsPushModelEnabled; }

class FNetPushObjectHandle
{
public:
	FNetPushObjectHandle(FNetHandle InNetHandle);

	bool IsValid() const;

private:
	friend struct FNetHandleLegacyPushModelHelper;

	explicit FNetPushObjectHandle(UEPushModelPrivate::FNetIrisPushObjectId PushId);
	FNetHandle GetNetHandle() const;
	UEPushModelPrivate::FNetIrisPushObjectId GetPushObjectId() const;

	FNetHandle NetHandle;
};

struct FNetHandleLegacyPushModelHelper
{
	static void InitPushModel();
	static void ShutdownPushModel();

	IRISCORE_API static void SetNetPushID(UObject* Object, FNetPushObjectHandle PushHandle);
	IRISCORE_API static void ClearNetPushID(UObject* Object);

private:
	static void MarkPropertyOwnerDirty(const UObject* Object, UEPushModelPrivate::FNetIrisPushObjectId PushId, const int32 RepIndex);
	static void MarkPropertiesOwnerDirty(const UObject* Object, UEPushModelPrivate::FNetIrisPushObjectId PushId, const int32 StartRepIndex, const int32 EndRepIndex);

	static void OptionallyMarkPropertyOwnerDirty(const UObject* Object, UEPushModelPrivate::FNetIrisPushObjectId PushId, const int32 RepIndex);
	static void OptionallyMarkPropertiesOwnerDirty(const UObject* Object, UEPushModelPrivate::FNetIrisPushObjectId PushId, const int32 StartRepIndex, const int32 EndRepIndex);
};


inline FNetPushObjectHandle::FNetPushObjectHandle(FNetHandle InNetHandle)
: NetHandle(InNetHandle)
{
}

inline FNetPushObjectHandle::FNetPushObjectHandle(UEPushModelPrivate::FNetIrisPushObjectId PushId)
: NetHandle(static_cast<uint64>(PushId))
{
}

inline bool FNetPushObjectHandle::IsValid() const
{
	return NetHandle.IsValid();
}

inline FNetHandle FNetPushObjectHandle::GetNetHandle() const
{
	return NetHandle;
}

inline UEPushModelPrivate::FNetIrisPushObjectId FNetPushObjectHandle::GetPushObjectId() const
{
	return NetHandle.GetInternalValue();
}

}

#define UE_NET_IRIS_SET_PUSH_ID(...) UE::Net::Private::FNetHandleLegacyPushModelHelper::SetNetPushID(__VA_ARGS__)
#define UE_NET_IRIS_CLEAR_PUSH_ID(...) UE::Net::Private::FNetHandleLegacyPushModelHelper::ClearNetPushID(__VA_ARGS__)

#define UE_NET_IRIS_INIT_LEGACY_PUSH_MODEL() UE::Net::Private::FNetHandleLegacyPushModelHelper::InitPushModel()
#define UE_NET_IRIS_SHUTDOWN_LEGACY_PUSH_MODEL()  UE::Net::Private::FNetHandleLegacyPushModelHelper::ShutdownPushModel()

#else

namespace UE::Net::Private
{

inline constexpr bool IsIrisPushModelEnabled(bool /* bIsPushModelEnabled */ = false) { return false; }

}

#define UE_NET_IRIS_SET_PUSH_ID(...)
#define UE_NET_IRIS_CLEAR_PUSH_ID(...)

#define UE_NET_IRIS_INIT_LEGACY_PUSH_MODEL() 
#define UE_NET_IRIS_SHUTDOWN_LEGACY_PUSH_MODEL() 

#endif
