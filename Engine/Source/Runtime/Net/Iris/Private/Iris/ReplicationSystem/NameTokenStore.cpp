// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NameTokenStore.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Hash/CityHash.h"
#include "Net/Core/Trace/NetTrace.h"
#include "UObject/CoreNet.h"

#define UE_NET_ENABLE_FNAME_TOKEN_LOG 1

#if UE_NET_ENABLE_FNAME_TOKEN_LOG
#	define UE_LOG_FNAMETOKEN(Format, ...)  UE_LOG(LogNetToken, Verbose, Format, ##__VA_ARGS__)
#else
#	define UE_LOG_FNAMETOKEN(...)
#endif

#define UE_LOG_FNAMETOKEN_WARNING(Format, ...)  UE_LOG(LogNetToken, Warning, Format, ##__VA_ARGS__)

namespace UE::Net
{

FNetToken FNameTokenStore::GetOrCreateToken(FName Name)
{
	FNetTokenStoreKey Key = GetOrCreateTokenStoreKey(Name);
	if (Key.IsValid())
	{
		const FNetToken ExistingToken = GetNetTokenFromKey(Key);
		if (ExistingToken.IsValid())
		{
			return ExistingToken;
		}
		else
		{
			const FNetToken NewToken = CreateAndStoreTokenForKey(Key);

			UE_LOG_FNAMETOKEN(TEXT("FNameTokenStore::GetOrCreateToken - Created %s for %s"), *NewToken.ToString(), *Name.ToString());

			return NewToken;
		}
	}

	return FNetToken();
}

FNetTokenDataStore::FNetTokenStoreKey FNameTokenStore::GetOrCreateTokenStoreKey(FName Name)
{
	{
		UE_IRIS_PARALLEL_EXPR(UE::TRWScopeLock FNameReadLock(NameTokenStoreMutex, SLT_ReadOnly));
		if (const FNetTokenStoreKey* ExistingKey = FNameToKey.Find(Name))
		{
			return *ExistingKey;
		}
	}

	const FNetTokenStoreKey NewKey = GetNextNetTokenStoreKey();
	if (NewKey.IsValid())
	{
		UE_IRIS_PARALLEL_EXPR(UE::TRWScopeLock FNameWriteLock(NameTokenStoreMutex, SLT_Write));
		StoredFNames.Add(Name);
		FNameToKey.Add(Name, NewKey);

		return NewKey;
	}

	return FNetTokenStoreKey();
}

FNameTokenStore::FNameTokenStore(FNetTokenStore& InTokenStore)
: FNetTokenDataStore(InTokenStore)
{
	// As we use an array for our storage we must match the size of the StoredTokens array.
	// We assume that Index 0 is invalid.
	StoredFNames.SetNum(StoredTokens.Num());
}

FName FNameTokenStore::ResolveToken(FNetToken Token, const FNetTokenStoreState* NetTokenStoreState) const
{
	const FNetTokenStoreState* TokenStoreState = TokenStore.IsLocalToken(Token) ? TokenStore.GetLocalNetTokenStoreState() : NetTokenStoreState;
	if (Token.IsValid() && ensureMsgf(TokenStoreState, TEXT("FNameTokenStore::ResolveToken Needs valid remote NetTokenStoreState to resolve remote %s"), *Token.ToString()))
	{
		const FNetTokenStoreKey StoreKey = GetTokenKey(Token, *TokenStoreState);
		if (StoreKey.IsValid() && StoreKey.GetKeyIndex() < (uint32)StoredFNames.Num())
		{
			return StoredFNames[StoreKey.GetKeyIndex()];
		}
		else
		{
			UE_LOGF(LogNetToken, Error, "FNameTokenStore::ResolveToken failed to resolve %ls in NetTokenDataStore: %ls", *Token.ToString(), *GetTokenStoreName().ToString());
		}
	}

	return FName();
}

void FNameTokenStore::WriteTokenData(FNetSerializationContext& Context, FNetTokenStoreKey TokenStoreKey) const
{
	UE_NET_TRACE_DYNAMIC_NAME_SCOPE(StoredFNames[TokenStoreKey.GetKeyIndex()], *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
	UE_LOG_FNAMETOKEN(TEXT("FNameTokenStore::WriteTokenData %s %s"), *(StoredTokens[TokenStoreKey.GetKeyIndex()].ToString()), *(StoredFNames[TokenStoreKey.GetKeyIndex()].ToString()));
	// $TODO: $IRIS: We can be a bit smarter here and utilize the string-number split of FNames to export less data.. JIRA: UE-221753
	WriteString(Context.GetBitStreamWriter(), StoredFNames[TokenStoreKey.GetKeyIndex()].ToString());
}

void FNameTokenStore::WriteTokenData(FArchive& Ar, FNetTokenStoreKey TokenStoreKey, UPackageMap* Map /*= nullptr*/) const
{
	UE_NET_TRACE_DYNAMIC_NAME_SCOPE(StoredFNames[TokenStoreKey.GetKeyIndex()], static_cast<FNetBitWriter&>(Ar), GetTraceCollector(static_cast<FNetBitWriter&>(Ar)), ENetTraceVerbosity::VeryVerbose);
	UE_LOG_FNAMETOKEN(TEXT("FNameTokenStore::WriteTokenData %s %s"), *(StoredTokens[TokenStoreKey.GetKeyIndex()].ToString()), *(StoredFNames[TokenStoreKey.GetKeyIndex()].ToString()));
	FName Name = StoredFNames[TokenStoreKey.GetKeyIndex()];
	UPackageMap::StaticSerializeName(Ar, Name);
}

FNetTokenDataStore::FNetTokenStoreKey FNameTokenStore::ReadTokenData(FNetSerializationContext& Context, const FNetToken& NetToken)
{
	UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(TokenScope, FName(), *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// Read the token data and add it to the store without assigning LocalToken
	FString Temp;
	ReadString(Reader, Temp);

	if (!Reader->IsOverflown())
	{
		FName Name(Temp);
		UE_NET_TRACE_SET_SCOPE_NAME(TokenScope, Name);

		return GetOrCreateTokenStoreKey(Name);
	}
	else
	{
		return FNetTokenStoreKey();
	}
}

FNetTokenDataStore::FNetTokenStoreKey FNameTokenStore::ReadTokenData(FArchive& Ar, const FNetToken& NetToken, UPackageMap* Map /*= nullptr*/)
{
	FName Name;
	UPackageMap::StaticSerializeName(Ar, Name);

	if (!Ar.IsError())
	{
		return GetOrCreateTokenStoreKey(Name);
	}
	else
	{
		return FNetTokenStoreKey();
	}
}

}
