// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/ObjectNetSerializer.h"
#include "HAL/PlatformMemory.h"
#include "Templates/AlignmentTemplates.h"
#include "AutoRTFM.h"

namespace UE::Net
{

// NetBlob
FNetBlob::FNetBlob(const FNetBlobCreationInfo& InCreationInfo)
: CreationInfo(InCreationInfo)
, RefCount(0)
{
}

FNetBlob::~FNetBlob()
{
	if (const FReplicationStateDescriptor* Descriptor = BlobDescriptor.GetReference())
	{
		if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState))
		{
			if (uint8* StateBuffer = QuantizedBlobState.GetStateBuffer())
			{
				QuantizedBlobState.Unprotect();
				Private::FReplicationStateOperationsInternal::FreeDynamicState(StateBuffer, Descriptor);
			}
		}
	}
}

void FNetBlob::SetState(const TRefCountPtr<const FReplicationStateDescriptor>& InBlobDescriptor, FQuantizedBlobState&& InQuantizedBlobState)
{
	BlobDescriptor = InBlobDescriptor;
	QuantizedBlobState = MoveTemp(InQuantizedBlobState);
}

void FNetBlob::SerializeCreationInfo(FNetSerializationContext& Context, const FNetBlobCreationInfo& CreationInfo)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	Writer->WriteBits((CreationInfo.Type == 0 ? 0U : 1U), 1U);
	if (CreationInfo.Type != 0)
	{
		Writer->WriteBits(CreationInfo.Type, 7U);
	}

	// Retain Reliable flag
	Writer->WriteBits(EnumHasAnyFlags(CreationInfo.Flags, ENetBlobFlags::Reliable) ? 1U : 0U, 1U);
}

void FNetBlob::DeserializeCreationInfo(FNetSerializationContext& Context, FNetBlobCreationInfo& OutCreationInfo)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	FNetBlobType Type = Reader->ReadBits(1U);
	if (Type != 0)
	{
		Type = Reader->ReadBits(7U);
	}

	const uint32 Reliable = Reader->ReadBits(1);
	ENetBlobFlags Flags = (Reliable ? ENetBlobFlags::Reliable : ENetBlobFlags::None);

	OutCreationInfo.Type = Type;
	OutCreationInfo.Flags = Flags;
}

void FNetBlob::SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const
{
	SerializeBlob(Context);
}

void FNetBlob::DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	check(BlobDescriptor.IsValid());
	DeserializeBlob(Context);
}

void FNetBlob::Serialize(FNetSerializationContext& Context) const
{
	SerializeBlob(Context);
}

void FNetBlob::Deserialize(FNetSerializationContext& Context)
{
	DeserializeBlob(Context);
}

void FNetBlob::CollectObjectReferences(FNetSerializationContext& Context, FNetReferenceCollector& Collector) const
{
	if (BlobDescriptor.IsValid() && QuantizedBlobState.GetStateBuffer())
	{
		const FNetSerializerChangeMaskParam InitStateChangeMaskInfo = { 0 };
		Private::FReplicationStateOperationsInternal::CollectReferences(Context, Collector, InitStateChangeMaskInfo, QuantizedBlobState.GetStateBuffer(), BlobDescriptor);
	}
}

void FNetBlob::SerializeBlob(FNetSerializationContext& Context) const
{
	if (BlobDescriptor.IsValid() && QuantizedBlobState.GetStateBuffer())
	{
		FReplicationStateOperations::Serialize(Context, QuantizedBlobState.GetStateBuffer(), BlobDescriptor);
	}
}

void FNetBlob::DeserializeBlob(FNetSerializationContext& Context)
{
	if (BlobDescriptor.IsValid() && QuantizedBlobState.GetStateBuffer())
	{
		FReplicationStateOperations::Deserialize(Context, QuantizedBlobState.GetStateBuffer(), BlobDescriptor);
	}
}

void FNetBlob::Release() const
{
	UE_AUTORTFM_ONCOMMIT(this)
	{
		if (--RefCount == 0)
		{
			delete this;
		}
	};
}

TArrayView<const FNetObjectReference> FNetBlob::GetNetObjectReferenceExports() const
{
	return MakeArrayView<const FNetObjectReference>(nullptr, 0);
};

TArrayView<const FNetToken> FNetBlob::GetNetTokenExports() const
{
	return MakeArrayView<const FNetToken>(nullptr, 0);
};

// NetObjectAttachment
FNetObjectAttachment::FNetObjectAttachment(const FNetBlobCreationInfo& CreationInfo)
: FNetBlob(CreationInfo)
{
}

FNetObjectAttachment::~FNetObjectAttachment()
{
}

void FNetObjectAttachment::SerializeObjectReference(FNetSerializationContext& Context) const
{
	WriteFullNetObjectReference(Context, NetObjectReference);
	WriteFullNetObjectReference(Context, TargetObjectReference);
}

void FNetObjectAttachment::DeserializeObjectReference(FNetSerializationContext& Context)
{
	ReadFullNetObjectReference(Context, NetObjectReference);
	ReadFullNetObjectReference(Context, TargetObjectReference);
}

void FNetObjectAttachment::SerializeSubObjectReference(FNetSerializationContext& Context, FNetRefHandle RefHandle) const
{
	WriteFullNetObjectReference(Context, TargetObjectReference);
}

void FNetObjectAttachment::DeserializeSubObjectReference(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	NetObjectReference = Private::FObjectReferenceCache::MakeNetObjectReference(RefHandle);
	ReadFullNetObjectReference(Context, TargetObjectReference);
}

FNetBlob::FQuantizedBlobState::FQuantizedBlobState(uint32 Size, uint32 Alignment, FQuantizedBlobState::EMemoryAllocationFlags InMemoryAllocationFlags)
: MemoryAllocationFlags(InMemoryAllocationFlags)
{
	if (InMemoryAllocationFlags == EMemoryAllocationFlags::None)
	{
		StateBuffer = static_cast<uint8*>(FMemory::MallocZeroed(Size, Alignment));
		AllocationSize = Size;
	}
	else
	{
		const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
		const uint32 NewAlignment = Align(Alignment, MemoryConstants.PageSize);
		AllocationSize = static_cast<uint32>(Align(Size, MemoryConstants.PageSize));
		StateBuffer = static_cast<uint8*>(FMemory::MallocZeroed(AllocationSize, NewAlignment));
	}
}


FNetBlob::FQuantizedBlobState::~FQuantizedBlobState()
{
	if (StateBuffer)
	{
		Unprotect();
		FMemory::Free(StateBuffer);
	}
}

void FNetBlob::FQuantizedBlobState::Protect()
{
	if (MemoryAllocationFlags == EMemoryAllocationFlags::Protectable)
	{
		UE_AUTORTFM_OPEN
		{
			constexpr bool bCanRead = true;
			constexpr bool bCanWrite = false;
			FPlatformMemory::PageProtect(StateBuffer, AllocationSize, bCanRead, bCanWrite);
		};
		UE_AUTORTFM_ONPREABORT(StateBuffer = this->StateBuffer, AllocationSize = this->AllocationSize)
		{
			constexpr bool bCanRead = true;
			constexpr bool bCanWrite = true;
			FPlatformMemory::PageProtect(StateBuffer, AllocationSize, bCanRead, bCanWrite);
		};
	}
}

void FNetBlob::FQuantizedBlobState::Unprotect()
{
	if (MemoryAllocationFlags == EMemoryAllocationFlags::Protectable)
	{
		// Marking a page as read and write does not require to be rolled back. It's the normal status of our memory.
		UE_AUTORTFM_OPEN
		{
			constexpr bool bCanRead = true;
			constexpr bool bCanWrite = true;
			FPlatformMemory::PageProtect(StateBuffer, AllocationSize, bCanRead, bCanWrite);
		};
	}
}

}
