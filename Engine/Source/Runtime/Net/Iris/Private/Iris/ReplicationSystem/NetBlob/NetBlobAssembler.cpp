// Copyright Epic Games, Inc. All Rights Reserved.
#include "Iris/ReplicationSystem/NetBlob/NetBlobAssembler.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandler.h"
#include "Iris/ReplicationSystem/NetBlob/RawDataNetBlob.h"
#include "Iris/ReplicationSystem/NetBlob/SequentialPartialNetBlobHandler.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Core/IrisLog.h"

namespace UE::Net
{

static const FName NetError_PartialNetBlobSequenceError("Out of order PartialNetBlob.");

FNetBlobAssembler::FNetBlobAssembler()
{
}

void FNetBlobAssembler::Init(const FNetBlobAssemblerInitParams& InitParams)
{
	PartialNetBlobHandlerConfig = InitParams.PartialNetBlobHandlerConfig;
	ensureMsgf(PartialNetBlobHandlerConfig != nullptr, TEXT("NetBlobAssembler requires a PartialNetBlobHandlerConfig"));
}

void FNetBlobAssembler::AddPartialNetBlob(FNetSerializationContext& Context, FNetRefHandle InRefHandle, const TRefCountPtr<FPartialNetBlob>& PartialNetBlob)
{
	bIsReadyToAssemble = false;

	if (!ensureMsgf(PartialNetBlobHandlerConfig != nullptr, TEXT("NetBlobAssembler requires a PartialNetBlobHandlerConfig")))
	{
		UE_LOGF(LogIris, Error, "NetBlobAssembler - Requires a PartialNetBlobHandlerConfig");
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	// We expect broken sequences to be handled as soon as they're reported. We've already reported the error once.
	if (bIsBrokenSequence)
	{
		Context.SetError(NetError_PartialNetBlobSequenceError);
		return;
	}

	const uint32 SequenceNumber = PartialNetBlob->GetSequenceNumber();
	const bool bIsFirstPart = PartialNetBlob->IsFirstPart();

	// Broken sequence detection. For reliable blobs the NextPartIndex and NextSequenceSumber must match. If reliability changes we need some additional validation.
	const bool bIsReliable = PartialNetBlob->IsReliable();
	if (SequenceNumber != NextSequenceSumber || bIsReliable != bIsProcessingReliable)
	{
		// Reliable sequences are expected to be fully processed before proceeding to a new sequence. If we're in the middle of processing everything needs to match.
		if (bIsProcessingReliable)
		{
			bIsBrokenSequence = true;
			Context.SetError(NetError_PartialNetBlobSequenceError);
			return;
		}

		// Gracefully handle going from unreliable to first part of blob, regardless of its reliability.
		if (!bIsFirstPart && SequenceNumber != NextSequenceSumber)
		{
			bIsBrokenSequence = true;
			if (bIsReliable)
			{
				Context.SetError(NetError_PartialNetBlobSequenceError);
			}
			return;
		}
	}

	bIsProcessingReliable = bIsReliable;

	if (bIsFirstPart)
	{
		const uint32 PartCount = PartialNetBlob->GetPartCount();
		NextSequenceSumber = SequenceNumber;
		LastPartSequenceNumber = SequenceNumber + PartialNetBlob->GetPartCount() - 1U;

		if (PartCount == 0)
		{
			bIsBrokenSequence = true;
			Context.SetError(GNetError_InvalidValue);
			UE_LOGF(LogIris, Error, "NetBlobAssembler - Expected partCount with first part");
			ensure(false);
			return;
		}

		if (PartCount > PartialNetBlobHandlerConfig->GetMaxPartCount())
		{
			bIsBrokenSequence = true;
			Context.SetError(GNetError_InvalidValue);
			UE_LOGF(LogIris, Error, "NetBlobAssembler - Received invalid PartCount %u configured MaxPartCount %u", PartCount, PartialNetBlobHandlerConfig->GetMaxPartCount());
			ensure(false);
			return;
		}

		const uint32 PayloadBitCount = PartialNetBlob->GetPayloadBitCount();
		// We allow part sizes to be lower than the config value in case it was hotfixed.
		if (PayloadBitCount > PartialNetBlobHandlerConfig->GetMaxPartBitCount())
		{
			bIsBrokenSequence = true;
			Context.SetError(GNetError_InvalidValue);
			UE_LOGF(LogIris, Error, "NetBlobAssembler - Received invalid PayloadBitCount %u configured MaxPartBitCount %u", PayloadBitCount, PartialNetBlobHandlerConfig->GetMaxPartBitCount());
			ensure(false);
			return;
		}

		RefHandle = InRefHandle;

		NetBlobCreationInfo = PartialNetBlob->GetOriginalCreationInfo();

		// The first part must be greater or equal to any subsequent parts or we will report an error.
		FirstPayloadBitCount = PayloadBitCount;

		// Prepare a bitstream that can hold the full blob.
		const uint32 MaxByteCountPerPart = (PayloadBitCount + 7U)/8U;
		Payload.SetNumUninitialized((MaxByteCountPerPart*PartCount + 3U)/4U);
		BitWriter.InitBytes(Payload.GetData(), Payload.Num()*4U);

		// Store partial payload.
		BitWriter.WriteBitStream(PartialNetBlob->GetPayload(), 0, PayloadBitCount);
	}
	else
	{
		if (InRefHandle != RefHandle)
		{
			bIsBrokenSequence = true;
			Context.SetError(NetError_PartialNetBlobSequenceError);
			return;
		}

		const uint32 PayloadBitCount = PartialNetBlob->GetPayloadBitCount();
		const uint32 PayloadByteCount = (PayloadBitCount + 7U)/8U;
		// All parts except the last one is expected to match the first part. The last part may be smaller.
		if (((PayloadBitCount != FirstPayloadBitCount) && (NextSequenceSumber != LastPartSequenceNumber)) || (PayloadBitCount > FirstPayloadBitCount))
		{
			bIsBrokenSequence = true;
			Context.SetError(GNetError_InvalidValue);
			UE_LOGF(LogIris, Error, "NetBlobAssembler - Received invalid PayloadBitCount %u", PayloadBitCount);
			ensure(false);
			return;
		}

		// Store partial payload.
		BitWriter.WriteBitStream(PartialNetBlob->GetPayload(), 0, PayloadBitCount);
	}

	if (NextSequenceSumber == LastPartSequenceNumber)
	{
		bIsReadyToAssemble = true;
	}
	else
	{
		++NextSequenceSumber;
	}
}

TRefCountPtr<FNetBlob> FNetBlobAssembler::Assemble(FNetSerializationContext& Context)
{
	bIsReadyToAssemble = false;

	INetBlobReceiver* BlobHandler = Context.GetNetBlobReceiver();
	checkSlow(BlobHandler != nullptr);

	const TRefCountPtr<FNetBlob>& NetBlob = BlobHandler->CreateNetBlob(NetBlobCreationInfo);
	if (Context.HasError() || !NetBlob.IsValid())
	{
		Context.SetError(GNetError_UnsupportedNetBlob);
		return nullptr;
	}

	BitWriter.CommitWrites();

	// Fast path for RawDataNetBlobs
	if (EnumHasAnyFlags(NetBlob->GetCreationInfo().Flags, ENetBlobFlags::RawDataNetBlob))
	{
		FRawDataNetBlob* RawDataNetBlob = static_cast<FRawDataNetBlob*>(NetBlob.GetReference());
		// The BitWriter is currently pointing to the payload data so we want to make sure we're not leaving
		// it in an undefined state after moving the payload.
		const uint32 PayloadBitCount = BitWriter.GetPosBits();
		BitWriter = FNetBitStreamWriter();
		RawDataNetBlob->SetRawData(MoveTemp(Payload), PayloadBitCount);
	}
	else
	{
		FNetBitStreamReader BitReader;
		BitReader.InitBits(Payload.GetData(), BitWriter.GetPosBits());

		FNetSerializationContext ReadContext = Context.MakeSubContext(&BitReader);
		ReadContext.SetTraceCollector(nullptr);

		if (RefHandle.IsValid())
		{
			NetBlob->DeserializeWithObject(ReadContext, RefHandle);
		}
		else
		{
			NetBlob->Deserialize(ReadContext);
		}

		if (ReadContext.HasErrorOrOverflow())
		{
			// Copy error
			if (ReadContext.HasError())
			{
				Context.SetError(ReadContext.GetError());
			}
			else
			{
				Context.SetError(GNetError_BitStreamOverflow);
			}

			return nullptr;
		}

		// Bitstream mismatch?
		if (BitWriter.GetPosBits() != BitReader.GetPosBits())
		{
			Context.SetError(GNetError_BitStreamError);
			return nullptr;
		}
	}

	return NetBlob;
}

}
