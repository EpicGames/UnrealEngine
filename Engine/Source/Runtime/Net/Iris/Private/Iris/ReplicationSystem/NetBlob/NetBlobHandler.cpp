// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/NetBlobHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetBlobHandler)

namespace UE::Net
{

const FName GNetError_UnsupportedNetBlob("Unsupported NetBlob type");

}

UNetBlobHandler::UNetBlobHandler()
: NetBlobType(UE::Net::InvalidNetBlobType)
{
}

UNetBlobHandler::~UNetBlobHandler()
{
}

void UNetBlobHandler::AddConnection(uint32 ConnectionId)
{
}

void UNetBlobHandler::RemoveConnection(uint32 ConnectionId)
{
}
