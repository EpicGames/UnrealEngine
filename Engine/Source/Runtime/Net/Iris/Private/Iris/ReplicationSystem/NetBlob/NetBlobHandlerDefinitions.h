// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "NetBlobHandlerDefinitions.generated.h"

namespace UE::Net::Private
{
	class FNetBlobHandlerManager;
}

USTRUCT()
struct FNetBlobHandlerDefinition
{
	GENERATED_BODY()

	/**
	 * UClass name of the UNetObjectHandler derived class.
	 * In order for a handler to be successfully registered via UReplicationSystem::RegisterNetBlobHandler
	 * the handler class must match one of the definitions.
	 */
	UPROPERTY()
	FName ClassName;
};

/** Configurable net blob handler definitions. */
UCLASS(MinimalAPI, transient, config=Engine)
class UNetBlobHandlerDefinitions : public UObject
{
	GENERATED_BODY()

protected:
	UNetBlobHandlerDefinitions();

private:
	friend UE::Net::Private::FNetBlobHandlerManager;

	UPROPERTY(Config)
	TArray<FNetBlobHandlerDefinition> NetBlobHandlerDefinitions;

#if WITH_AUTOMATION_WORKER
public:
	TArray<FNetBlobHandlerDefinition>& ReadWriteHandlerDefinitions() { return NetBlobHandlerDefinitions; }
#endif
};
