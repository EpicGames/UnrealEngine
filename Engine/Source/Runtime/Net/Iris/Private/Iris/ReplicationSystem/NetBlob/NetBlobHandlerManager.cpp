// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/NetBlobHandlerManager.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandlerDefinitions.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Core/IrisLog.h"
#include "Containers/ArrayView.h"

namespace UE::Net::Private
{

FNetBlobHandlerManager::FNetBlobHandlerManager()
{
}

void FNetBlobHandlerManager::Init()
{
	const UNetBlobHandlerDefinitions* BlobHandlerDefinitions = GetDefault<UNetBlobHandlerDefinitions>();
	// Check if FNetBlob::SerializeCreationInfo needs to use more bits for blob type.
	checkf(BlobHandlerDefinitions->NetBlobHandlerDefinitions.Num() < 128, TEXT("Excessive amount of NetBlobHandlers: %d. This breaks net serialization."), BlobHandlerDefinitions->NetBlobHandlerDefinitions.Num());
	Handlers.SetNum(BlobHandlerDefinitions->NetBlobHandlerDefinitions.Num());
}

bool FNetBlobHandlerManager::RegisterHandler(UNetBlobHandler* Handler)
{
	if (Handler == nullptr)
	{
		return false;
	}

	if (!ensureMsgf(Handler->NetBlobType == InvalidNetBlobType, TEXT("NetBlobHandler of class %s has already been registered."), ToCStr(Handler->GetClass()->GetName())))
	{
		return false;
	}

	const FString& ClassName = Handler->GetClass()->GetName();
	const UNetBlobHandlerDefinitions* BlobHandlerDefinitions = GetDefault<UNetBlobHandlerDefinitions>();
	for (const FNetBlobHandlerDefinition& Definition : MakeArrayView(BlobHandlerDefinitions->NetBlobHandlerDefinitions))
	{
		if (Definition.ClassName.ToString() == ClassName)
		{
			const uint32 Index = static_cast<uint32>(&Definition - BlobHandlerDefinitions->NetBlobHandlerDefinitions.GetData());
			Handler->NetBlobType = Index;
			Handlers[Index] = Handler;
			return true;
		}
	}

	UE_LOGF(LogIris, Warning, "Handler of class %ls was not found in the NetBlobHandlerDefinitions", ToCStr(Handler->GetClass()->GetName()));
	return false;
}

TRefCountPtr<FNetBlob> FNetBlobHandlerManager::CreateNetBlob(const FNetBlobCreationInfo& CreationInfo) const
{
	if (!ensureMsgf(CreationInfo.Type < uint32(Handlers.Num()), TEXT("Unknown NetBlob type %u"), CreationInfo.Type))
	{
		return nullptr;
	}

	UNetBlobHandler* Handler = Handlers[CreationInfo.Type].Get();
	if (Handler == nullptr)
	{
		UE_LOGF(LogIris, Warning, "No handler registered for NetBlob type %u", CreationInfo.Type);
		return nullptr;
	}

	const TRefCountPtr<FNetBlob>& Blob = Handler->CreateNetBlob(CreationInfo);
#if !UE_BUILD_SHIPPING
	ensure(!Blob.IsValid() || (Blob->GetCreationInfo().Type == CreationInfo.Type));
#endif

	return Blob;
}

void FNetBlobHandlerManager::OnNetBlobReceived(FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& Blob)
{
	if (!Blob.IsValid())
	{
		return;
	}

	const FNetBlobCreationInfo& CreationInfo = Blob->GetCreationInfo();
	if (!ensure(CreationInfo.Type < uint32(Handlers.Num())))
	{
		Context.SetError(GNetError_UnsupportedNetBlob);
		return;
	}

	UNetBlobHandler* Handler = Handlers[CreationInfo.Type].Get();
	if (Handler == nullptr)
	{
		UE_LOGF(LogIris, Warning, "No handler registered for NetBlob type %u", CreationInfo.Type);
		return; 
	}

	return Handler->OnNetBlobReceived(Context, Blob);
}

void FNetBlobHandlerManager::AddConnection(uint32 ConnectionId) const
{
	for (const TWeakObjectPtr<UNetBlobHandler>& Handler : Handlers)
	{
		if (!Handler.IsValid())
		{
			continue;
		}

		Handler->AddConnection(ConnectionId);
	}
}

void FNetBlobHandlerManager::RemoveConnection(uint32 ConnectionId)
{
}

}
