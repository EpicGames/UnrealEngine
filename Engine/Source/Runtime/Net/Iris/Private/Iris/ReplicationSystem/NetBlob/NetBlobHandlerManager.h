// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandler.h"

namespace UE::Net::Private
{

class FNetBlobHandlerManager final : public INetBlobReceiver
{
public:
	FNetBlobHandlerManager();

	void Init();

	/** Returns true if the handler was successfully registered. */
	bool RegisterHandler(UNetBlobHandler* Handler);

	/** Creates a NetBlob of the specific type. */
	virtual TRefCountPtr<FNetBlob> CreateNetBlob(const FNetBlobCreationInfo&) const override;

	/** Calls the appropriate blob handler's OnNetBlobReceived() method. */
	virtual void OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& Blob) override;

	/** Calls AddConnection on each registered handler. */
	void AddConnection(uint32 ConnectionId) const;

	/** Calls RemoveConnection on each registered handler. */
	void RemoveConnection(uint32 ConnectionId);

private:
	TArray<TWeakObjectPtr<UNetBlobHandler>> Handlers;
};

}
