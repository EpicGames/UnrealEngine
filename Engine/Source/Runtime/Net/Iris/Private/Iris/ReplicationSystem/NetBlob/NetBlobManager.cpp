// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/NetBlobManager.h"

#include "HAL/IConsoleManager.h"

#include "Iris/ReplicationSystem/NetBlob/NetObjectBlobHandler.h"
#include "Iris/ReplicationSystem/NetBlob/NetRPCHandler.h"
#include "Iris/ReplicationSystem/NetBlob/PartialNetObjectAttachmentHandler.h"

#include "Iris/ReplicationSystem/ReplicationConnections.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationWriter.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"

#include "Iris/Core/NetObjectReference.h"
#include "Iris/Core/IrisLog.h"

#include "Net/Core/Trace/NetDebugName.h"

namespace UE::Net::Private
{

static TAutoConsoleVariable<int32> CVarEnableIrisRPCs(TEXT("net.Iris.EnableRPCs"), 1, TEXT( "If > 0 let Iris replicate and execute RPCs."));

static bool bThrottleRPCWarnings = true;
static FAutoConsoleVariableRef CVarThrottleRPCWarnings(TEXT("net.Iris.ThrottleRPCWarnings"), bThrottleRPCWarnings, TEXT("Only log send failure warnings once per RPC type."));

static bool bEnableRPCSendWarnings = true;
static FAutoConsoleVariableRef CVarEnableRPCSendWarnings(TEXT("net.Iris.EnableRPCSendWarnings"), bEnableRPCSendWarnings, TEXT("Whether to warn about RPC send failures or not. Defaults to true"));

static bool bAllowRPCsOnDormantObjects = true;
static FAutoConsoleVariableRef CVarAllowRPCsOnDormantObjects(TEXT("net.Iris.RPC.AllowOnDormantObjects"), bAllowRPCsOnDormantObjects, TEXT("When true allow RPCs to be sent on dormant objects. When false block all RPCs from the moment the dormant change is requested."));

static bool bAutoNetFlushOnDormantRPC = true;
static FAutoConsoleVariableRef CVarAutoNetFlushOnDormantRPC(TEXT("net.Iris.RPC.AutoNetFlushOnDormantObjects"), bAutoNetFlushOnDormantRPC, TEXT("When true we will make an implicit NetFlush request when an RPC is sent on a dormant object. When false no replicated properties would be sent along with the RPC."));

FNetBlobManager::FNetBlobManager()
{
}

void FNetBlobManager::Init(FNetBlobManagerInitParams& InitParams)
{
	BlobHandlerManager.Init();

	ReplicationSystem = InitParams.ReplicationSystem;
	Connections = &InitParams.ReplicationSystem->GetReplicationSystemInternal()->GetConnections();
	NetRefHandleManager = &InitParams.ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
	ObjectReferenceCache = &InitParams.ReplicationSystem->GetReplicationSystemInternal()->GetObjectReferenceCache();
	bIsServer = ReplicationSystem->IsServer();
	bSendAttachmentsWithObject = InitParams.bSendAttachmentsWithObject; 
	bAllowObjectReplication = ReplicationSystem->AllowObjectReplication();

	RegisterDefaultHandlers();

	AttachmentSendQueue.Init(this);
}

bool FNetBlobManager::RegisterNetBlobHandler(UNetBlobHandler* Handler)
{
	return BlobHandlerManager.RegisterHandler(Handler);
}

bool FNetBlobManager::QueueNetObjectAttachment(uint32 ConnectionId, const FNetObjectReference& TargetRef, const TRefCountPtr<FNetObjectAttachment>& Attachment, ENetObjectAttachmentSendPolicyFlags SendFlags)
{
	if (!Attachment.IsValid())
	{
		return false;
	}

	if (!Connections->IsValidConnection(ConnectionId))
	{
		UE_LOGF(LogIris, Warning, "Dropping attachment to invalid connection %u.", ConnectionId);
		return true;
	}

	FRPCOwner OwnerInfo;

	OwnerInfo.CallerRef = TargetRef;
	OwnerInfo.TargetRef = TargetRef;

	bool bCanSendRpc = GetRootObjectAndSubObjectIndicesFromAnyHandle(TargetRef.GetRefHandle(), OwnerInfo.RootObjectIndex, OwnerInfo.SubObjectIndex);

	if (!bCanSendRpc)
	{
		if (!TargetRef.IsValid())
		{
			UE_LOGF(LogIris, Warning, "QueueNetObjectAttachment %ls Failed due to invalid Target. Unable to resolve target reference %ls.", *Attachment->GetNetObjectReference().ToString(), *TargetRef.ToString());
			return false;
		}

		//$IRIS TODO: It's possible the Target and Caller will be differrent from the RootObjectIndex and SubObjectIndex since we use the ReplicatedOuter instead of the true Root.
		//This probably happens when the outer list looks like this: Actor->ActorComponent->SubObject1->SubObject2. 

		// If the TargetRef is a valid reference but not replicated, then the outer must be used instead
		OwnerInfo.CallerRef = ObjectReferenceCache->GetReplicatedOuter(TargetRef, ReplicationSystem);

		// Can that outer send RPCs
		bCanSendRpc = GetRootObjectAndSubObjectIndicesFromAnyHandle(OwnerInfo.CallerRef.GetRefHandle(), OwnerInfo.RootObjectIndex, OwnerInfo.SubObjectIndex);
		if (!bCanSendRpc)
		{
			UE_LOGF(LogIris, Warning, "QueueNetObjectAttachment %ls Failed due to invalid Outer. Unable to resolve outer reference %ls (index: %u) for target reference %ls (index: %u).", 
				*Attachment->GetNetObjectReference().ToString(), ToCStr(OwnerInfo.CallerRef.ToString()), OwnerInfo.RootObjectIndex, ToCStr(TargetRef.ToString()), OwnerInfo.SubObjectIndex);

			return false;
		}
	}

	Attachment->SetNetObjectReference(OwnerInfo.CallerRef, OwnerInfo.TargetRef);
	AttachmentSendQueue.Enqueue(ConnectionId, OwnerInfo.RootObjectIndex, OwnerInfo.SubObjectIndex, Attachment, SendFlags);
	return true;
}

bool FNetBlobManager::SendMulticastRPC(const FSendRPCContext& Context, const void* Parameters, UE::Net::ENetObjectAttachmentSendPolicyFlags SendFlags)
{
	if (CVarEnableIrisRPCs.GetValueOnGameThread() <= 0)
	{
		return false;
	}

	UNetRPCHandler* Handler = RPCHandler.Get();
	if (Handler == nullptr)
	{
		return false;
	}

	// May the RPC be sent?
	if ((Context.Function->FunctionFlags & (bIsServer ? (FUNC_NetClient | FUNC_NetMulticast) : FUNC_NetServer)) == 0)
	{
		checkf(false, TEXT("Trying to call RPC %s in the wrong direction."), ToCStr(Context.Function->GetName()));
		return true;
	}

	// Check if there are any connections at all
	const FNetBitArray& ValidConnections = Connections->GetValidConnections();
	if (ValidConnections.FindFirstOne() == FNetBitArray::InvalidIndex)
	{
		return true;
	}

	FRPCOwner OwnerInfo;
	if (!GetRPCOwner(OwnerInfo, Context))
	{
		return false;
	}
	
	// If we prevent RPCs on dormant objects (allowed by default)
	const bool bIsObjectDormant = NetRefHandleManager->GetWantToBeDormantInternalIndices().IsBitSet(OwnerInfo.RootObjectIndex);
	if (!bAllowRPCsOnDormantObjects && bIsObjectDormant)
	{
		// If the object is dormant but requested a FlushNet we still allow him to send RPCs
		// This also means objects that went dormant on the current frame can also send RPCs.
		if (!NetRefHandleManager->GetDormantObjectsPendingFlushNet().IsBitSet(OwnerInfo.RootObjectIndex))
		{
			return false;
		}
	}

	const TRefCountPtr<FNetRPC>& RPC = Handler->CreateRPC(OwnerInfo.CallerRef, Context.Function, Parameters);
	if (!RPC.IsValid())
	{
		UE_LOGF(LogIris, Warning, "Unable to create RPC for function %ls.", ToCStr(Context.Function->GetName()));
		return true;
	}

	// Force a NetFlush when sending an RPC on a dormant object
	if (bAutoNetFlushOnDormantRPC && bIsObjectDormant)
	{
		UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UObjectReplicationBridge>();
		Bridge->NetFlushDormantObject(ObjectReferenceCache->GetObjectReferenceHandleFromObject(Context.RootObject));
	}

	RPC->SetNetObjectReference(OwnerInfo.CallerRef, OwnerInfo.TargetRef);
	AttachmentSendQueue.Enqueue(OwnerInfo.RootObjectIndex, OwnerInfo.SubObjectIndex, reinterpret_cast<const TRefCountPtr<FNetObjectAttachment>&>(RPC), SendFlags, Connections->GetOpenConnections());

	return true;
}

bool FNetBlobManager::SendUnicastRPC(uint32 ConnectionId, const FSendRPCContext& Context, const void* Parameters, UE::Net::ENetObjectAttachmentSendPolicyFlags SendFlags)
{
	if (CVarEnableIrisRPCs.GetValueOnGameThread() <= 0)
	{
		return false;
	}

	UNetRPCHandler* Handler = RPCHandler.Get();
	if (Handler == nullptr)
	{
		return false;
	}

	// If NetServer, NetClient, or NetMulticast flags are present, filter the RPC based on them. If not, send the RPC in either directon.
	if ((Context.Function->FunctionFlags & FUNC_NetServer) && bIsServer)
	{
		checkf(false, TEXT("Trying to call server RPC %s in the wrong direction."), ToCStr(Context.Function->GetName()));
		return true;
	}

	if ((Context.Function->FunctionFlags & (FUNC_NetClient | FUNC_NetMulticast)) && !bIsServer)
	{
		checkf(false, TEXT("Trying to call client RPC %s in the wrong direction."), ToCStr(Context.Function->GetName()));
		return true;
	}

	if (!Connections->IsValidConnection(ConnectionId))
	{
		UE_LOGF(LogIris, Warning, "Trying to call RPC on non-existing connection %u.", ConnectionId);
		return true;
	}

	if (!Connections->IsOpenConnection(ConnectionId))
	{
		// This connection is shutting down and only flushing existing reliable data, not sending new RPCs.
		return true;
	}

	FRPCOwner OwnerInfo;
	if (!GetRPCOwner(OwnerInfo, Context))
	{
		return false;
	}

	// If we prevent RPCs on dormant objects (allowed by default)
	const bool bIsObjectDormant = NetRefHandleManager->GetWantToBeDormantInternalIndices().IsBitSet(OwnerInfo.RootObjectIndex);
	if (!bAllowRPCsOnDormantObjects && bIsObjectDormant)
	{
		// If the object is dormant but requested a FlushNet we still allow him to send RPCs
		// This also means objects that went dormant on the current frame can also send RPCs.
		if (!NetRefHandleManager->GetDormantObjectsPendingFlushNet().IsBitSet(OwnerInfo.RootObjectIndex))
		{
			return false;
		}
	}

	const TRefCountPtr<FNetRPC>& RPC = Handler->CreateRPC(OwnerInfo.CallerRef, Context.Function, Parameters);
	if (!RPC.IsValid())
	{
		UE_LOGF(LogIris, Warning, "Unable to create RPC for function %ls.", ToCStr(Context.Function->GetName()));
		return true;
	}

	// Force a NetFlush when sending an RPC on a dormant object
	if (bAutoNetFlushOnDormantRPC && bIsObjectDormant)
	{
		UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UObjectReplicationBridge>();
		Bridge->NetFlushDormantObject(ObjectReferenceCache->GetObjectReferenceHandleFromObject(Context.RootObject));
	}

	RPC->SetNetObjectReference(OwnerInfo.CallerRef, OwnerInfo.TargetRef);
	AttachmentSendQueue.Enqueue(ConnectionId, OwnerInfo.RootObjectIndex, OwnerInfo.SubObjectIndex, reinterpret_cast<const TRefCountPtr<FNetObjectAttachment>&>(RPC), SendFlags);
	return true;
}

bool FNetBlobManager::GetRPCOwner(FRPCOwner& OutOwnerInfo, const FSendRPCContext& Context) const
{
	bool bCanSendRpc = false;

	// If a root object is sending an RPC
	if (Context.SubObject == nullptr)
	{
		OutOwnerInfo.TargetRef = ObjectReferenceCache->GetOrCreateObjectReference(Context.RootObject, ReplicationSystem);
		OutOwnerInfo.CallerRef = OutOwnerInfo.TargetRef;

		bCanSendRpc = GetRootObjectIndicesFromHandle(OutOwnerInfo.TargetRef.GetRefHandle(), OutOwnerInfo.RootObjectIndex);

		if (!bCanSendRpc)
		{
			bool bLogRPCFailed = true;
			if (UE::Net::Private::bThrottleRPCWarnings)
			{
				bool& bWasAlreadyLogged = RPCWarningThrottler.FindOrAdd(Context.Function->GetFName(), false);
				bLogRPCFailed = !bWasAlreadyLogged;
				bWasAlreadyLogged = true;
			}
			UE_CLOGF(bEnableRPCSendWarnings && bLogRPCFailed, LogIris, Warning, "SendRPC %ls for %ls Failed. This root object is not yet replicated (RefHandle: %ls Index: %u).",
					ToCStr(Context.Function->GetName()), *GetNameSafe(Context.RootObject), ToCStr(OutOwnerInfo.CallerRef.GetRefHandle().ToString()), OutOwnerInfo.RootObjectIndex);
		}
	}
	// If a subobject is sending an RPC
	else
	{
		const FNetRefHandle SubObjectNetRef = ObjectReferenceCache->GetObjectReferenceHandleFromObject(Context.SubObject);

		// If the subobject can be referenced
		if (SubObjectNetRef.IsValid())
		{
			OutOwnerInfo.TargetRef = ObjectReferenceCache->GetOrCreateObjectReference(Context.SubObject, ReplicationSystem);
			OutOwnerInfo.CallerRef = OutOwnerInfo.TargetRef;

			check(OutOwnerInfo.TargetRef.GetRefHandle() == SubObjectNetRef);

			bCanSendRpc = GetRootObjectAndSubObjectIndicesFromSubObjectHandle(OutOwnerInfo.TargetRef.GetRefHandle(), OutOwnerInfo.RootObjectIndex, OutOwnerInfo.SubObjectIndex);
		}
		
		// Send the RPC via the Root object if the subobject is not capable
		if (!bCanSendRpc)
		{
			OutOwnerInfo.TargetRef = ObjectReferenceCache->GetOrCreateObjectReference(Context.SubObject, ReplicationSystem);
			OutOwnerInfo.CallerRef = ObjectReferenceCache->GetOrCreateObjectReference(Context.RootObject, ReplicationSystem);

			bCanSendRpc = GetRootObjectIndicesFromHandle(OutOwnerInfo.CallerRef.GetRefHandle(), OutOwnerInfo.RootObjectIndex);
		}
		
		if (!bCanSendRpc)
		{
			bool bLogRPCFailed = true;
			if (UE::Net::Private::bThrottleRPCWarnings)
			{
				bool& bWasAlreadyLogged = RPCWarningThrottler.FindOrAdd(Context.Function->GetFName(), false);
				bLogRPCFailed = !bWasAlreadyLogged;
				bWasAlreadyLogged = true;
			}

			UE_CLOGF(bLogRPCFailed, LogIris, Warning, "SendRPC %ls for %ls::%ls Failed. The root object (RefHandle: %ls Index: %u) and subobject (RefHandle: %ls Index: %u) is not yet replicated.",
					ToCStr(Context.Function->GetName()), *GetNameSafe(Context.RootObject), *GetNameSafe(Context.SubObject), ToCStr(OutOwnerInfo.CallerRef.ToString()), OutOwnerInfo.RootObjectIndex, ToCStr(OutOwnerInfo.TargetRef.ToString()), OutOwnerInfo.SubObjectIndex);
		}
	}

	return bCanSendRpc;
}

bool FNetBlobManager::GetRootObjectIndicesFromHandle(FNetRefHandle RootObjectRefHandle, FInternalNetRefIndex& OutRootObjectIndex) const
{
	if (!RootObjectRefHandle.IsValid())
	{
		return false;
	}

	const FInternalNetRefIndex ObjectIndex = NetRefHandleManager->GetInternalIndex(RootObjectRefHandle);
	if (ObjectIndex == InvalidInternalNetRefIndex)
	{
		return false;
	}

	checkf(NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex).SubObjectRootIndex == InvalidInternalNetRefIndex, TEXT("Object %s (index:%u) (netref:%s) is not a rootobject"),
		*GetNameSafe(NetRefHandleManager->GetReplicatedObjectInstance(ObjectIndex)), ObjectIndex, ToCStr(RootObjectRefHandle.ToString()));

	OutRootObjectIndex = ObjectIndex;

	return true;
}

bool FNetBlobManager::GetRootObjectAndSubObjectIndicesFromSubObjectHandle(FNetRefHandle SubObjectRefHandle, FInternalNetRefIndex& OutRootObjectIndex, FInternalNetRefIndex& OutSubObjectIndex) const
{
	if (!SubObjectRefHandle.IsValid())
	{
		return false;
	}

	const FInternalNetRefIndex ObjectIndex = NetRefHandleManager->GetInternalIndex(SubObjectRefHandle);
	if (ObjectIndex == InvalidInternalNetRefIndex)
	{
		return false;
	}

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);

	checkf(ObjectData.SubObjectRootIndex != InvalidInternalNetRefIndex, TEXT("SubObject %s (index:%u) (netref:%s) does not have a rootobject"),
		*GetNameSafe(NetRefHandleManager->GetReplicatedObjectInstance(ObjectIndex)), ObjectIndex, ToCStr(SubObjectRefHandle.ToString()));

	OutRootObjectIndex = ObjectData.SubObjectRootIndex;
	OutSubObjectIndex = ObjectIndex;

	return OutRootObjectIndex != InvalidInternalNetRefIndex;
}

bool FNetBlobManager::GetRootObjectAndSubObjectIndicesFromAnyHandle(FNetRefHandle AnyRefHandle, FInternalNetRefIndex& OutRootObjectIndex, FInternalNetRefIndex& OutSubObjectIndex) const
{
	if (!AnyRefHandle.IsValid())
	{
		return false;
	}

	const FInternalNetRefIndex ObjectIndex = NetRefHandleManager->GetInternalIndex(AnyRefHandle);
	if (ObjectIndex == InvalidInternalNetRefIndex)
	{
		return false;
	}

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
	// If it's a sub object
	if (ObjectData.SubObjectRootIndex != InvalidInternalNetRefIndex)
	{
		OutRootObjectIndex = ObjectData.SubObjectRootIndex;
		OutSubObjectIndex = ObjectIndex;
	}
	// If it's a root object
	else
	{
		OutRootObjectIndex = ObjectIndex;
		OutSubObjectIndex = InvalidInternalNetRefIndex;
	}

	return true;
}

void FNetBlobManager::ProcessOOBNetObjectAttachmentSendQueue(FNetBitArray& OutConnetionsPendingImmediateSend)
{
	AttachmentSendQueue.PrepareAndProcessOOBAttachmentQueue(Connections, NetRefHandleManager, OutConnetionsPendingImmediateSend);
}

void FNetBlobManager::ProcessNetObjectAttachmentSendQueue(EProcessMode ProcessMode)
{
	AttachmentSendQueue.PrepareProcessQueue(Connections, NetRefHandleManager);
	AttachmentSendQueue.ProcessQueue(ProcessMode);
}

void FNetBlobManager::ResetNetObjectAttachmentSendQueue()
{
	AttachmentSendQueue.ResetProcessQueue();
}

void FNetBlobManager::AddConnection(uint32 ConnectionId)
{
	BlobHandlerManager.AddConnection(ConnectionId);
}

void FNetBlobManager::RemoveConnection(uint32 ConnectionId)
{
	BlobHandlerManager.RemoveConnection(ConnectionId);
}

void FNetBlobManager::RegisterDefaultHandlers()
{
	// NetRPCHandler
	{
		RPCHandler = TStrongObjectPtr<UNetRPCHandler>(NewObject<UNetRPCHandler>());
		RPCHandler->Init(*ReplicationSystem);
		if (!RegisterNetBlobHandler(RPCHandler.Get()))
		{
#if !WITH_AUTOMATION_WORKER
			checkf(false, TEXT("%s"), TEXT("Unable to register RPC handler. RPCs cannot be sent or received."));
#endif
			RPCHandler.Reset();
		}
	}

	// PartialNetObjectAttachmentHandler
	{
		PartialNetObjectAttachmentHandlerConfig = GetDefault<UPartialNetObjectAttachmentHandlerConfig>();
		FPartialNetObjectAttachmentHandlerInitParams InitParams = {};
		InitParams.ReplicationSystem  = ReplicationSystem;
		InitParams.Config = PartialNetObjectAttachmentHandlerConfig;
		PartialNetObjectAttachmentHandler = TStrongObjectPtr<UPartialNetObjectAttachmentHandler>(NewObject<UPartialNetObjectAttachmentHandler>());
		PartialNetObjectAttachmentHandler->Init(InitParams);
		if (!RegisterNetBlobHandler(PartialNetObjectAttachmentHandler.Get()))
		{
#if !WITH_AUTOMATION_WORKER
			checkf(false, TEXT("%s"), TEXT("Unable to register PartialNetObjectAttachment handler. Attachments cannot be split."));
#endif
			PartialNetObjectAttachmentHandler.Reset();
		}
	}

	// NetObjectBlobHandler
	{
		NetObjectBlobHandler = TStrongObjectPtr<UNetObjectBlobHandler>(NewObject<UNetObjectBlobHandler>());
		if (!RegisterNetBlobHandler(NetObjectBlobHandler.Get()))
		{
#if !WITH_AUTOMATION_WORKER
			checkf(false, TEXT("%s"), TEXT("Unable to register NetObjectBlobHandler handler. Replicated objects cannot be split."));
#endif
			NetObjectBlobHandler.Reset();
		}
	}
}

// FNetObjectAttachmentQueue
FNetBlobManager::FNetObjectAttachmentSendQueue::FNetObjectAttachmentSendQueue()
: Manager(nullptr)
, bHasMulticastAttachments(false)
{
}

void FNetBlobManager::FNetObjectAttachmentSendQueue::Init(FNetBlobManager* InManager)
{
	Manager = InManager;
}

void FNetBlobManager::FNetObjectAttachmentSendQueue::Enqueue(uint32 ConnectionId, FInternalNetRefIndex OwnerIndex, FInternalNetRefIndex SubObjectIndex, const TRefCountPtr<FNetObjectAttachment>& Attachment, ENetObjectAttachmentSendPolicyFlags SendFlags)
{
	const bool bScheduleUsingOOBAttachmentQueue = EnumHasAnyFlags(SendFlags, ENetObjectAttachmentSendPolicyFlags::ScheduleAsOOB);
	FQueue& TargetQueue = bScheduleUsingOOBAttachmentQueue ? ScheduleAsOOBAttachmentQueue : AttachmentQueue;

	FNetObjectAttachmentQueueEntry& QueueEntry = TargetQueue.AddDefaulted_GetRef();
	QueueEntry.ConnectionId = ConnectionId;
	QueueEntry.OwnerIndex = OwnerIndex;
	QueueEntry.SubObjectIndex = SubObjectIndex;
	QueueEntry.SendFlags = SendFlags;
	QueueEntry.Attachment = Attachment;
}

void FNetBlobManager::FNetObjectAttachmentSendQueue::Enqueue(FInternalNetRefIndex OwnerIndex, FInternalNetRefIndex SubObjectIndex, const TRefCountPtr<FNetObjectAttachment>& Attachment, ENetObjectAttachmentSendPolicyFlags SendFlags, FNetBitArray OpenConnections)
{
	const bool bScheduleUsingOOBAttachmentQueue = EnumHasAnyFlags(SendFlags, ENetObjectAttachmentSendPolicyFlags::ScheduleAsOOB);
	FQueue& TargetQueue = bScheduleUsingOOBAttachmentQueue ? ScheduleAsOOBAttachmentQueue : AttachmentQueue;

	FNetObjectAttachmentQueueEntry& QueueEntry = TargetQueue.AddDefaulted_GetRef();
	QueueEntry.ConnectionId = 0;
	QueueEntry.OwnerIndex = OwnerIndex;
	QueueEntry.SubObjectIndex = SubObjectIndex;
	QueueEntry.SendFlags = SendFlags;
	QueueEntry.Attachment = Attachment;
	QueueEntry.MulticastConnections = MoveTemp(OpenConnections);

	bHasMulticastAttachments = true;
}

void FNetBlobManager::FNetObjectAttachmentSendQueue::PrepareAndProcessOOBAttachmentQueue(FReplicationConnections* InConnections, const FNetRefHandleManager* InNetRefHandleManager, FNetBitArray& OutConnectionsPendingImmediateSend)
{
	check(!ProcessContext.IsValid());

	if (ProcessContext.IsValid())
	{
		return;
	}

	if (ScheduleAsOOBAttachmentQueue.Num() <= 0)
	{
		OutConnectionsPendingImmediateSend.ClearAllBits();
		return;
	}

	// Init context to process OOBAttachmentQueue
	ProcessContext.QueueToProcess = &ScheduleAsOOBAttachmentQueue;
	ProcessContext.Connections = InConnections;
	ProcessContext.NetRefHandleManager = InNetRefHandleManager;	
	ProcessContext.AttachmentsToObjectsInScope.Init(ScheduleAsOOBAttachmentQueue.Num());
	ProcessContext.ConnectionsPendingSendInPostDispatch.Init(InConnections->GetValidConnections().GetNumBits());

	if (bHasMulticastAttachments)
	{
		const FNetBitArray& ValidConnections = InConnections->GetValidConnections();
		const FNetBitArrayView ReplicatingConnections = MakeNetBitArrayView(ValidConnections);

		ProcessContext.ConnectionIds.SetNum(ReplicatingConnections.CountSetBits());
		if (!ProcessContext.ConnectionIds.IsEmpty())
		{
			ReplicatingConnections.GetSetBitIndices(0, ~0, ProcessContext.ConnectionIds.GetData(), ProcessContext.ConnectionIds.Num());
		}
	}

	uint32 CurrentEntryIndex = 0U;
	for (const FNetObjectAttachmentQueueEntry& Entry : MakeArrayView(ScheduleAsOOBAttachmentQueue))
	{
		ProcessContext.AttachmentsToObjectsInScope.SetBit(CurrentEntryIndex);
		++CurrentEntryIndex;
	}

	ProcessQueue(EProcessMode::ProcessObjectsInScope);

	OutConnectionsPendingImmediateSend.InitAndCopy(ProcessContext.ConnectionsPendingSendInPostDispatch);

	// Reset Context
	ProcessContext.Reset();
	ScheduleAsOOBAttachmentQueue.Reset();	
}

void FNetBlobManager::FNetObjectAttachmentSendQueue::PrepareProcessQueue(FReplicationConnections* InConnections, const FNetRefHandleManager* InNetRefHandleManager)
{
	if (ProcessContext.IsValid())
	{
		return;
	}

	// If we have entries in the ScheduleAsOOBAttachmentQueue that was not processed during PostTickDispatch (might have been posted after PostTickDispatch) we make sure to process them now.
	AttachmentQueue.Append(ScheduleAsOOBAttachmentQueue);
	ScheduleAsOOBAttachmentQueue.Reset();

	if (AttachmentQueue.Num() <= 0)
	{
		return;
	}

	// Init context
	ProcessContext.QueueToProcess = &AttachmentQueue;
	ProcessContext.Connections = InConnections;
	ProcessContext.NetRefHandleManager = InNetRefHandleManager;	
	ProcessContext.AttachmentsToObjectsGoingOutOfScope.Init(AttachmentQueue.Num());
	ProcessContext.AttachmentsToObjectsInScope.Init(AttachmentQueue.Num());
	ProcessContext.ConnectionsPendingSendInPostDispatch.Init(InConnections->GetValidConnections().GetNumBits());

	if (bHasMulticastAttachments)
	{
		const FNetBitArray& ValidConnections = InConnections->GetValidConnections();
		const FNetBitArrayView ReplicatingConnections = MakeNetBitArrayView(ValidConnections);

		ProcessContext.ConnectionIds.SetNum(ReplicatingConnections.CountSetBits());
		if (!ProcessContext.ConnectionIds.IsEmpty())
		{
			ReplicatingConnections.GetSetBitIndices(0, ~0, ProcessContext.ConnectionIds.GetData(), ProcessContext.ConnectionIds.Num());
		}
	}

	if (Manager->AllowObjectReplication())
	{
		// Figure out if we have any attachments to objects going out of scope.
		const FNetBitArrayView ScopableObjects = InNetRefHandleManager->GetCurrentFrameScopableInternalIndices();
		const FNetBitArrayView PrevScopableObjects = InNetRefHandleManager->GetPrevFrameScopableInternalIndices();

		uint32 CurrentEntryIndex = 0U;
		for (const FNetObjectAttachmentQueueEntry& Entry : MakeArrayView(AttachmentQueue))
		{
			const uint32 TargetInternalObjectIndex = Entry.SubObjectIndex != InvalidInternalNetRefIndex ? Entry.SubObjectIndex : Entry.OwnerIndex;

			const bool bIsAttachmentToObjectGoingOutOfScope = !ScopableObjects.GetBit(TargetInternalObjectIndex) && PrevScopableObjects.GetBit(TargetInternalObjectIndex);
			if (bIsAttachmentToObjectGoingOutOfScope)
			{
				ProcessContext.AttachmentsToObjectsGoingOutOfScope.SetBit(CurrentEntryIndex);
			}
			else
			{
				ProcessContext.AttachmentsToObjectsInScope.SetBit(CurrentEntryIndex);
			}
			++CurrentEntryIndex;
		}
	}
	else
	{
		uint32 CurrentEntryIndex = 0U;
		for (const FNetObjectAttachmentQueueEntry& Entry : MakeArrayView(AttachmentQueue))
		{
			ProcessContext.AttachmentsToObjectsInScope.SetBit(CurrentEntryIndex);
			++CurrentEntryIndex;
		}
	}
}

void FNetBlobManager::FNetObjectAttachmentSendQueue::ResetProcessQueue()
{
	// Clear queue
	AttachmentQueue.Reset();
	bHasMulticastAttachments = false;	
	ProcessContext.Reset();
}

bool FNetBlobManager::HasUnprocessedReliableAttachments(FInternalNetRefIndex InternalIndex) const
{
	return AttachmentSendQueue.HasUnprocessedReliableAttachments(InternalIndex);
}

bool FNetBlobManager::HasAnyUnprocessedReliableAttachments() const
{
	return AttachmentSendQueue.HasAnyUnprocessedReliableAttachments();
}

bool FNetBlobManager::FNetObjectAttachmentSendQueue::HasUnprocessedReliableAttachments(FInternalNetRefIndex InternalIndex)  const
{
	// For the moment we only need to check the AttachmentQueue as reliable attachments are not schedules as immediate.
	return AttachmentQueue.ContainsByPredicate([&InternalIndex](const FNetObjectAttachmentQueueEntry& Entry) { return (Entry.OwnerIndex == InternalIndex || Entry.SubObjectIndex == InternalIndex) && Entry.Attachment->IsReliable();} );
}

bool FNetBlobManager::FNetObjectAttachmentSendQueue::HasAnyUnprocessedReliableAttachments()  const
{
	// For the moment we only need to check the AttachmentQueue as reliable attachments are not schedules as immediate.
	return AttachmentQueue.ContainsByPredicate([](const FNetObjectAttachmentQueueEntry& Entry) { return Entry.Attachment->IsReliable(); });
}

void FNetBlobManager::FNetObjectAttachmentSendQueue::ProcessQueue(EProcessMode ProcessMode)
{
	if (!ProcessContext.IsValid())
	{
		return;
	}

	const FNetBitArray& IndicesToProcess = (ProcessMode == EProcessMode::ProcessObjectsGoingOutOfScope) ? ProcessContext.AttachmentsToObjectsGoingOutOfScope : ProcessContext.AttachmentsToObjectsInScope;

	TArray<TRefCountPtr<FNetBlob>> PartialNetBlobs;
	const FNetObjectAttachmentQueueEntry* Entries = ProcessContext.QueueToProcess->GetData();
	const uint32 NumEntries = ProcessContext.QueueToProcess->Num();

	// Verify that we have not missed to prepare the process context
	check(ProcessContext.IsValid() && IndicesToProcess.GetNumBits() == NumEntries);

	IndicesToProcess.ForAllSetBits([this, Entries, &PartialNetBlobs](uint32 Index)
	{
		const FNetObjectAttachmentQueueEntry& Entry = Entries[Index];

		PartialNetBlobs.Reset();

		const TRefCountPtr<FNetObjectAttachment>& Attachment = Entry.Attachment;
		const FReplicationStateDescriptor* ReplicationStateDescriptor = Attachment->GetReplicationStateDescriptor();

		const bool bMulticast = Entry.ConnectionId == 0;
		const bool bHasConnectionSpecificSerialization = ReplicationStateDescriptor && EnumHasAnyFlags(ReplicationStateDescriptor->Traits, EReplicationStateTraits::HasConnectionSpecificSerialization);
		const bool bSendInPostTickDispatch = EnumHasAnyFlags(Entry.SendFlags, ENetObjectAttachmentSendPolicyFlags::SendInPostTickDispatch);

		const bool bShouldSendAttachmentsWithObject = EnumHasAnyFlags(Entry.SendFlags, ENetObjectAttachmentSendPolicyFlags::ScheduleAsOOB) ? false : Manager->bSendAttachmentsWithObject;

		if (!(bMulticast && bHasConnectionSpecificSerialization) && !PreSerializeAndSplitNetBlob(Entry.ConnectionId, Attachment, PartialNetBlobs, bShouldSendAttachmentsWithObject))
		{
			checkf(false, TEXT("Unable to split %s NetObjectAttachment."), (EnumHasAnyFlags(Attachment->GetCreationInfo().Flags, ENetBlobFlags::Reliable) ? TEXT("reliable") : TEXT("unreliable")));
			return;
		}

		const bool bIsMultiPartAttachment = (PartialNetBlobs.Num() > 1);
		TArrayView<const TRefCountPtr<FNetBlob>> AttachmentsView = MakeArrayView(PartialNetBlobs);
		if (bMulticast)
		{
			if (bIsMultiPartAttachment)
			{
				UE_LOGF(LogIris, Warning, "Splitting multicast net object attachment %ls.", ReplicationStateDescriptor != nullptr && ReplicationStateDescriptor->DebugName ? ReplicationStateDescriptor->DebugName->Name : TEXT("Unknown"));
			}

			const FNetBlobCreationInfo& BlobCreationInfo = Attachment->GetCreationInfo();
			const bool bIsReliableRPC = EnumHasAnyFlags(BlobCreationInfo.Flags, ENetBlobFlags::Reliable);
			for (uint32 ConnectionId : MakeArrayView(ProcessContext.ConnectionIds))
			{
				// Objects won't be prioritized until there's a view so let's avoid queuing multicast attachments.
				if (!bIsReliableRPC && ProcessContext.Connections->GetReplicationView(ConnectionId).Views.Num() <= 0)
				{
					continue;
				}

				// Don't send RPCs to connections that were already closing when the RPC was called/queued
				if (!Entry.MulticastConnections.IsBitSet(ConnectionId))
				{
					continue;
				}

				if (bHasConnectionSpecificSerialization)
				{
					PartialNetBlobs.Reset();
					if (!PreSerializeAndSplitNetBlob(ConnectionId, Attachment, PartialNetBlobs, bShouldSendAttachmentsWithObject))
					{
						checkf(false, TEXT("Unable to split %s NetObjectAttachment with connection specific serialization."), (EnumHasAnyFlags(Attachment->GetCreationInfo().Flags, ENetBlobFlags::Reliable) ? TEXT("reliable") : TEXT("unreliable")));
						continue;
					}

					AttachmentsView = MakeArrayView(PartialNetBlobs);
				}

				FReplicationConnection* Connection = ProcessContext.Connections->GetConnection(ConnectionId);
				// We're only iterating over valid connections so the Connection pointer must be valid.
				const bool bWasEnqueued = Connection->ReplicationWriter->QueueNetObjectAttachments(Entry.OwnerIndex, Entry.SubObjectIndex, AttachmentsView, Entry.SendFlags);
				if (bWasEnqueued && bSendInPostTickDispatch)
				{
					ProcessContext.ConnectionsPendingSendInPostDispatch.SetBit(ConnectionId);
				}
			}
		}
		else
		{
			if (FReplicationConnection* Connection = ProcessContext.Connections->GetConnection(Entry.ConnectionId))
			{
				const bool bWasEnqueued = Connection->ReplicationWriter->QueueNetObjectAttachments(Entry.OwnerIndex, Entry.SubObjectIndex, AttachmentsView, Entry.SendFlags);
				if (bWasEnqueued && bSendInPostTickDispatch)
				{
					ProcessContext.ConnectionsPendingSendInPostDispatch.SetBit(Entry.ConnectionId);
				}	
			}
		}
	});
}

bool FNetBlobManager::FNetObjectAttachmentSendQueue::PreSerializeAndSplitNetBlob(uint32 ConnectionId, const TRefCountPtr<FNetObjectAttachment>& Attachment, TArray<TRefCountPtr<FNetBlob>>& OutPartialNetBlobs, bool bInSendAttachmentsWithObject) const
{
	if (!Manager->PartialNetObjectAttachmentHandler.IsValid())
	{
		OutPartialNetBlobs.Add(reinterpret_cast<const TRefCountPtr<FNetBlob>&>(Attachment));
		return true;
	}

	return Manager->PartialNetObjectAttachmentHandler->PreSerializeAndSplitNetBlob(ConnectionId, Attachment, OutPartialNetBlobs, bInSendAttachmentsWithObject);
}

}
