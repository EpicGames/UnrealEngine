// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"

#include "Iris/ReplicationSystem/InternalNetRefIndexManager.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"

#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandlerManager.h"

#include "Net/Core/NetBitArray.h"

#include "UObject/StrongObjectPtr.h"

class UNetBlobHandler;
class UNetObjectBlobHandler;
class UPartialNetObjectAttachmentHandler;
class UPartialNetObjectAttachmentHandlerConfig;
class UNetRPCHandler;

namespace UE::Net
{
	class FNetObjectReference;

	typedef uint32 FInternalNetRefIndex;

	namespace Private
	{
		class FNetRefHandleManager;
		class FObjectReferenceCache;
		class FReplicationConnections;
	}
}

namespace UE::Net::Private
{

struct FNetBlobManagerInitParams
{
	UReplicationSystem* ReplicationSystem = nullptr;
	bool bSendAttachmentsWithObject = false;
};

class FNetBlobManager
{
public:
	FNetBlobManager();
	
	void Init(FNetBlobManagerInitParams& InitParams);

	bool RegisterNetBlobHandler(UNetBlobHandler* Handler);

	bool AllowObjectReplication() const { return bAllowObjectReplication; }

	bool QueueNetObjectAttachment(uint32 ConnectionId, const FNetObjectReference& TargetRef, const TRefCountPtr<FNetObjectAttachment>& Attachment, ENetObjectAttachmentSendPolicyFlags SendFlags = ENetObjectAttachmentSendPolicyFlags::None);

	struct FSendRPCContext
	{
		const UObject* RootObject = nullptr;
		const UObject* SubObject = nullptr;
		const UFunction* Function = nullptr;
	};

	// Multicast RPC
	bool SendMulticastRPC(const FSendRPCContext& Context, const void* Parameters, ENetObjectAttachmentSendPolicyFlags SendFlags = ENetObjectAttachmentSendPolicyFlags::None);

	// Unicast RPC
	bool SendUnicastRPC(uint32 ConnectionId, const FSendRPCContext& Context, const void* Parameters, ENetObjectAttachmentSendPolicyFlags SendFlags = ENetObjectAttachmentSendPolicyFlags::None);

	bool HasUnprocessedReliableAttachments(FInternalNetRefIndex InternalIndex) const;
	bool HasAnyUnprocessedReliableAttachments() const;

	enum class EProcessMode 
	{
		ProcessObjectsGoingOutOfScope,
		ProcessObjectsInScope,
	};

	FNetBitArrayView GetConnectionsPendingImmediateSend() const;

	void ProcessOOBNetObjectAttachmentSendQueue(FNetBitArray& OutConnectionsPendingImmediateSend);
	void ProcessNetObjectAttachmentSendQueue(EProcessMode ProcessMode);
	void ResetNetObjectAttachmentSendQueue();

	FNetBlobHandlerManager& GetNetBlobHandlerManager() { return BlobHandlerManager; }
	const FNetBlobHandlerManager& GetNetBlobHandlerManager() const { return BlobHandlerManager; }

	const UPartialNetObjectAttachmentHandler* GetPartialNetObjectAttachmentHandler() const { return PartialNetObjectAttachmentHandler.Get(); }
	const UNetObjectBlobHandler* GetNetObjectBlobHandler() const { return NetObjectBlobHandler.Get(); }

	// Connection handling
	void AddConnection(uint32 ConnectionId);
	void RemoveConnection(uint32 ConnectionId);

private:

	void RegisterDefaultHandlers();

	struct FRPCOwner
	{
		// The replicated object responsible for carrying (sending) the RPC.
		FNetObjectReference CallerRef;

		// The object the RPC will be applied to.
		FNetObjectReference TargetRef;

		FInternalNetRefIndex RootObjectIndex = InvalidInternalNetRefIndex;
		FInternalNetRefIndex SubObjectIndex = InvalidInternalNetRefIndex;
	};
	bool GetRPCOwner(FRPCOwner& OutOwnerInfo, const FSendRPCContext& Context) const;

	/** 
	* Validates that RootObjectRefHandle is a true root object and return it's index
	* @return True if the root object handle is valid and has been replicated.
	*/
	bool GetRootObjectIndicesFromHandle(FNetRefHandle RootObjectRefHandle, FInternalNetRefIndex& OutRootObjectIndex) const;

	/**
	* Validates that the SubObjectRefHandle is a true subobject and return it's index and the index of it's root object.
	* @return True if the subobject handle is valid and has been replicated.
	*/
	bool GetRootObjectAndSubObjectIndicesFromSubObjectHandle(FNetRefHandle SubObjectRefHandle, FInternalNetRefIndex& OutRootObjectIndex, FInternalNetRefIndex& OutSubObjectIndex) const;

	/**
	 * Receives the handle of a root object or a sub object and returns the index of the root object and subobject if there is one.
	 * @return True if the object handle is valid has been replicated.
	 */
	bool GetRootObjectAndSubObjectIndicesFromAnyHandle(FNetRefHandle AnyRefHandle, FInternalNetRefIndex& OutRootObjectIndex, FInternalNetRefIndex& OutSubObjectIndex) const;

	class FNetObjectAttachmentSendQueue
	{
	public:
		FNetObjectAttachmentSendQueue();

		void Init(FNetBlobManager* Manager);

		// Unicast
		void Enqueue(uint32 ConnectionId, FInternalNetRefIndex OwnerIndex, FInternalNetRefIndex SubObjectIndex, const TRefCountPtr<FNetObjectAttachment>& Attachment, ENetObjectAttachmentSendPolicyFlags SendFlags);

		// Multicast
		void Enqueue(FInternalNetRefIndex OwnerIndex, FInternalNetRefIndex SubObjectIndex, const TRefCountPtr<FNetObjectAttachment>& Attachment, ENetObjectAttachmentSendPolicyFlags SendFlags, FNetBitArray OpenConnections);

		void PrepareProcessQueue(FReplicationConnections* InConnections, const FNetRefHandleManager* InNetRefHandleManager);
		void ProcessQueue(EProcessMode ProcessMode);
		void ResetProcessQueue();
		void PrepareAndProcessOOBAttachmentQueue(FReplicationConnections* InConnections, const FNetRefHandleManager* InNetRefHandleManager, FNetBitArray& OutConnetionsPendingImmediateSend);

		bool HasUnprocessedReliableAttachments(FInternalNetRefIndex InternalIndex)  const;
		bool HasAnyUnprocessedReliableAttachments()  const;
	
	private:
		struct FNetObjectAttachmentQueueEntry
		{
			uint32 ConnectionId;
			FInternalNetRefIndex OwnerIndex;
			FInternalNetRefIndex SubObjectIndex;
			ENetObjectAttachmentSendPolicyFlags SendFlags;
			TRefCountPtr<FNetObjectAttachment> Attachment;
			FNetBitArray MulticastConnections;
		};
		typedef TArray<FNetObjectAttachmentQueueEntry> FQueue;

		bool PreSerializeAndSplitNetBlob(uint32 ConnectionId, const TRefCountPtr<FNetObjectAttachment>& Attachment, TArray<TRefCountPtr<FNetBlob>>& OutPartialNetBlobs, bool bInSendAttachmentsWithObject) const;

		FNetBlobManager* Manager;
		FQueue AttachmentQueue;
		FQueue ScheduleAsOOBAttachmentQueue;		
		bool bHasMulticastAttachments;

		struct FProcessQueueContext
		{
			FNetBitArray AttachmentsToObjectsGoingOutOfScope;
			FNetBitArray AttachmentsToObjectsInScope;
			FNetBitArray ConnectionsPendingSendInPostDispatch;

			TArray<uint32> ConnectionIds;
			FReplicationConnections* Connections = nullptr;
			const FNetRefHandleManager* NetRefHandleManager = nullptr;
			FQueue* QueueToProcess = nullptr;

			void Reset()
			{
				Connections = nullptr;
				NetRefHandleManager = nullptr;
				QueueToProcess = nullptr;
				ConnectionsPendingSendInPostDispatch.ClearAllBits();
			}

			bool IsValid() const { return NetRefHandleManager != nullptr; }
		};
		FProcessQueueContext ProcessContext;
	};


	FNetBlobHandlerManager BlobHandlerManager;
	FNetObjectAttachmentSendQueue AttachmentSendQueue;
	TStrongObjectPtr<UNetRPCHandler> RPCHandler;
	TStrongObjectPtr<UPartialNetObjectAttachmentHandler> PartialNetObjectAttachmentHandler;
	TStrongObjectPtr<UNetObjectBlobHandler> NetObjectBlobHandler;

	/** Track if a warning was already logged for a specific RPC. */
	mutable TMap<FName, bool> RPCWarningThrottler;

	UReplicationSystem* ReplicationSystem = nullptr;
	FObjectReferenceCache* ObjectReferenceCache = nullptr;
	FReplicationConnections* Connections = nullptr;
	const UPartialNetObjectAttachmentHandlerConfig* PartialNetObjectAttachmentHandlerConfig = nullptr;
	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	bool bIsServer = false;
	bool bSendAttachmentsWithObject = false;
	bool bAllowObjectReplication = false;
};

}
