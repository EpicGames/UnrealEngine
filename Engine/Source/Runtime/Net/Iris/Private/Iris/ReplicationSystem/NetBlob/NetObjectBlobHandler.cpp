// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/NetObjectBlobHandler.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetObjectBlobHandler)

namespace UE::Net::Private
{

FNetObjectBlob::FNetObjectBlob(const UE::Net::FNetBlobCreationInfo& CreationInfo)
: FRawDataNetBlob(CreationInfo)
{
}

void FNetObjectBlob::SerializeHeader(FNetSerializationContext& Context, const FHeader& Header)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(Header.ObjectCount, 16U);
}

void FNetObjectBlob::DeserializeHeader(FNetSerializationContext& Context, FHeader& OutHeader)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	OutHeader.ObjectCount = Reader->ReadBits(16U);
}

}

UNetObjectBlobHandler::UNetObjectBlobHandler()
: UNetBlobHandler()
{
}

UNetObjectBlobHandler::~UNetObjectBlobHandler()
{
}

TRefCountPtr<UE::Net::Private::FNetObjectBlob> UNetObjectBlobHandler::CreateNetObjectBlob(const TArrayView<const uint32> RawData, uint32 RawDataBitCount) const
{
	using namespace UE::Net;

	FNetBlobCreationInfo CreationInfo;
	CreationInfo.Type = GetNetBlobType();
	CreationInfo.Flags = ENetBlobFlags::Reliable;
	FNetObjectBlob* Blob = new FNetObjectBlob(CreationInfo);
	Blob->SetRawData(RawData, RawDataBitCount);
	return Blob;
}

TRefCountPtr<UE::Net::FNetBlob> UNetObjectBlobHandler::CreateNetBlob(const FNetBlobCreationInfo& CreationInfo) const
{
	FNetObjectBlob* Blob = new FNetObjectBlob(CreationInfo);
	return Blob;
}

void UNetObjectBlobHandler::OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>&)
{
	ensureMsgf(false, TEXT("%s"), TEXT("NetObjectBlobHandler expects the blobs to be assembled and deserialized using a special path. This function is not expected to be called."));
	Context.SetError(UE::Net::GNetError_UnsupportedNetBlob);
}
