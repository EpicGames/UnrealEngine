// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandler.h"
#include "Iris/ReplicationSystem/NetBlob/RawDataNetBlob.h"
#include "NetObjectBlobHandler.generated.h"

namespace UE::Net::Private
{

class FNetObjectBlob : public FRawDataNetBlob
{
public:
	struct FHeader
	{
		uint32 ObjectCount;
	};

	FNetObjectBlob(const UE::Net::FNetBlobCreationInfo&);
	
	static void SerializeHeader(FNetSerializationContext& Context, const FHeader& Header);
	static void DeserializeHeader(FNetSerializationContext& Context, FHeader& OutHeader);
};

}

/**
 * NetBlobHandler used for huge replicated objects. This blob will be split into PartialNetBlobs.
 */
UCLASS(transient, MinimalAPI)
class UNetObjectBlobHandler final : public UNetBlobHandler
{
	GENERATED_BODY()

	using FNetObjectBlob = UE::Net::Private::FNetObjectBlob;

public:
	UNetObjectBlobHandler();
	virtual ~UNetObjectBlobHandler();

	TRefCountPtr<FNetObjectBlob> CreateNetObjectBlob(const TArrayView<const uint32> RawData, uint32 RawDataBitCount) const;

private:
	virtual TRefCountPtr<FNetBlob> CreateNetBlob(const FNetBlobCreationInfo&) const override;
	virtual void OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>&) override;
};
