// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/NetRPC.h"

#include "HAL/IConsoleManager.h"
#include "Net/Core/Connection/RPCDoSDetection.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/Core/Misc/NetContext.h"

#include "Iris/Core/BitTwiddling.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"

#include "Iris/ReplicationSystem/NetBlob/NetRPCHandler.h"

#include "Iris/ReplicationSystem/InternalNetRefIndexManager.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"

#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/ObjectNetSerializer.h"

#include "Containers/ArrayView.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "UObject/CoreNetContext.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/RemoteExecutor.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogIrisRpc, Log, All);

namespace UE::Net::Private
{

static bool bEnableRPCServerStateStompDetection = false;
static FAutoConsoleVariableRef CVarEnableRPCStateStompDetection(
	TEXT("net.Iris.RPC.ServerStompDetectionEnabled"),
	bEnableRPCServerStateStompDetection,
	TEXT("If enabled we can detect if RPC state is stomped between quantization and serialization on a dedicated server.")
);

static bool NetRPC_GetFunctionLocator(const UReplicationSystem* ReplicationSystem, const FNetObjectReference& ObjectReference, const UFunction* Function, FNetRPC::FFunctionLocator& OutFunctionLocator, const FReplicationStateMemberFunctionDescriptor*& OutFunctionDescriptor);

static UObject* NetRPC_GetObject(FNetSerializationContext& Context, const FNetObjectReference& ObjectReference);
static UObject* NetRPC_GetObject(FNetSerializationContext& Context, const FNetObjectReference& RootObjectReference, const FNetObjectReference& SubObjectReference);
static UObject* NetRPC_GetRootObject(FNetSerializationContext& Context, const FNetObjectReference& ObjectReference);

static bool NetRPC_GetFunctionAndObject(FNetSerializationContext& Context, const FNetObjectReference& RootObjectReference, const FNetObjectReference& SubObjectReferece, const FNetRPC::FFunctionLocator& FunctionLocator, const FReplicationStateMemberFunctionDescriptor*& OutFunctionDescriptor, TWeakObjectPtr<UObject>& OutObject);

static FString NetRPC_GetDebugObjectRefName(FNetSerializationContext& Context, const FNetObjectReference& ObjectReference, UObject* ObjectPtr = nullptr);

static const FName NetError_InvalidNetObjectReference("Invalid NetObjectRefererence");
static const FName NetError_UnknownFunction("Unknown RPC");
static const FName NetError_FunctionCallNotAllowed("RPC call not allowed");

FNetRPC::FNetRPC(const FNetBlobCreationInfo& CreationInfo)
: FNetObjectAttachment(CreationInfo)
, FunctionLocator({})
, Function(nullptr)
{
}

FNetRPC::~FNetRPC()
{
}

TArrayView<const FNetObjectReference> FNetRPC::GetNetObjectReferenceExports() const
{
	return ReferencesToExport.IsValid() ? MakeArrayView(*ReferencesToExport) : MakeArrayView<const FNetObjectReference>(nullptr, 0);
}

void FNetRPC::SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const
{
	UE_NET_TRACE_DYNAMIC_NAME_SCOPE(BlobDescriptor->DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	const uint32 HeaderPos = Writer.GetPosBits();
	InternalSerializeHeader(Context);
	if (Context.HasError())
	{
		return;
	}

	SerializeFunctionLocator(Context);
	if (Context.HasError())
	{
		return;
	}

	InternalSerializeSubObjectReference(Context, RefHandle);
	if (Context.HasError())
	{
		return;
	}

	InternalSerializeBlob(Context);
	if (Context.HasError())
	{
		return;
	}

	if (!Writer.IsOverflown())
	{
		// Re-serialize the final size value in the header
		const uint32 RPCSize = Writer.GetPosBits() - HeaderPos;
		FNetBitStreamWriteScope WriteScope(Writer, HeaderPos);
		InternalSerializeHeader(Context, RPCSize);
	}
}

void FNetRPC::DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	// We don't know the function name until much later.
	UE_NET_TRACE_NAMED_SCOPE(TraceScope, RPC, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	// Store the next valid position after the NetRPC data.
	const uint32 HeaderPos = Context.GetBitStreamReader()->GetPosBits();
	const uint32 PostNetRPCPos = HeaderPos + InternalDeserializeHeader(Context);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	DeserializeFunctionLocator(Context);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	InternalDeserializeSubObjectReference(Context, RefHandle);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	bool bResolveSucceeded = ResolveFunctionAndObject(Context);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}
	
	if (!bResolveSucceeded)
	{
		UE_LOGF(LogIrisRpc, Error, "DeserializeWithObject::Skipping RPC due missing object or function.");

		// Stop deserializing and seek past the entire payload if the resolve failed
		Context.GetBitStreamReader()->Seek(PostNetRPCPos);
		return;
	}

	UE_NET_TRACE_SET_SCOPE_NAME(TraceScope, BlobDescriptor->DebugName);
	InternalDeserializeBlob(Context);
	
	if (!Context.HasErrorOrOverflow())
	{
		// Just because the serialization didn't detect an error doesn't mean everything is ok. Validate stream position.
		if (PostNetRPCPos != Context.GetBitStreamReader()->GetPosBits())
		{
			UE_LOGF(LogIrisRpc, Error, "Bitstream mismatch while deserializing function %ls. Actual stream position: %u Expected stream position: %u", ToCStr(BlobDescriptor->DebugName), Context.GetBitStreamReader()->GetPosBits(), PostNetRPCPos);
			ensureMsgf(PostNetRPCPos == Context.GetBitStreamReader()->GetPosBits(), TEXT("Bitstream mismatch while deserializing function %s. Actual stream position: %u Expected stream position: %u"), ToCStr(BlobDescriptor->DebugName), Context.GetBitStreamReader()->GetPosBits(), PostNetRPCPos);
			Context.GetBitStreamReader()->Seek(PostNetRPCPos);
			Context.SetError(GNetError_BitStreamError);
			// Make sure the RPC won't be exeuted regardless of how errors are handled.
			Function = nullptr;
		}
	}
}

void FNetRPC::Serialize(FNetSerializationContext& Context) const
{
	UE_NET_TRACE_DYNAMIC_NAME_SCOPE(BlobDescriptor->DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();
	
	const uint32 HeaderPos = Writer.GetPosBits();
	InternalSerializeHeader(Context);
	if (Context.HasError())
	{
		return;
	}

	InternalSerializeObjectReference(Context);
	if (Context.HasError())
	{
		return;
	}

	SerializeFunctionLocator(Context);
	if (Context.HasError())
	{
		return;
	}

	InternalSerializeBlob(Context);
	if (Context.HasError())
	{
		return;
	}

	if (!Writer.IsOverflown())
	{
		// Re-serialize the final size value in the header
		const uint32 RPCSize = Writer.GetPosBits() - HeaderPos;
		FNetBitStreamWriteScope WriteScope(Writer, HeaderPos);
		InternalSerializeHeader(Context, RPCSize);
	}
}

void FNetRPC::Deserialize(FNetSerializationContext& Context)
{
	UE_NET_TRACE_NAMED_SCOPE(TraceScope, RPC, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	// Store the next valid bit position after the NetRPC data.
	const uint32 HeaderPos = Context.GetBitStreamReader()->GetPosBits();
	const uint32 PostNetRPCPos = HeaderPos + InternalDeserializeHeader(Context);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	InternalDeserializeObjectReference(Context);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	DeserializeFunctionLocator(Context);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}
	
	bool bResolveSucceeded = ResolveFunctionAndObject(Context);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	if (!bResolveSucceeded)
	{
		UE_LOGF(LogIrisRpc, Error, "DeserializeWithObject::Skipping RPC due missing object or function.");
		// Stop deserializing and seek past the entire payload when the Resolve fails
		Context.GetBitStreamReader()->Seek(PostNetRPCPos);
		return;
	}
	
	UE_NET_TRACE_SET_SCOPE_NAME(TraceScope, BlobDescriptor->DebugName);

	InternalDeserializeBlob(Context);
	
	if (!Context.HasErrorOrOverflow())
	{
		if (PostNetRPCPos != Context.GetBitStreamReader()->GetPosBits())
		{
			UE_LOGF(LogIrisRpc, Error, "DeserializeWithObject::RPC %ls did not read expected number of bits ErrorContext: %ls", ToCStr(BlobDescriptor->DebugName), *Context.PrintReadJournal())
			ensure(PostNetRPCPos == Context.GetBitStreamReader()->GetPosBits());
		}
	}
}

void FNetRPC::InternalSerializeHeader(FNetSerializationContext& Context, int32 PayloadSize /*= -1*/) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// Pre-serialize the header before we know the final RPC payload size
	if (PayloadSize == -1)
	{
		UE_NET_TRACE_SCOPE(RPCSize, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		Writer->WriteBits(0u, HeaderSizeBitCount);
	}
	// Write the final size once available
	else
	{
		checkf(PayloadSize > 0 && PayloadSize <= MaxRpcSizeInBits, TEXT("FNetRPC can only support a payload size of %u bits (RPC %s cost %u bits)"), MaxRpcSizeInBits, ToCStr(Function->GetName()), PayloadSize);
		Writer->WriteBits(PayloadSize, HeaderSizeBitCount);
	}
}

uint32 FNetRPC::InternalDeserializeHeader(FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	UE_NET_TRACE_SCOPE(RPCSize, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	return Reader->ReadBits(HeaderSizeBitCount);
}

void FNetRPC::SerializeFunctionLocator(FNetSerializationContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	UE_NET_TRACE_SCOPE(FunctionLocator, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	const uint16 MaxValue = FMath::Max(FunctionLocator.DescriptorIndex, FunctionLocator.FunctionIndex);
	const uint32 NibbleCount = (GetBitsNeeded(MaxValue | uint16(1)) + 3) >> 2U;

	Writer->WriteBits(NibbleCount - 1U, 2U);
	Writer->WriteBits(FunctionLocator.DescriptorIndex, NibbleCount*4U);
	Writer->WriteBits(FunctionLocator.FunctionIndex, NibbleCount*4U);
}

void FNetRPC::DeserializeFunctionLocator(FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	UE_NET_TRACE_SCOPE(FunctionLocator, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	const uint32 NibbleCount = Reader->ReadBits(2U) + 1U;
	FunctionLocator.DescriptorIndex = static_cast<uint16>(Reader->ReadBits(NibbleCount*4U));
	FunctionLocator.FunctionIndex = static_cast<uint16>(Reader->ReadBits(NibbleCount*4U));
}

void FNetRPC::InternalSerializeObjectReference(FNetSerializationContext& Context) const
{
	UE_NET_TRACE_SCOPE(TargetObject, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	SerializeObjectReference(Context);
}

void FNetRPC::InternalDeserializeObjectReference(FNetSerializationContext& Context)
{
	UE_NET_TRACE_SCOPE(TargetObject, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	DeserializeObjectReference(Context);
}

void FNetRPC::InternalSerializeSubObjectReference(FNetSerializationContext& Context, FNetRefHandle RefHandle) const
{
	UE_NET_TRACE_SCOPE(TargetObject, *(Context.GetBitStreamWriter()), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	SerializeSubObjectReference(Context, RefHandle);
}

void FNetRPC::InternalDeserializeSubObjectReference(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	UE_NET_TRACE_SCOPE(TargetObject, *(Context.GetBitStreamReader()), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	DeserializeSubObjectReference(Context, RefHandle);
}

void FNetRPC::InternalSerializeBlob(FNetSerializationContext& Context) const
{
	UE_NET_TRACE_SCOPE(FunctionParams, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	SerializeBlob(Context);
}

void FNetRPC::InternalDeserializeBlob(FNetSerializationContext& Context)
{
	UE_NET_TRACE_SCOPE(FunctionParams, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	DeserializeBlob(Context);
}

bool FNetRPC::ResolveFunctionAndObject(FNetSerializationContext& Context)
{
	// At this point we need a valid handle and FunctionLocator
	if (!NetObjectReference.GetRefHandle().IsValid())
	{
		// This can occur if sending side had queued up rpcs to object being invalidated
		UE_LOGF(LogIrisRpc, Verbose, "%s FAILED (invalid handle): NetRef=%ls TargetRef=%ls FuncLocator=(%u:%u)",
			__FUNCTION__, *NetObjectReference.ToString(), *TargetObjectReference.ToString(), FunctionLocator.DescriptorIndex, FunctionLocator.FunctionIndex);
		return false;
	}

	const FReplicationStateMemberFunctionDescriptor* FunctionDescriptor = nullptr;
	if (!NetRPC_GetFunctionAndObject(Context, NetObjectReference, TargetObjectReference, FunctionLocator, FunctionDescriptor, ObjectPtr))
	{
		UObject* RefObject = NetRPC_GetObject(Context, NetObjectReference, TargetObjectReference);
		UE_LOGF(LogIrisRpc, Warning, "%s FAILED (GetFunctionAndObject): NetRef=%ls TargetRef=%ls ResolvedObject=%ls FuncLocator=(%u:%u)",
			__FUNCTION__, *NetObjectReference.ToString(), *TargetObjectReference.ToString(), *GetNameSafe(RefObject),
			FunctionLocator.DescriptorIndex, FunctionLocator.FunctionIndex);
		return false;
	}

	Function = FunctionDescriptor->Function;

	// Patch up NetBlobFlags based on function flags.
	if (Function)
	{
		if ((Function->FunctionFlags & FUNC_NetReliable) != 0)
		{
			CreationInfo.Flags |= ENetBlobFlags::Reliable;
		}

		// The sending side will set Ordered on unicast/reliable RPCs so we're restoring that flag. Unicast RPCs are ordered with respect to other reliable and unicast RPCs whereas multicast RPCs are not.
		if ((Function->FunctionFlags & FUNC_NetMulticast) == 0)
		{
			CreationInfo.Flags |= UE::Net::ENetBlobFlags::Ordered;
		}
	}

	// Set the BlobDescriptor even if it has zero size so that we can trace with a meaningful name.
	BlobDescriptor = FunctionDescriptor->Descriptor;

	if (FunctionDescriptor->Descriptor->InternalSize)
	{
		QuantizedBlobState = FQuantizedBlobState(FunctionDescriptor->Descriptor->InternalSize, FunctionDescriptor->Descriptor->InternalAlignment);
	}

	return true;
}

FNetRPC* FNetRPC::Create(UReplicationSystem* ReplicationSystem, const FNetBlobCreationInfo& CreationInfo, const FNetObjectReference& ObjectReference, const UFunction* Function, const void* FunctionParameters)
{
	while (const UFunction* SuperFunction = Function->GetSuperFunction())
	{
		Function = SuperFunction;
	};

	FFunctionLocator FunctionLocator = {};
	const FReplicationStateMemberFunctionDescriptor* FunctionDescriptor = nullptr;
	if (!NetRPC_GetFunctionLocator(ReplicationSystem, ObjectReference, Function, FunctionLocator, FunctionDescriptor))
	{
		return nullptr;
	}

	const FReplicationStateDescriptor* BlobDescriptor = FunctionDescriptor->Descriptor;
	FQuantizedBlobState QuantizedBlobState;

	// Don't spend CPU cycles on quantizing zero parameters
	if (BlobDescriptor != nullptr && BlobDescriptor->InternalSize)
	{
#if WITH_SERVER_CODE
		const FQuantizedBlobState::EMemoryAllocationFlags MemoryAllocationFlags = bEnableRPCServerStateStompDetection ? FQuantizedBlobState::EMemoryAllocationFlags::Protectable : FQuantizedBlobState::EMemoryAllocationFlags::None;
#else
		const FQuantizedBlobState::EMemoryAllocationFlags MemoryAllocationFlags = FQuantizedBlobState::EMemoryAllocationFlags::None;
#endif
		QuantizedBlobState = FQuantizedBlobState(FunctionDescriptor->Descriptor->InternalSize, FunctionDescriptor->Descriptor->InternalAlignment, MemoryAllocationFlags);

		// Setup Context
		FNetSerializationContext Context;
		FInternalNetSerializationContext InternalContext(ReplicationSystem);
		
		Context.SetInternalContext(&InternalContext);
		
		// The proxy uses a single, shared object reference cache and can contain references from multiple game servers, which means it can differ from
		// the object reference caches of the individual connected clients. For this reason, any RPC arguments forwarded from a client to a game server
		// through the proxy that are static object references should always use objects paths. This avoids the proxy generating a new object reference
		// using a FNetRefHandle from the proxy's object reference cache when it doesn't exist on the client (the game server expects it's clients to have
		// an object reference cache that is a subset of it's own cache).
		if (ReplicationSystem->GetProxyType() == EProxyType::Backend)
		{
			InternalContext.bForceObjectPathSerialization = 1;
		}

		// Quantize the function parameters
		FReplicationStateOperations::Quantize(Context, QuantizedBlobState.GetStateBuffer(), static_cast<const uint8*>(FunctionParameters), BlobDescriptor);

#if WITH_SERVER_CODE
		if (MemoryAllocationFlags == FQuantizedBlobState::EMemoryAllocationFlags::Protectable)
		{
			QuantizedBlobState.Protect();
		}
#endif
	}

	FNetRPC* NetRPC = new FNetRPC(CreationInfo);
	NetRPC->SetFunctionLocator(FunctionLocator);
	NetRPC->Function = Function;
	if (BlobDescriptor != nullptr)
	{
		if (BlobDescriptor->HasObjectReference())
		{
			// Collect all references and add them to potential exports
			using namespace UE::Net::Private;
			{
				FNetSerializationContext LocalContext;
				FNetReferenceCollector Collector(ENetReferenceCollectorTraits::OnlyCollectReferencesThatCanBeExported);
				const FNetSerializerChangeMaskParam InitStateChangeMaskInfo = { 0 };
				FReplicationStateOperationsInternal::CollectReferences(LocalContext, Collector, InitStateChangeMaskInfo, QuantizedBlobState.GetStateBuffer(), BlobDescriptor);

				if (Collector.GetCollectedReferences().Num())
				{
					NetRPC->ReferencesToExport = MakeUnique<FNetRPCExportsArray>();
					for (const FNetReferenceCollector::FReferenceInfo& Info : MakeArrayView(Collector.GetCollectedReferences()))
					{
						NetRPC->ReferencesToExport->AddUnique(Info.Reference);
					}

					NetRPC->CreationInfo.Flags |= ENetBlobFlags::HasExports;
				}
			}
		}

		NetRPC->SetState(BlobDescriptor, MoveTemp(QuantizedBlobState));
	}

	return NetRPC;
}

class FNetRPCDoSScope
{
public:
	FNetRPCDoSScope(FRPCDoSDetection* InRPCDoS)
		: RPCDoS(InRPCDoS)
	{
	}

	~FNetRPCDoSScope()
	{
		if (bEnabled && RPCDoS != nullptr)
		{
			RPCDoS->PostReceivedRPC();
		}
	}

	void SetEnabled(bool bInEnabled)
	{
		bEnabled = bInEnabled;
	}

private:
	FRPCDoSDetection* RPCDoS = nullptr;

	bool bEnabled = false;
};

void FNetRPC::CallFunction(FNetRPCCallContext& CallContext)
{
#if UE_NET_IRIS_CSV_STATS
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(HandleRPC);
#endif

	FNetSerializationContext& Context = CallContext.GetNetSerializationContext();

	const UReplicationSystem* ReplicationSystem = Context.GetInternalContext()->ReplicationSystem;

	// Check whether we are ok with calling the function
	UObject* Object = ObjectPtr.Get();

	if (Object == nullptr)
	{
		Object = NetRPC_GetObject(Context, NetObjectReference, TargetObjectReference);
	}

	if (Object == nullptr || Function == nullptr)
	{
		const FString DebugFuncName = Function ? Function->GetName() : FString("unknown");
		if (!TargetObjectReference.IsValid())
		{
			UE_LOGF(LogIrisRpc, Error, "Rejected RPC (%u:%u) '%ls' due to missing object or function for object: %ls.",
				FunctionLocator.DescriptorIndex, FunctionLocator.FunctionIndex,
				*DebugFuncName,
				*NetRPC_GetDebugObjectRefName(Context, NetObjectReference, Object));
		}
		else
		{
			UE_LOGF(LogIrisRpc, Error, "Rejected RPC (%u:%u) '%ls' due to missing object or function for subobject: %ls of rootobject: %ls",
				FunctionLocator.DescriptorIndex, FunctionLocator.FunctionIndex,
				*DebugFuncName,
				*NetRPC_GetDebugObjectRefName(Context, TargetObjectReference, Object), *NetRPC_GetDebugObjectRefName(Context, NetObjectReference));
		}
		return;
	}

	// Make sure the root object has an instance protocol
	{
		const FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
		const FNetRefHandle RootObjectNetRefHandle = NetObjectReference.GetRefHandle();
		const FInternalNetRefIndex InternalIndex = NetRefHandleManager.GetInternalIndex(RootObjectNetRefHandle);

		ensureMsgf(InternalIndex != InvalidInternalNetRefIndex, TEXT("Processing NetRPC for an object before it's internal index was assigned. NetRefHandle: %s | Instance: %s"),
			*RootObjectNetRefHandle.ToString(), *GetNameSafe(Object));

		// If the object stopped replicating locally, discard all received RPCs so they don't get executed.
		if (!NetRefHandleManager.HasInstanceProtocol(InternalIndex))
		{
			UE_LOGF(LogIrisRpc, Warning, "Rejected RPC: %ls due to object: %ls having stopped replication",
				*(Function ? Function->GetName() : FString("unknown")),
				*NetRPC_GetDebugObjectRefName(Context, NetObjectReference, Object));

			return;
		}
	}

	if ((Function->FunctionFlags & FUNC_Net) == 0)
	{
		UE_LOGF(LogIrisRpc, Error, "Rejected %ls function %ls due to it not being a Net function. %ls : %ls", (Function->FunctionFlags & FUNC_NetReliable ? TEXT("reliable") : TEXT("unreliable")), ToCStr(Function->GetName()), *NetObjectReference.ToString(), *Object->GetFullName());
		Context.SetError(NetError_FunctionCallNotAllowed);
		return;
	}

	const bool bIsServer = ReplicationSystem->IsServer();
	if (bIsServer)
	{
		if ((Function->FunctionFlags & (FUNC_NetClient | FUNC_NetMulticast)) != 0)
		{
			UE_LOGF(LogIrisRpc, Error, "Rejected %ls client RPC function %ls due to this being the server. %ls : %ls", (Function->FunctionFlags & FUNC_NetReliable ? TEXT("reliable") : TEXT("unreliable")), ToCStr(Function->GetName()), *NetObjectReference.ToString(), *Object->GetFullName());
			Context.SetError(NetError_FunctionCallNotAllowed);
			return;
		}

		if (!IsServerAllowedToExecuteRPC(Context))
		{
			UE_LOGF(LogIrisRpc, Verbose, "Rejected %ls RPC function %ls due to target not being owned by the connection. %ls : %ls", (Function->FunctionFlags & FUNC_NetReliable ? TEXT("reliable") : TEXT("unreliable")), ToCStr(Function->GetName()), *NetObjectReference.ToString(), *Object->GetFullName()); 
			return;
		}

	}
	else
	{
		if ((Function->FunctionFlags & FUNC_NetServer) != 0)
		{
			UE_LOGF(LogIrisRpc, Error, "Rejected %ls server RPC function %ls due to this being the client. %ls : %ls", (Function->FunctionFlags & FUNC_NetReliable ? TEXT("reliable") : TEXT("unreliable")), ToCStr(Function->GetName()), *NetObjectReference.ToString(), *Object->GetFullName());
			return;
		}
	}

	if (UE_LOG_ACTIVE(LogIrisRpc, Verbose))
	{
		bool bLogRpc = true;

		// Suppress spammy engine RPCs. This could be made a configurable list in the future.
		const FString& FunctionName = Function->GetName();
		if (   FunctionName.Contains(TEXT("ServerUpdateCamera"))
			|| FunctionName.Contains(TEXT("ClientAckGoodMove"))
			|| FunctionName.Contains(TEXT("ServerMove")))
		{
			bLogRpc = false;
		}
		
		UE_CLOGF(bLogRpc, LogIrisRpc, Verbose, "Calling %s RPC function %ls for %ls : %ls", (Function->FunctionFlags & FUNC_NetReliable ? "reliable" : "unreliable"), ToCStr(Function->GetName()), *NetObjectReference.ToString(), *Object->GetFullName());
	}

	FRPCDoSDetection* RPCDoS = Context.GetInternalContext()->RPCDoS;
	FNetRPCDoSScope RPCDoSScope(RPCDoS);

	if (RPCDoS != nullptr && bIsServer && RPCDoS->IsRPCDoSDetectionEnabled() /* && !Connection->IsReplay() */)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RPCDosDetection_Iris);

		FName FunctionName = Function->GetFName();

		RPCDoSScope.SetEnabled(true);

		if (UNLIKELY(RPCDoS->ShouldMonitorReceivedRPC()))
		{
			ERPCNotifyResult Result = RPCDoS->NotifyReceivedRPC(Object, Function, FunctionName);

			if (UNLIKELY(Result == ERPCNotifyResult::BlockRPC))
			{
				RPCDoS->TrackBlockedRPC(FunctionName);

				return;
			}
		}
		else
		{
			RPCDoS->LightweightReceivedRPC(Function, FunctionName);
		}
	}

	// Setup function parameters
	uint8* FunctionParameters = nullptr;
	if (Function->ParmsSize > 0)
	{
		check(BlobDescriptor.IsValid());

		FunctionParameters = static_cast<uint8*>(FMemory_Alloca(Function->ParmsSize));
		FMemory::Memzero(FunctionParameters, Function->ParmsSize);
		if (!EnumHasAnyFlags(BlobDescriptor->Traits, EReplicationStateTraits::IsSourceTriviallyConstructible))
		{
			const FReplicationStateMemberDescriptor* MemberDescriptors = BlobDescriptor->MemberDescriptors;
			const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors = BlobDescriptor->MemberPropertyDescriptors;
			const FProperty** MemberProperties = BlobDescriptor->MemberProperties;
			for (uint32 MemberIt = 0, MemberCount = BlobDescriptor->MemberCount; MemberIt < MemberCount; ++MemberIt)
			{
				const FReplicationStateMemberPropertyDescriptor& MemberPropertyDescriptor = MemberPropertyDescriptors[MemberIt];
		
				// InitializeValue operates on the entire static array so make sure not to call it other than for the first element.
				if (MemberPropertyDescriptor.ArrayIndex == 0)
				{
					const FProperty* Property = MemberProperties[MemberIt];
					// We will dequantize all parameters to this buffer so we don't care if zero is the wrong value as it will be overwritten anyway.
					// So we only need to initialize the value if it's complex, such as having virtual functions.
					if (!Property->HasAnyPropertyFlags(CPF_ZeroConstructor | CPF_IsPlainOldData))
					{
						const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
						Property->InitializeValue(FunctionParameters + MemberDescriptor.ExternalMemberOffset);
					}
				}
			}
		}
		
		FReplicationStateOperations::Dequantize(Context, FunctionParameters, QuantizedBlobState.GetStateBuffer(), BlobDescriptor);
	}

	// Since the replicated FunctionLocator references the SuperFunction, we must lookup the actual function to call from the target object to properly support BP derived functions.
	// See: JIRA: UE-220400
	const UFunction* ActualFunction = Object->FindFunction(Function->GetFName());
	if (ActualFunction == nullptr)
	{
		ActualFunction = Function;
	}

	// Forward function
	if (const FForwardNetRPCCallMulticastDelegate& Delegate = CallContext.GetForwardNetRPCCallDelegate(); Delegate.IsBound())
	{
		UObject* RootObject = NetRPC_GetRootObject(Context, NetObjectReference);
		UObject* SubObject = (Object != RootObject ? Object : static_cast<UObject*>(nullptr));
		Delegate.Broadcast(RootObject, SubObject, const_cast<UFunction*>(ActualFunction), FunctionParameters);
	}

	// Call function
	if (ReplicationSystem->AllowExecuteRPC())
	{
#if IRIS_CLIENT_PROFILER_ENABLE
		UE::Net::FClientProfiler::RecordRPC(ActualFunction->GetFName());
#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
		UE::RemoteExecutor::OnRegionBeginDelegate.ExecuteIfBound(ActualFunction->GetFName());
		UE_AUTORTFM_ONABORT()
		{
			UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound("Abort");
		};
		UE_AUTORTFM_ONCOMMIT()
		{
			UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound("Commit");
		};
#endif

		UE::Net::FScopedNetContextRPC CallingRPC;
		UE::Net::Private::FScopedRemoteRPCMode ReceivingRemoteRPC(ActualFunction, UE::Net::Private::ERemoteFunctionMode::Receiving);
		Object->ProcessEvent(const_cast<UFunction*>(ActualFunction), FunctionParameters);
	}

	// Deinitialize function parameters
	if (FunctionParameters != nullptr)
	{

		if (!EnumHasAnyFlags(BlobDescriptor->Traits, EReplicationStateTraits::IsSourceTriviallyDestructible))
		{
			for (TFieldIterator<FProperty> ParamIt(Function); ParamIt && (ParamIt->PropertyFlags & (CPF_Parm | CPF_ReturnParm)) == CPF_Parm; ++ParamIt)
			{
				ParamIt->DestroyValue_InContainer(FunctionParameters);
			}
		}
	}
}

bool FNetRPC::IsServerAllowedToExecuteRPC(FNetSerializationContext& Context) const
{
	const FNetRefHandle Handle = NetObjectReference.GetRefHandle();
	const UReplicationSystem* ReplicationSystem = Context.GetInternalContext()->ReplicationSystem;

	const uint32 OwningConnectionId = ReplicationSystem->GetOwningNetConnection(Handle);
	const uint32 ExecutingConnectionId = Context.GetLocalConnectionId();
	const bool bTargetIsOwnedByConnection = OwningConnectionId == ExecutingConnectionId;
	return bTargetIsOwnedByConnection;
}

static bool NetRPC_GetFunctionLocator(const UReplicationSystem* ReplicationSystem, const FNetObjectReference& ObjectReference, const UFunction* Function, FNetRPC::FFunctionLocator& OutFunctionLocator, const FReplicationStateMemberFunctionDescriptor*& OutFunctionDescriptor)
{
	const UObjectReplicationBridge* Bridge = Cast<UObjectReplicationBridge>(ReplicationSystem->GetReplicationBridge());
	if (!ensure(Bridge != nullptr))
	{
		return false;
	}

	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(ObjectReference.GetRefHandle());
	if (!ensure(Protocol != nullptr))
	{
		return false;
	}

	for (const FReplicationStateDescriptor*& Descriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
	{
		for (const FReplicationStateMemberFunctionDescriptor& FunctionDescriptor : MakeArrayView(Descriptor->MemberFunctionDescriptors, Descriptor->FunctionCount))
		{
			if (FunctionDescriptor.Function == Function)
			{
				OutFunctionLocator.DescriptorIndex = static_cast<uint16>(&Descriptor - Protocol->ReplicationStateDescriptors);
				OutFunctionLocator.FunctionIndex = static_cast<uint16>(&FunctionDescriptor - Descriptor->MemberFunctionDescriptors);
				OutFunctionDescriptor = &FunctionDescriptor;
				return true;
			}
		}
	}

	return false;
}

static UObject* NetRPC_GetObject(FNetSerializationContext& Context, const FNetObjectReference& RootObjectReference, const FNetObjectReference& SubObjectReference)
{
	return NetRPC_GetObject(Context, SubObjectReference.IsValid() ? SubObjectReference : RootObjectReference);
}

static UObject* NetRPC_GetObject(FNetSerializationContext& Context, const FNetObjectReference& ObjectReference)
{
	FInternalNetSerializationContext* InternalContext = Context.GetInternalContext();
	return InternalContext->ObjectReferenceCache->ResolveObjectReference(ObjectReference, InternalContext->ResolveContext);
}

static UObject* NetRPC_GetRootObject(FNetSerializationContext& Context, const FNetObjectReference& ObjectReference)
{
	FInternalNetSerializationContext* InternalContext = Context.GetInternalContext();
	const FNetRefHandleManager& NetRefHandleManager = InternalContext->ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
	
	FNetRefHandle OwnerRefHandle = ObjectReference.GetRefHandle();
	const FInternalNetRefIndex InternalIndex = NetRefHandleManager.GetInternalIndex(OwnerRefHandle);
	if (!ensureMsgf(InternalIndex != InvalidInternalNetRefIndex, TEXT("Unable to find InternalIndex for object reference %s"), ToCStr(ObjectReference.ToString())))
	{
		return nullptr;
	}

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager.GetReplicatedObjectDataNoCheck(InternalIndex);
	if (ObjectData.SubObjectRootIndex != InvalidInternalNetRefIndex)
	{
		const FNetRefHandleManager::FReplicatedObjectData& RootObjectData = NetRefHandleManager.GetReplicatedObjectDataNoCheck(ObjectData.SubObjectRootIndex);
		OwnerRefHandle = RootObjectData.RefHandle;
	}

	UObject* RootObject = InternalContext->ObjectReferenceCache->ResolveObjectReferenceHandle(OwnerRefHandle, InternalContext->ResolveContext);
	return RootObject;
}

static bool NetRPC_GetFunctionAndObject(FNetSerializationContext& Context, const FNetObjectReference& ObjectReference, const FNetObjectReference& SubObjectReference, const FNetRPC::FFunctionLocator& FunctionLocator, const FReplicationStateMemberFunctionDescriptor*& OutFunctionDescriptor, TWeakObjectPtr<UObject>& OutObject)
{
	const UReplicationSystem* ReplicationSystem = Context.GetInternalContext()->ReplicationSystem;
	const UObjectReplicationBridge* Bridge = Cast<UObjectReplicationBridge>(ReplicationSystem->GetReplicationBridge());
	if (!ensure(Bridge != nullptr))
	{
		Context.SetError(NetError_InvalidNetObjectReference);
		return false;
	}

	UObject* RefObject = NetRPC_GetObject(Context, ObjectReference, SubObjectReference);
	if (!RefObject)
	{
		// Ignore this RPC and continue processing the rest of the data
		UE_LOGF(LogIrisRpc, Warning, "%s: Object resolution FAILED. ObjRef=%ls SubObjRef=%ls Handle=%ls",
			__FUNCTION__, *ObjectReference.ToString(), *SubObjectReference.ToString(), *ObjectReference.GetRefHandle().ToString());
        return false;
	}

	const FReplicationProtocol* Protocol = ReplicationSystem->GetReplicationProtocol(ObjectReference.GetRefHandle());
	if (!Protocol)
	{
		// Ignore this RPC and continue processing the rest of the data, Note: this might for example occur if we have incoming RPC data from client to an object that has been destroyed on server.
  		UE_LOGF(LogIrisRpc, Verbose, "ReplicationProtocol doesn't exist for %ls (Connection %u). Ignoring RPC (%u|%u), this is most likely due to object %ls no longer being replicated.", *GetNameSafe(RefObject), Context.GetLocalConnectionId(), FunctionLocator.DescriptorIndex, FunctionLocator.FunctionIndex, *ObjectReference.ToString());
		return false;
	}

	if (!ensure(FunctionLocator.DescriptorIndex < Protocol->ReplicationStateCount))
	{
		Context.SetError(NetError_UnknownFunction);
		return false;
	}

	const FReplicationStateDescriptor* Descriptor = Protocol->ReplicationStateDescriptors[FunctionLocator.DescriptorIndex];
	if (!ensure(FunctionLocator.FunctionIndex < Descriptor->FunctionCount))
	{
		Context.SetError(NetError_UnknownFunction);
		return false;
	}

	OutFunctionDescriptor = &Descriptor->MemberFunctionDescriptors[FunctionLocator.FunctionIndex];

	OutObject = RefObject;

	return true;
}

static FString NetRPC_GetDebugObjectRefName(FNetSerializationContext& Context, const FNetObjectReference& ObjectReference, UObject* ObjectPtr)
{
	if (!ObjectPtr)
	{
		ObjectPtr = NetRPC_GetObject(Context, ObjectReference);
	}
	return FString::Printf(TEXT("%s (%s)"), *GetNameSafe(ObjectPtr), *ObjectReference.ToString());
}

} // end namespace UE::Net::Private
