// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "UObject/WeakObjectPtr.h"

namespace UE::Net
{
	class FNetRPCCallContext;
}

namespace UE::Net::Private
{

class FNetRPC final : public FNetObjectAttachment
{
public:
	struct FFunctionLocator
	{
		// Which ReplicationStateDescriptor in the Object's protocol the function information can be found
		uint16 DescriptorIndex;
		// Which index in to the MemberFunctionDescriptor array in the above ReplicationStateDescriptor contains the function information
		uint16 FunctionIndex;
	};

	FNetRPC(const FNetBlobCreationInfo& CreationInfo);

	static FNetRPC* Create(UReplicationSystem* ReplicationSystem, const FNetBlobCreationInfo& CreationInfo, const FNetObjectReference& ObjectReference, const UFunction* Function, const void* FunctionParameters);

	void CallFunction(FNetRPCCallContext& Context);

	void SetFunctionLocator(const FFunctionLocator& InfFunctionLocator) { FunctionLocator = InfFunctionLocator; }
	const FFunctionLocator& GetFunctionLocator() const { return FunctionLocator; }

private:
	virtual ~FNetRPC();

	virtual TArrayView<const FNetObjectReference> GetNetObjectReferenceExports() const override final;

	virtual void SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const override;
	virtual void DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) override;

	virtual void Serialize(FNetSerializationContext& Context) const override;
	virtual void Deserialize(FNetSerializationContext& Context) override;

	void InternalSerializeHeader(FNetSerializationContext& Context, int32 PayloadSize=-1) const;
	uint32 InternalDeserializeHeader(FNetSerializationContext& Context);

	void SerializeFunctionLocator(FNetSerializationContext& Context) const;
	void DeserializeFunctionLocator(FNetSerializationContext& Context);

	void InternalSerializeObjectReference(FNetSerializationContext& Context) const;
	void InternalDeserializeObjectReference(FNetSerializationContext& Context);

	void InternalSerializeSubObjectReference(FNetSerializationContext& Context, FNetRefHandle RefHandle) const;
	void InternalDeserializeSubObjectReference(FNetSerializationContext& Context, FNetRefHandle RefHandle);

	void InternalSerializeBlob(FNetSerializationContext& Context) const;
	void InternalDeserializeBlob(FNetSerializationContext& Context);

	bool ResolveFunctionAndObject(FNetSerializationContext& Context);

	bool IsServerAllowedToExecuteRPC(FNetSerializationContext& Context) const;

private:

	static constexpr uint32 HeaderSizeBitCount = 24U;
	static constexpr uint32 MaxRpcSizeInBits = (1U << HeaderSizeBitCount) - 1U;

private:
	// If we have exports we normally expect the number to be low
	typedef TArray<FNetObjectReference, TInlineAllocator<4>> FNetRPCExportsArray;

	FFunctionLocator FunctionLocator;
	const UFunction* Function;
	TWeakObjectPtr<UObject> ObjectPtr;
	TUniquePtr<FNetRPCExportsArray> ReferencesToExport;
};

}
