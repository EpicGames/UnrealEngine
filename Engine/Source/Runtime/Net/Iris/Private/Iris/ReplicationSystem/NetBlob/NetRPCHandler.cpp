// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/NetRPCHandler.h"
#include "Iris/ReplicationSystem/NetBlob/NetRPC.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetRPCHandler)

UNetRPCHandler::UNetRPCHandler()
{
}

UNetRPCHandler::~UNetRPCHandler()
{
}

void UNetRPCHandler::Init(UReplicationSystem& InReplicationSystem)
{
	ReplicationSystem = &InReplicationSystem;
}

TRefCountPtr<UE::Net::Private::FNetRPC> UNetRPCHandler::CreateRPC(const UE::Net::FNetObjectReference& ObjectReference, const UFunction* Function, const void* Parameters) const
{
	FNetBlobCreationInfo CreationInfo;
	CreationInfo.Type = GetNetBlobType();
	CreationInfo.Flags = ((Function->FunctionFlags & FUNC_NetReliable) != 0) ? UE::Net::ENetBlobFlags::Reliable : UE::Net::ENetBlobFlags::None;
	// Unicast RPCs should be ordered with respect to other reliable and unicast RPCs.
	if ((Function->FunctionFlags & FUNC_NetMulticast) == 0)
	{
		CreationInfo.Flags |= UE::Net::ENetBlobFlags::Ordered;
	}

	FNetRPC* RPC = FNetRPC::Create(ReplicationSystem, CreationInfo, ObjectReference, Function, Parameters);
	return RPC;
}

TRefCountPtr<UE::Net::FNetBlob> UNetRPCHandler::CreateNetBlob(const FNetBlobCreationInfo& CreationInfo) const
{
	FNetRPC* RPC = new FNetRPC(CreationInfo);
	return RPC;
}

void UNetRPCHandler::OnNetBlobReceived(UE::Net::FNetSerializationContext& NetContext, const TRefCountPtr<FNetBlob>& NetBlob)
{
	const UE::Net::FForwardNetRPCCallMulticastDelegate& ForwardNetRPCCallDelegate = ReplicationSystem->GetReplicationSystemInternal()->GetForwardNetRPCCallMulticastDelegate();
	UE::Net::FNetRPCCallContext CallContext(NetContext, ForwardNetRPCCallDelegate);
	FNetRPC* RPC = static_cast<FNetRPC*>(NetBlob.GetReference());
	RPC->CallFunction(CallContext);
}
