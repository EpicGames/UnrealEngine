// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Iris/ReplicationSystem/NetBlob/NetRPC.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandler.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"
#include "NetRPCHandler.generated.h"

namespace UE::Net
{

class FNetRPCCallContext
{
public:
	FNetRPCCallContext(FNetSerializationContext& NetContext, const FForwardNetRPCCallMulticastDelegate& ForwardNetRPCCallDelegate);

	FNetSerializationContext& GetNetSerializationContext();
	const FForwardNetRPCCallMulticastDelegate& GetForwardNetRPCCallDelegate() const;

private:
	FNetSerializationContext& NetContext;
	const FForwardNetRPCCallMulticastDelegate& ForwardNetRPCCallDelegate;
};

}

UCLASS(transient, MinimalAPI)
class UNetRPCHandler final : public UNetBlobHandler
{
	GENERATED_BODY()

	using FNetRPC = UE::Net::Private::FNetRPC;

public:
	UNetRPCHandler();
	virtual ~UNetRPCHandler();

	void Init(UReplicationSystem& ReplicationSystem);

	TRefCountPtr<UE::Net::Private::FNetRPC> CreateRPC(const UE::Net::FNetObjectReference& ObjectReference, const UFunction* Function, const void* Parameters) const;

private:
	virtual TRefCountPtr<FNetBlob> CreateNetBlob(const FNetBlobCreationInfo&) const override;
	virtual void OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& NetBlob) override;

	UReplicationSystem* ReplicationSystem = nullptr;
};

namespace UE::Net
{

inline FNetRPCCallContext::FNetRPCCallContext(FNetSerializationContext& InNetContext, const FForwardNetRPCCallMulticastDelegate& InForwardNetRPCCallDelegate)
: NetContext(InNetContext)
, ForwardNetRPCCallDelegate(InForwardNetRPCCallDelegate)
{
}

inline FNetSerializationContext& FNetRPCCallContext::GetNetSerializationContext()
{
	return NetContext;
}

inline const FForwardNetRPCCallMulticastDelegate& FNetRPCCallContext::GetForwardNetRPCCallDelegate() const
{
	return ForwardNetRPCCallDelegate;
}

}
