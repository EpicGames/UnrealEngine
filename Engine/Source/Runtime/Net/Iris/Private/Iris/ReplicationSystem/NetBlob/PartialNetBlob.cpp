// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/PartialNetBlob.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net::Private
{
	/*
	 * We need a unique sequence number per split blob to detect out of order sequences. In theory this requirement is only needed for split blobs sent via the same queue or whatever mechanism is used.
	 * As the overhead of large sequence numbers should be relatively low compared to the split payload size and sharing a global sequence number avoids bloating the splitting API we have this one shared atomic.
	 * Each split NetBlob will reserve unique sequence numbers for all of its parts. If there are multiple ReplicationSystems sharing the atomic adds relatively little growing of the sequence numbers.
	 * Splitting blobs is a special case mainly used to enable replicating very large objects.
	 */
	static std::atomic<uint32> PartialNetBlobGlobalSequenceNumber;
}

namespace UE::Net
{

FPartialNetBlob::FPartialNetBlob(const FNetBlobCreationInfo& CreationInfo)
: FNetBlob(CreationInfo)
, OriginalCreationInfo({})
{
}

TArrayView<const FNetObjectReference> FPartialNetBlob::GetNetObjectReferenceExports() const
{
	return MakeConstArrayView<>(ObjectReferenceExportsArray);
}

TArrayView<const FNetToken> FPartialNetBlob::GetNetTokenExports() const
{
	return MakeConstArrayView<>(NetTokenExportsArray);
}

void FPartialNetBlob::SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const
{
	InternalSerialize(Context);
}

void FPartialNetBlob::DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	InternalDeserialize(Context);
}

void FPartialNetBlob::Serialize(FNetSerializationContext& Context) const
{
	InternalSerialize(Context);
}

void FPartialNetBlob::Deserialize(FNetSerializationContext& Context)
{
	InternalDeserialize(Context);
}

void FPartialNetBlob::InternalSerialize(FNetSerializationContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// Use user provided debug name as outer scope. Terminate scope immediately if none was provided.
	UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(UserProvidedScope, &DebugName, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	if (DebugName.Name == nullptr)
	{
		UE_NET_TRACE_EXIT_NAMED_SCOPE(UserProvidedScope);
	}
	UE_NET_TRACE_SCOPE(PartialNetBlob, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(SequenceScope, static_cast<const TCHAR*>(nullptr), *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
#if UE_NET_TRACE_ENABLED
	if (FNetTrace::GetNetTraceVerbosityEnabled(ENetTraceVerbosity::VeryVerbose))
	{
		TStringBuilder<64> Builder;
		if (IsFirstPart())
		{
			Builder.Appendf(TEXT("Seq %u First part of %u"), SequenceNumber, PartCount);
		}
		else
		{
			Builder.Appendf(TEXT("Seq %u"), SequenceNumber);
		}
		UE_NET_TRACE_SET_SCOPE_NAME(SequenceScope, Builder.ToString());
	}
#endif // UE_NET_TRACE_ENABLED

	WritePackedUint32(Writer, SequenceNumber);
	if (Writer->WriteBool(IsFirstPart()))
	{
		WritePackedUint16(Writer, PartCount - 1U);

		UE_NET_TRACE_SCOPE(OriginalCreationInfo, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		SerializeCreationInfo(Context, OriginalCreationInfo);
	}

	InternalSerializeBlob(Context);
}

void FPartialNetBlob::InternalDeserialize(FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	UE_NET_TRACE_SCOPE(PartialNetBlob, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(SequenceScope, static_cast<const TCHAR*>(nullptr), *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

	SequenceNumber = ReadPackedUint32(Reader);
	SequenceFlags = ESequenceFlags::None;
	if (Reader->ReadBool())
	{
		SequenceFlags = ESequenceFlags::IsFirstPart;
		PartCount = ReadPackedUint16(Reader) + 1U;

		UE_NET_TRACE_SCOPE(OriginalCreationInfo, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		DeserializeCreationInfo(Context, OriginalCreationInfo);
	}

	InternalDeserializeBlob(Context);

#if UE_NET_TRACE_ENABLED
	if (FNetTrace::GetNetTraceVerbosityEnabled(ENetTraceVerbosity::VeryVerbose))
	{
		TStringBuilder<64> Builder;
		if (IsFirstPart())
		{
			Builder.Appendf(TEXT("Seq %u First part of %u"), SequenceNumber, PartCount);
		}
		else
		{
			Builder.Appendf(TEXT("Seq %u"), SequenceNumber);
		}
		UE_NET_TRACE_SET_SCOPE_NAME(SequenceScope, Builder.ToString());
	}
#endif // UE_NET_TRACE_ENABLED
}

void FPartialNetBlob::InternalSerializeBlob(FNetSerializationContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	UE_NET_TRACE_SCOPE(Payload, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	WritePackedUint16(Writer, PayloadBitCount);
	Writer->WriteBitStream(Payload.GetData(), 0U, PayloadBitCount);
}

void FPartialNetBlob::InternalDeserializeBlob(FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	UE_NET_TRACE_SCOPE(Payload, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	PayloadBitCount = ReadPackedUint16(Reader);
	Payload.SetNumUninitialized((PayloadBitCount + 31U)/32U);
	Reader->ReadBitStream(Payload.GetData(), PayloadBitCount);
}

bool FPartialNetBlob::SplitNetBlob(FNetSerializationContext& Context, const FNetBlobCreationInfo& CreationInfo, const FPartialNetBlob::FSplitParams& SplitParams, const TRefCountPtr<FNetBlob>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs)
{
	check(SplitParams.MaxPartBitCount > 31U && SplitParams.MaxPartBitCount < 65536U && SplitParams.MaxPartCount > 0 && SplitParams.MaxPartCount < 65536U);
	if (!Blob.IsValid())
	{
		return false;
	}

	// We have no idea what the internals of the FNetBlob look like. We must serialize it to a temporary buffer.

	const uint32 MaxTotalBitCount = (SplitParams.MaxPartBitCount*SplitParams.MaxPartCount) & ~31U;

	// Want the part bit count to be a multiple of 32 so we can safely memcpy.
	const uint32 MaxPartBitCount = SplitParams.MaxPartBitCount & ~31U;

	// Allocate a temporary buffer that is 128KB to begin with.
	TArray<uint32> Payload;
	constexpr uint32 InitialPayloadBitCountAttempt = 128U*1024U*8U;
	uint32 CurrentPayloadBitCount = FPlatformMath::Min(MaxTotalBitCount, InitialPayloadBitCountAttempt);

	// Trial and error. We don't want to allocate a ton of memory to begin with as we're not likely to need much.
	bool bSuccess = false;
	do
	{
		Payload.SetNumUninitialized(CurrentPayloadBitCount/32U);

		FNetBitStreamWriter Writer;
		Writer.InitBytes(Payload.GetData(), static_cast<uint32>(Payload.Num())*4U);

		FNetSerializationContext SubContext = Context.MakeSubContext(&Writer);
		Private::FNetExportRollbackScope ExportRollBack(SubContext);

		if (SplitParams.bSerializeWithObject)
		{
			Blob->SerializeWithObject(SubContext, SplitParams.NetObjectReference.GetRefHandle());
		}
		else
		{
			Blob->Serialize(SubContext);
		}

		// If there was an actual error we are unlikely to succeed.
		if (SubContext.HasError())
		{
			return false;
		}

		// If get a bitstream overflow then grow the buffer and retry.
		bSuccess = !Writer.IsOverflown();
		if (!bSuccess)
		{
			// Again, we don't know how much buffer space we need. Double it.
			const uint32 NewPayloadBitCount = FPlatformMath::Min(2U*CurrentPayloadBitCount, MaxTotalBitCount);
			if (NewPayloadBitCount <= CurrentPayloadBitCount)
			{
				// Buffer space exhausted. We cannot split the blob.
				return false;
			}

			CurrentPayloadBitCount = NewPayloadBitCount;
			continue;
		}

		// Adjust the payload bit count to the final value
		Writer.CommitWrites();
		CurrentPayloadBitCount = Writer.GetPosBits();
	} while (!bSuccess);

	// At this point we've successfully serialized the blob to our buffer. Time to split.
	{
		FPayloadSplitParams PayloadSplitParams;
		PayloadSplitParams.DebugName = SplitParams.DebugName;
		PayloadSplitParams.CreationInfo = CreationInfo;
		PayloadSplitParams.OriginalBlob = Blob.GetReference();
		PayloadSplitParams.OriginalCreationInfo = Blob->GetCreationInfo();
		PayloadSplitParams.Payload = Payload.GetData();
		PayloadSplitParams.PayloadBitCount = CurrentPayloadBitCount;
		PayloadSplitParams.PartBitCount = MaxPartBitCount;
		if (Private::FNetExportContext* ExportContext = Context.GetExportContext())
		{			
			PayloadSplitParams.ObjectReferencesPendingExport = MakeConstArrayView<>(ExportContext->GetBatchExports().ReferencesPendingExportInCurrentBatch);
			PayloadSplitParams.NetTokensPendingExport = MakeConstArrayView<>(ExportContext->GetBatchExports().NetTokensPendingExportInCurrentBatch);
		}

		SplitPayload(PayloadSplitParams, OutPartialBlobs);
	}

	return true;
}

bool FPartialNetBlob::SplitNetBlob(const FNetBlobCreationInfo& CreationInfo, const FSplitParams& SplitParams, const TRefCountPtr<FRawDataNetBlob>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs)
{
	check(SplitParams.MaxPartBitCount > 31U && SplitParams.MaxPartBitCount < 65536U && SplitParams.MaxPartCount > 0 && SplitParams.MaxPartCount < 65536U);
	if (!Blob.IsValid())
	{
		return false;
	}

	FPayloadSplitParams PayloadSplitParams = {};
	PayloadSplitParams.DebugName = SplitParams.DebugName;
	PayloadSplitParams.CreationInfo = CreationInfo;
	PayloadSplitParams.OriginalBlob = Blob.GetReference();
	PayloadSplitParams.OriginalCreationInfo = Blob->GetCreationInfo();
	PayloadSplitParams.Payload = Blob->GetRawData().GetData();
	PayloadSplitParams.PayloadBitCount = Blob->GetRawDataBitCount();
	PayloadSplitParams.PartBitCount = SplitParams.MaxPartBitCount & ~31U;

	SplitPayload(PayloadSplitParams, OutPartialBlobs);
	return true;
}

void FPartialNetBlob::SplitPayload(const FPartialNetBlob::FPayloadSplitParams& SplitParams, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs)
{
	const uint32 PartialBlobCount = (SplitParams.PayloadBitCount + SplitParams.PartBitCount - 1U)/SplitParams.PartBitCount;
	OutPartialBlobs.Reserve(OutPartialBlobs.Num() + int32(PartialBlobCount));
	
	// Reserve sequence numbers for all parts. 
	uint32 SequenceNumber = 0;
	// We do not currently run SplitPayload in a closed AutoRTFM scope, but it is acceptable for the sequence number to
	// increase and never roll back, so this makes it safe to access this atomic in a closed scope in the future
	UE_AUTORTFM_OPEN
	{
		SequenceNumber = Private::PartialNetBlobGlobalSequenceNumber;
		Private::PartialNetBlobGlobalSequenceNumber += PartialBlobCount;
	};
	

	uint32 PayloadBitOffset = 0U;

	// Do we have exports	
	for (uint32 PartIt = 0, PartEndIt = PartialBlobCount; PartIt != PartEndIt; ++PartIt)
	{
		const bool bIsFirstPart = PartIt == 0U;

		FPartialNetBlob* PartialBlob = new FPartialNetBlob(SplitParams.CreationInfo);
		PartialBlob->SetDebugName(SplitParams.DebugName);
		PartialBlob->OriginalCreationInfo = SplitParams.OriginalCreationInfo;
		PartialBlob->PartCount = static_cast<uint16>(PartialBlobCount);
		PartialBlob->SequenceFlags = (bIsFirstPart ? ESequenceFlags::IsFirstPart : ESequenceFlags::None);
		PartialBlob->SequenceNumber = SequenceNumber++;
		if (bIsFirstPart && (EnumHasAnyFlags(SplitParams.OriginalCreationInfo.Flags, ENetBlobFlags::HasExports) || !SplitParams.ObjectReferencesPendingExport.IsEmpty() || !SplitParams.NetTokensPendingExport.IsEmpty()))
		{
			PartialBlob->OriginalBlob = SplitParams.OriginalBlob;
			PartialBlob->CreationInfo.Flags |= ENetBlobFlags::HasExports;
			PartialBlob->ObjectReferenceExportsArray.Append(SplitParams.ObjectReferencesPendingExport);
			PartialBlob->NetTokenExportsArray.Append(SplitParams.NetTokensPendingExport);
			if (EnumHasAnyFlags(SplitParams.OriginalCreationInfo.Flags, ENetBlobFlags::HasExports) && SplitParams.OriginalBlob)
			{
				for (TConstArrayView<FNetObjectReference> ObjectReferences = SplitParams.OriginalBlob->CallGetNetObjectReferenceExports(); const FNetObjectReference& ObjectReference : ObjectReferences)
				{
					PartialBlob->ObjectReferenceExportsArray.AddUnique(ObjectReference);
				}
			}
		}

		// Copy relevant data from our temporary buffer.
		const uint32 PartialBlobBitCount = FPlatformMath::Min(SplitParams.PayloadBitCount - PayloadBitOffset, SplitParams.PartBitCount);
		const uint32 PartialBlobWordCount = (PartialBlobBitCount + 31U)/32U;
		const uint32 PayloadWordOffset = PayloadBitOffset/32U;
		PartialBlob->PayloadBitCount = static_cast<uint16>(PartialBlobBitCount);
		PartialBlob->Payload.SetNumUninitialized(PartialBlobWordCount);
		FPlatformMemory::Memcpy(PartialBlob->Payload.GetData(), SplitParams.Payload + PayloadWordOffset, PartialBlobWordCount*4U);

		PayloadBitOffset += SplitParams.PartBitCount;

		OutPartialBlobs.Add(PartialBlob);
	}
}

}
