// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/PartialNetObjectAttachmentHandler.h"
#include "Iris/ReplicationSystem/NetBlob/PartialNetBlob.h"
#include "Iris/ReplicationSystem/NetBlob/ShrinkWrapNetBlob.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetExportContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PartialNetObjectAttachmentHandler)

UPartialNetObjectAttachmentHandler::UPartialNetObjectAttachmentHandler()
: USequentialPartialNetBlobHandler()
{
}

UPartialNetObjectAttachmentHandler::~UPartialNetObjectAttachmentHandler()
{
}

void UPartialNetObjectAttachmentHandler::Init(const FPartialNetObjectAttachmentHandlerInitParams& InitParams)
{
	ensure(InitParams.Config->GetBitCountSplitThreshold() <= 65536);

	FSequentialPartialNetBlobHandlerInitParams ParentInitParams = {};
	ParentInitParams.ReplicationSystem = InitParams.ReplicationSystem;
	ParentInitParams.Config = InitParams.Config;
	Super::Init(ParentInitParams);
}

bool UPartialNetObjectAttachmentHandler::PreSerializeAndSplitNetBlob(uint32 ConnectionId, const TRefCountPtr<UE::Net::FNetObjectAttachment>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs, bool bSerializeWithObject)
{
	using namespace UE::Net;
	
	uint32 BitCountSplitThreshold =  GetConfig()->GetBitCountSplitThreshold() & ~31U;	

	const FNetBlobCreationInfo& BlobCreationInfo = Blob->GetCreationInfo();
	const bool bIsReliable = EnumHasAnyFlags(BlobCreationInfo.Flags, ENetBlobFlags::Reliable);
	if (!bIsReliable)
	{
		BitCountSplitThreshold = (ReplicationSystem->IsServer() ? GetConfig()->GetServerUnreliableBitCountSplitThreshold() : GetConfig()->GetClientUnreliableBitCountSplitThreshold()) & ~31U;
	}

	TArray<uint32> Payload;
	Payload.AddUninitialized(BitCountSplitThreshold/32U);

	FNetBitStreamWriter Writer;
	Writer.InitBytes(Payload.GetData(), Payload.Num()*4U);

	Private::FInternalNetSerializationContext InternalContext(ReplicationSystem);
	FNetSerializationContext SerializationContext(&Writer);
	// $IRIS TODO Cannot share serialization if there are serializers with connection specific serialization 
	SerializationContext.SetLocalConnectionId(ConnectionId);
	SerializationContext.SetInternalContext(&InternalContext);

	// Setup ExportScope to capture non object reference exports.
	Private::FNetExportContext::FBatchExports BatchExports;
	Private::FNetExports::FExportScope ExportScope = NetExports.MakeExportScope(SerializationContext, BatchExports);

	{
		Private::FNetExportRollbackScope ExportRollBack(SerializationContext);

		if (bSerializeWithObject)
		{
			Blob->SerializeWithObject(SerializationContext, Blob->GetNetObjectReference().GetRefHandle());
		}
		else
		{
			Blob->Serialize(SerializationContext);
		}

		// Errors are bad. We cannot recover from this.
		if (SerializationContext.HasError())
		{
			return false;
		}
	}

	Writer.CommitWrites();

	// Blob needs splitting!
	if (Writer.IsOverflown())
	{
		// This will redo all the serialization work, but it should rarely happen.
		if (bSerializeWithObject)
		{
			return Super::SplitNetBlob(SerializationContext, Blob->GetNetObjectReference(), reinterpret_cast<const TRefCountPtr<FNetBlob>&>(Blob), OutPartialBlobs);
		}
		else
		{
			return Super::SplitNetBlob(SerializationContext, reinterpret_cast<const TRefCountPtr<FNetBlob>&>(Blob), OutPartialBlobs);
		}
	}
	else
	{
		FShrinkWrapNetObjectAttachment* ShrinkWrapNetBlob = new FShrinkWrapNetObjectAttachment(SerializationContext, Blob, MoveTemp(Payload), Writer.GetPosBits());
		OutPartialBlobs.AddDefaulted_GetRef() = ShrinkWrapNetBlob;
		return true;
	}
}

bool UPartialNetObjectAttachmentHandler::SplitRawDataNetBlob(const TRefCountPtr<UE::Net::FRawDataNetBlob>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs, const UE::Net::FNetDebugName* InDebugName) const
{
	using namespace UE::Net;

	const uint32 BitCountSplitThreshold =  GetConfig()->GetBitCountSplitThreshold() & ~31U;
	if (Blob->GetRawDataBitCount() > BitCountSplitThreshold)
	{
		const UPartialNetObjectAttachmentHandlerConfig* ThisConfig = GetConfig();

		FNetBlobCreationInfo CreationInfo = {};
		CreationInfo.Type = GetNetBlobType();
		CreationInfo.Flags = ENetBlobFlags::Reliable;

		FPartialNetBlob::FSplitParams SplitParams = {};
		SplitParams.MaxPartBitCount = ThisConfig->GetMaxPartBitCount();
		SplitParams.MaxPartCount = ThisConfig->GetMaxPartCount();
		SplitParams.bSerializeWithObject = false;
		if (InDebugName != nullptr)
		{
			SplitParams.DebugName = *InDebugName;
		}

		return FPartialNetBlob::SplitNetBlob(CreationInfo, SplitParams, Blob, OutPartialBlobs);
	}
	// No splitting required.
	else
	{
		OutPartialBlobs.AddDefaulted_GetRef() = Blob.GetReference();
		return true;
	}
}
