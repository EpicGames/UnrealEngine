// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Iris/ReplicationSystem/NetBlob/SequentialPartialNetBlobHandler.h"
#include "Iris/ReplicationSystem/NetBlob/RawDataNetBlob.h"
#include "Iris/ReplicationSystem/NetExports.h"
#include "PartialNetObjectAttachmentHandler.generated.h"

UCLASS()
class UPartialNetObjectAttachmentHandlerConfig : public USequentialPartialNetBlobHandlerConfig
{
	GENERATED_BODY()

public:
	uint32 GetBitCountSplitThreshold() const { return BitCountSplitThreshold; }
	uint32 GetClientUnreliableBitCountSplitThreshold() const { return ClientUnreliableBitCountSplitThreshold; }
	uint32 GetServerUnreliableBitCountSplitThreshold() const { return ServerUnreliableBitCountSplitThreshold; }

private:
	/** How many bits a payload should have to recommend a split. Should be higher than MaxPartBitCount as splitting adds overhead. */
	UPROPERTY(Config)
	uint32 BitCountSplitThreshold = (128 + 64)*8;

	/** How many bits a unreliable payload should have to recommend a split on the client. Should be higher than MaxPartBitCount as splitting adds overhead. */
	UPROPERTY(Config)
	uint32 ClientUnreliableBitCountSplitThreshold = (850)*8;

	/** How many bits a unreliable payload should have to recommend a split on the server. Should be higher than MaxPartBitCount as splitting adds overhead. */
	UPROPERTY(Config)
	uint32 ServerUnreliableBitCountSplitThreshold = (256)*8;
};

struct FPartialNetObjectAttachmentHandlerInitParams
{
	UReplicationSystem* ReplicationSystem;
	const UPartialNetObjectAttachmentHandlerConfig* Config;
};

/**
 * NetBlobHandler that can split and assemble very large NetObjectAttachments.
 */
UCLASS(transient, MinimalAPI)
class UPartialNetObjectAttachmentHandler final : public USequentialPartialNetBlobHandler
{
	GENERATED_BODY()

public:
	UPartialNetObjectAttachmentHandler();
	virtual ~UPartialNetObjectAttachmentHandler();

	void Init(const FPartialNetObjectAttachmentHandlerInitParams& InitParams);

	/** Serializes the NetBlob and either store the serialized version in a new NetBlob or splits into multiple partial NetBlobs. */
	bool PreSerializeAndSplitNetBlob(uint32 ConnectionId, const TRefCountPtr<UE::Net::FNetObjectAttachment>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs, bool bSerializeWithObject);

	/** Splits a RawDataNetBlob. The blob must have been created by a registered NetBlobHandler in order to be reconstructed on the receiving side. */
	bool SplitRawDataNetBlob(const TRefCountPtr<UE::Net::FRawDataNetBlob>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs, const UE::Net::FNetDebugName* InDebugName) const;

	const UPartialNetObjectAttachmentHandlerConfig* GetConfig() const;
private:
	UE::Net::Private::FNetExports NetExports;
};

inline const UPartialNetObjectAttachmentHandlerConfig* UPartialNetObjectAttachmentHandler::GetConfig() const
{
	return CastChecked<const UPartialNetObjectAttachmentHandlerConfig>(Super::GetConfig(), ECastCheckedType::NullAllowed);
}
