// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/RawDataNetBlob.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net
{

FRawDataNetBlob::FRawDataNetBlob(const FNetBlobCreationInfo& InCreationInfo)
: FNetBlob(FNetBlobCreationInfo{InCreationInfo.Type, InCreationInfo.Flags | ENetBlobFlags::RawDataNetBlob})
{
}

void FRawDataNetBlob::SetRawData(TArray<uint32>&& InRawData, uint32 InRawDataBitCount)
{
	RawData = MoveTemp(InRawData);
	RawDataBitCount = InRawDataBitCount;
}

void FRawDataNetBlob::SetRawData(const TArrayView<const uint32> InRawData, uint32 InRawDataBitCount)
{
	RawData = InRawData;
	RawDataBitCount = InRawDataBitCount;
}

void FRawDataNetBlob::InternalSerialize(FNetSerializationContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	UE_NET_TRACE_SCOPE(RawDataNetBlob, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	WritePackedUint32(Writer, RawDataBitCount);
	Writer->WriteBitStream(RawData.GetData(), 0, RawDataBitCount);
}

void FRawDataNetBlob::InternalDeserialize(FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	UE_NET_TRACE_SCOPE(RawDataNetBlob, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	RawDataBitCount = ReadPackedUint32(Reader);
	RawData.SetNumUninitialized((RawDataBitCount + 31U)/32U);
	Reader->ReadBitStream(RawData.GetData(), RawDataBitCount);
}

void FRawDataNetBlob::SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const
{
	InternalSerialize(Context);
}

void FRawDataNetBlob::DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	InternalDeserialize(Context);
}


void FRawDataNetBlob::Serialize(FNetSerializationContext& Context) const
{
	InternalSerialize(Context);
}

void FRawDataNetBlob::Deserialize(FNetSerializationContext& Context)
{
	InternalDeserialize(Context);
}

}
