// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/ReliableNetBlobQueue.h"
#include "Iris/IrisConfigInternal.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandlerManager.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net::Private
{

static const FName NetError_ReliableQueueFull("Reliable attachment queue full");
static const FName NetError_InvalidSequence("Invalid sequence number");

// ReliableNetBlobQueue
FReliableNetBlobQueue::FReliableNetBlobQueue()
: Sent{}
, Acked{}
, FirstSeq(0)
, LastSeq(0)
, UnsentBlobCount(0)
{
}

FReliableNetBlobQueue::~FReliableNetBlobQueue()
{
}

// If everything is sent and acked and the first sequence index is zero then it's safe to destroy this instance.
bool FReliableNetBlobQueue::IsSafeToDestroy() const
{
	if (FirstSeq != LastSeq)
	{
		return false;
	}

	if (SequenceToIndex(FirstSeq) != 0)
	{
		return false;
	}

	return true;
}

uint32 FReliableNetBlobQueue::Serialize(FNetSerializationContext& Context, FReliableNetBlobQueue::FReplicationRecord& OutRecord)
{
	FNetRefHandle InvalidNetHandle;
	constexpr bool bSerializeWithObject = false;
	return SerializeInternal(Context, InvalidNetHandle, OutRecord, bSerializeWithObject);
}

uint32 FReliableNetBlobQueue::SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle, FReliableNetBlobQueue::FReplicationRecord& OutRecord)
{
	constexpr bool bSerializeWithObject = true;
	return SerializeInternal(Context, RefHandle, OutRecord, bSerializeWithObject);
}

uint32 FReliableNetBlobQueue::SerializeInternal(FNetSerializationContext& Context, FNetRefHandle RefHandle, FReliableNetBlobQueue::FReplicationRecord& OutRecord, const bool bSerializeWithObject)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	const FObjectReferenceCache* ObjectReferenceCache = Context.GetInternalContext()->ObjectReferenceCache;

	uint32 PrevHasMoreBlobsWritePos = 0;
	uint32 SerializedCount = 0;
	uint32 PrevWrittenSeq = ~0U;
	uint32 WrittenIndex = 0;
	uint32 WrittenCount[MaxWriteSequenceCount] = {};
	uint32 WrittenSeq[MaxWriteSequenceCount];
	for (uint32& Sequence : WrittenSeq)
	{
		Sequence = ~0U;
	}

	for (uint32 Seq = FirstSeq, EndSeq = LastSeq; Seq < EndSeq; ++Seq)
	{
		const uint32 Index = SequenceToIndex(Seq);
		if (IsIndexSent(Index))
		{
			continue;
		}

		// Disjoint sequence handling
		// First in sequence?
		if (WrittenSeq[WrittenIndex] == ~0U)
		{
			WrittenSeq[WrittenIndex] = Seq;
		}
		// If broken sequence start the next one.
		else if (PrevWrittenSeq + 1U != Seq)
		{
			++WrittenIndex;
			// There's limited support for disjoint sequences in the replication record.
			if (WrittenIndex >= UE_ARRAY_COUNT(WrittenCount))
			{
				break;
			}
			WrittenSeq[WrittenIndex] = Seq;
		}

		FNetBitStreamRollbackScope RollbackScope(*Writer);
		FNetExportRollbackScope ExportRollbackScope(Context);

		// GetRefCount() is not const.
		TRefCountPtr<FNetBlob>& Attachment = NetBlobs[Index];

		// If this sequence is disjoint from the previous sequence we need to serialize the full index.
		// It's important that the sequence number is sent first so we can validate it before receiving exports and payload.
		if (Writer->WriteBool(Seq != PrevWrittenSeq + 1U))
		{
			Writer->WriteBits(Index, IndexBitCount);
		}

		// Unreliable blobs may have been released.
		const bool bHasData = Attachment.GetRefCount() > 0;
		if (Writer->WriteBool(bHasData))
		{
			UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(ReliabilityScope, GetReliabilityNetDebugName(Attachment->IsReliable()), *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

			// If we have exports, append them, if attachment is rolled back we will roll back any appended exports as well.
			FNetExportContext* ExportContext = Attachment->HasExports() ? Context.GetExportContext() : nullptr;
			if (ExportContext)
			{
				ObjectReferenceCache->AddPendingExports(Context, Attachment->CallGetNetObjectReferenceExports());
				ExportContext->AddPendingExports(Attachment->CallGetNetTokenExports());
			}

			Attachment->SerializeCreationInfo(Context, Attachment->GetCreationInfo());
			if (bSerializeWithObject)
			{
				Attachment->SerializeWithObject(Context, RefHandle);
			}
			else
			{
				Attachment->Serialize(Context);
			}
		}

		const uint32 HasMoreBlobsWritePos = Writer->GetPosBits();
		Writer->WriteBool(false); // Don't know yet if there's more to come

		if (Writer->IsOverflown())
		{
			ExportRollbackScope.Rollback();
			break;
		}
		// We did manage to serialize yet another blob
		else
		{
			if (SerializedCount > 0)
			{
				FNetBitStreamWriteScope WriteScope(*Writer, PrevHasMoreBlobsWritePos);
				Writer->WriteBool(true);
			}

			++SerializedCount;

			++WrittenCount[WrittenIndex];
			if (WrittenCount[WrittenIndex] == MaxSequenceLength)
			{
				++WrittenIndex;
				if (WrittenIndex >= UE_ARRAY_COUNT(WrittenCount))
				{
					break;
				}
			}

			PrevWrittenSeq = Seq;
			PrevHasMoreBlobsWritePos = HasMoreBlobsWritePos;
		}
	}

	// Assemble replication record
	FReplicationRecord Record;
	for (uint32 Index = 0; Index < MaxWriteSequenceCount; ++Index)
	{
		Record.Sequences[Index].Number = WrittenSeq[Index] & (MaxUnackedBlobCount - 1U);
		Record.Sequences[Index].Count = WrittenCount[Index] & MaxSequenceLength;
	}
	OutRecord = Record;

	return SerializedCount;
}

void FReliableNetBlobQueue::CommitReplicationRecord(const FReliableNetBlobQueue::FReplicationRecord& Record)
{
	for (uint32 Index = 0, EndIndex = MaxWriteSequenceCount; Index != EndIndex; ++Index)
	{
		const uint32 Count = Record.Sequences[Index].Count;
		UnsentBlobCount -= Count;
		for (uint32 Seq = Record.Sequences[Index].Number, EndSeq = Seq + Count; Seq != EndSeq; ++Seq)
		{
			const uint32 BlobIndex = SequenceToIndex(Seq);
			SetIndexIsSent(BlobIndex);

			// Release unreliable blobs. They should not be resent.
			TRefCountPtr<FNetBlob>& RefCountBlob = NetBlobs[BlobIndex];
			if (const FNetBlob* Blob = RefCountBlob.GetReference())
			{
				if (!Blob->IsReliable())
				{
					RefCountBlob.SafeRelease();
				}
			}
		}
	}
}

SIZE_T FReliableNetBlobQueue::GetUnreliableCount() const
{
	SIZE_T UnreliableCount = 0U;

	for (uint32 Seq = FirstSeq; Seq < LastSeq; ++Seq)
	{
		const uint32 Index = SequenceToIndex(Seq);
		const TRefCountPtr<FNetBlob>& RefCntBlob = NetBlobs[Index];
		if (const FNetBlob* Blob = RefCntBlob.GetReference())
		{
			UnreliableCount += Blob && !Blob->IsReliable() ? 1 : 0;
		}
	}

	return UnreliableCount;
}

void FReliableNetBlobQueue::DropUnreliable()
{
	for (uint32 Seq = FirstSeq; Seq < LastSeq; ++Seq)
	{
		const uint32 Index = SequenceToIndex(Seq);

		TRefCountPtr<FNetBlob>& RefCntBlob = NetBlobs[Index];
		const FNetBlob* Blob = RefCntBlob.GetReference();
		if (Blob && !Blob->IsReliable())
		{
			RefCntBlob.SafeRelease();
		}
	}
}

uint32 FReliableNetBlobQueue::Deserialize(FNetSerializationContext& Context)
{
	FNetRefHandle InvalidNetHandle;
	return DeserializeInternal(Context, InvalidNetHandle, false);
}

uint32 FReliableNetBlobQueue::DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	return DeserializeInternal(Context, RefHandle, true);
}

uint32 FReliableNetBlobQueue::DeserializeInternal(FNetSerializationContext& Context, FNetRefHandle RefHandle, const bool bSerializeWithObject)
{
	INetBlobReceiver* BlobReceiver = Context.GetNetBlobReceiver();
	checkSlow(BlobReceiver != nullptr);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	FObjectReferenceCache* ObjectReferenceCache = Context.GetInternalContext()->ObjectReferenceCache;

	uint32 Index = ~0U;
	uint32 DeserializedCount = 0;
	bool bHasMoreBlobs = false;
	do
	{
		if (const bool bIsNewSequence = Reader->ReadBool())
		{
			Index = Reader->ReadBits(IndexBitCount);
		}
		else
		{
			Index = (Index + 1) % MaxUnackedBlobCount;
		}

		uint32 ReceivedSeq = FirstSeq - (FirstSeq % MaxUnackedBlobCount) + Index;
		if (ReceivedSeq < FirstSeq)
		{
			ReceivedSeq += MaxUnackedBlobCount;
		}

		if (!IsValidReceiveSequence(ReceivedSeq) || IsIndexAcked(Index))
		{
			UE_LOGF(LogIris, Warning, "Invalid reliable sequence number. ReceivedSeq: %u FirstSeq: %u LastSeq: %u IsAcked: %u", ReceivedSeq, FirstSeq, LastSeq, unsigned(IsIndexAcked(Index)));
			Context.SetError(NetError_InvalidSequence);

			return DeserializedCount;
		}

		TRefCountPtr<FNetBlob> Blob;
		if (const bool bHasData = Reader->ReadBool())
		{
			UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(ReliabilityScope, GetReliabilityNetDebugName(false), *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

			FNetBlobCreationInfo CreationInfo;
			FNetBlob::DeserializeCreationInfo(Context, CreationInfo);
			Blob = BlobReceiver->CreateNetBlob(CreationInfo);
			if (!Blob.IsValid())
			{
				UE_LOGF(LogIris, Warning, "%s", "Unable to create blob.");
				Context.SetError(GNetError_UnsupportedNetBlob);

				return DeserializedCount;
			}

			if (bSerializeWithObject)
			{
				Blob->DeserializeWithObject(Context, RefHandle);
			}
			else
			{
				Blob->Deserialize(Context);
			}

			UE_NET_TRACE_SET_SCOPE_NAME(ReliabilityScope, GetReliabilityNetDebugName(Blob->IsReliable()));
		}
		
		bHasMoreBlobs = Reader->ReadBool();

		// Break if something went wrong with deserialization.
		if (Context.HasErrorOrOverflow())
		{
			UE_LOGF(LogIris, Warning, "Failed to deserialize reliable attachments for %ls", *RefHandle.ToString());
			break;
		}

		NetBlobs[Index] = MoveTemp(Blob);
		LastSeq = FMath::Max(LastSeq, ReceivedSeq + 1U);
		SetIndexIsAcked(Index);

		++DeserializedCount;
	} while (bHasMoreBlobs);

	return DeserializedCount;
}

bool FReliableNetBlobQueue::Enqueue(const TRefCountPtr<FNetBlob>& Blob)
{
	if (IsSendWindowFull())
	{
		return false;
	}

	++UnsentBlobCount;

	const uint32 Index = SequenceToIndex(LastSeq);
	++LastSeq;

	NetBlobs[Index] = Blob;
	return true;
}

const TRefCountPtr<FNetBlob>* FReliableNetBlobQueue::Peek()
{
	for (; FirstSeq < LastSeq; ++FirstSeq)
	{
		const uint32 Index = SequenceToIndex(FirstSeq);
		if (!IsIndexAcked(Index))
		{
			return nullptr;
		}

		if (NetBlobs[Index].GetRefCount() > 0)
		{
			return &NetBlobs[Index];
		}

		// We skip over empty blobs and ack them.
		ClearIndexIsAcked(Index);
	}

	return nullptr;
}

void FReliableNetBlobQueue::Pop()
{
	const uint32 Index = SequenceToIndex(FirstSeq);
	checkSlow(IsIndexAcked(Index) && (FirstSeq < LastSeq));
	NetBlobs[Index].SafeRelease();
	ClearIndexIsAcked(Index);
	++FirstSeq;

	// $TODO. For a memory optimization one can change the storage implementation and free the memory here if everything is acked.
}


void FReliableNetBlobQueue::DequeueUnreliable(TArray<TRefCountPtr<FNetBlob>>& Unreliable)
{
	for (uint32 Seq = FirstSeq; Seq < LastSeq; ++Seq)
	{
		const uint32 Index = SequenceToIndex(Seq);
		if (!IsIndexAcked(Index))
		{
			continue;
		}

		TRefCountPtr<FNetBlob>& RefCntBlob = NetBlobs[Index];
		if (RefCntBlob.GetRefCount() > 0)
		{
			if (const bool bIsUnReliable = !RefCntBlob->IsReliable())
			{
				Unreliable.Emplace(MoveTemp(RefCntBlob));

				// Advance the ack window if possible.
				if (Seq == FirstSeq)
				{
					ClearIndexIsAcked(Index);
					++FirstSeq;
				}
			}
		}
	}
}

void FReliableNetBlobQueue::ProcessPacketDeliveryStatus(EPacketDeliveryStatus Status, const FReliableNetBlobQueue::FReplicationRecord& Record)
{
	switch (Status)
	{
	case EPacketDeliveryStatus::Delivered:
	{
		OnPacketDelivered(Record);
		break;
	}
	case EPacketDeliveryStatus::Lost:
	{
		OnPacketDropped(Record);
		break;
	}
	case EPacketDeliveryStatus::Discard:
	{
		// Pretend that it was delivered.
		OnPacketDelivered(Record);
		break;
	}
	default:
	{
		break;
	}
	}
}

void FReliableNetBlobQueue::OnPacketDelivered(const FReliableNetBlobQueue::FReplicationRecord& Record)
{
	// Mark blobs as acked
	for (uint32 SeqIt = 0, EndSeqIt = MaxWriteSequenceCount; SeqIt != EndSeqIt; ++SeqIt)
	{
		const uint32 Count = Record.Sequences[SeqIt].Count;
		for (uint32 Seq = Record.Sequences[SeqIt].Number, EndSeq = Seq + Count; Seq != EndSeq; ++Seq)
		{
			const uint32 Index = SequenceToIndex(Seq);

			SetIndexIsAcked(Index);

			// Release blob as quickly as possible.
			NetBlobs[Index].SafeRelease();
		}
	}

	// Remove acked blobs, allowing for more blobs to be added to the queue.
	PopInOrderAckedBlobs();
}

void FReliableNetBlobQueue::OnPacketDropped(const FReliableNetBlobQueue::FReplicationRecord& Record)
{
	// Mark blobs as unsent
	for (uint32 SeqIt = 0, EndSeqIt = MaxWriteSequenceCount; SeqIt != EndSeqIt; ++SeqIt)
	{
		const uint32 Count = Record.Sequences[SeqIt].Count;
		UnsentBlobCount += Count;
		for (uint32 Seq = Record.Sequences[SeqIt].Number, EndSeq = Seq + Count; Seq != EndSeq; ++Seq)
		{
			ClearSequenceIsSent(Seq);
		}
	}
}

void FReliableNetBlobQueue::PopInOrderAckedBlobs()
{
	for (; FirstSeq != LastSeq; ++FirstSeq)
	{
		const uint32 Index = SequenceToIndex(FirstSeq);
		if (!IsIndexAcked(Index))
		{
			return;
		}

		ClearIndexIsAcked(Index);
		ClearIndexIsSent(Index);
	}

	// $TODO. For a memory optimization one can change the storage implementation and free the memory here if everything is acked.
}

}
