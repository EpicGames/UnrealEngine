// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/SequentialPartialNetBlobHandler.h"
#include "Iris/ReplicationSystem/NetBlob/PartialNetBlob.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetSerializationContext.h"


// 

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequentialPartialNetBlobHandler)
USequentialPartialNetBlobHandler::USequentialPartialNetBlobHandler()
: ReplicationSystem(nullptr)
, Config(nullptr)
{
}

void USequentialPartialNetBlobHandler::Init(const FSequentialPartialNetBlobHandlerInitParams& InitParams)
{
	ReplicationSystem = InitParams.ReplicationSystem;
	Config = InitParams.Config;
}

bool USequentialPartialNetBlobHandler::SplitNetBlob(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs, const UE::Net::FNetDebugName* InDebugName) const
{
	using namespace UE::Net;

	FNetBlobCreationInfo CreationInfo = {};
	CreationInfo.Type = GetNetBlobType();
	CreationInfo.Flags = Blob->GetCreationInfo().Flags & (ENetBlobFlags::Ordered | ENetBlobFlags::Reliable);

	FPartialNetBlob::FSplitParams SplitParams = {};
	SplitParams.MaxPartBitCount = Config->GetMaxPartBitCount();
	SplitParams.MaxPartCount = Config->GetMaxPartCount();
	if (InDebugName == nullptr)
	{
		if (const FReplicationStateDescriptor* Descriptor = Blob->GetReplicationStateDescriptor())
		{
			InDebugName = Descriptor->DebugName;
		}
	}
	if (InDebugName != nullptr)
	{
		SplitParams.DebugName = *InDebugName;
	}

	return FPartialNetBlob::SplitNetBlob(Context, CreationInfo, SplitParams, Blob, OutPartialBlobs);
}

bool USequentialPartialNetBlobHandler::SplitNetBlob(UE::Net::FNetSerializationContext& Context, const UE::Net::FNetObjectReference& NetObjectReference, const TRefCountPtr<FNetBlob>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs, const UE::Net::FNetDebugName* InDebugName) const
{
	using namespace UE::Net;

	FNetBlobCreationInfo CreationInfo = {};
	CreationInfo.Type = GetNetBlobType();
	CreationInfo.Flags = Blob->GetCreationInfo().Flags & (ENetBlobFlags::Ordered | ENetBlobFlags::Reliable);

	FPartialNetBlob::FSplitParams SplitParams = {};
	SplitParams.MaxPartBitCount = Config->GetMaxPartBitCount();
	SplitParams.MaxPartCount = Config->GetMaxPartCount();
	SplitParams.NetObjectReference = NetObjectReference;
	SplitParams.bSerializeWithObject = true;
	if (InDebugName == nullptr)
	{
		if (const FReplicationStateDescriptor* Descriptor = Blob->GetReplicationStateDescriptor())
		{
			InDebugName = Descriptor->DebugName;
		}
	}
	if (InDebugName != nullptr)
	{
		SplitParams.DebugName = *InDebugName;
	}

	return FPartialNetBlob::SplitNetBlob(Context, CreationInfo, SplitParams, Blob, OutPartialBlobs);
}

TRefCountPtr<UE::Net::FNetBlob> USequentialPartialNetBlobHandler::CreateNetBlob(const FNetBlobCreationInfo& CreationInfo) const
{
	using namespace UE::Net;

	FPartialNetBlob* Blob = new FPartialNetBlob(CreationInfo);
	return Blob;
}

void USequentialPartialNetBlobHandler::OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& NetBlob)
{
	ensureMsgf(false, TEXT("%s"), TEXT("SequentialPartialNetBlobHandler expects the blobs to be assembled via FNetBlobAssembler and then further processed, e.g. via the original blob type's NetBlobHandler. This function is not expected to be called."));
	Context.SetError(UE::Net::GNetError_UnsupportedNetBlob);
}
