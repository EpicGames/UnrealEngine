// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetBlob/ShrinkWrapNetBlob.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net
{

FShrinkWrapNetBlob::FShrinkWrapNetBlob(FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& InOriginalBlob, TArray<uint32>&& Payload, uint32 PayloadBitCount)
: FNetBlob(InOriginalBlob->GetCreationInfo())
, OriginalBlob(InOriginalBlob)
, SerializedBlob(MoveTemp(Payload))
, SerializedBlobBitCount(PayloadBitCount)
{
	NetTokenExportsArray = Context.GetExportContext()->GetBatchExports().NetTokensPendingExportInCurrentBatch;
	CreationInfo.Flags |= NetTokenExportsArray.Num() ? ENetBlobFlags::HasExports : ENetBlobFlags::None;
}

TArrayView<const FNetObjectReference> FShrinkWrapNetBlob::GetNetObjectReferenceExports() const
{
	return OriginalBlob->CallGetNetObjectReferenceExports();
}

TArrayView<const FNetToken> FShrinkWrapNetBlob::GetNetTokenExports() const
{
	return MakeArrayView<const FNetToken>(NetTokenExportsArray.GetData(), NetTokenExportsArray.Num());
}

void FShrinkWrapNetBlob::SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const
{
	if (Context.GetTraceCollector() != nullptr && OriginalBlob.IsValid())
	{
		OriginalBlob->SerializeWithObject(Context, RefHandle);
	}
	else
	{
		InternalSerialize(Context);
	}
}

void FShrinkWrapNetBlob::DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	checkf(false, TEXT("%s"), TEXT("This function should not be called. Contact the networking team."));
}

void FShrinkWrapNetBlob::Serialize(FNetSerializationContext& Context) const
{
	if (Context.GetTraceCollector() != nullptr && OriginalBlob.IsValid())
	{
		OriginalBlob->Serialize(Context);
	}
	else
	{
		InternalSerialize(Context);
	}
}

void FShrinkWrapNetBlob::Deserialize(FNetSerializationContext& Context)
{
	checkf(false, TEXT("%s"), TEXT("This function should not be called. Contact the networking team."));
}

void FShrinkWrapNetBlob::InternalSerialize(FNetSerializationContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	UE_NET_TRACE_SCOPE(ShrinkWrapNetBlob, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	// No need to serialize the blob bit count as on the receiving side the original NetBlob will do all the deserialization.
	constexpr uint32 SerializedBlobBitOffset = 0U;
	Writer->WriteBitStream(SerializedBlob.GetData(), SerializedBlobBitOffset, SerializedBlobBitCount);
}


FShrinkWrapNetObjectAttachment::FShrinkWrapNetObjectAttachment(FNetSerializationContext& Context, const TRefCountPtr<FNetObjectAttachment>& InOriginalBlob, TArray<uint32>&& Payload, uint32 PayloadBitCount)
: FNetBlob(InOriginalBlob->GetCreationInfo())
, OriginalBlob(InOriginalBlob)
, SerializedBlob(MoveTemp(Payload))
, SerializedBlobBitCount(PayloadBitCount)
{
	NetTokenExportsArray = Context.GetExportContext()->GetBatchExports().NetTokensPendingExportInCurrentBatch;
	CreationInfo.Flags |= NetTokenExportsArray.Num() ? ENetBlobFlags::HasExports : ENetBlobFlags::None;
}

TArrayView<const FNetObjectReference> FShrinkWrapNetObjectAttachment::GetNetObjectReferenceExports() const
{
	return OriginalBlob->CallGetNetObjectReferenceExports();
}

TArrayView<const FNetToken> FShrinkWrapNetObjectAttachment::GetNetTokenExports() const
{
	return MakeArrayView<const FNetToken>(NetTokenExportsArray.GetData(), NetTokenExportsArray.Num());
}

void FShrinkWrapNetObjectAttachment::SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const
{
	if (Context.GetTraceCollector() != nullptr && OriginalBlob.IsValid())
	{
		OriginalBlob->SerializeWithObject(Context, RefHandle);
	}
	else
	{
		// Add target exports
		if (Private::FNetExportContext* ExportContext = Context.GetExportContext())
		{
			const Private::FObjectReferenceCache* ObjectReferenceCache = Context.GetInternalContext()->ObjectReferenceCache;
			ObjectReferenceCache->AddPendingExport(*ExportContext, OriginalBlob->GetTargetObjectReference());
		}
		InternalSerialize(Context);
	}
}

void FShrinkWrapNetObjectAttachment::DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle)
{
	checkf(false, TEXT("%s"), TEXT("This function should not be called. Contact the networking team."));
}

void FShrinkWrapNetObjectAttachment::Serialize(FNetSerializationContext& Context) const
{
	if (Context.GetTraceCollector() != nullptr && OriginalBlob.IsValid())
	{
		OriginalBlob->Serialize(Context);
	}
	else
	{
		// Add target exports
		if (Private::FNetExportContext* ExportContext = Context.GetExportContext())
		{
			const Private::FObjectReferenceCache* ObjectReferenceCache = Context.GetInternalContext()->ObjectReferenceCache;		
			ObjectReferenceCache->AddPendingExport(*ExportContext, OriginalBlob->GetNetObjectReference());
			ObjectReferenceCache->AddPendingExport(*ExportContext, OriginalBlob->GetTargetObjectReference());
		}
		InternalSerialize(Context);
	}
}

void FShrinkWrapNetObjectAttachment::Deserialize(FNetSerializationContext& Context)
{
	checkf(false, TEXT("%s"), TEXT("This function should not be called. Contact the networking team."));
}

void FShrinkWrapNetObjectAttachment::InternalSerialize(FNetSerializationContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	UE_NET_TRACE_SCOPE(ShrinkWrapNetObjectAttachment, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	// No need to serialize the blob bit count as on the receiving side the original NetBlob will do all the deserialization.
	constexpr uint32 SerializedBlobBitOffset = 0U;
	Writer->WriteBitStream(SerializedBlob.GetData(), SerializedBlobBitOffset, SerializedBlobBitCount);
}

}
