// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetDependencyData.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "UObject/CoreNetTypes.h"

namespace UE::Net::Private
{

FNetDependencyData::FNetDependencyData()
{
}

void FNetDependencyData::FreeStoredDependencyDataForObject(FInternalNetRefIndex InternalIndex)
{
	if (FDependencyInfo* Entry = DependencyInfos.Find(InternalIndex))
	{
		for (const uint32 ArrayIndex : Entry->ArrayIndices)
		{
			if (ArrayIndex != FDependencyInfo::InvalidCacheIndex)
			{
				checkSlow(DependentObjectsStorage[ArrayIndex].Num() == 0);
				DependentObjectsStorage.RemoveAt(ArrayIndex);
			}
		}

		if (Entry->SubObjectConditionalArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			SubObjectConditionalsStorage.RemoveAt(Entry->SubObjectConditionalArrayIndex);
		}

		if (Entry->DependentObjectsInfoArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			DependentObjectInfosStorage.RemoveAt(Entry->DependentObjectsInfoArrayIndex);
		}

		if (Entry->CreationDependentArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			CreationDependentInfosStorage.RemoveAt(Entry->CreationDependentArrayIndex);
		}

		if (Entry->CreationDependencyArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			CreationDependencyInfosStorage.RemoveAt(Entry->CreationDependencyArrayIndex);
		}

		DependencyInfos.Remove(InternalIndex);
	}
}

FNetDependencyData::FDependencyInfo& FNetDependencyData::GetOrCreateCacheEntry(FInternalNetRefIndex InternalIndex)
{
	FDependencyInfo* Entry = DependencyInfos.Find(InternalIndex);

	if (!Entry)
	{
		Entry = &DependencyInfos.Add(InternalIndex);
		*Entry = FDependencyInfo();
	}

	return *Entry;
}

FNetDependencyData::FDependentObjectInfoArray& FNetDependencyData::GetOrCreateDependentObjectInfoArray(FInternalNetRefIndex OwnerIndex)
{
	FDependencyInfo& Entry = GetOrCreateCacheEntry(OwnerIndex);
	
	if (Entry.DependentObjectsInfoArrayIndex == FDependencyInfo::InvalidCacheIndex)
	{
		FSparseArrayAllocationInfo AllocInfo = DependentObjectInfosStorage.AddUninitialized();
		Entry.DependentObjectsInfoArrayIndex = AllocInfo.Index;

		FDependentObjectInfoArray* DependentObjectsInfoArrayIndex = new (AllocInfo.Pointer) FDependentObjectInfoArray();
		
		return *DependentObjectsInfoArrayIndex;
	}
	else
	{
		return DependentObjectInfosStorage[Entry.DependentObjectsInfoArrayIndex];
	}
}

FNetDependencyData::FSubObjectConditionalsArray& FNetDependencyData::GetOrCreateSubObjectConditionalsArray(FInternalNetRefIndex OwnerIndex)
{
	FDependencyInfo& Entry = GetOrCreateCacheEntry(OwnerIndex);
	check(Entry.ArrayIndices[EArrayType::ChildSubObjects] != FDependencyInfo::InvalidCacheIndex);
	
	if (Entry.SubObjectConditionalArrayIndex == FDependencyInfo::InvalidCacheIndex)
	{
		FSparseArrayAllocationInfo AllocInfo = SubObjectConditionalsStorage.AddUninitialized();
		Entry.SubObjectConditionalArrayIndex = AllocInfo.Index;

		FSubObjectConditionalsArray* SubObjectConditionalsArray = new (AllocInfo.Pointer) FSubObjectConditionalsArray();
		
		// Make sure that we initialize the conditionals to match the number of SubObjects
		const int32 NumChildSubObjects = DependentObjectsStorage[Entry.ArrayIndices[EArrayType::ChildSubObjects]].Num();
		static_assert(COND_None == 0, "Can't use SetNumZeroed() to initialize COND_None");
		SubObjectConditionalsArray->SetNumZeroed(NumChildSubObjects);

		return *SubObjectConditionalsArray;
	}
	else
	{
		return SubObjectConditionalsStorage[Entry.SubObjectConditionalArrayIndex];
	}
}

FNetDependencyData::FInternalNetIndexArray& FNetDependencyData::GetOrCreateInternalChildSubObjectsArray(FInternalNetRefIndex OwnerIndex, FSubObjectConditionalsArray*& OutSubObjectConditionals)
{
	FDependencyInfo& Entry = GetOrCreateCacheEntry(OwnerIndex);
	if (Entry.ArrayIndices[ChildSubObjects] == FDependencyInfo::InvalidCacheIndex)
	{
		// Allocate storage for subobjects / atomic dependencies
		FSparseArrayAllocationInfo AllocInfo = DependentObjectsStorage.AddUninitialized();
		Entry.ArrayIndices[ChildSubObjects] = AllocInfo.Index;

		FInternalNetIndexArray* InternalIndexArray = new (AllocInfo.Pointer) FInternalNetIndexArray();
		OutSubObjectConditionals = nullptr;
		return *InternalIndexArray;
	}
	else
	{
		OutSubObjectConditionals = Entry.SubObjectConditionalArrayIndex != FDependencyInfo::InvalidCacheIndex ? &SubObjectConditionalsStorage[Entry.SubObjectConditionalArrayIndex] : nullptr;
		return DependentObjectsStorage[Entry.ArrayIndices[ChildSubObjects]];
	}
}

FNetDependencyData::FCreationDependentInfoArray& FNetDependencyData::GetOrCreateCreationDependentInfoArray(FInternalNetRefIndex ParentIndex)
{
	FDependencyInfo& Entry = GetOrCreateCacheEntry(ParentIndex);
	
	if (Entry.CreationDependentArrayIndex == FDependencyInfo::InvalidCacheIndex)
	{
		FSparseArrayAllocationInfo AllocInfo = CreationDependentInfosStorage.AddUninitialized();
		Entry.CreationDependentArrayIndex = AllocInfo.Index;

		FCreationDependentInfoArray* InfoArray = new (AllocInfo.Pointer) FCreationDependentInfoArray();
		
		return *InfoArray;
	}
	else
	{
		return CreationDependentInfosStorage[Entry.CreationDependentArrayIndex];
	}
}

void FNetDependencyData::FreeCreationDependentInfoArray(FInternalNetRefIndex ParentIndex)
{
	if (FDependencyInfo* Entry = DependencyInfos.Find(ParentIndex))
	{
		if (Entry->CreationDependentArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			CreationDependentInfosStorage.RemoveAt(Entry->CreationDependentArrayIndex);
			Entry->CreationDependentArrayIndex = FDependencyInfo::InvalidCacheIndex;
		}
	}
}

FNetDependencyData::FCreationDependencyInfoArray& FNetDependencyData::GetOrCreateCreationDependencyInfoArray(FInternalNetRefIndex ChildIndex)
{
	FDependencyInfo& Entry = GetOrCreateCacheEntry(ChildIndex);
	
	if (Entry.CreationDependencyArrayIndex == FDependencyInfo::InvalidCacheIndex)
	{
		FSparseArrayAllocationInfo AllocInfo = CreationDependencyInfosStorage.AddUninitialized();
		Entry.CreationDependencyArrayIndex = AllocInfo.Index;

		FCreationDependencyInfoArray* InfoArray = new (AllocInfo.Pointer) FCreationDependencyInfoArray();
		
		return *InfoArray;
	}
	else
	{
		return CreationDependencyInfosStorage[Entry.CreationDependencyArrayIndex];
	}
}

void FNetDependencyData::FreeCreationDependencyInfoArray(FInternalNetRefIndex ChildIndex)
{
	if (FDependencyInfo* Entry = DependencyInfos.Find(ChildIndex))
	{
		if (Entry->CreationDependencyArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			CreationDependencyInfosStorage.RemoveAt(Entry->CreationDependencyArrayIndex);
			Entry->CreationDependencyArrayIndex = FDependencyInfo::InvalidCacheIndex;
		}
	}
}

FNetDependencyData::FInternalNetIndexArray& FNetDependencyData::GetOrCreateInternalIndexArray(FInternalNetRefIndex OwnerIndex, EArrayType ArrayType)
{
	FDependencyInfo& Entry = GetOrCreateCacheEntry(OwnerIndex);
	if (Entry.ArrayIndices[ArrayType] == FDependencyInfo::InvalidCacheIndex)
	{
		// Allocate storage for subobjects / atomic dependencies
		FSparseArrayAllocationInfo AllocInfo = DependentObjectsStorage.AddUninitialized();
		Entry.ArrayIndices[ArrayType] = AllocInfo.Index;

		FInternalNetIndexArray* InternalIndexArray = new (AllocInfo.Pointer) FInternalNetIndexArray();

		return *InternalIndexArray;
	}
	else
	{
		return DependentObjectsStorage[Entry.ArrayIndices[ArrayType]];
	}
}

}