// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"

namespace UE::Net
{
	typedef uint32 FInternalNetRefIndex;
}

namespace UE::Net::Private
{
	typedef int8 FLifeTimeConditionStorage;
}

namespace UE::Net::Private
{

struct FChildSubObjectsInfo
{
	const FInternalNetRefIndex* ChildSubObjects = nullptr;
	const FLifeTimeConditionStorage* SubObjectLifeTimeConditions = nullptr;
	uint32 NumSubObjects = 0U;
	bool bReplicateChildSubObjectsAfterParent = true;
};

struct FDependentObjectInfo
{
	FInternalNetRefIndex NetRefIndex = 0U;
	EDependentObjectSchedulingHint SchedulingHint = EDependentObjectSchedulingHint::Default;
};

class FNetDependencyData
{
public:
	FNetDependencyData();

	typedef TArray<FInternalNetRefIndex, TInlineAllocator<8>> FInternalNetIndexArray;
	typedef TArray<FLifeTimeConditionStorage, TInlineAllocator<8>> FSubObjectConditionalsArray;
	typedef TArray<FDependentObjectInfo, TInlineAllocator<8>> FDependentObjectInfoArray;
	typedef TArray<FInternalNetRefIndex, TInlineAllocator<8>> FCreationDependentInfoArray;
	typedef TArray<FInternalNetRefIndex, TInlineAllocator<2>> FCreationDependencyInfoArray;

	enum EArrayType
	{
		SubObjects = 0U,
		ChildSubObjects,
		DependentParentObjects,
		CreationDependencies,
		Count
	};

	template<EArrayType TypeIndex>
	FInternalNetIndexArray& GetOrCreateInternalIndexArray(FInternalNetRefIndex InternalIndex)
	{
		static_assert(TypeIndex != EArrayType::Count, "Invalid array type index");
		return GetOrCreateInternalIndexArray(InternalIndex, TypeIndex);
	};

	FDependentObjectInfoArray& GetOrCreateDependentObjectInfoArray(FInternalNetRefIndex InternalIndex);
	FDependentObjectInfoArray* GetDependentObjectInfoArray(FInternalNetRefIndex InternalIndex)
	{
		const FDependencyInfo* Entry = DependencyInfos.Find(InternalIndex);
		const uint32 ArrayIndex = Entry ? Entry->DependentObjectsInfoArrayIndex : FDependencyInfo::InvalidCacheIndex;
		if (ArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			return &DependentObjectInfosStorage[ArrayIndex];
		}
		return nullptr;
	}

	TArrayView<const FDependentObjectInfo> GetDependentObjectInfoArray(FInternalNetRefIndex InternalIndex) const
	{
		const FDependencyInfo* Entry = DependencyInfos.Find(InternalIndex);
		const uint32 ArrayIndex = Entry ? Entry->DependentObjectsInfoArrayIndex : FDependencyInfo::InvalidCacheIndex;
		if (ArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			return MakeArrayView(DependentObjectInfosStorage[ArrayIndex]);
		}
		return MakeArrayView<const FDependentObjectInfo>(nullptr, 0);
	}

	FSubObjectConditionalsArray& GetOrCreateSubObjectConditionalsArray(FInternalNetRefIndex InternalIndex);

	FInternalNetIndexArray& GetOrCreateInternalChildSubObjectsArray(FInternalNetRefIndex InternalIndex, FSubObjectConditionalsArray*& OutSubObjectConditionals);

	bool GetInternalChildSubObjectAndConditionalArrays(FInternalNetRefIndex InternalIndex, FInternalNetIndexArray*& OutChildSubObjectsArray, FSubObjectConditionalsArray*& OutSubObjectConditionals)
	{
		const FDependencyInfo* Entry = DependencyInfos.Find(InternalIndex);
		const uint32 ArrayIndex = Entry ? Entry->ArrayIndices[ChildSubObjects] : FDependencyInfo::InvalidCacheIndex;

		if (ArrayIndex == FDependencyInfo::InvalidCacheIndex)
		{
			return false;
		}

		OutChildSubObjectsArray = &DependentObjectsStorage[ArrayIndex];
		OutSubObjectConditionals = Entry->SubObjectConditionalArrayIndex != FDependencyInfo::InvalidCacheIndex ? &SubObjectConditionalsStorage[Entry->SubObjectConditionalArrayIndex] : nullptr;
		return true;
	}

	bool GetChildSubObjects(FInternalNetRefIndex InternalIndex, FChildSubObjectsInfo& OutInfo) const
	{
		const FDependencyInfo* Entry = DependencyInfos.Find(InternalIndex);
		const uint32 ArrayIndex = Entry ? Entry->ArrayIndices[ChildSubObjects] : FDependencyInfo::InvalidCacheIndex;
		if (ArrayIndex == FDependencyInfo::InvalidCacheIndex)
		{
			return false;
		}

		OutInfo.ChildSubObjects = DependentObjectsStorage[ArrayIndex].GetData();			
		OutInfo.SubObjectLifeTimeConditions = Entry->SubObjectConditionalArrayIndex != FDependencyInfo::InvalidCacheIndex ? SubObjectConditionalsStorage[Entry->SubObjectConditionalArrayIndex].GetData() : nullptr;
		OutInfo.NumSubObjects = DependentObjectsStorage[ArrayIndex].Num();
		return true;
	}

	FCreationDependentInfoArray& GetOrCreateCreationDependentInfoArray(FInternalNetRefIndex ParentIndex);

	void FreeCreationDependentInfoArray(FInternalNetRefIndex ParentIndex);

	FCreationDependentInfoArray* GetCreationDependentInfoArray(FInternalNetRefIndex ParentIndex)
	{
		const FDependencyInfo* Entry = DependencyInfos.Find(ParentIndex);
		const uint32 ArrayIndex = Entry ? Entry->CreationDependentArrayIndex : FDependencyInfo::InvalidCacheIndex;
		if (ArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			return &CreationDependentInfosStorage[ArrayIndex];
		}
		return nullptr;
	}

	const FCreationDependentInfoArray* GetCreationDependentInfoArray(FInternalNetRefIndex ParentIndex) const
	{
		return const_cast<FNetDependencyData*>(this)->GetCreationDependentInfoArray(ParentIndex);
	}

	/** Create or return the creation dependency list of an object */
	FCreationDependencyInfoArray& GetOrCreateCreationDependencyInfoArray(FInternalNetRefIndex ChildIndex);
	
	/** Liberate the creation dependency list assigned to an object */
	void FreeCreationDependencyInfoArray(FInternalNetRefIndex ChildIndex);

	FCreationDependencyInfoArray* GetCreationDependencyInfoArray(FInternalNetRefIndex ChildIndex)
	{
		const FDependencyInfo* Entry = DependencyInfos.Find(ChildIndex);
		const uint32 ArrayIndex = Entry ? Entry->CreationDependencyArrayIndex : FDependencyInfo::InvalidCacheIndex;
		if (ArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			return &CreationDependencyInfosStorage[ArrayIndex];
		}
		return nullptr;
	}

	const FCreationDependencyInfoArray* GetCreationDependencyInfoArray(FInternalNetRefIndex ChildIndex) const
	{
		return const_cast<FNetDependencyData*>(this)->GetCreationDependencyInfoArray(ChildIndex);
	}

	template<EArrayType TypeIndex>
	FInternalNetIndexArray* GetInternalIndexArray(FInternalNetRefIndex InternalIndex)
	{
		static_assert(TypeIndex != EArrayType::Count, "Invalid array type index");
		const FDependencyInfo* Entry = DependencyInfos.Find(InternalIndex);
		const uint32 ArrayIndex = Entry ? Entry->ArrayIndices[TypeIndex] : FDependencyInfo::InvalidCacheIndex;
		if (ArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			return &DependentObjectsStorage[ArrayIndex];
		}
		return nullptr;
	}

	template<EArrayType TypeIndex>
	TArrayView<const FInternalNetRefIndex> GetInternalIndexArray(FInternalNetRefIndex InternalIndex) const
	{
		static_assert(TypeIndex != EArrayType::Count, "Invalid array type index");

		const FDependencyInfo* Entry = DependencyInfos.Find(InternalIndex);
		const uint32 ArrayIndex = Entry ? Entry->ArrayIndices[TypeIndex] : FDependencyInfo::InvalidCacheIndex;
		if (ArrayIndex != FDependencyInfo::InvalidCacheIndex)
		{
			return MakeArrayView(DependentObjectsStorage[ArrayIndex]);
		}
		return MakeArrayView<const FInternalNetRefIndex>(nullptr, 0);
	}

	void FreeStoredDependencyDataForObject(FInternalNetRefIndex InternalIndex);
	
private:
	
	struct FDependencyInfo
	{
		constexpr static uint32 InvalidCacheIndex = ~(0U);
		uint32 ArrayIndices[EArrayType::Count] = { InvalidCacheIndex, InvalidCacheIndex, InvalidCacheIndex, InvalidCacheIndex };
		uint32 SubObjectConditionalArrayIndex = InvalidCacheIndex;
		uint32 DependentObjectsInfoArrayIndex = InvalidCacheIndex;
		uint32 CreationDependentArrayIndex = InvalidCacheIndex;
		uint32 CreationDependencyArrayIndex = InvalidCacheIndex;
	};

private:
	FInternalNetIndexArray& GetOrCreateInternalIndexArray(FInternalNetRefIndex InternalIndex, EArrayType Type);
	FDependencyInfo& GetOrCreateCacheEntry(FInternalNetRefIndex InternalIndex);
	
	// Map to track the replicated objects with subObjects or dependencies
	TMap<FInternalNetRefIndex, FDependencyInfo> DependencyInfos;

	// Storage for DependentObjects and SubObjects
	TSparseArray<FInternalNetIndexArray> DependentObjectsStorage;

	// Storage for SubObject conditionals
	TSparseArray<FSubObjectConditionalsArray> SubObjectConditionalsStorage;

	// Storage for DependentObjects traits and info (bound on the Parent)
	TSparseArray<FDependentObjectInfoArray> DependentObjectInfosStorage;

	// Storage for creation dependent infos (bound on the Parent)
	TSparseArray<FCreationDependentInfoArray> CreationDependentInfosStorage;

	// Storage for creation dependency infos (bound on the Child)
	TSparseArray<FCreationDependencyInfoArray> CreationDependencyInfosStorage;

};

} // end namespace UE::Net::Private
