// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetExports.h"
#include "Iris/PacketControl/PacketNotification.h"

namespace UE::Net::Private
{

void FNetExports::InitExportRecordForPacket()
{
	CurrentExportInfo.NetHandleExportCount = static_cast<uint32>(NetHandleExports.Count());
	CurrentExportInfo.NetTokenExportCount = static_cast<uint32>(NetTokenExports.Count());
}

void FNetExports::CommitExportsToRecord(FExportScope& ExportScope)
{
	const FNetExportContext::FBatchExports& BatchExports = ExportScope.ExportContext.GetBatchExports();
	for (FNetRefHandle Handle : BatchExports.HandlesExportedInCurrentBatch)
	{
		NetHandleExports.Enqueue(Handle);
	}
	for (FNetToken Token : BatchExports.NetTokensExportedInCurrentBatch)
	{
		NetTokenExports.Enqueue(Token);
	}
}

void FNetExports::PushExportRecordForPacket()
{
	FExportInfo Info;
	Info.NetHandleExportCount = static_cast<uint32>(NetHandleExports.Count()) - CurrentExportInfo.NetHandleExportCount;
	Info.NetTokenExportCount = static_cast<uint32>(NetTokenExports.Count()) - CurrentExportInfo.NetTokenExportCount;
	ExportRecord.Enqueue(Info);
}

void FNetExports::AcknowledgeBatchExports(const FNetExportContext::FBatchExports& BatchExports)
{
	AcknowledgedExports.AcknowledgedExportedHandles.Append(BatchExports.HandlesExportedInCurrentBatch);
	AcknowledgedExports.AcknowledgedExportedNetTokens.Append(BatchExports.NetTokensExportedInCurrentBatch);
}

FNetExports::FExportInfo FNetExports::PopExportRecord()
{
	check(ExportRecord.Count());

	const FExportInfo ExportInfo = ExportRecord.Peek();
	ExportRecord.Pop();

	return ExportInfo;
}

void FNetExports::ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status)
{
	FExportInfo ExportInfo = PopExportRecord();
	if (Status == UE::Net::EPacketDeliveryStatus::Delivered)
	{
		uint32 NetHandleExportCount = ExportInfo.NetHandleExportCount;
		while (NetHandleExportCount)
		{
			FNetRefHandle Handle = NetHandleExports.PeekNoCheck();
			AcknowledgedExports.AcknowledgedExportedHandles.Add(Handle);
			NetHandleExports.PopNoCheck();
			--NetHandleExportCount;
		}
		uint32 NetTokenExportCount = ExportInfo.NetTokenExportCount;
		while (NetTokenExportCount)
		{
			FNetToken Token = NetTokenExports.PeekNoCheck();
			AcknowledgedExports.AcknowledgedExportedNetTokens.Add(Token);
			NetTokenExports.PopNoCheck();
			--NetTokenExportCount;
		}
	}
	else
	{
		NetHandleExports.PopNoCheck(ExportInfo.NetHandleExportCount);
		NetTokenExports.PopNoCheck(ExportInfo.NetTokenExportCount);
	}
}

}
