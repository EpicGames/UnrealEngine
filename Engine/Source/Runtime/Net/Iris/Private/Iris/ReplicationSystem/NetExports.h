// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "Iris/Serialization/NetExportContext.h"

namespace UE::Net
{
	enum class EPacketDeliveryStatus : uint8;
}

namespace UE::Net::Private
{

class FNetExports
{
public:

	// Simple scope to make sure we set the correct ExportContext and restore the old one when we exit the scope
	class FExportScope
	{
	public:
		~FExportScope();

	private:
		friend class FNetExports;

		FExportScope(FNetSerializationContext& InContext, const FNetExportContext::FAcknowledgedExports& InAcknowledgedExports, FNetExportContext::FBatchExports& BatchExports);

		FNetExportContext ExportContext;
		FNetSerializationContext& Context;
		FNetExportContext* OldExportContext;
	};

public:

	// Call at the beginning of the packet to store the current state of exports
	void InitExportRecordForPacket();

	// Commit exports from exported during current batch
	void CommitExportsToRecord(FExportScope& ExportScope);

	// Call at the end of the packet to store record of any new exports committed during the frame
	void PushExportRecordForPacket();

	void ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status);

	// Explicitly acknowledge exports, this is used for exports originating from out of band batches 
	void AcknowledgeBatchExports(const FNetExportContext::FBatchExports& BatchExports);

	// Construct a new export scope
	FExportScope MakeExportScope(UE::Net::FNetSerializationContext& Context, FNetExportContext::FBatchExports& BatchExports) { return FExportScope(Context, AcknowledgedExports, BatchExports); }

private:

	struct FExportInfo
	{
		uint32 NetHandleExportCount;
		uint32 NetTokenExportCount;
	};

	FExportInfo PopExportRecord();

private:

	TResizableCircularQueue<FExportInfo> ExportRecord;
	TResizableCircularQueue<FNetRefHandle> NetHandleExports;
	TResizableCircularQueue<FNetToken> NetTokenExports;

	FExportInfo CurrentExportInfo;

	// Export state
	FNetExportContext::FAcknowledgedExports AcknowledgedExports;
};

inline FNetExports::FExportScope::FExportScope(FNetSerializationContext& InContext, const FNetExportContext::FAcknowledgedExports& AcknowledgedExports, FNetExportContext::FBatchExports& BatchExports)
	: ExportContext(AcknowledgedExports, BatchExports)
	, Context(InContext)
	, OldExportContext(InContext.GetExportContext())
{
	Context.SetExportContext(&ExportContext);
}

inline FNetExports::FExportScope::~FExportScope()
{
	Context.SetExportContext(OldExportContext);
}


}
