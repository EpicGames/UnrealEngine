// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetObjectFactory.h"

#include "Iris/Core/IrisLog.h"

#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"

#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializationContext.h"

#include "UObject/RemoteExecutor.h"

// Define this on both client and server to add extra header information and quickly detect errors in the header serialization/deserialization code

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetObjectFactory)
#ifndef IRIS_CREATIONHEADER_BITGUARD
	#define IRIS_CREATIONHEADER_BITGUARD 0
#endif

namespace UE::Net::Private
{
	static const FName CreateAndFillHeaderTransactionName("UNetObjectFactory::CreateAndFillHeader");
}


//------------------------------------------------------------------------
// UNetObjectFactory
//------------------------------------------------------------------------

void UNetObjectFactory::Init(UE::Net::FNetObjectFactoryId InId, UObjectReplicationBridge* InBridge)
{
	FactoryId = InId;
	Bridge = InBridge;

	OnInit();
}

void UNetObjectFactory::Deinit()
{
	OnDeinit();

	Bridge = nullptr;
}

void UNetObjectFactory::PostReceiveUpdate()
{
	OnPostReceiveUpdate();
}

TUniquePtr<UE::Net::FNetObjectCreationHeader> UNetObjectFactory::CreateHeader(UE::Net::FNetRefHandle Handle, UE::Net::FReplicationProtocolIdentifier ProtocolId)
{
	using namespace UE::Net;

	const bool bIsValid = Bridge->IsReplicatedHandle(Handle);
	if (UNLIKELY(!bIsValid))
	{
		ensureMsgf(bIsValid, TEXT("%s::CreateLocalHeader received invalid or non-replicated handle: %s"), *GetNameSafe(GetClass()), *Handle.ToString());
		return nullptr;
	}

	// Ask the derived class to allocate and fill the header
	TUniquePtr<FNetObjectCreationHeader> Header;
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	if (UE::Net::Private::FBridgeSerialization::IsReplicatingTransactionally(Bridge) && !AutoRTFM::IsTransactional())
	{
		UE::RemoteExecutor::ExecuteTransactional(UE::Net::Private::CreateAndFillHeaderTransactionName, [this, &Header, Handle]()
			{
				Header = CreateAndFillHeader(Handle);
			});
	}
	else
#endif
	{
		Header = CreateAndFillHeader(Handle);
	}

	if (LIKELY(Header.IsValid()))
	{
		Header->SetProtocolId(ProtocolId);
		Header->SetFactoryId(FactoryId);
		return Header;
	}

	return nullptr;
}

bool UNetObjectFactory::WriteHeader(UE::Net::FNetRefHandle Handle, UE::Net::FNetSerializationContext& Serialization, const UE::Net::FNetObjectCreationHeader* Header)
{
	using namespace UE::Net;

	FNetBitStreamWriter* Writer = Serialization.GetBitStreamWriter();
	check(Writer);

	check(Header->GetNetFactoryId() == FactoryId);
	check(FactoryId != InvalidNetObjectFactoryId);

	// FactoryID is always serialized first since the bridge will read it first to find the right factory 
	Writer->WriteBits(Header->GetNetFactoryId(), FNetObjectFactoryRegistry::GetMaxBits());
	Writer->WriteBits(Header->GetProtocolId(), 32);

#if IRIS_CREATIONHEADER_BITGUARD
	// Preserialize bits that will hold the header bit size
	const uint32 StartPos = Writer->GetPosBits();
	Writer->WriteBits(0, 32);
#endif

	
	const bool bSuccess = SerializeHeader(FCreationHeaderContext(Handle, Bridge, this, Serialization), Header);

#if IRIS_CREATIONHEADER_BITGUARD
	if (bSuccess && !Writer->IsOverflown())
	{
		// Go back and write the final header bit size
		const uint32 BitsWritten = Writer->GetPosBits() - StartPos;
		FNetBitStreamWriteScope WriteScope(*Writer, StartPos);
		Writer->WriteBits(BitsWritten, 32);
	}
#endif

	return bSuccess && !Writer->IsOverflown();
}

TUniquePtr<UE::Net::FNetObjectCreationHeader> UNetObjectFactory::ReadHeader(UE::Net::FNetRefHandle Handle, UE::Net::FNetSerializationContext& Serialization)
{
	using namespace UE::Net;

	FNetBitStreamReader* Reader = Serialization.GetBitStreamReader();
	check(Reader);

	// FactoryId was already read by the Bridge
	const FReplicationProtocolIdentifier ProtocolId = Reader->ReadBits(32);

#if IRIS_CREATIONHEADER_BITGUARD
	// Find the amount of bits we are expected to read
	const uint32 StartPos = Reader->GetPosBits();
	const uint32 ExpectedReadBits = Reader->ReadBits(32);
#endif

	TUniquePtr<FNetObjectCreationHeader> Header = CreateAndDeserializeHeader(FCreationHeaderContext(Handle, Bridge, this, Serialization));	

#if IRIS_CREATIONHEADER_BITGUARD
	const uint32 ActualReadBits = Reader->GetPosBits() - StartPos;
	if (ActualReadBits != ExpectedReadBits)
	{
		Reader->DoOverflow();
		UE_LOGF(LogIris, Error, "Found deserialization error in %ls for %ls. Header: %ls. Source wrote %u bits but we read %u bits (delta: %d)", *GetName(), *Handle.ToString(), Header.IsValid() ? *Header->ToString() : TEXT("invalid"), ExpectedReadBits, ActualReadBits, (ActualReadBits-ExpectedReadBits));
		ensureMsgf(ActualReadBits == ExpectedReadBits, TEXT("Found deserialization error in %s for %s"), *GetName(), *Handle.ToString());
		return nullptr;
	}
#endif

	if (LIKELY(Header.IsValid() && !Reader->IsOverflown()))
	{
		Header->SetFactoryId(FactoryId);
		Header->SetProtocolId(ProtocolId);
		return Header;
	}

	return nullptr;
}

//------------------------------------------------------------------------

const TCHAR* LexToString(UE::Net::EDetachReason Reason)
{
	using namespace UE::Net;

	switch (Reason)
	{
		case EDetachReason::Stopped:
		{
			return TEXT("Stopped");
		}

		case EDetachReason::NoLongerRelevant:
		{
			return TEXT("NoLongerRelevant");
		}

		case EDetachReason::StaticDestroyed:
		{
			return TEXT("StaticDestroyed");
		}

		case EDetachReason::TornOff:
		{
			return TEXT("TornOff");
		}

		case EDetachReason::RootObjectStillRelevant:
		{
			return TEXT("RootObjectStillRelevant");
		}

		default:
		{
			ensure(false);
			return TEXT("Missing");
		}
	}
}
