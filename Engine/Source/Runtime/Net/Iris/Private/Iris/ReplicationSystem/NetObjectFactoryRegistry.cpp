// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"

#include "Iris/ReplicationSystem/NetObjectFactory.h"
#include "Iris/Core/IrisLog.h"

namespace UE::Net::Private
{
	bool bIsFactoryRegistrationAllowed = true;
} // end namespace UE::Net::Private

namespace UE::Net 
{

TArray<FNetObjectFactoryRegistry::FFactoryData, TFixedAllocator<FNetObjectFactoryRegistry::MaxFactories>>  FNetObjectFactoryRegistry::NetFactories;

void FNetObjectFactoryRegistry::SetFactoryRegistrationAllowed(bool bAllowed)
{
	UE::Net::Private::bIsFactoryRegistrationAllowed = bAllowed;
}

void FNetObjectFactoryRegistry::RegisterFactory(UClass* FactoryClass, FName FactoryName)
{
	using namespace UE::Net::Private;

	// Factories cannot be modified while Iris NetDrivers exist
	if (!bIsFactoryRegistrationAllowed)
	{
		UE_LOGF(LogIris, Error, "FNetObjectFactoryRegistry::RegisterFactory cannot register factory: %ls name: %ls because it was called while Iris replication was already started", *GetNameSafe(FactoryClass), *FactoryName.ToString());
		check(bIsFactoryRegistrationAllowed);
		return;
	}

	check(FactoryClass);

	if (FactoryName.IsNone())
	{
		checkf(!FactoryName.IsNone(), TEXT("FNetObjectFactoryRegistry::RegisterFactory cannot register %s due to invalid name"), *GetNameSafe(FactoryClass));
		return;
	}

	if (NetFactories.Num() >= MaxFactories)
	{
		UE_LOGF(LogIris, Error, "FNetObjectFactoryRegistry::RegisterFactory already has %u factories registered. Cannot register factory: %ls with name: %ls", MaxFactories, *GetNameSafe(FactoryClass), *FactoryName.ToString());
		ensure(false);
		return;
	}

	// Make sure the name is unique
	for (const FFactoryData& FactoryData : NetFactories)
	{
		if (FactoryData.Name == FactoryName)
		{
			UE_LOGF(LogIris, Error, "FNetObjectFactoryRegistry::RegisterFactory cannot register factory: %ls with name: %ls. This name is already used by factory: %ls id: %u", *GetNameSafe(FactoryClass), *FactoryName.ToString(), *GetNameSafe(FactoryData.NetFactoryClass.Get()), FactoryData.Id);
			ensure(false);
			return;
		}
	}

	// Make sure the class is of the correct type
	if (!FactoryClass->IsChildOf<UNetObjectFactory>())
	{
		checkf(FactoryClass->IsChildOf<UNetObjectFactory>(), TEXT("FNetObjectFactoryRegistry::RegisterFactory factory: %s name: %s is not derived from UNetObjectFactory."), *GetNameSafe(FactoryClass), *FactoryName.ToString());
		return;
	}

	// Find an invalid factory entry
	int32 Index = NetFactories.IndexOfByPredicate([](const FFactoryData& rhs)
	{
		return rhs.Name == NAME_None;
	});

	if (Index != INDEX_NONE)
	{
		NetFactories[Index] = FFactoryData
		{
			.Name = FactoryName,
			.Id = IntCastChecked<FNetObjectFactoryId>(Index),
			.NetFactoryClass = FactoryClass,
		};
	}
	else
	{
		Index = NetFactories.Emplace(FFactoryData
		{
			.Name = FactoryName,
			.NetFactoryClass = FactoryClass,
		});
		NetFactories[Index].Id = IntCastChecked<FNetObjectFactoryId>(Index);
	}

	UE_LOGF(LogIris, Verbose, "FNetObjectFactoryRegistry::RegisterFactory has registered factory: %ls name: %ls id: %u", *GetNameSafe(NetFactories[Index].NetFactoryClass.Get()), *NetFactories[Index].Name.ToString(), NetFactories[Index].Id);
}

void FNetObjectFactoryRegistry::UnregisterFactory(FName FactoryName)
{
	using namespace UE::Net::Private;

	// Factories cannot be modified while Iris NetDrivers exist
	if (!bIsFactoryRegistrationAllowed)
	{
		UE_LOGF(LogIris, Error, "FNetObjectFactoryRegistry::UnregisterFactory cannot unregister factory name: %ls because it was called while Iris replication was already started", *FactoryName.ToString());
		check(bIsFactoryRegistrationAllowed);
		return;
	}

	for (int32 Index=0; Index < NetFactories.Num(); ++Index)
	{
		const FFactoryData& Data = NetFactories[Index];
		if (Data.Name == FactoryName)
		{
			UE_LOGF(LogIris, Verbose, "FNetObjectFactoryRegistry::UnregisterFactory is unregistering factory: %ls name: %ls id: %u", *GetNameSafe(Data.NetFactoryClass.Get()), *Data.Name.ToString(), Data.Id);
			
			// Reset the entry to keep the Id's mapped to their index
			NetFactories[Index] = FFactoryData{};
			return;
		}
	}

	UE_LOGF(LogIris, Error, "FNetObjectFactoryRegistry::UnregisterFactory could not find any factories using name: %ls", *FactoryName.ToString());
	ensure(false);
}
	  
FNetObjectFactoryId FNetObjectFactoryRegistry::GetFactoryIdFromName(FName FactoryName)
{
	for (const FFactoryData& Data : NetFactories)
	{
		if (Data.Name == FactoryName)
		{
			return Data.Id;
		}
	}

	return InvalidNetObjectFactoryId;
}

bool FNetObjectFactoryRegistry::IsValidFactoryId(FNetObjectFactoryId Id)
{
	if (NetFactories.IsValidIndex(Id))
	{
		return !NetFactories[Id].Name.IsNone();
	}

	return false;
}

} // end namespace UE::Net