// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"

namespace UE::Net
{

FString FNetRefHandle::ToString() const
{
	if (IsCompleteHandle())
	{
		const uint32 ReplicationSystemIdToDisplay = GetReplicationSystemId();
		return FString::Printf(TEXT("NetRefHandle (Id=%" UINT64_FMT "):(RepSystemId=%u)"), GetId(), ReplicationSystemIdToDisplay);
	}
	else
	{
		return FString::Printf(TEXT("NetRefHandle (Id=%" UINT64_FMT "):(RepSystemId=?)"), GetId());
	}
}

FString FNetRefHandle::ToCompactString() const
{
	if (IsCompleteHandle())
	{
		const uint32 ReplicationSystemIdToDisplay = GetReplicationSystemId();
		return FString::Printf(TEXT("%" UINT64_FMT ":%u"), GetId(), ReplicationSystemIdToDisplay);
	}
	else
	{
		return FString::Printf(TEXT("%" UINT64_FMT ":?"), GetId());
	}
}

}

FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const UE::Net::FNetRefHandle& NetRefHandle) 
{ 	
	if (NetRefHandle.IsCompleteHandle())
	{
		const uint32 ReplicationSystemIdToDisplay = NetRefHandle.GetReplicationSystemId();
		Builder.Appendf(TEXT("NetRefHandle (Id=%" UINT64_FMT "):(RepSystemId=%u)"), NetRefHandle.GetId(), ReplicationSystemIdToDisplay);
	}
	else
	{
		Builder.Appendf(TEXT("NetRefHandle (Id=%" UINT64_FMT "):(RepSystemId=?)"), NetRefHandle.GetId());
	}

	return Builder;
}

FAnsiStringBuilderBase& operator<<(FAnsiStringBuilderBase& Builder, const UE::Net::FNetRefHandle& NetRefHandle)
{
	if (NetRefHandle.IsCompleteHandle())
	{
		const uint32 ReplicationSystemIdToDisplay = NetRefHandle.GetReplicationSystemId();
		Builder.Appendf("NetRefHandle (Id=%" UINT64_FMT "):(RepSystemId=%u)", NetRefHandle.GetId(), ReplicationSystemIdToDisplay);
	}
	else
	{
		Builder.Appendf("NetRefHandle (Id=%" UINT64_FMT "):(RepSystemId=?)", NetRefHandle.GetId());
	}

	return Builder;
}

FArchive& operator<<(FArchive& Ar, UE::Net::FNetRefHandle& RefHandle)
{
	using namespace UE::Net;

	if (Ar.IsSaving())
	{
		bool bIsValid = RefHandle.IsValid();
		Ar.SerializeBits(&bIsValid, 1U);
		if (bIsValid)
		{
			uint64 IdBits = RefHandle.GetId();
			Ar.SerializeIntPacked64(IdBits);
		}
	}
	else if (Ar.IsLoading())
	{
		FNetRefHandle Handle;

		bool bIsValid = false;
		Ar.SerializeBits(&bIsValid, 1U);
		if (bIsValid)
		{
			uint64 NetId = 0U;
			Ar.SerializeIntPacked64(NetId);		
			if (!Ar.IsError())
			{
				Handle = Private::FNetRefHandleManager::MakeNetRefHandleFromId(NetId);
			}
		}

		RefHandle = Handle;
	}
	return Ar;
}
