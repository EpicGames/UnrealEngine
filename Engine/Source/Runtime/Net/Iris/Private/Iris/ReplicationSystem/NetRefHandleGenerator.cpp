// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetRefHandleGenerator.h"
#include "Iris/Core/BitTwiddling.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"

void URemoteObjectNetRefHandleGenerator::Bind(UE::Net::FNetRefHandleGeneratorBindParams& Params)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (ensure(Params.ObjectToNetRefHandleSerial))
	{
		Params.ObjectToNetRefHandleSerial->BindLambda([](const TObjectPtr<const UObject>& ObjectPtr)
		{
			const FRemoteObjectId RemoteObjectId = ObjectPtr.GetRemoteId();
			const uint64 RemoteObjectIdValue = RemoteObjectId.GetIdNumber();

			// Shift away the reserved bits in FRemoteObjectId so we're just left with the server id and serial number
			// which will be embedded in the NetRefHandle serial.
			const uint64 Serial = (RemoteObjectIdValue >> REMOTE_OBJECT_RESERVED_BIT_SIZE);

			UE_LOGF(LogIris, Verbose, "URemoteObjectNetRefHandleGenerator: Mapping FNetRefHandle from FRemoteObjectId for object %ls", *ObjectPtr->GetName());

			return Serial;
		});
	}

	if (ensure(Params.NetRefHandleSerialToObject))
	{
		Params.NetRefHandleSerialToObject->BindLambda([](const uint64 Serial)
		{
			// Shift left to made room for the reserved bits on FRemoteObjectId.
			const uint64 RemoteObjectIdValue = (Serial << REMOTE_OBJECT_RESERVED_BIT_SIZE);
			const FRemoteObjectId RemoteObjectId = FRemoteObjectId::CreateFromInt(RemoteObjectIdValue).GetLocalized();

			return TObjectPtr<UObject>(FObjectPtr(RemoteObjectId));
		});
	}
#else
	ensureMsgf(0, TEXT("URemoteObjectNetRefHandleGenerator: Requires UE_WITH_REMOTE_OBJECT_HANDLE to be defined."));
	UE_LOGF(LogIris, Error, "URemoteObjectNetRefHandleGenerator: Requires UE_WITH_REMOTE_OBJECT_HANDLE to be defined.");
#endif
}

void UMultiServerNetRefHandleGenerator::Bind(UE::Net::FNetRefHandleGeneratorBindParams& Params)
{
	NextSerial = InitialSerial;

	const uint64 LocalNumServerIdBits = NumServerIdBits;
	const uint32 LocalServerId = ServerId;

	if (LocalNumServerIdBits <= 0 || NumServerIdBits >= UE_NET_IRIS_NETREFHANDLE_SERIAL_SIZE)
	{
		UE_LOGF(LogIris, Error, "UMultiServerNetRefHandleGenerator: An invalid value of NumServerIdBits was provided.");
		return;
	}

	const uint64 NumSerialBits = UE_NET_IRIS_NETREFHANDLE_SERIAL_SIZE - NumServerIdBits;
	if (NumSerialBits <= 0)
	{
		UE_LOGF(LogIris, Error, "UMultiServerNetRefHandleGenerator: No space has been reserved for the serial number.")
		return;
	}

	check(UE_NET_IRIS_NETREFHANDLE_SERIAL_SIZE <= 64);
	check(LocalNumServerIdBits + NumSerialBits == UE_NET_IRIS_NETREFHANDLE_SERIAL_SIZE);

	const uint64 LargestServerId = UE::Net::LargestBitValueForBits(LocalNumServerIdBits);
	const uint64 LargestSerial = UE::Net::LargestBitValueForBits(NumSerialBits);

	if (!ensureMsgf(ServerId < LargestServerId, TEXT("UMultiServerNetRefHandleGenerator: The server id %u is larger than the largest supported value of %llu"), ServerId, LargestServerId))
	{
		return;
	}

	UE_LOGF(LogIris, Log, "UMultiServerNetRefHandleGenerator: ServerId=%u NumServerIdBits=%llu InitialSerial=%llu", ServerId, LocalNumServerIdBits, NextSerial);

	if (ensure(Params.ObjectToNetRefHandleSerial))
	{
		Params.ObjectToNetRefHandleSerial->BindLambda([this, LargestSerial, LocalNumServerIdBits, LocalServerId](const TObjectPtr<const UObject>& ObjectPtr)
		{
			const uint64 Serial = NextSerial;

			if (!ensureMsgf(Serial < LargestSerial, TEXT("UMultiServerNetRefHandleGenerator: The serial %llu is larger than the largest supported value of %llu"), Serial, LargestSerial))
			{
				// This is equivelant to FNetRefHandle::GetInvalid().
				return 0ULL;
			}

			NextSerial++;

			const uint64 FinalNetRefHandleSerial = (Serial << LocalNumServerIdBits) | LocalServerId;

			UE_LOGF(LogIris, Verbose, "UMultiServerNetRefHandleGenerator: Assigning FNetRefHandle serial %llu (ServerId=%u Serial=%llu) for object %ls", 
				FinalNetRefHandleSerial,
				LocalServerId,
				Serial,
				*GetNameSafe(ObjectPtr));

			return FinalNetRefHandleSerial;
		});
	}
}
