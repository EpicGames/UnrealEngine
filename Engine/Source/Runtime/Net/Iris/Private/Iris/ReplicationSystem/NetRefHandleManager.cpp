// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetRefHandleManager.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"

#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetSerializationContext.h"

#include "Misc/ScopeLock.h"

#include "Net/Core/Trace/NetDebugName.h"
#include "Net/Core/Trace/NetTrace.h"

#include "ProfilingDebugging/CsvProfiler.h"

#include "ReplicationOperationsInternal.h"
#include "ReplicationProtocol.h"
#include "ReplicationProtocolManager.h"

#include "Stats/StatsMisc.h"

#include "UObject/CoreNetTypes.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include "HAL/IConsoleManager.h"

CSV_DEFINE_CATEGORY(IrisCommon, true);

namespace UE::Net::Private
{

static bool bAllowCreateNetObjectForNetRefHandlePendingDestroy = true;
static FAutoConsoleVariableRef CVarAllowCreateNetObjectForNetRefHandlePendingDestroy(
	TEXT("net.Iris.AllowCreateNetObjectForNetRefHandlePendingDestroy"),
	bAllowCreateNetObjectForNetRefHandlePendingDestroy,
	TEXT("If set to true we allow creation of a new NetObject for a NetRefHandle pending destroy that might still be replicating data to client. This is currently not fully supported and can lead to issues on the client.")
);

static int32 WarnIfPendingDestroyUpdateCount = 256;
static FAutoConsoleVariableRef CVarWarnIfPendingDestroyUpdateCount(
	TEXT("net.Iris.WarnIfPendingDestroyUpdateCount"),
	WarnIfPendingDestroyUpdateCount,
	TEXT("Output warning if an NetObject is stuck in pending destroy for more than WarnIfPendingDestroyUpdateCount updates.")
);

static_assert(GetDetachReasonBitsNeeded() <= 8U, "EInternalDetachReason is a uint8");

const TCHAR* LexToString(EInternalDetachReason DetachReason)
{
	switch (DetachReason)
	{
		case EInternalDetachReason::Normal:
			return TEXT("Normal");
		case EInternalDetachReason::NotRelevant:
			return TEXT("NotRelevant");
		case EInternalDetachReason::StaticDestroyed:
			return TEXT("StaticDestroyed");
		case EInternalDetachReason::ProxyReuse:
			return TEXT("ProxyReuse");
		case EInternalDetachReason::TornOff:
			return TEXT("TornOff");
		case EInternalDetachReason::Max:
		{
			ensureMsgf(false, TEXT("Max entry should never be used except for calculating the bits needed"));
			return TEXT("Max");
		}
		default:
		{
			ensure(false);
			return TEXT("Missing");
		};
	}
}

static bool IsValidIndex(FInternalNetRefIndex InternalIndex)
{
	return InternalIndex != InvalidInternalNetRefIndex;
}

FNetRefHandleManager::FNetRefHandleManager(FReplicationProtocolManager& InReplicationProtocolManager)
	: ReplicationProtocolManager(InReplicationProtocolManager)
{
	static_assert(InvalidInternalNetRefIndex == 0, "InvalidInternalNetRefIndex has an unexpected value");
}

void FNetRefHandleManager::Init(const FInitParams& InitParams)
{
	if (InitParams.InitialNetRefHandleIndex > 0)
	{
		NextStaticHandleIndex = InitParams.InitialNetRefHandleIndex;
	}

	if (InitParams.InitialNetRefHandleIndex > 0)
	{
		NextDynamicHandleIndex = InitParams.InitialNetRefHandleIndex;
	}

	MaxActiveObjectCount = FNetBitArray::RoundUpToMaxWordBitCount(InitParams.MaxActiveObjectCount);
	InternalNetRefIndexGrowSize = InitParams.InternalNetRefIndexGrowSize > 0 ? FNetBitArray::RoundUpToMaxWordBitCount(InitParams.InternalNetRefIndexGrowSize) : MaxActiveObjectCount;

	ReplicationSystemId = InitParams.ReplicationSystemId;

	// Must be a minimum of 1 to account for InvalidInternalNetRefIndex.
	uint32 PreAllocatedNetChunkedArrayCount = FMath::Clamp(InitParams.NetChunkedArrayCount, 1U, MaxActiveObjectCount);

	// Calculate the highest internal index possible with the current preallocated buffers. Must be a minimum of 0 to support InvalidInternalNetRefIndex.
	HighestNetChunkedArrayInternalIndex = PreAllocatedNetChunkedArrayCount - 1;

	// Initialize NetObjectList configs
	CurrentMaxInternalNetRefIndex = InitParams.InternalNetRefIndexInitSize > 0 ? FMath::Min(InitParams.InternalNetRefIndexInitSize, MaxActiveObjectCount) : MaxActiveObjectCount;
	CurrentMaxInternalNetRefIndex = FNetBitArray::RoundUpToMaxWordBitCount(CurrentMaxInternalNetRefIndex);

	UE_LOGF(LogIris, Log, "NetRefHandleManager: Configured with MaxActiveObjectCount=%d, MaxInternalNetRefIndex: %u, Grow=%u, NetChunkedArray: Init=%u|Highest=%u",
		MaxActiveObjectCount, CurrentMaxInternalNetRefIndex, InternalNetRefIndexGrowSize, PreAllocatedNetChunkedArrayCount, HighestNetChunkedArrayInternalIndex);

	// Initialize TNetChunkedArrays with PreAllocatedObjectCount
	ReplicatedObjectData = TNetChunkedArray<FReplicatedObjectData>(PreAllocatedNetChunkedArrayCount, EInitMemory::Constructor);
	ReplicatedObjectRefCount = TNetChunkedArray<uint16>(PreAllocatedNetChunkedArrayCount, EInitMemory::Zero);
	ReplicatedObjectStateBuffers = TNetChunkedArray<uint8*>(PreAllocatedNetChunkedArrayCount, EInitMemory::Zero);
	ReplicatedInstances = TNetChunkedArray<TObjectPtr<UObject>>(PreAllocatedNetChunkedArrayCount, EInitMemory::Zero);

	// For convenience we initialize ReplicatedObjectData for InvalidInternalNetRefIndex so that GetReplicatedObjectDataNoCheck returns something useful.
	ReplicatedObjectData[InvalidInternalNetRefIndex].Reset();

	// Init all NetBitArrays here
	{
		InitNetBitArray(&ScopeFrameData.CurrentFrameScopableInternalIndices);
		InitNetBitArray(&ScopeFrameData.PrevFrameScopableInternalIndices);
		InitNetBitArray(&GlobalScopableInternalIndices);
		InitNetBitArray(&RelevantObjectsInternalIndices);
		InitNetBitArray(&PolledObjectsInternalIndices);
		InitNetBitArray(&DirtyObjectsToQuantize);
		InitNetBitArray(&AssignedInternalIndices);
		InitNetBitArray(&SubObjectInternalIndices);
		InitNetBitArray(&DependentObjectInternalIndices);
		InitNetBitArray(&ObjectsWithCreationDependents);
		InitNetBitArray(&ObjectsWithCreationDependencies);
		InitNetBitArray(&ObjectsWithDirtyCreationDependencies);
		InitNetBitArray(&ObjectsWithDependentObjectsInternalIndices);
		InitNetBitArray(&DestroyedStartupObjectInternalIndices);
		InitNetBitArray(&WantToBeDormantInternalIndices);
		InitNetBitArray(&ObjectsWithPreUpdate);
		InitNetBitArray(&DormantObjectsPendingFlushNet);
		InitNetBitArray(&ObjectsWithFullPushBasedDirtiness);
	}

	// Mark the invalid index as used
	AssignedInternalIndices.SetBit(0);
}

void FNetRefHandleManager::Deinit()
{
	AssignedInternalIndices.ClearBit(0);
	AssignedInternalIndices.ForAllSetBits([this](uint32 InternalIndex) 
	{
		this->InternalDestroyNetObject(InternalIndex);
	});

	OwnedNetBitArrays.Empty();

	ensureMsgf(OnMaxInternalNetRefIndexIncreased.IsBound() == false, TEXT("FNetRefHandleManager still has delegates registered to OnMaxInternalNetRefIndexIncreased while deinitializing."));
}

void FNetRefHandleManager::InitNetBitArray(FNetBitArray* NetBitArray)
{
	NetBitArray->Init(CurrentMaxInternalNetRefIndex);
	OwnedNetBitArrays.Add(NetBitArray);
}

FInternalNetRefIndex FNetRefHandleManager::GrowNetObjectLists()
{
	check(AssignedInternalIndices.GetNumBits() == CurrentMaxInternalNetRefIndex);

	// The old max is the next available index
	const FInternalNetRefIndex NextFreeIndex = CurrentMaxInternalNetRefIndex;

	// We already are at the max, return InvalidIndex and abort.
	if (CurrentMaxInternalNetRefIndex >= MaxActiveObjectCount)
	{
		return InvalidInternalNetRefIndex;
	}

    CurrentMaxInternalNetRefIndex += InternalNetRefIndexGrowSize;

	if (CurrentMaxInternalNetRefIndex > MaxActiveObjectCount)
	{
		// Last grow possibility before we cause a critical failure
		CurrentMaxInternalNetRefIndex = MaxActiveObjectCount;
	}

	UE_LOGF(LogIris, Log, "FNetRefHandleManager::GrowNetObjectLists grew MaxInternalIndex from %u to %u (+%u)", NextFreeIndex, CurrentMaxInternalNetRefIndex, CurrentMaxInternalNetRefIndex - NextFreeIndex);

	MaxInternalNetRefIndexIncreased(CurrentMaxInternalNetRefIndex);

	return NextFreeIndex;
}

void FNetRefHandleManager::MaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalNetRefIndex)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FNetRefHandleManager_MaxInternalNetRefIndexIncreased);
	CSV_CUSTOM_STAT(IrisCommon, MaxInternalIndexIncreasedCount, 1, ECsvCustomStatOp::Accumulate);

	// Start by reallocating all the NetBitArrays we own
	for (FNetBitArray* NetBitArray : OwnedNetBitArrays)
	{
		NetBitArray->SetNumBits(NewMaxInternalNetRefIndex);
	}

	// Tell other systems to increase their lists too
	OnMaxInternalNetRefIndexIncreased.Broadcast(NewMaxInternalNetRefIndex);
}

void FNetRefHandleManager::GrowNetChunkedArrayBuffers(FInternalNetRefIndex InternalIndex)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FNetRefHandleManager_GrowNetChunkedArrayBuffers);
	CSV_CUSTOM_STAT(IrisCommon, DynamicNetChunkedArrayGrowCount, 1, ECsvCustomStatOp::Accumulate);

	// This call will add the necessary number of elements and chunks to ReplicatedObjectRefCount
	// to accommodate InternalIndex. Once this is done, we determine how many more elements could be
	// added to the array without adding a new chunk and ensure all of the other buffers have this
	// many elements. 
	// 
	// This optimization will reduce the number of calls to AddToIndexUninitialized() and AddToIndexZeroed()
	// but assumes each buffer is going to have the same number of elements all the time.
	ReplicatedObjectRefCount.AddToIndexUninitialized(InternalIndex);
	const int32 LargestIndexInCurrentChunk = ReplicatedObjectRefCount.Capacity() - 1;

	ReplicatedObjectRefCount.AddToIndexUninitialized(LargestIndexInCurrentChunk);
	ReplicatedObjectStateBuffers.AddToIndexZeroed(LargestIndexInCurrentChunk);
	ReplicatedInstances.AddToIndexZeroed(LargestIndexInCurrentChunk);
	ReplicatedObjectData.AddToIndexZeroed(LargestIndexInCurrentChunk);
		
	HighestNetChunkedArrayInternalIndex = static_cast<uint32>(LargestIndexInCurrentChunk);

	OnNetChunkedArrayIncrease.Broadcast(HighestNetChunkedArrayInternalIndex);
}

uint64 FNetRefHandleManager::GetNextNetRefHandleSerial(uint64 HandleId) const
{
	if constexpr (FNetRefHandle::MaxSerial == 0)
	{
		return 0;
	}

	uint64 NextHandleId = (HandleId + 1) % FNetRefHandle::MaxSerial;
	if (NextHandleId == 0)
	{
		++NextHandleId;
	}
	return NextHandleId;
}

FInternalNetRefIndex FNetRefHandleManager::GetNextFreeInternalIndex() const
{
	const uint32 NextFreeIndex = AssignedInternalIndices.FindFirstZero();
	return NextFreeIndex != FNetBitArray::InvalidIndex ? NextFreeIndex : InvalidInternalNetRefIndex;
}

FInternalNetRefIndex FNetRefHandleManager::InternalCreateNetObject(const FNetRefHandle NetRefHandle, const FNetHandle GlobalHandle, const FCreateNetObjectParams& Params)
{
	if (ActiveObjectCount >= MaxActiveObjectCount)
	{
		UE_LOGF(LogIris, Error, "NetRefHandleManager: Maximum active object count reached (%d/%d).", ActiveObjectCount, MaxActiveObjectCount);
		ensureMsgf(false, TEXT("NetRefHandleManager: Maximum active object count reached (%d/%d)."), ActiveObjectCount, MaxActiveObjectCount);
		return InvalidInternalNetRefIndex;
	}

	// Verify that the handle is free
	if (RefHandleToInternalIndex.Contains(NetRefHandle))
	{
		UE_LOGF(LogIris, Error, "NetRefHandleManager::InternalCreateNetObject %ls already exists", *NetRefHandle.ToString());
		ensureMsgf(false, TEXT("NetRefHandleManager::InternalCreateNetObject %s already exists"), *NetRefHandle.ToString());
		return InvalidInternalNetRefIndex;
	}

	// Verify that we are not trying to StartRepliation for an object using the same NetRefHandle as one that is pending destroy.
	for (const FPendingDestroyInfo& PendingDestroyInfo : PendingDestroyInfos)
	{
		if (PendingDestroyInfo.RefHandle == NetRefHandle)
		{
			if (ReplicatedObjectRefCount[PendingDestroyInfo.InternalIndex] > 0)
			{
				if (!bAllowCreateNetObjectForNetRefHandlePendingDestroy)
				{
					UE_LOGF(LogIris, Error, "NetRefHandleManager::InternalCreateNetObject Cannot create NetObject %ls for object already replicated as %ls marked for destroy. Only supported if using AsyncStopReplication.", *NetRefHandle.ToString(), *PrintObjectFromIndex(PendingDestroyInfo.InternalIndex));
					ensureMsgf(false, TEXT("NetRefHandleManager::InternalCreateNetObject Cannot create NetObject %s for object marked for destroy. Only supported if using AsyncStopReplication."), *NetRefHandle.ToString());
					return InvalidInternalNetRefIndex;
				}
				else
				{
					UE_LOGF(LogIris, Verbose, "NetRefHandleManager::InternalCreateNetObject Created NetObject %ls for object already replicated as %ls marked for destroy.", *NetRefHandle.ToString(), *PrintObjectFromIndex(PendingDestroyInfo.InternalIndex));	
					break;
				}
			}
		}
	}

	uint32 InternalIndex = GetNextFreeInternalIndex();

	// Try to grow the NetObjectLists if no more indexes are available.
	if (InternalIndex == InvalidInternalNetRefIndex)
	{
		InternalIndex = GrowNetObjectLists();

		// If we could not grow anymore, kill the process now. The system cannot replicate objects anymore and the game behavior is undefined.
		if (InternalIndex == InvalidInternalNetRefIndex)
		{
			UE_LOGF(LogIris, Fatal, "NetRefHandleManager: Hit the maximum limit of active replicated objects: %u. Aborting since we cannot replicate %ls", MaxActiveObjectCount, Params.ReplicationProtocol->DebugName->Name);
			return InvalidInternalNetRefIndex;
		}
	}

	UE_LOGF(LogIris, Verbose, "FNetRefHandleManager::InternalCreateNetObject: (InternalIndex: %u) (%ls)", InternalIndex, *NetRefHandle.ToString());

	// Track the largest internal index and grow internal buffers if necessary.
	if (InternalIndex > HighestNetChunkedArrayInternalIndex)
	{
		GrowNetChunkedArrayBuffers(InternalIndex);
	}

	// Store data;
	FReplicatedObjectData& Data = ReplicatedObjectData[InternalIndex];
	Data.Reset();
	Data.RefHandle = NetRefHandle;
	Data.NetHandle = GlobalHandle;
	Data.Protocol = Params.ReplicationProtocol;
	Data.bShouldPropagateChangedStates = 1U;
	Data.NetFactoryId = Params.NetFactoryId;
	Data.IrisAsyncLoadingPriority = Params.IrisAsyncLoadingPriority;

	ObjectsWithPreUpdate.ClearBit(InternalIndex);
	ObjectsWithFullPushBasedDirtiness.ClearBit(InternalIndex);
	
	ReplicatedObjectStateBuffers[InternalIndex] = nullptr;

	++ActiveObjectCount;

	// Add map entry so that we can map from NetRefHandle to InternalIndex
	RefHandleToInternalIndex.Add(NetRefHandle, InternalIndex);
		
	// Add mapping from global handle to InternalIndex to speed up lookups for ReplicationSystem public API
	if (GlobalHandle.IsValid())
	{
		NetHandleToInternalIndex.Add(GlobalHandle, InternalIndex);
	}

	// Mark Handle index as assigned and scopable for now
	AssignedInternalIndices.SetBit(InternalIndex);
	GlobalScopableInternalIndices.SetBit(InternalIndex);

	// When a handle is first created, it is not set to be a subobject
	SubObjectInternalIndices.ClearBit(InternalIndex);

	// Need a full copy if set, normally only needed for new objects.
	Data.bNeedsFullCopyAndQuantize = 1U;

	// Make sure we do a full poll of all properties the first time the object gets polled.
	Data.bWantsFullPoll = 1U;

	ReplicatedObjectRefCount[InternalIndex] = 0;

	return InternalIndex;
}

void FNetRefHandleManager::AttachInstanceProtocol(FInternalNetRefIndex InternalIndex, const FReplicationInstanceProtocol* InstanceProtocol, UObject* Instance)
{
	if (ensure((InternalIndex != InvalidInternalNetRefIndex) && InstanceProtocol))
	{
		FReplicatedObjectData& Data = ReplicatedObjectData[InternalIndex];
		Data.InstanceProtocol = InstanceProtocol;

		check(ReplicatedInstances[InternalIndex] == nullptr);
		ReplicatedInstances[InternalIndex] = Instance;

		ObjectsWithPreUpdate.SetBitValue(InternalIndex, EnumHasAnyFlags(InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::NeedsPreSendUpdate));
		ObjectsWithFullPushBasedDirtiness.SetBitValue(InternalIndex, EnumHasAnyFlags(InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::HasFullPushBasedDirtiness));
	}
}

const FReplicationInstanceProtocol* FNetRefHandleManager::DetachInstanceProtocol(FInternalNetRefIndex InternalIndex)
{
	if (ensure(InternalIndex != InvalidInternalNetRefIndex))
	{
		FReplicatedObjectData& Data = ReplicatedObjectData[InternalIndex];
		const FReplicationInstanceProtocol* InstanceProtocol = Data.InstanceProtocol;
	
		Data.InstanceProtocol = nullptr;
		ReplicatedInstances[InternalIndex] = nullptr;

		ObjectsWithPreUpdate.ClearBit(InternalIndex);
		ObjectsWithFullPushBasedDirtiness.ClearBit(InternalIndex);
		
		return InstanceProtocol;
	}

	return nullptr;
}

bool FNetRefHandleManager::HasInstanceProtocol(FInternalNetRefIndex InternalIndex) const
{
	check(InternalIndex == InvalidInternalNetRefIndex || AssignedInternalIndices.GetBit(InternalIndex));
	return ReplicatedObjectData[InternalIndex].InstanceProtocol != nullptr;
}

FNetRefHandle FNetRefHandleManager::AllocateNetRefHandle(bool bIsStatic)
{
	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock Lock(HandleSerialCriticalSection));
	uint64& NextHandleSerial = bIsStatic ? NextStaticHandleIndex : NextDynamicHandleIndex;

	FNetRefHandle NewHandle = AllocateNetRefHandleFromSerial(NextHandleSerial, bIsStatic);

	// Bump NextHandleSerial
	NextHandleSerial = GetNextNetRefHandleSerial(NextHandleSerial);

	return NewHandle;
}

FNetRefHandle FNetRefHandleManager::AllocateNetRefHandleFromSerial(uint64 Serial, bool bIsStatic)
{
	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock Lock(HandleSerialCriticalSection));
	FNetRefHandle NewHandle = MakeNetRefHandleFromSerial(Serial, bIsStatic, ReplicationSystemId);

	// Verify that the handle is free
	if (RefHandleToInternalIndex.Contains(NewHandle))
	{
		checkf(false, TEXT("FNetRefHandleManager::AllocateNetHandle - Handle %s already exists!"), *NewHandle.ToString());

		return FNetRefHandle();
	}

	return NewHandle;
}

FNetRefHandle FNetRefHandleManager::CreateNetObject(FNetRefHandle WantedHandle, FNetHandle GlobalHandle, const FCreateNetObjectParams& Params)
{
	FNetRefHandle NetRefHandle = WantedHandle;

	const FInternalNetRefIndex InternalIndex = InternalCreateNetObject(NetRefHandle, GlobalHandle, Params);
	if (InternalIndex != InvalidInternalNetRefIndex)
	{
		FReplicatedObjectData& Data = ReplicatedObjectData[InternalIndex];
		Data.bIsLocal = 1;

		const UE::Net::FReplicationProtocol* ReplicationProtocol = Params.ReplicationProtocol;
		// We're dependent on the protocol staying valid for the lifetime of the object.
		ReplicationProtocol->AddRef();
		
		// Allocate storage for outgoing data.
		ReplicatedObjectStateBuffers[InternalIndex] = AllocateStateBuffer(EStateBufferType::Outgoing, ReplicationProtocol);

		return NetRefHandle;
	}
	else
	{
		return FNetRefHandle();
	}
}

// Create NetRefHandle not owned by us
FNetRefHandle FNetRefHandleManager::CreateNetObjectFromRemote(FNetRefHandle WantedHandle, const FCreateNetObjectParams& Params)
{
	if (!ensureMsgf(WantedHandle.IsValid() && !WantedHandle.IsCompleteHandle(), TEXT("FNetRefHandleManager::CreateNetObjectFromRemote Expected WantedHandle %s to be valid and incomplete"), *WantedHandle.ToString()))
	{
		return FNetRefHandle();
	}

	check(Params.NetFactoryId != UE::Net::InvalidNetObjectFactoryId);

	FNetRefHandle NetRefHandle = MakeNetRefHandle(WantedHandle.GetId(), ReplicationSystemId);

	const FInternalNetRefIndex InternalIndex = InternalCreateNetObject(NetRefHandle, FNetHandle(), Params);
	if (InternalIndex != InvalidInternalNetRefIndex)
	{
		FReplicatedObjectData& Data = ReplicatedObjectData[InternalIndex];

		const UE::Net::FReplicationProtocol* ReplicationProtocol = Params.ReplicationProtocol;
		// We're dependent on the protocol staying valid for the lifetime of the object.
		ReplicationProtocol->AddRef();

		// Allocate storage for incoming data
		Data.ReceiveStateBuffer = AllocateStateBuffer(EStateBufferType::Incoming, ReplicationProtocol);

		return NetRefHandle;
	}
	else
	{
		return FNetRefHandle();
	}
}

void FNetRefHandleManager::ResetDestroyedStartupObject(FInternalNetRefIndex InternalIndex)
{
	// Cleanup cross reference to destruction info
	if (DestroyedStartupObjectInternalIndices.GetBit(InternalIndex))
	{
		DestroyedStartupObjectInternalIndices.ClearBit(InternalIndex);
		uint32 OtherInternalIndex = 0U;
		if (DestroyedStartupObject.RemoveAndCopyValue(InternalIndex, OtherInternalIndex))
		{
			DestroyedStartupObject.Remove(OtherInternalIndex);
		}
	}
}

void FNetRefHandleManager::InternalDestroyNetObject(FInternalNetRefIndex InternalIndex)
{
	UE_LOGF(LogIris, Verbose, "FNetRefHandleManager::InternalDestroyNetObject: %ls", *PrintObjectFromIndex(InternalIndex));

	FReplicatedObjectData& Data = ReplicatedObjectData[InternalIndex];

	uint8* StateBuffer = ReplicatedObjectStateBuffers[InternalIndex];
	// Free any allocated resources
	if (EnumHasAnyFlags(Data.Protocol->ProtocolTraits, EReplicationProtocolTraits::HasDynamicState))
	{
		FNetSerializationContext FreeContext;
		FInternalNetSerializationContext InternalContext;
		FreeContext.SetInternalContext(&InternalContext);
		if (StateBuffer != nullptr)
		{
			FReplicationProtocolOperationsInternal::FreeDynamicState(FreeContext, StateBuffer, Data.Protocol);
		}
		if (Data.ReceiveStateBuffer != nullptr)
		{
			FReplicationProtocolOperationsInternal::FreeDynamicState(FreeContext, Data.ReceiveStateBuffer, Data.Protocol);
		}
	}
	
	// If this is a RootObject, remove all subobjects from the list
	if (FNetDependencyData::FInternalNetIndexArray* SubObjectArray = SubObjects.GetInternalIndexArray<FNetDependencyData::EArrayType::SubObjects>(InternalIndex))
	{
		for (FInternalNetRefIndex SubObjectInternalIndex : *SubObjectArray)
		{
			InternalRemoveSubObject(InternalIndex, SubObjectInternalIndex, false);
		}
		SubObjectArray->Reset(0U);
	}
	// Clear ChildSubObjectArray
	if (FNetDependencyData::FInternalNetIndexArray* ChildSubObjectArray = SubObjects.GetInternalIndexArray<FNetDependencyData::EArrayType::ChildSubObjects>(InternalIndex))
	{
		ChildSubObjectArray->Reset(0U);
	}

	// If we are a subobject remove from owner and hierarchical parents
	if (Data.SubObjectRootIndex != InvalidInternalNetRefIndex)
	{
		InternalRemoveSubObject(Data.SubObjectRootIndex, InternalIndex);
	}

	// Remove from all dependent object relationships and clear data
	InternalRemoveAllDependencies(InternalIndex);

	// Free all stored data for object
	SubObjects.FreeStoredDependencyDataForObject(InternalIndex);

	// Decrease protocol refcount
	Data.Protocol->Release();
	if (Data.Protocol->GetRefCount() == 0)
	{
		ReplicationProtocolManager.DestroyReplicationProtocol(Data.Protocol);
	}

	FMemory::Free(StateBuffer);
	FMemory::Free(Data.ReceiveStateBuffer);

	// Clear pointer to state buffer
	ReplicatedObjectStateBuffers[InternalIndex] = nullptr;

	// Remove cached creation header.
	if (Data.bHasCachedCreationInfo)
	{
		CachedCreationHeaders.Remove(InternalIndex);
	}

	UE_NET_TRACE_NETHANDLE_DESTROYED(Data.RefHandle);

	Data.Reset();

	// tracking
	AssignedInternalIndices.ClearBit(InternalIndex);

	// Restore internal state
	ClearStateForFreedInternalIndex(InternalIndex);

	// Cleanup cross reference to destruction info
	ResetDestroyedStartupObject(InternalIndex);

	--ActiveObjectCount;
}

void FNetRefHandleManager::ClearStateForFreedInternalIndex(FInternalNetRefIndex FreedInternalIndex)
{
	GlobalScopableInternalIndices.ClearBit(FreedInternalIndex);
	ObjectsWithPreUpdate.ClearBit(FreedInternalIndex);
	SubObjectInternalIndices.ClearBit(FreedInternalIndex);
	ObjectsWithDependentObjectsInternalIndices.ClearBit(FreedInternalIndex);
	WantToBeDormantInternalIndices.ClearBit(FreedInternalIndex);
	DormantObjectsPendingFlushNet.ClearBit(FreedInternalIndex);
	ObjectsWithFullPushBasedDirtiness.ClearBit(FreedInternalIndex);
}

FNetRefHandle FNetRefHandleManager::CreateHandleForDestructionInfo(FNetRefHandle Handle, const FCreateNetObjectParams& Params)
{
	// Create destruction info handle carrying destruction info
	constexpr bool bIsStaticHandle = false;
	FNetRefHandle AllocatedHandle = AllocateNetRefHandle(bIsStaticHandle);
	FNetRefHandle DestructionInfoHandle = CreateNetObject(AllocatedHandle, FNetHandle(), Params);

	if (DestructionInfoHandle.IsValid())
	{
		const uint32 DestructionInfoInternalIndex = GetInternalIndex(DestructionInfoHandle);
		const uint32 DestroyedInternalIndex = GetInternalIndex(Handle);
		
		// Mark the internal index as a destruction info
		GetReplicatedObjectDataNoCheck(DestructionInfoInternalIndex).bIsDestructionInfo = true;
		DestroyedStartupObjectInternalIndices.SetBit(DestructionInfoInternalIndex);
				
		// If the destroyed object is replicated we must make sure that we do not add it to scope by accident
		if (DestroyedInternalIndex)
		{			
			DestroyedStartupObject.FindOrAdd(DestructionInfoInternalIndex, DestroyedInternalIndex);
		
			// Mark the replicated index as destroyed
			DestroyedStartupObjectInternalIndices.SetBit(DestroyedInternalIndex);
			DestroyedStartupObject.FindOrAdd(DestroyedInternalIndex, DestructionInfoInternalIndex);
		}
	}

	return DestructionInfoHandle;
}

void FNetRefHandleManager::RemoveFromScope(FInternalNetRefIndex InternalIndex)
{
	// Can only remove an object from scope if it is assignable
	if (ensure(AssignedInternalIndices.GetBit(InternalIndex)))
	{
		GlobalScopableInternalIndices.ClearBit(InternalIndex);
	}
}

void FNetRefHandleManager::DestroyNetObjectByIndex(FInternalNetRefIndex InternalIndex, bool bKeepInScopeAndMarkForFlush)
{
	if (ensure(AssignedInternalIndices.GetBit(InternalIndex)))
	{
		FReplicatedObjectData& Data = ReplicatedObjectData[InternalIndex];

		// If object already is marked for flush or pending destroy, we only need to remove it from scope.
		if (!(Data.bFlush || Data.bNetObjectIsPendingDestroy))
		{
			// Just to catch misuse by bad code.
			FInternalNetRefIndex FoundInternalIndex = RefHandleToInternalIndex.FindAndRemoveChecked(Data.RefHandle);
			if (FoundInternalIndex != InternalIndex)
			{
				UE_LOGF(LogIris, Error, "FNetRefHandleManager::DestroyNetObjectByIndex: Trying to destroy NetObject with conflicting InternalIndex: ( %u ) != ( %u) . Please fix calling code.", InternalIndex, FoundInternalIndex);
				ensureMsgf(false, TEXT("Trying to destroy NetObject with conflicting InternalIndex"));
				return;
			}
			
			// Remove mapping from global handle to internal index
			NetHandleToInternalIndex.Remove(Data.NetHandle);

			// We always defer the actual destroy
			PendingDestroyInfos.Emplace(InternalIndex, Data.RefHandle, UpdatePendingDestroyCounter);

			// Before we can start replicating this RefHandle again, we need to first finish replication of previous incarnations of the object.
			InternalIndicesReplicatingNetRefHandle.FindOrAdd(Data.RefHandle).Add(InternalIndex);

			// Track that this object is marked for destroy so we can avoid calling it again
			Data.bNetObjectIsPendingDestroy = true;
		}

		if (bKeepInScopeAndMarkForFlush)
		{
			// Mark as flush
			Data.bFlush = true;
		}
		else
		{
			// Remove from scopable objects if not already done
			GlobalScopableInternalIndices.ClearBit(InternalIndex);
		}
	}
}

void FNetRefHandleManager::DestroyObjectsPendingDestroy()
{
	IRIS_PROFILER_SCOPE(FNetRefHandleManager_DestroyObjectsPendingDestroy);

	TArray<FInternalNetRefIndex> FreedInternalIndices;
	FreedInternalIndices.Reserve(PendingDestroyInfos.Num());

	// Destroy Objects pending destroy
	for (int32 It = PendingDestroyInfos.Num() - 1; It >= 0; --It)
	{
		const FInternalNetRefIndex InternalIndex = PendingDestroyInfos[It].InternalIndex;

		// Do not fully destroy objects until they
		// - are removed from GlobalScopableInternalIndices
		// - are not referenced by any ReplicationWriter
		// - do not have any subobjects
		if (!GlobalScopableInternalIndices.GetBit(InternalIndex) && ReplicatedObjectRefCount[InternalIndex] == 0 && GetSubObjects(InternalIndex).Num() <= 0)
		{
			FreedInternalIndices.Add(InternalIndex);

			InternalDestroyNetObject(InternalIndex);

			// Remove InternalIndex from array of objects replicating the same handle
			if (TArray<FInternalNetRefIndex, TInlineAllocator<2>>* Entry = InternalIndicesReplicatingNetRefHandle.Find(PendingDestroyInfos[It].RefHandle))
			{
				Entry->Remove(InternalIndex);
				if (Entry->IsEmpty())
				{
					InternalIndicesReplicatingNetRefHandle.Remove(PendingDestroyInfos[It].RefHandle);
				}
			}

			PendingDestroyInfos.RemoveAtSwap(It);
		}
		else if ((WarnIfPendingDestroyUpdateCount > 0) && uint32(UpdatePendingDestroyCounter - PendingDestroyInfos[It].PendingDestroyUpdateCount) >= uint32(WarnIfPendingDestroyUpdateCount))
		{
			UE_LOGF(LogIris, Warning, "FNetRefHandleManager::DestroyObjectsPendingDestroy: Object %ls IsReferenced %u", *PrintObjectFromIndex(InternalIndex), ReplicatedObjectRefCount[InternalIndex]);
			if (ReplicatedObjectRefCount[InternalIndex] == 0)
			{
				for (FInternalNetRefIndex SubObjectIndex : GetSubObjects(InternalIndex))
				{
					UE_LOGF(LogIris, Warning, "FNetRefHandleManager::DestroyObjectsPendingDestroy: Due to SubObjectsObject %ls IsReferenced %u", *PrintObjectFromIndex(SubObjectIndex), ReplicatedObjectRefCount[SubObjectIndex]);
				}
			}
			PendingDestroyInfos[It].PendingDestroyUpdateCount = UpdatePendingDestroyCounter;	
		}
	}
	CSV_CUSTOM_STAT(IrisCommon, PendingDestroyInternalIndicesCount, (float)PendingDestroyInfos.Num(), ECsvCustomStatOp::Set);

	if (OnInternalNetRefIndicesFreed.IsBound())
	{
		OnInternalNetRefIndicesFreed.Broadcast(MakeConstArrayView(FreedInternalIndices));
	}

	// We use a counter instead of time to avoid issues when debugging
	++UpdatePendingDestroyCounter;
}

bool FNetRefHandleManager::AddSubObject(FNetRefHandle ParentObjectHandle, FNetRefHandle SubObjectHandle, FNetRefHandle RelativeOtherSubObjectHandle, EAddSubObjectFlags Flags)
{
	check(ParentObjectHandle != SubObjectHandle);

	// validate objects
	const FInternalNetRefIndex ParentObjectInternalIndex = GetInternalIndex(ParentObjectHandle);
	const FInternalNetRefIndex SubObjectInternalIndex = GetInternalIndex(SubObjectHandle);

	const bool bIsValidParent = ensure(ParentObjectInternalIndex != InvalidInternalNetRefIndex);
	const bool bIsValidSubObject = ensure(SubObjectInternalIndex != InvalidInternalNetRefIndex);

	if (!bIsValidParent || !bIsValidSubObject)
	{
		return false;
	}

	const FInternalNetRefIndex RootObjectInternalIndex = GetRootObjectInternalIndexOfAnyObject(ParentObjectInternalIndex);

#if UE_NET_TRACE_ENABLED
	const FNetRefHandle RootObjectHandle = GetReplicatedObjectDataNoCheck(RootObjectInternalIndex).RefHandle;
	UE_NET_TRACE_SUBOBJECT(RootObjectHandle, SubObjectHandle);
#endif

	const FInternalNetRefIndex RelativeOtherSubObjectInternalIndex = EnumHasAnyFlags(Flags, EAddSubObjectFlags::InsertAtStartOrAfterOther) ? GetInternalIndex(RelativeOtherSubObjectHandle) : InvalidInternalNetRefIndex;
	return InternalAddSubObject(RootObjectInternalIndex, ParentObjectInternalIndex, SubObjectInternalIndex, RelativeOtherSubObjectInternalIndex, Flags);
}

bool FNetRefHandleManager::InternalAddSubObject(FInternalNetRefIndex RootObjectInternalIndex, FInternalNetRefIndex OptionalParentSubObjectInternalIndex, FInternalNetRefIndex SubObjectInternalIndex, FInternalNetRefIndex RelativeOtherSubObjectInternalIndex, EAddSubObjectFlags Flags)
{
	using namespace UE::Net::Private;

	FReplicatedObjectData& SubObjectData = GetReplicatedObjectDataNoCheck(SubObjectInternalIndex);
	if (!ensureMsgf(SubObjectData.SubObjectRootIndex == InvalidInternalNetRefIndex, TEXT("FNetRefHandleManager::AddSubObject %s is already marked as a subobject"), ToCStr(SubObjectData.RefHandle.ToString())))
	{
		return false;
	}

	// Add the subobject to the RootObject, this array contains all subobjects associated with the root in the order they are added
	FNetDependencyData::FInternalNetIndexArray& SubObjectArray = SubObjects.GetOrCreateInternalIndexArray<FNetDependencyData::SubObjects>(RootObjectInternalIndex);
	SubObjectArray.Add(SubObjectInternalIndex);
	
	SubObjectData.SubObjectRootIndex = RootObjectInternalIndex;

	// Mark the object as a subobject
	SetIsSubObject(SubObjectInternalIndex, true);

	FInternalNetRefIndex ParentOfSubObjectIndex = RootObjectInternalIndex;
	
	if (OptionalParentSubObjectInternalIndex != InvalidInternalNetRefIndex && OptionalParentSubObjectInternalIndex != RootObjectInternalIndex)
	{
		// Make sure the non-rootobject Parent is a subobject of the same RootObject
		const bool bIsValidOuter = SubObjectArray.Contains(OptionalParentSubObjectInternalIndex);
		if (ensureMsgf(bIsValidOuter, TEXT("ParentSubObjectInternalIndex %s must be a Subobject of %s"), *PrintObjectFromIndex(OptionalParentSubObjectInternalIndex), *PrintObjectFromIndex(RootObjectInternalIndex)))
		{
			ParentOfSubObjectIndex = OptionalParentSubObjectInternalIndex;
		}
	}

	// Add the subobject to its Parent's list which is what we use to define replication order.
	FNetDependencyData::FSubObjectConditionalsArray* SubObjectConditionalsArray = nullptr;
	FNetDependencyData::FInternalNetIndexArray& ChildSubObjectArray = SubObjects.GetOrCreateInternalChildSubObjectsArray(ParentOfSubObjectIndex, SubObjectConditionalsArray);

	if (EnumHasAnyFlags(Flags, EAddSubObjectFlags::InsertAtStart))
	{
		// Add at the start
		ChildSubObjectArray.Insert(SubObjectInternalIndex, 0);
		if (SubObjectConditionalsArray)
		{
			SubObjectConditionalsArray->Insert(ELifetimeCondition::COND_None, 0);
		}
	}
	else if (EnumHasAnyFlags(Flags, EAddSubObjectFlags::InsertAtStartOrAfterOther))
	{
		// Insert the object right after the other one, or at the start if it wasn't in the list
		const int32 RelativeIndex = RelativeOtherSubObjectInternalIndex != InvalidInternalNetRefIndex ? ChildSubObjectArray.Find(RelativeOtherSubObjectInternalIndex) : INDEX_NONE;
		const int32 InsertIndex = (RelativeIndex == INDEX_NONE) ? 0 : (RelativeIndex + 1);

		ChildSubObjectArray.Insert(SubObjectInternalIndex, InsertIndex);
		if (SubObjectConditionalsArray)
		{
			SubObjectConditionalsArray->Insert(ELifetimeCondition::COND_None, InsertIndex);
		}
	}
	else
	{
		// At at the end
		ChildSubObjectArray.Add(SubObjectInternalIndex);
		if (SubObjectConditionalsArray)
		{
			SubObjectConditionalsArray->Add(ELifetimeCondition::COND_None);
		}
	}

	if (ObjectsWithCreationDependencies.GetBit(RootObjectInternalIndex))
	{
		// Let filtering know that this subobject might be filtered by creation dependencies
		ObjectsWithDirtyCreationDependencies.SetBit(SubObjectInternalIndex);
	}
	
	SubObjectData.SubObjectParentIndex = ParentOfSubObjectIndex;
	SubObjectData.bReplicateChildSubObjectsAfterParent = EnumHasAnyFlags(Flags, EAddSubObjectFlags::ReplicateChildSubObjectsAfterParent) ? 1U : 0U;

	return true;
}

void FNetRefHandleManager::InternalRemoveSubObject(FInternalNetRefIndex RootObjectInternalIndex, FInternalNetRefIndex SubObjectInternalIndex, bool bRemoveFromSubObjectArray)
{
	// both must be valid
	if (RootObjectInternalIndex == InvalidInternalNetRefIndex || SubObjectInternalIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	FReplicatedObjectData& SubObjectData = GetReplicatedObjectDataNoCheck(SubObjectInternalIndex);
	check(SubObjectData.SubObjectRootIndex == RootObjectInternalIndex);
		
	if (bRemoveFromSubObjectArray)
	{
		// Remove subobject from the root object's list
		if (FNetDependencyData::FInternalNetIndexArray* SubObjectArray = SubObjects.GetInternalIndexArray<FNetDependencyData::EArrayType::SubObjects>(RootObjectInternalIndex))
		{
			SubObjectArray->Remove(SubObjectInternalIndex);
		}

		// Remove subobject from the parent's list
		if (SubObjectData.SubObjectParentIndex != InvalidInternalNetRefIndex)
		{
			FNetDependencyData::FInternalNetIndexArray* ChildSubObjectArray;
			FNetDependencyData::FSubObjectConditionalsArray* SubObjectConditionsArray;

			if (SubObjects.GetInternalChildSubObjectAndConditionalArrays(SubObjectData.SubObjectParentIndex, ChildSubObjectArray, SubObjectConditionsArray))
			{
				const int32 ArrayIndex = ChildSubObjectArray->Find(SubObjectInternalIndex);
				if (ensureMsgf(ArrayIndex != INDEX_NONE, TEXT("Subobject: %s not found in the list of the parent"), *PrintObjectFromIndex(SubObjectInternalIndex)))
				{
					ChildSubObjectArray->RemoveAt(ArrayIndex);
					if (SubObjectConditionsArray)
					{
						SubObjectConditionsArray->RemoveAt(ArrayIndex);
						check(SubObjectConditionsArray->Num() == ChildSubObjectArray->Num());
					}
				}
			}
		}
	}

	if (ObjectsWithCreationDependencies.GetBit(SubObjectData.SubObjectRootIndex))
	{
		// Let filtering know that creation dependency data may need to be cleared for this subobject
		ObjectsWithDirtyCreationDependencies.SetBit(SubObjectInternalIndex);
	}

	SubObjectData.SubObjectRootIndex = InvalidInternalNetRefIndex;
	SubObjectData.SubObjectParentIndex = InvalidInternalNetRefIndex;

	SetIsSubObject(SubObjectInternalIndex, false);
}

void FNetRefHandleManager::RemoveSubObject(FNetRefHandle Handle)
{
	FInternalNetRefIndex SubObjectInternalIndex = GetInternalIndex(Handle);
	checkSlow(SubObjectInternalIndex);

	if (SubObjectInternalIndex != InvalidInternalNetRefIndex)
	{
		const FInternalNetRefIndex RootObjectInternalIndex = ReplicatedObjectData[SubObjectInternalIndex].SubObjectRootIndex;
		if (RootObjectInternalIndex != InvalidInternalNetRefIndex)
		{
			InternalRemoveSubObject(RootObjectInternalIndex, SubObjectInternalIndex);
		}
	}
}

bool FNetRefHandleManager::SetSubObjectNetCondition(FInternalNetRefIndex SubObjectInternalIndex, FLifeTimeConditionStorage SubObjectCondition, bool& bOutWasModified)
{
	if (ensure(SubObjectInternalIndex != InvalidInternalNetRefIndex))
	{
		const FInternalNetRefIndex SubObjectParentIndex = ReplicatedObjectData[SubObjectInternalIndex].SubObjectParentIndex;
		if (ensure(SubObjectParentIndex != InvalidInternalNetRefIndex))
		{
			FNetDependencyData::FInternalNetIndexArray* SubObjectsArray;
			FNetDependencyData::FSubObjectConditionalsArray* SubObjectConditionals;
			
			if (SubObjects.GetInternalChildSubObjectAndConditionalArrays(SubObjectParentIndex, SubObjectsArray, SubObjectConditionals))
			{
				const int32 SubObjectArrayIndex = SubObjectsArray->Find(SubObjectInternalIndex);
				if (ensure(SubObjectArrayIndex != INDEX_NONE))
				{
					if (!SubObjectConditionals)
					{
						// No need to create up the conditionals array if we are not setting a condition.
						if (SubObjectCondition == ELifetimeCondition::COND_None)
						{
							
							bOutWasModified = false;
							return true;

						}
						SubObjectConditionals = &SubObjects.GetOrCreateSubObjectConditionalsArray(SubObjectParentIndex);
					}

					check(SubObjectConditionals->Num() == SubObjectsArray->Num());

					const FLifeTimeConditionStorage OldCondition = (*SubObjectConditionals)[SubObjectArrayIndex];
					bOutWasModified = OldCondition != SubObjectCondition;

					(*SubObjectConditionals)[SubObjectArrayIndex] = SubObjectCondition;
					
					return true;
				}
			}

		}
	}

	bOutWasModified = false;
	return false;
}

FNetRefHandle FNetRefHandleManager::GetRootObjectOfAnyObject(FNetRefHandle NetRefHandle) const
{
	const FInternalNetRefIndex InternalIndex = GetInternalIndex(NetRefHandle);
	if (!IsValidIndex(InternalIndex))
	{
		return FNetRefHandle();
	}

	// Find the rootobject for subojects, otherwise just use the passed index since its a root.
	const FInternalNetRefIndex RootObjectInternalIndex = ReplicatedObjectData[InternalIndex].IsSubObject() ? ReplicatedObjectData[InternalIndex].SubObjectRootIndex : InternalIndex;

	return IsValidIndex(RootObjectInternalIndex) ? ReplicatedObjectData[RootObjectInternalIndex].RefHandle : FNetRefHandle();
}

FNetRefHandle FNetRefHandleManager::GetRootObjectOfSubObject(FNetRefHandle SubObjectRefHandle) const
{
	const FInternalNetRefIndex SubObjectInternalIndex = GetInternalIndex(SubObjectRefHandle);
	const FInternalNetRefIndex OwnerInternalIndex = IsValidIndex(SubObjectInternalIndex) ? ReplicatedObjectData[SubObjectInternalIndex].SubObjectRootIndex : InvalidInternalNetRefIndex;

	return IsValidIndex(OwnerInternalIndex) ? ReplicatedObjectData[OwnerInternalIndex].RefHandle : FNetRefHandle();
}

FInternalNetRefIndex FNetRefHandleManager::GetRootObjectInternalIndexOfAnyObject(FInternalNetRefIndex InternalIndex) const
{
	// Find the rootobject for subojects, otherwise just use the passed index since it is a root or invalid.
	const FReplicatedObjectData& ObjectData = ReplicatedObjectData[InternalIndex];
	return ObjectData.IsSubObject() ? ObjectData.SubObjectRootIndex : InternalIndex;
}

FInternalNetRefIndex FNetRefHandleManager::GetRootObjectInternalIndexOfSubObject(FInternalNetRefIndex SubObjectIndex) const
{
	// If InvalidInternalNetRefIndex is passed we can rely on that data being default initialized so it will return InvalidInternalNetRefIndex too.
	return ReplicatedObjectData[SubObjectIndex].SubObjectRootIndex;
}

bool FNetRefHandleManager::AddDependentObject(FNetRefHandle ParentRefHandle, FNetRefHandle DependentObjectRefHandle, EDependentObjectSchedulingHint SchedulingHint)
{
	if (ParentRefHandle == DependentObjectRefHandle)
	{
		UE_LOGF(LogIris, Error, "FNetRefHandleManager::AddDependentObject: ParentObject %ls cannot be dependent on itself. Please fix calling code.", *PrintObjectFromNetRefHandle(ParentRefHandle));
		ensureMsgf(ParentRefHandle != DependentObjectRefHandle, TEXT("ParentObject %s cannot be dependent on itself."), *PrintObjectFromNetRefHandle(ParentRefHandle));
		return false;
	}

	// validate objects
	FInternalNetRefIndex ParentInternalIndex = GetInternalIndex(ParentRefHandle);
	FInternalNetRefIndex DependentObjectInternalIndex = GetInternalIndex(DependentObjectRefHandle);

	const bool bIsValidOwner = ensure(ParentInternalIndex != InvalidInternalNetRefIndex);
	const bool bIsValidDependentObject = ensure(DependentObjectInternalIndex != InvalidInternalNetRefIndex);

	if (!(bIsValidOwner && bIsValidDependentObject))
	{
		return false;
	}

	FReplicatedObjectData& DependentObjectData = GetReplicatedObjectDataNoCheck(DependentObjectInternalIndex);
	FReplicatedObjectData& ParentObjectData = GetReplicatedObjectDataNoCheck(ParentInternalIndex);

	// SubObjects cannot have dependent objects or be a dependent object (for now)
	if (DependentObjectData.IsSubObject() || ParentObjectData.IsSubObject() || SubObjectInternalIndices.GetBit(DependentObjectInternalIndex) || SubObjectInternalIndices.GetBit(ParentInternalIndex))
	{
		UE_LOGF(LogIris, Warning, "FNetRefHandleManager::AddDependentObject: SubObjects cannot have or be dependent objects. Parent: %ls child: %ls.", *PrintObjectFromNetRefHandle(ParentRefHandle), *PrintObjectFromNetRefHandle(DependentObjectRefHandle));
		ensure(!DependentObjectData.IsSubObject() && !ParentObjectData.IsSubObject());
		ensure(!SubObjectInternalIndices.GetBit(DependentObjectInternalIndex));
		ensure(!SubObjectInternalIndices.GetBit(ParentInternalIndex));
		return false;
	}

	// DebugObjects cannot have dependent objects or be a dependent object (for now)
	if (DependentObjectData.IsDebugObject() || ParentObjectData.IsDebugObject())
	{
		UE_LOGF(LogIris, Warning, "FNetRefHandleManager::AddDependentObject: DebugObjects cannot have or be dependent objects. Parent: %ls child: %ls.", *PrintObjectFromNetRefHandle(ParentRefHandle), *PrintObjectFromNetRefHandle(DependentObjectRefHandle));
		ensure(!DependentObjectData.IsDebugObject());
		ensure(!ParentObjectData.IsDebugObject());
		return false;
	}

	FNetDependencyData::FDependentObjectInfoArray& ParentDependentObjectsArray = SubObjects.GetOrCreateDependentObjectInfoArray(ParentInternalIndex);
	FNetDependencyData::FInternalNetIndexArray& DependentParentObjectArray = SubObjects.GetOrCreateInternalIndexArray<FNetDependencyData::DependentParentObjects>(DependentObjectInternalIndex);
	
	// Make sure parent didn't set the child as a dependent already
	{
		const FDependentObjectInfo* DependentInfo = ParentDependentObjectsArray.FindByPredicate([DependentObjectInternalIndex](const FDependentObjectInfo& Entry) { return Entry.NetRefIndex == DependentObjectInternalIndex;});
		if (DependentInfo)
		{
			// Make sure the children is also dependent to the Parent
			checkf(DependentParentObjectArray.Find(ParentInternalIndex) != INDEX_NONE, TEXT("FNetRefHandleManager::AddDependentObject: Parent: %s already has child: %s as dependent but not the inverse."), 
				*PrintObjectFromNetRefHandle(ParentRefHandle), *PrintObjectFromNetRefHandle(DependentObjectRefHandle));

			// If they were already dependent there is no side-effect, unless the scheduler hint would have been changed by the new call.
			UE_LOGF(LogIris, Warning, "FNetRefHandleManager::AddDependentObject: Parent: %ls already has child: %ls as a dependent", *PrintObjectFromNetRefHandle(ParentRefHandle), *PrintObjectFromNetRefHandle(DependentObjectRefHandle));
			ensureMsgf(DependentInfo->SchedulingHint == SchedulingHint, TEXT("FNetRefHandleManager::AddDependentObject: Conflicting scheduling hint between Child: %s and Parent: %s. Requested %s but was already set to %s"),
				*PrintObjectFromNetRefHandle(DependentObjectRefHandle), *PrintObjectFromNetRefHandle(ParentRefHandle), LexToString(SchedulingHint), LexToString(DependentInfo->SchedulingHint));
			return false;
		}
	}

	// If child was already set as dependent on the parent there is a logic error somewhere.
	checkf(DependentParentObjectArray.Find(ParentInternalIndex) == INDEX_NONE, TEXT("FNetRefHandleManager::AddDependentObject: Child: %s already dependent of Parent: %s but not the inverse."), *PrintObjectFromNetRefHandle(DependentObjectRefHandle), *PrintObjectFromNetRefHandle(ParentRefHandle));

	// Make sure adding this dependency wouldn't create a circular dependency chain.
	if (AddingDependencyWouldCreateCircularDependency(ParentInternalIndex, DependentObjectInternalIndex))
	{
		UE_LOGF(LogIris, Warning, "FNetRefHandleManager::AddDependentObject: Child: %ls has an implicit or direct dependency on Parent: %ls. Denying circular dependency.", *PrintObjectFromNetRefHandle(DependentObjectRefHandle), *PrintObjectFromNetRefHandle(ParentRefHandle));
		ensureMsgf(false, TEXT("FNetRefHandleManager::AddDependentObject: Child: %s has an implicit or direct dependency on Parent: %s. Denying circular dependency."), *PrintObjectFromNetRefHandle(DependentObjectRefHandle), *PrintObjectFromNetRefHandle(ParentRefHandle));
		return false;
	}

	// Add dependent to parent's dependent object list
	FDependentObjectInfo DependentObjectInfo;
	DependentObjectInfo.NetRefIndex = DependentObjectInternalIndex;
	DependentObjectInfo.SchedulingHint = SchedulingHint;
	ParentDependentObjectsArray.Add(DependentObjectInfo);

	// Add parent to dependent's list
	DependentParentObjectArray.Add(ParentInternalIndex);

	// Update cached info to avoid to do map lookups to find out if we are a dependent object or have dependent objects
	DependentObjectData.bIsDependentObject = true;
	ParentObjectData.bHasDependentObjects = true;
	ObjectsWithDependentObjectsInternalIndices.SetBit(ParentInternalIndex);
	DependentObjectInternalIndices.SetBit(DependentObjectInternalIndex);

	return true;
}

void FNetRefHandleManager::RemoveDependentObject(FNetRefHandle DependentHandle)
{
	FInternalNetRefIndex DependentInternalIndex = GetInternalIndex(DependentHandle);

	if (DependentInternalIndex != InvalidInternalNetRefIndex)
	{
		InternalRemoveAllDependencies(DependentInternalIndex);
	}
}

bool FNetRefHandleManager::AddCreationDependency(FNetRefHandle Parent, FNetRefHandle Child)
{
	if (Parent == Child)
	{
		ensureMsgf(false, TEXT("Cannot add a creation dependency on yourself: %s"), *PrintObjectFromNetRefHandle(Child));
		return false;
	}

	FInternalNetRefIndex ParentIndex = GetInternalIndex(Parent);
	FInternalNetRefIndex ChildIndex = GetInternalIndex(Child);

	return InternalAddCreationDependency(ParentIndex, ChildIndex);
}

void FNetRefHandleManager::RemoveCreationDependency(FNetRefHandle Parent, FNetRefHandle Child)
{
	FInternalNetRefIndex ParentIndex = GetInternalIndex(Parent);
	FInternalNetRefIndex ChildIndex = GetInternalIndex(Child);

	InternalRemoveCreationDependency(ParentIndex, ChildIndex);
}

bool FNetRefHandleManager::InternalAddCreationDependency(FInternalNetRefIndex ParentIndex, FInternalNetRefIndex ChildIndex)
{
	if (ParentIndex == InvalidInternalNetRefIndex || ChildIndex == InvalidInternalNetRefIndex)
	{
		ensureMsgf(false, TEXT("AddCreationDependency failed due to invalid reference. Parent: %s | Child: %s"), *PrintObjectFromIndex(ParentIndex), *PrintObjectFromIndex(ChildIndex));
		return false;
	}

	if (IsSubObject(ParentIndex) || IsSubObject(ChildIndex))
	{
		UE_LOGF(LogIris, Error, "AddCreationDependency only works on RootObjects. Parent: %ls | Child: %ls", *PrintObjectFromIndex(ParentIndex), *PrintObjectFromIndex(ChildIndex));
		ensureMsgf(false, TEXT("AddCreationDependency not supported on subobjects"));
		return false;
	}

	// Add child to parent's list.
	FNetDependencyData::FCreationDependentInfoArray& CreationDependentInfo = SubObjects.GetOrCreateCreationDependentInfoArray(ParentIndex);
	// Only add the child to the list if it's not already present.
	if (CreationDependentInfo.Find(ChildIndex) == INDEX_NONE)
	{
		CreationDependentInfo.Add(ChildIndex);
		ObjectsWithCreationDependents.SetBit(ParentIndex);
	}

	// Add parent to child's list.
	FNetDependencyData::FCreationDependencyInfoArray& CreationDependencyInfo = SubObjects.GetOrCreateCreationDependencyInfoArray(ChildIndex);
	// Only add the parent to the list if it's not already present.
	if (CreationDependencyInfo.Find(ParentIndex) == INDEX_NONE)
	{
		CreationDependencyInfo.Add(ParentIndex);
		ObjectsWithCreationDependencies.SetBit(ChildIndex);
	}

	// The child might need to become filtered by its new creation dependency, so dirty it for the creation dependency update.
	ObjectsWithDirtyCreationDependencies.SetBit(ChildIndex);

	return true;
}

void FNetRefHandleManager::InternalRemoveCreationDependency(FInternalNetRefIndex ParentIndex, FInternalNetRefIndex ChildIndex)
{
	if (ParentIndex == InvalidInternalNetRefIndex || ChildIndex == InvalidInternalNetRefIndex)
	{
		ensureMsgf(false, TEXT("RemoveCreationDependency failed due to invalid reference. Parent: %s | Child: %s"), *PrintObjectFromIndex(ParentIndex), *PrintObjectFromIndex(ChildIndex));
		return;
	}

	// Remove child from parent's list.
	FNetDependencyData::FCreationDependentInfoArray* CreationDependentInfo = SubObjects.GetCreationDependentInfoArray(ParentIndex);
	if (CreationDependentInfo)
	{
		CreationDependentInfo->Remove(ChildIndex);
		if (CreationDependentInfo->IsEmpty())
		{
			ObjectsWithCreationDependents.ClearBit(ParentIndex);
		}
	}

	// Remove parent from child's list.
	FNetDependencyData::FCreationDependencyInfoArray* CreationDependencyInfo = SubObjects.GetCreationDependencyInfoArray(ChildIndex);
	if (CreationDependencyInfo)
	{
		CreationDependencyInfo->Remove(ParentIndex);

		if (CreationDependencyInfo->IsEmpty())
		{
			ObjectsWithCreationDependencies.ClearBit(ChildIndex);
			// We don't free the array because we usually add a new dependency right after or the object will stop replicating soon and the array will be freed there.
		}
	}

	// The child might need to become relevant by losing a creation dependency, so dirty it for the creation dependency update.
	ObjectsWithDirtyCreationDependencies.SetBit(ChildIndex);
}

TConstArrayView<FInternalNetRefIndex> FNetRefHandleManager::GetCreationDependents(FInternalNetRefIndex ParentInternalIndex) const
{
	if (ObjectsWithCreationDependents.IsBitSet(ParentInternalIndex) == false)
	{
		return MakeArrayView<const FInternalNetRefIndex>(nullptr, 0);
	}

	const FNetDependencyData::FCreationDependentInfoArray* CreationDependentArray = SubObjects.GetCreationDependentInfoArray(ParentInternalIndex);
	
	if (!CreationDependentArray)
	{
		return MakeArrayView<const FInternalNetRefIndex>(nullptr, 0);
	}
	
	return MakeArrayView(*CreationDependentArray);
}

TConstArrayView<FInternalNetRefIndex> FNetRefHandleManager::GetCreationDependencies(FInternalNetRefIndex ChildInternalIndex) const
{
	if (ObjectsWithCreationDependencies.IsBitSet(ChildInternalIndex) == false)
	{
		return MakeArrayView<const FInternalNetRefIndex>(nullptr, 0);
	}

	const FNetDependencyData::FCreationDependencyInfoArray* CreationDependencyArray = SubObjects.GetCreationDependencyInfoArray(ChildInternalIndex);
	
	if (!CreationDependencyArray)
	{
		return MakeArrayView<const FInternalNetRefIndex>(nullptr, 0);
	}
	
	return MakeArrayView(*CreationDependencyArray);
}

void FNetRefHandleManager::InternalRemoveDependentObject(FInternalNetRefIndex ParentInternalIndex, FInternalNetRefIndex DependentInternalIndex, ERemoveDependentObjectFlags Flags)
{
	if (EnumHasAnyFlags(Flags, ERemoveDependentObjectFlags::RemoveFromDependentParentObjects))
	{
		if (FNetDependencyData::FInternalNetIndexArray* ParentObjectArray = SubObjects.GetInternalIndexArray<FNetDependencyData::DependentParentObjects>(DependentInternalIndex))
		{
			ParentObjectArray->Remove(ParentInternalIndex);
			if (ParentObjectArray->Num() == 0)
			{
				FReplicatedObjectData& DependentObjectData = GetReplicatedObjectDataNoCheck(DependentInternalIndex);
				DependentObjectData.bIsDependentObject = false;
				DependentObjectInternalIndices.ClearBit(DependentInternalIndex);
			}
		}
	}

	if (EnumHasAnyFlags(Flags, ERemoveDependentObjectFlags::RemoveFromParentDependentObjects))
	{
		FReplicatedObjectData& ParentObjectData = GetReplicatedObjectDataNoCheck(ParentInternalIndex);
		FNetDependencyData::FDependentObjectInfoArray* ParentDependentObjectsArray = ParentObjectData.bHasDependentObjects ? SubObjects.GetDependentObjectInfoArray(ParentInternalIndex) : nullptr;
		if (ParentDependentObjectsArray)
		{
			const int32 ArrayIndex =  ParentDependentObjectsArray->FindLastByPredicate([DependentInternalIndex](const FDependentObjectInfo& Entry) { return Entry.NetRefIndex == DependentInternalIndex;});
			if (ArrayIndex != INDEX_NONE)
			{
				ParentDependentObjectsArray->RemoveAt(ArrayIndex);
			}

			if (ParentDependentObjectsArray->Num() == 0)
			{
				ParentObjectData.bHasDependentObjects = false;
				ObjectsWithDependentObjectsInternalIndices.ClearBit(ParentInternalIndex);
			}
		}
	}
}

void FNetRefHandleManager::InternalRemoveAllDependencies(FInternalNetRefIndex DependentInternalIndex)
{
	// Remove from all parents
	if (FNetDependencyData::FInternalNetIndexArray* ParentObjectArray = SubObjects.GetInternalIndexArray<FNetDependencyData::DependentParentObjects>(DependentInternalIndex))
	{
		for (FInternalNetRefIndex ParentInternalIndex : *ParentObjectArray)
		{
			// Flag is set to only update data on the parent to avoid modifying the array we iterate over
			InternalRemoveDependentObject(ParentInternalIndex, DependentInternalIndex, ERemoveDependentObjectFlags::RemoveFromParentDependentObjects);
		}
		ParentObjectArray->Reset();
	}

	// Remove from our dependents
	if (FNetDependencyData::FDependentObjectInfoArray* DependentObjectArray = SubObjects.GetDependentObjectInfoArray(DependentInternalIndex))
	{
		for (const FDependentObjectInfo& ChildDependentObjectInfo : *DependentObjectArray)
		{
			// Flag is set to only update data on the childDependentObject to avoid modifying the array we iterate over
			InternalRemoveDependentObject(DependentInternalIndex, ChildDependentObjectInfo.NetRefIndex, ERemoveDependentObjectFlags::RemoveFromDependentParentObjects);
		}
		DependentObjectArray->Reset();		
	}

	if (FNetDependencyData::FCreationDependentInfoArray* CreationDependentsArray = SubObjects.GetCreationDependentInfoArray(DependentInternalIndex))
	{
		// Remove the creation dependencies between this object and its dependents.
		for (int32 DependentsArrayIndex = CreationDependentsArray->Num() - 1; DependentsArrayIndex >= 0; --DependentsArrayIndex)
		{
			FInternalNetRefIndex ChildIndex = CreationDependentsArray->GetData()[DependentsArrayIndex];
			InternalRemoveCreationDependency(DependentInternalIndex, ChildIndex);
		}

		CreationDependentsArray->Reset();
	}

	if (FNetDependencyData::FCreationDependencyInfoArray* CreationDependenciesArray = SubObjects.GetCreationDependencyInfoArray(DependentInternalIndex))
	{
		// Remove the creation dependencies between this object and its dependencies.
		for (int32 DependenciesArrayIndex = CreationDependenciesArray->Num() - 1; DependenciesArrayIndex >= 0; --DependenciesArrayIndex)
		{
			FInternalNetRefIndex ParentIndex = CreationDependenciesArray->GetData()[DependenciesArrayIndex];
			InternalRemoveCreationDependency(ParentIndex, DependentInternalIndex);
		}

		CreationDependenciesArray->Reset();
	}

	FReplicatedObjectData& DependentObjectData = GetReplicatedObjectDataNoCheck(DependentInternalIndex);

	// Clear out flags on this object	
	DependentObjectData.bIsDependentObject = false;
	DependentObjectData.bHasDependentObjects = false;
	ObjectsWithDependentObjectsInternalIndices.ClearBit(DependentInternalIndex);
	DependentObjectInternalIndices.ClearBit(DependentInternalIndex);
	ObjectsWithCreationDependents.ClearBit(DependentInternalIndex);
	ObjectsWithCreationDependencies.ClearBit(DependentInternalIndex);
}

void FNetRefHandleManager::RemoveDependentObject(FNetRefHandle ParentHandle, FNetRefHandle DependentHandle)
{
	// Validate objects
	FInternalNetRefIndex ParentInternalIndex = GetInternalIndex(ParentHandle);
	FInternalNetRefIndex DependentInternalIndex = GetInternalIndex(DependentHandle);

	if ((ParentInternalIndex == InvalidInternalNetRefIndex) || (DependentInternalIndex == InvalidInternalNetRefIndex))
	{
		return;
	}

	InternalRemoveDependentObject(ParentInternalIndex, DependentInternalIndex);
}

bool FNetRefHandleManager::AddingDependencyWouldCreateCircularDependency(FInternalNetRefIndex ParentInternalIndex, FInternalNetRefIndex DependentInternalIndex) const
{
	TArray<FInternalNetRefIndex, TInlineAllocator<64>> DependentObjects;
	DependentObjects.Push(DependentInternalIndex);

	do 
	{
		if (const FInternalNetRefIndex ObjectIndex = DependentObjects.Pop(EAllowShrinking::No))
		{
			if (ObjectsWithDependentObjectsInternalIndices.GetBit(ObjectIndex))
			{
				TConstArrayView<FDependentObjectInfo> DependentObjectInfos = GetDependentObjectInfos(ObjectIndex);
				DependentObjects.Reserve(DependentObjects.Num() + DependentObjectInfos.Num());
				for (const FDependentObjectInfo& DependentObjectInfo : DependentObjectInfos)
				{
					if (DependentObjectInfo.NetRefIndex == ParentInternalIndex)
					{
						return true;
					}

					DependentObjects.Add(DependentObjectInfo.NetRefIndex);
				}
			}
		}
	}
	while (!DependentObjects.IsEmpty());

	return false;
}

void FNetRefHandleManager::SetIsDebugObject(FNetRefHandle Handle)
{
	FInternalNetRefIndex ObjectInternalIndex = GetInternalIndex(Handle);
	if (ObjectInternalIndex != InvalidInternalNetRefIndex && ensureMsgf(ReplicatedObjectRefCount[ObjectInternalIndex] == 0U, TEXT("SetIsDebugObject can only be called before we have started to replicate an object. %s"), *PrintObjectFromIndex(ObjectInternalIndex)))
	{
		ReplicatedObjectData[ObjectInternalIndex].bIsDebugObject = 1U;
	}
}

void FNetRefHandleManager::SetShouldPropagateChangedStates(FInternalNetRefIndex ObjectInternalIndex, bool bShouldPropagateChangedStates)
{
	if (ObjectInternalIndex != InvalidInternalNetRefIndex)
	{
		if (bShouldPropagateChangedStates)
		{
			// Currently we do not support re-enabling state propagation
			// $IRIS: $TODO: Implement method to force dirty all changes 
			// https://jira.it.epicgames.com/browse/UE-127368

			checkf(false, TEXT("Re-enabling state change propagation is currently Not implemented."));			
			return;
		}

		ReplicatedObjectData[ObjectInternalIndex].bShouldPropagateChangedStates = bShouldPropagateChangedStates ? 1U : 0U;
	}
}

void FNetRefHandleManager::SetShouldPropagateChangedStates(FNetRefHandle Handle, bool bShouldPropagateChangedStates)
{
	FInternalNetRefIndex ObjectInternalIndex = GetInternalIndex(Handle);
	return SetShouldPropagateChangedStates(ObjectInternalIndex, bShouldPropagateChangedStates);
}

uint8* FNetRefHandleManager::AllocateStateBuffer(FNetRefHandleManager::EStateBufferType BufferType, const FReplicationProtocol* Protocol)
{
	uint8* StateBuffer = static_cast<uint8*>(FMemory::MallocZeroed(FPlatformMath::Max(Protocol->InternalTotalSize, 1U), Protocol->InternalTotalAlignment));
	if (BufferType == EStateBufferType::Outgoing && EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasConditionalChangeMask))
	{
		// Enable all conditions by default. This is to be compatible with the implementation of FRepChangedPropertyTracker where we have hooks into ReplicationSystem.
		FNetBitArrayView ConditionalChangeMask(reinterpret_cast<uint32*>(StateBuffer + Protocol->GetConditionalChangeMaskOffset()), Protocol->ChangeMaskBitCount, FNetBitArrayView::NoResetNoValidate);
		ConditionalChangeMask.SetAllBits();
	}

	return StateBuffer;
}

void FNetRefHandleManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (TObjectPtr<UObject>& Object : ReplicatedInstances)
	{
		Collector.AddReferencedObject(Object);
	}
}

void FNetRefHandleManager::StoreCachedCreationInfo(FInternalNetRefIndex InternalIndex, TUniquePtr<FNetObjectCreationHeader>&& Header)
{
	if (InternalIndex != InvalidInternalNetRefIndex)
	{
		FReplicatedObjectData& ObjectData = GetReplicatedObjectDataNoCheck(InternalIndex);
		ObjectData.bHasCachedCreationInfo = 1U;
		CachedCreationHeaders.Add(InternalIndex, MoveTemp(Header));
	}
}

void FNetRefHandleManager::ClearCachedCreationInfo(FInternalNetRefIndex InternalIndex)
{
	if (InternalIndex != InvalidInternalNetRefIndex)
	{
		FReplicatedObjectData& ObjectData = GetReplicatedObjectDataNoCheck(InternalIndex);
		ObjectData.bHasCachedCreationInfo = 0U;
		CachedCreationHeaders.Remove(InternalIndex);
	}
}

const FNetObjectCreationHeader* FNetRefHandleManager::GetCachedCreationInfo(FInternalNetRefIndex InternalIndex) const
{
	if (const TUniquePtr<const FNetObjectCreationHeader>* CachedHeader = CachedCreationHeaders.Find(InternalIndex))
	{
		return CachedHeader->Get();
	}
	else
	{
		return nullptr;
	}
}


TConstArrayView<FInternalNetRefIndex> FNetRefHandleManager::GetInternalIndicesReplicatingNetRefHandle(FNetRefHandle NetRefHandle) const
{
	const TArray<FInternalNetRefIndex, TInlineAllocator<2>>* Entry = InternalIndicesReplicatingNetRefHandle.Find(NetRefHandle);
	if (!Entry)
	{
		return MakeConstArrayView<FInternalNetRefIndex>(nullptr, 0);
	}
	else
	{
		return MakeConstArrayView(*Entry);
	}
}


FNetRefHandle FNetRefHandleManager::MakeNetRefHandle(uint64 Id, uint32 ReplicationSystemId)
{
	check(Id <= FNetRefHandle::MaxId);
	check(ReplicationSystemId < FNetRefHandle::MaxReplicationSystemId);

	FNetRefHandle Handle;

	Handle.Fields.Static = Id & 1;
	Handle.Fields.Serial = Id >> 1;
	Handle.Fields.ReplicationSystemId = ReplicationSystemId + 1U;

	return Handle;
}

FNetRefHandle FNetRefHandleManager::MakeNetRefHandleFromId(uint64 Id)
{
	// This is called on the receiving end when deserializing replicated objects. We don't want to crash on bit stream errors leading to invalid handle IDs being read.
	ensure(Id <= FNetRefHandle::MaxId);

	FNetRefHandle Handle;

	Handle.Fields.Static = Id & 1;
	Handle.Fields.Serial = Id >> 1;
	Handle.Fields.ReplicationSystemId = 0U;

	return Handle;
}

FNetRefHandle FNetRefHandleManager::MakeNetRefHandleFromSerial(uint64 Serial, bool bIsStatic, uint32 ReplicationSystemId)
{
	check(Serial <= FNetRefHandle::MaxSerial);

	FNetRefHandle Handle;

	Handle.Fields.Static = bIsStatic;
	Handle.Fields.Serial = Serial;
	Handle.Fields.ReplicationSystemId = ReplicationSystemId + 1U;

	return Handle;
}

void FNetRefHandleManager::OnPreSendUpdate()
{
	// The current frame scope is based on all indexes assigned up to this point.
	ScopeFrameData.CurrentFrameScopableInternalIndices.Copy(GlobalScopableInternalIndices);

	// Allow the list to be read.
	ScopeFrameData.bIsValid = true;
}

void FNetRefHandleManager::OnPostSendUpdate()
{
	// Store the scope for the next frame.
	ScopeFrameData.PrevFrameScopableInternalIndices.Copy(ScopeFrameData.CurrentFrameScopableInternalIndices);

	// From here no-one should access the ScopeFrameData
	ScopeFrameData.bIsValid = false;

	CSV_CUSTOM_STAT(IrisCommon, ActiveReplicatedObjectCount, (float)ActiveObjectCount, ECsvCustomStatOp::Set);
}

FString FNetRefHandleManager::PrintObjectFromIndex(FInternalNetRefIndex ObjectIndex) const
{
	if (ObjectIndex != InvalidInternalNetRefIndex)
	{
		const bool bIsIndexAssigned = AssignedInternalIndices.GetBit(ObjectIndex);
		if (!bIsIndexAssigned)
		{
			// If the index isn't bound to any object (possibly invalid or obsolete)
			return FString::Printf(TEXT("UnassignedObject (InternalIndex: %u)"), ObjectIndex);
		}

		const FReplicatedObjectData& ObjectData = GetReplicatedObjectDataNoCheck(ObjectIndex);
		const FNetRefHandle NetRefHandle = ObjectData.RefHandle;

		if (ObjectData.SubObjectRootIndex == InvalidInternalNetRefIndex)
		{
			return FString::Printf(TEXT("RootObject %s (InternalIndex: %u) (%s) Protocol: (%s)"), *GetNameSafe(ReplicatedInstances[ObjectIndex]), ObjectIndex, *NetRefHandle.ToString(), ObjectData.Protocol->DebugName->Name);
		}
		else
		{
			const FNetRefHandle RootNetRefHandle = GetNetRefHandleFromInternalIndex(ObjectData.SubObjectRootIndex);
			return FString::Printf(TEXT("SubObject %s (InternalIndex: %u) (%s) tied to RootObject %s (InternalIndex: %u) (%s) Protocol: (%s)"), 
								*GetNameSafe(ReplicatedInstances[ObjectIndex]), ObjectIndex, *NetRefHandle.ToString(),
								*GetNameSafe(ReplicatedInstances[ObjectData.SubObjectRootIndex]), ObjectData.SubObjectRootIndex, *RootNetRefHandle.ToString(), ObjectData.Protocol->DebugName->Name);
		}
	}
	else
	{
		return FString("InvalidObject (InternalIndex: Invalid)");
	}
}

FString FNetRefHandleManager::PrintObjectFromNetRefHandle(FNetRefHandle ObjectHandle) const
{ 
	const FInternalNetRefIndex ObjectIndex = GetInternalIndex(ObjectHandle);
	if (ObjectIndex != InvalidInternalNetRefIndex)
	{
		return PrintObjectFromIndex(ObjectIndex);
	}
	else
	{
		return FString::Printf(TEXT("NetObject None (InternalIndex: None) (%s)"), *ObjectHandle.ToString());
	}
	
}

} // end namespace UE::Net::Private
