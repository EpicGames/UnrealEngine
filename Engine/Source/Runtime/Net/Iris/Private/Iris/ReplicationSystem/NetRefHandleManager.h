// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"

#include "CoreTypes.h"

#include "Delegates/Delegate.h"

#include "Iris/Core/NetChunkedArray.h"

#include "Iris/IrisConstants.h"

#include "Iris/ReplicationState/ReplicationStateDescriptor.h"

#include "Iris/ReplicationSystem/InternalNetRefIndexManager.h"
#include "Iris/ReplicationSystem/NetDependencyData.h"
#include "Iris/ReplicationSystem/NetObjectFactory.h"
#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/NetRefHandleManagerTypes.h"
#include "Iris/ReplicationSystem/ObjectReferenceTypes.h"
#include "Iris/ReplicationSystem/ReplicationBridgeTypes.h"

#include "Misc/TransactionallySafeCriticalSection.h"

#include "Net/Core/NetBitArray.h"
#include "Net/Core/NetHandle/NetHandle.h"

#include "UObject/ObjectPtr.h"

class FReferenceCollector;
namespace UE::Net
{
	struct FReplicationProtocol;
	struct FReplicationInstanceProtocol;

	namespace Private
	{
		class FReplicationProtocolManager;
	}
}

namespace UE::Net::Private
{

enum class EAddSubObjectFlags : uint32
{
	None = 0U,
	/** Insert the subject at the start of the parents list */
	InsertAtStart = 1U, 
	/** Insert at the start of the parents list or right after the relative handle if one is set */
	InsertAtStartOrAfterOther = InsertAtStart << 1U,
	/** If set, children of this SubObject will replicate after the parent */
	ReplicateChildSubObjectsAfterParent  = InsertAtStartOrAfterOther << 1U,
};
ENUM_CLASS_FLAGS(EAddSubObjectFlags);

enum class ERemoveDependentObjectFlags : uint32
{
	None = 0U,
	RemoveFromDependentParentObjects = 1U,
	RemoveFromParentDependentObjects = RemoveFromDependentParentObjects << 1U,
	All = RemoveFromDependentParentObjects | RemoveFromParentDependentObjects,
};
ENUM_CLASS_FLAGS(ERemoveDependentObjectFlags);

// Internal class to manage NetHandles and their internal data
class FNetRefHandleManager
{
public:

	// We need to store some internal data for Replicated objects
	// $TODO: This should be split up into separate array according to usage patterns, it is getting a bit too large
	struct FReplicatedObjectData
	{
		FReplicatedObjectData() = default;

		// Do not support moving/copying. We should modify entries in the array directly.
		FReplicatedObjectData(FReplicatedObjectData&&) = delete;
		FReplicatedObjectData& operator=(FReplicatedObjectData&&) = delete;
		FReplicatedObjectData(const FReplicatedObjectData&) = delete;
		FReplicatedObjectData& operator=(const FReplicatedObjectData&) = delete;

		void Reset()
		{
			new (this) FReplicatedObjectData();
		}

		FNetRefHandle RefHandle;
		FNetHandle NetHandle;
		
		const FReplicationProtocol* Protocol = nullptr;
		const FReplicationInstanceProtocol* InstanceProtocol = nullptr;
		uint8* ReceiveStateBuffer = nullptr;
		
		/** Subobjects only: Internal index of the RootObject of this subobject */
		FInternalNetRefIndex SubObjectRootIndex = InvalidInternalNetRefIndex;
		
		/** Subobjects only: Internal index of the ParentObject of this subobject */
		FInternalNetRefIndex SubObjectParentIndex = InvalidInternalNetRefIndex;

		/** SubObjects only: Determine if child subobjects should replicate before or after parent */
		uint16 bReplicateChildSubObjectsAfterParent : 1U = 0;
		uint16 bShouldPropagateChangedStates : 1U = 0;
		uint16 bTearOff : 1U = 0;
		uint16 bFlush : 1U = 0;
		uint16 bIsDependentObject : 1U = 0;
		uint16 bHasDependentObjects : 1U = 0;
		uint16 bAllowDestroyInstanceFromRemote : 1U = 0;
		uint16 bNeedsFullCopyAndQuantize : 1U = 0;
		uint16 bWantsFullPoll : 1U = 0;
		uint16 bIsAsyncStopping : 1U = 0;
		/** Set to true when the object had StopReplication called directly on it */
		uint16 bPendingEndReplication : 1U = 0;
		uint16 bHasCachedCreationInfo : 1U = 0;
		uint16 bIsDebugObject : 1U = 0;
		uint16 bNetObjectIsPendingDestroy : 1U = 0;
		uint16 bIsDestructionInfo : 1U = 0;
		uint16 bIsLocal : 1U = 0;

		/** The factory responsible for instantiating this object */
		FNetObjectFactoryId NetFactoryId = InvalidNetObjectFactoryId;

		/** The async loading priority for references collected from this object */
		EIrisAsyncLoadingPriority IrisAsyncLoadingPriority = EIrisAsyncLoadingPriority::Default;

		/** Stores the detach reason while the object is stopping replication */
		EInternalDetachReason InternalDetachReason = EInternalDetachReason::Normal;

		/** Returns true if this is a root object */
		bool IsRootObject() const { return SubObjectRootIndex == InvalidInternalNetRefIndex; }
	
		/** Returns true if this is a SubObject */
		bool IsSubObject() const { return SubObjectRootIndex != InvalidInternalNetRefIndex; }

		/** Returns true if this is a child dependent object  */
		bool IsDependentObject() const { return bIsDependentObject; }

		/** Returns true if this is a considered a debugobject  */
		bool IsDebugObject() const { return bIsDebugObject; }
	};

	// We want to handle both actual Networked and "network addressable" handles in the same way
	// As we expect the number of FNetRefHandles to be much greater than the actual number of Replicated Objects we
	// use a map to map from Handle to InternalIndex, only Replicated Objects will ever be assigned an InternalNetHandle
	typedef TMap<FNetRefHandle, FInternalNetRefIndex> FRefHandleMap;
	typedef TMap<FNetHandle, FInternalNetRefIndex> FNetHandleMap;

public:

	/** External configuration variables used to initialize the NetRefHandleManager */
	struct FInitParams
	{
		uint32 ReplicationSystemId = 0;
		uint32 MaxActiveObjectCount = 0;
		uint32 InternalNetRefIndexInitSize = 0;
		uint32 NetChunkedArrayCount = 0;
		uint32 InternalNetRefIndexGrowSize = 0;
		uint32 InitialNetRefHandleIndex = 1;
	};

	FNetRefHandleManager(FReplicationProtocolManager& InReplicationProtocolManager);

	void Init(const FInitParams& InitParams);
	void Deinit();

	/** Callback triggered at the beginning of PreSendUpdate. Used to sync current frame data. */
	void OnPreSendUpdate();

	/** Callback triggered at the end of SendUpdate. Used to clear current frame data. */
	void OnPostSendUpdate();

	// Return true if this is a scopable index
	bool IsScopableIndex(FInternalNetRefIndex InternalIndex) const { return GlobalScopableInternalIndices.GetBit(InternalIndex); }

	static FNetRefHandle MakeNetRefHandle(uint64 Id, uint32 ReplicationSystemId);
	static FNetRefHandle MakeNetRefHandleFromId(uint64 Id);
	static FNetRefHandle MakeNetRefHandleFromSerial(uint64 Serial, bool bIsStatic, uint32 ReplicationSystemId);

	// Returns a valid handle using a bespoke identifier.
	FNetRefHandle AllocateNetRefHandleFromSerial(uint64 Serial, bool bIsStatic);

	// Returns a valid handle using an internally generated identifier.
	FNetRefHandle AllocateNetRefHandle(bool bIsStatic);

	// Create local Net Object
	FNetRefHandle CreateNetObject(FNetRefHandle WantedHandle, FNetHandle GlobalHandle, const FCreateNetObjectParams& Params);

	// Create NetObject on request from remote
	FNetRefHandle CreateNetObjectFromRemote(FNetRefHandle WantedHandle, const FCreateNetObjectParams& Params);

	// Attach Instance protocol to handle
	// Instance can be null, we only track the Instance for legacy support
	void AttachInstanceProtocol(FInternalNetRefIndex InternalIndex, const FReplicationInstanceProtocol* InstanceProtocol, UObject* Instance);

	// Used when we want to detach the instance protocol due to async destroy
	const FReplicationInstanceProtocol* DetachInstanceProtocol(FInternalNetRefIndex InternalIndex);

	/** Does this object has an instance protocol attached to it */
	bool HasInstanceProtocol(FInternalNetRefIndex InternalIndex) const;

	// Creates a DestructionInfo Handle which is used to replicate persistently destroyed static objects when late joining or streaming in levels with static objects that already has been destroyed
	// Returns a new handle used to replicate the destruction info, and adds a cross-reference to the original handle being destroyed if that still exists 
	// Handle is a reference to an already replicated Handle that is about to be destroyed
	FNetRefHandle CreateHandleForDestructionInfo(FNetRefHandle Handle, const FCreateNetObjectParams& Params);

	// Reset state of object tracked as internal index.
	void ResetDestroyedStartupObject(FInternalNetRefIndex InternalIndex);
	
	void DestroyNetObjectByIndex(FInternalNetRefIndex InternalIndex, bool bKeepInScopeAndMarkForFlush = false);

	// Mark object as no longer scopable, object will be removed from scope for all connections
	void RemoveFromScope(FInternalNetRefIndex InternalIndex);

	struct FPendingDestroyInfo
	{
		FInternalNetRefIndex InternalIndex;
		FNetRefHandle RefHandle;
		uint32 PendingDestroyUpdateCount = 0U;
	};

	TConstArrayView<FPendingDestroyInfo> GetObjectsPendingDestroy() const { return MakeArrayView(PendingDestroyInfos); }
	void DestroyObjectsPendingDestroy();

	const FReplicatedObjectData& GetReplicatedObjectDataNoCheck(FInternalNetRefIndex InternalIndex) const { return ReplicatedObjectData[InternalIndex]; }
	FReplicatedObjectData& GetReplicatedObjectDataNoCheck(FInternalNetRefIndex InternalIndex) { return ReplicatedObjectData[InternalIndex]; }

	inline const FReplicatedObjectData& GetReplicatedObjectData(FInternalNetRefIndex InternalIndex) const;

	inline const uint8* GetReplicatedObjectStateBufferNoCheck(FInternalNetRefIndex InternalObjectIndex) const { return ReplicatedObjectStateBuffers[InternalObjectIndex]; }
	inline uint8* GetReplicatedObjectStateBufferNoCheck(FInternalNetRefIndex InternalObjectIndex) { return ReplicatedObjectStateBuffers[InternalObjectIndex]; }
	inline const TNetChunkedArray<uint8*>& GetReplicatedObjectStateBuffers() const { return ReplicatedObjectStateBuffers; }

	inline EIrisAsyncLoadingPriority GetIrisAsyncLoadingPriority(FInternalNetRefIndex InternalIndex) const;

	/** Returns true if the handle has been assigned an internal index and it's replicated object has been bound to local storage data. */
	inline bool IsNetRefHandleAssigned(FNetRefHandle Handle) const;

	/** Returns true if the InternaIndex belongs to a replicated object owned by the local peer. */
	inline bool IsLocal(FInternalNetRefIndex InternalIndex) const;

	/** Returns true if the handle belongs to a replicated object owned by the local peer. */
	inline bool IsLocalNetRefHandle(FNetRefHandle Handle) const;

	/** Returns true if the handle is for a remote replicated object. */
	inline bool IsRemoteNetRefHandle(FNetRefHandle Handle) const;

	/** Returns true if the handle is associated with a root object */
	inline bool IsRootObject(FNetRefHandle Handle) const;

	/** Extract a full handle from an incomplete one consisting of only an index. */
	inline FNetRefHandle GetCompleteNetRefHandle(FNetRefHandle IncompleteHandle) const;
	
	// Get Handle from internal index
	inline FNetRefHandle GetNetRefHandleFromInternalIndex(FInternalNetRefIndex InternalIndex) const;

	// Get internal index from handle
	inline FInternalNetRefIndex GetInternalIndex(FNetRefHandle Handle) const;

	// Get internal index from NetHandle
	inline FInternalNetRefIndex GetInternalIndexFromNetHandle(FNetHandle Handle) const;

	/** All scopable internal indices of the ReplicationSystem. Always up to date but should mostly be accessed in operations executed outside PreSendUpdate. */
	const FNetBitArrayView GetGlobalScopableInternalIndices() const { return MakeNetBitArrayView(GlobalScopableInternalIndices); }

	/** All scopable internal indices of the current frame at the start of PreSendUpdate. Only accessible during that operation. */
	const FNetBitArrayView GetCurrentFrameScopableInternalIndices() const { check(ScopeFrameData.bIsValid); return MakeNetBitArrayView(ScopeFrameData.CurrentFrameScopableInternalIndices); }

	/** All scopable internal indices of the previous PreSendUpdate. Only accessible during that operation */
	const FNetBitArrayView GetPrevFrameScopableInternalIndices() const { check(ScopeFrameData.bIsValid); return MakeNetBitArrayView(ScopeFrameData.PrevFrameScopableInternalIndices); }

	/** List of objects that are always relevant or currently relevant to at least one connection. */
	FNetBitArrayView GetRelevantObjectsInternalIndices() const { return MakeNetBitArrayView(RelevantObjectsInternalIndices); }

	/** List of objects that we polled this frame */
	FNetBitArrayView GetPolledObjectsInternalIndices() const { return MakeNetBitArrayView(PolledObjectsInternalIndices); }

	/** List of objects that have dirty state data that needs to be quantized */
	FNetBitArrayView GetDirtyObjectsToQuantize() const { return MakeNetBitArrayView(DirtyObjectsToQuantize); }

	// Get bitarray for all internal indices that currently are assigned
	const FNetBitArray& GetAssignedInternalIndices() const { return AssignedInternalIndices; }
	const FNetBitArrayView GetAssignedInternalIndicesView() const { return MakeNetBitArrayView(AssignedInternalIndices); }

	// SubObjects
	const FNetBitArray& GetSubObjectInternalIndices() const { return SubObjectInternalIndices; }
	const FNetBitArrayView GetSubObjectInternalIndicesView() const { return MakeNetBitArrayView(SubObjectInternalIndices); }

	bool AddSubObject(FNetRefHandle ParentObjectHandle, FNetRefHandle SubObjectHandle, FNetRefHandle RelativeOtherSubObjectHandle, EAddSubObjectFlags Flags = EAddSubObjectFlags::None);
	bool AddSubObject(FNetRefHandle ParentObjectHandle, FNetRefHandle SubObjectHandle, EAddSubObjectFlags Flags = EAddSubObjectFlags::None);
	void RemoveSubObject(FNetRefHandle SubObjectHandle);
	
	FNetRefHandle GetRootObjectOfAnyObject(FNetRefHandle NetRefHandle) const;
	FNetRefHandle GetRootObjectOfSubObject(FNetRefHandle SubObjectHandle) const;

	// If InternalIndex is a root object or invalid it will be returned. If it's a subobject then its root object index will be returned.
	FInternalNetRefIndex GetRootObjectInternalIndexOfAnyObject(FInternalNetRefIndex InternalIndex) const;

	// If SubObjectIndex is valid and a subobject then its root object index is returned, otherwise InvalidInternalNetRefIndex is returned.
	FInternalNetRefIndex GetRootObjectInternalIndexOfSubObject(FInternalNetRefIndex SubObjectIndex) const;

	bool SetSubObjectNetCondition(FInternalNetRefIndex SubObjectInternalIndex, FLifeTimeConditionStorage SubObjectCondition, bool& bOutWasModified);

	bool IsRootObject(FInternalNetRefIndex InternalIndex) const
	{
		return SubObjectInternalIndices.IsBitSet(InternalIndex) == false;
	}

	bool IsSubObject(FInternalNetRefIndex InternalIndex) const
	{
		return SubObjectInternalIndices.GetBit(InternalIndex);
	}

	// DependentObjects
	const FNetBitArrayView GetDependentObjectInternalIndices() const { return MakeNetBitArrayView(DependentObjectInternalIndices); }
	const FNetBitArrayView GetObjectsWithDependentObjectsInternalIndices() const { return MakeNetBitArrayView(ObjectsWithDependentObjectsInternalIndices); }
	
	bool AddDependentObject(FNetRefHandle ParentHandle, FNetRefHandle DependentHandle, EDependentObjectSchedulingHint SchedulingHint);
	void RemoveDependentObject(FNetRefHandle ParentHandle, FNetRefHandle DependentHandle);

	// Creation dependency API
	bool AddCreationDependency(FNetRefHandle Parent, FNetRefHandle Child);
	void RemoveCreationDependency(FNetRefHandle Parent, FNetRefHandle Child);

private:
	bool InternalAddCreationDependency(FInternalNetRefIndex ParentIndex, FInternalNetRefIndex ChildIndex);
	void InternalRemoveCreationDependency(FInternalNetRefIndex ParentIndex, FInternalNetRefIndex ChildIndex);

public:
	const FNetBitArrayView GetObjectsWithCreationDependents() const
	{
		return MakeNetBitArrayView(ObjectsWithCreationDependents);
	}

	TConstArrayView<FInternalNetRefIndex> GetCreationDependents(FInternalNetRefIndex ParentInternalIndex) const;

	const FNetBitArrayView GetObjectsWithCreationDependencies() const
	{
		return MakeNetBitArrayView(ObjectsWithCreationDependencies);
	}

	TConstArrayView<FInternalNetRefIndex> GetCreationDependencies(FInternalNetRefIndex ChildInternalIndex) const;

	FNetBitArrayView GetObjectsWithDirtyCreationDependencies() const
	{
		return MakeNetBitArrayView(ObjectsWithDirtyCreationDependencies);
	}

	// Remove DependentHandles from all dependent object tracking
	// $TODO: This might need a pass ObjectReplicationBridge as well since most likely also must restore filter/polling logic
	void RemoveDependentObject(FNetRefHandle DependentHandle);
	
	void SetShouldPropagateChangedStates(FNetRefHandle Handle, bool bShouldPropagateChangedStates);
	void SetShouldPropagateChangedStates(FInternalNetRefIndex ObjectInternalIndex, bool bShouldPropagateChangedStates);

	void SetIsDebugObject(FNetRefHandle Handle);

	/** Return the maximum internal index that NetBitArrays and TArrays are currently allocated for. */
	FInternalNetRefIndex GetCurrentMaxInternalNetRefIndex() const { return CurrentMaxInternalNetRefIndex; };

	/** The absolute maximum replicated objects that can be registered in the ReplicationSystem. Hitting this limit will cause a critical failure. */
	uint32 GetMaxActiveObjectCount() const { return MaxActiveObjectCount; }

	/** The current amount of replicated objects registered for replication. */
	uint32 GetActiveObjectCount() const { return ActiveObjectCount; }
	
	// We do refcount objects tracked by each connection in order to know when it is safe to reuse an InternalIndex
	void AddNetObjectRef(FInternalNetRefIndex InternalIndex) { ++ReplicatedObjectRefCount[InternalIndex]; }
	void ReleaseNetObjectRef(FInternalNetRefIndex InternalIndex) { check(ReplicatedObjectRefCount[InternalIndex] > 0); --ReplicatedObjectRefCount[InternalIndex]; }
	uint16 GetNetObjectRefCount(FInternalNetRefIndex ObjectInternalIndex) const { return ReplicatedObjectRefCount[ObjectInternalIndex]; }

	// Get dependent objects for the given ParentIndex
	TArrayView<const FDependentObjectInfo> GetDependentObjectInfos(FInternalNetRefIndex ParentIndex) const { return SubObjects.GetDependentObjectInfoArray(ParentIndex); }

	// Get all parents of the given DependentIndex
	inline TArrayView<const FInternalNetRefIndex> GetDependentObjectParents(FInternalNetRefIndex DependentIndex) const;

	// Get all subobjects for the given OwnerIndex, Note: only valid for the root
	inline TArrayView<const FInternalNetRefIndex> GetSubObjects(FInternalNetRefIndex OwnerIndex) const;

	// Get child subobjects for a object, used when we do hierarchical operations such as conditional serialization
	inline TArrayView<const FInternalNetRefIndex> GetChildSubObjects(FInternalNetRefIndex ParentObjectIndex) const;

	// Get child subobjects and conditionals array if one exists, if there are no child subobjects the method returns false
	inline bool GetChildSubObjects(FInternalNetRefIndex OwnerIndex, FChildSubObjectsInfo& OutInfo) const;

	/** Get the map that translates RefHandles into InternalIndexes */
	const FRefHandleMap& GetReplicatedHandles() const { return RefHandleToInternalIndex; }

	// Get the replicated object represented by a given internal index.
	UObject* GetReplicatedObjectInstance(FInternalNetRefIndex ObjectIndex) const { return ReplicatedInstances[ObjectIndex]; }

	/** Get the array of all held object pointers */
	const TNetChunkedArray<TObjectPtr<UObject>>& GetReplicatedInstances() const { return ReplicatedInstances; }

	void AddReferencedObjects(FReferenceCollector& Collector);

	/** IsDestructionInfo will return true if it's a destruction info for a destroyed startup object. */
	inline bool IsDestructionInfo(FInternalNetRefIndex InternalIndex) const;

	/** IsDestroyedStartupObject will return true if it's a destroyed startup object or the destruction info for a destroyed startup object. */
	bool IsDestroyedStartupObject(FInternalNetRefIndex InternalIndex) const { return DestroyedStartupObjectInternalIndices.GetBit(InternalIndex); }

	/** Will return the destruction info object index if passing a destroyed startup object index and vice versa. */
	inline uint32 GetOriginalDestroyedStartupObjectIndex(FInternalNetRefIndex InternalIndex) const;

	const FNetBitArrayView GetDestroyedStartupObjectInternalIndices() const { return MakeNetBitArrayView(DestroyedStartupObjectInternalIndices); }

	/** List of replicated objects that want to be dormant */
	const FNetBitArrayView GetWantToBeDormantInternalIndices() const { return MakeNetBitArrayView(WantToBeDormantInternalIndices); }
	FNetBitArrayView GetWantToBeDormantInternalIndices() { return MakeNetBitArrayView(WantToBeDormantInternalIndices); }

	/** List of dormant objects that requested a FlushNet */
	const FNetBitArrayView GetDormantObjectsPendingFlushNet() const { return MakeNetBitArrayView(DormantObjectsPendingFlushNet); }
	FNetBitArrayView GetDormantObjectsPendingFlushNet() { return MakeNetBitArrayView(DormantObjectsPendingFlushNet); }

	/** Get Objects that are flagged for PreUpdate (aka PreReplication) */
	FNetBitArrayView GetObjectsWithPreUpdate() const { return MakeNetBitArrayView(ObjectsWithPreUpdate); }

	/** Get Objects that fully supports pushbased dirtiness */
	FNetBitArrayView GetObjectsWithFullPushBasedDirtiness() const { return MakeNetBitArrayView(ObjectsWithFullPushBasedDirtiness); }

public:

	/** Return a string to identify the object linked to an index in logs */
	[[nodiscard]] FString PrintObjectFromIndex(FInternalNetRefIndex ObjectIndex) const;
	[[nodiscard]] FString PrintObjectFromNetRefHandle(FNetRefHandle ObjectHandle) const;

public:

	/** Delegate that will notify when the highest NetChunkedArray internal index has increased (Highest = Max-1)*/
	using FOnNetChunkedArrayIncrease = TMulticastDelegate<void(FInternalNetRefIndex HighestInternalIndex)>;
	FOnNetChunkedArrayIncrease::RegistrationType& GetOnNetChunkedArrayIncreaseDelegate() const { return OnNetChunkedArrayIncrease; };

	/** Return the highest internal index that NetChunkedArrays are currently allocated for. */
	uint32 GetHighestNetChunkedArrayInternalIndex() const { return HighestNetChunkedArrayInternalIndex; };

	/** Delegate that will notify when the NetObjectLists (eg. NetBitArray and TArray indexed via FInternalNetRefIndex) have a new maximum index to support */
	using FOnMaxInternalNetRefIndexIncreased = TMulticastDelegate<void(FInternalNetRefIndex MaxInternalIndex)>;
	FOnMaxInternalNetRefIndexIncreased::RegistrationType& GetOnMaxInternalNetRefIndexIncreasedDelegate() const { return OnMaxInternalNetRefIndexIncreased; };

	/** Delegate that will notify when one or more NetRefInternalIndices have been freed and can be re-assigned to new objects */
	using FOnInternalNetRefIndicesFreed = TMulticastDelegate<void(const TConstArrayView<FInternalNetRefIndex>& FreedIndices)>;
	FOnInternalNetRefIndicesFreed::RegistrationType& GetOnInternalNetRefIndicesFreedDelegate() const { return OnInternalNetRefIndicesFreed; };

	/** Store cached creation info allowing us to send data for objects for which we no longer has an instance */
	void StoreCachedCreationInfo(FInternalNetRefIndex, TUniquePtr<FNetObjectCreationHeader>&& Header);

	/** Get cached creation header */
	const FNetObjectCreationHeader* GetCachedCreationInfo(FInternalNetRefIndex InternalIndex) const;

	/** Clear cached creation info  */
	void ClearCachedCreationInfo(FInternalNetRefIndex InternalIndex);

	/** Get internal indices if more than one internal indices are replicating the same NetRefHandle during flush. */
	TConstArrayView<FInternalNetRefIndex> GetInternalIndicesReplicatingNetRefHandle(FNetRefHandle NetRefHandle) const;

public:

	// Iterate over all dependent objects and their dependent objects
	template <typename T>
	void ForAllDependentObjectsRecursive(FInternalNetRefIndex ObjectIndex, T&& Functor) const
	{
		if (ObjectsWithDependentObjectsInternalIndices.GetBit(ObjectIndex))
		{
			for (const FDependentObjectInfo& DependentObjectInfo : GetDependentObjectInfos(ObjectIndex))
			{
				Functor(DependentObjectInfo.NetRefIndex);
				ForAllDependentObjectsRecursive(DependentObjectInfo.NetRefIndex, Functor);
			}
			return;
		}
	};

private:
	enum class EStateBufferType : unsigned
	{
		Outgoing,
		Incoming,
	};

	void InitNetBitArray(FNetBitArray* NetBitArray);

	FInternalNetRefIndex InternalCreateNetObject(const FNetRefHandle NetRefHandle, const FNetHandle GlobalHandle, const FCreateNetObjectParams& Params);

	void InternalDestroyNetObject(FInternalNetRefIndex InternalIndex);

	/** Clear the status of all internal bit arrays when an internal index is put back in the free list */
	void ClearStateForFreedInternalIndex(FInternalNetRefIndex FreedInternalIndex);

	uint64 GetNextNetRefHandleSerial(uint64 HandleIndex) const;

	// Get the next free internal index, returns InvalidInternalNetRefIndex if a free one cannot be found
	FInternalNetRefIndex GetNextFreeInternalIndex() const;

	bool InternalAddSubObject(FInternalNetRefIndex RootObjectInternalIndex, FInternalNetRefIndex ParentSubObjectInternalIndex, FInternalNetRefIndex SubObjectInternalIndex, FInternalNetRefIndex RelativeOtherSubObjectInternalIndex, EAddSubObjectFlags Flags);

	inline void SetIsSubObject(FInternalNetRefIndex InternalIndex, bool IsSubObject) { SubObjectInternalIndices.SetBitValue(InternalIndex, IsSubObject); }
	void InternalRemoveSubObject(FInternalNetRefIndex RootObjectInternalIndex, FInternalNetRefIndex SubObjectInternalIndex, bool bRemoveFromSubObjectArray = true);

	void InternalRemoveDependentObject(FInternalNetRefIndex ParentInternalIndex, FInternalNetRefIndex DependentInternalIndex, ERemoveDependentObjectFlags Flags = ERemoveDependentObjectFlags::All);
	void InternalRemoveAllDependencies(FInternalNetRefIndex DependentInternalIndex);
	bool AddingDependencyWouldCreateCircularDependency(FInternalNetRefIndex ParentInternalIndex, FInternalNetRefIndex DependentInternalIndex) const;

	/**
	 * Grow the NetObjectLists to fit more internal indexes.
	 * Returns the newest available index if the increase worked or InvalidInternalNetRefIndex if we hit the maximum.
	 */
	FInternalNetRefIndex GrowNetObjectLists();

	/** Grow memory buffers when we use an index higher than the previous maximum. */
	void GrowNetChunkedArrayBuffers(FInternalNetRefIndex InternalIndex);

	/** Deal with the increase to the max index and tell other systems about it */
	void MaxInternalNetRefIndexIncreased(uint32 NewBitCount);

	static uint8* AllocateStateBuffer(EStateBufferType BufferType, const FReplicationProtocol* Protocol);

private:

	// Max allowed replicated object count
	uint32 MaxActiveObjectCount = 0;

	// The current replicated object count
	uint32 ActiveObjectCount = 0;

	// By how many indexes to grow the NetObjectLists when we hit the highest index.
	FInternalNetRefIndex InternalNetRefIndexGrowSize = 0;

	// The maximum internal index that NetObjectLists need to be able to reference
	FInternalNetRefIndex CurrentMaxInternalNetRefIndex = 0;

	// The highest internal index currently allocated for NetChunkedArrays
	uint32 HighestNetChunkedArrayInternalIndex = 0;

	uint32 ReplicationSystemId = UE::Net::InvalidReplicationSystemId;

	FRefHandleMap RefHandleToInternalIndex;
	FNetHandleMap NetHandleToInternalIndex;

	struct FScopeFrameData
	{
		FScopeFrameData() : bIsValid(false) { }

		// Controls if the frame data can be read or not
		uint32 bIsValid : 1;

		// Bitset used in order to track assigned internal which are scopable
		FNetBitArray CurrentFrameScopableInternalIndices;

		// Which internal indices were used last net frame. This can be used to find out which ones are new and deleted this frame. 
		FNetBitArray PrevFrameScopableInternalIndices;
	};

	/** Bitset used in order to track assigned internal which are scopable */
	FNetBitArray GlobalScopableInternalIndices;

	/** Stores scope lists relevant to the current frame. Only valid during SendUpdate(). */
	FScopeFrameData ScopeFrameData;

	/** This contains the ScopableInternalIndices list minus filtered objects that are not relevant to any connection this frame. */
	FNetBitArray RelevantObjectsInternalIndices;

	/** List of objects that we polled this frame */
	FNetBitArray PolledObjectsInternalIndices;

	/** List of the objects that are considered dirty and for whom we will quantize their state data */
	FNetBitArray DirtyObjectsToQuantize;

	// Bitset containing all internal indices that are assigned
	FNetBitArray AssignedInternalIndices;

	// Bitset containing all internal indices that are SubObjects
	FNetBitArray SubObjectInternalIndices;

	// Bitset containing all internal indices that are dependentObjects
	FNetBitArray DependentObjectInternalIndices;

	// Objects who need to exist before their children can be created.
	FNetBitArray ObjectsWithCreationDependents;

	// Objects that cannot be created until the parents they depend on also exist
	FNetBitArray ObjectsWithCreationDependencies;

	// Objects that need to have their filter status re-evaluated due to a change in creation dependency
	FNetBitArray ObjectsWithDirtyCreationDependencies;

	// Bitset containing all internal indices that have dependentObjects
	FNetBitArray ObjectsWithDependentObjectsInternalIndices;

	// Bitset marking internal indices of static objects that are permanently destroyed
	// We use this for special scoping
	FNetBitArray DestroyedStartupObjectInternalIndices;

	// Bitset marking internal indices that wants to be dormant
	FNetBitArray WantToBeDormantInternalIndices;

	// Set of dormant objects that have requested a NetFlushDormancy
	FNetBitArray DormantObjectsPendingFlushNet;

	// Bitset marking internal indices that is flagged as requiring PreUpdate
	FNetBitArray ObjectsWithPreUpdate;

	// Bitset marking internal indices that is flagged as supporting FullPushBasedDirtiness
	FNetBitArray ObjectsWithFullPushBasedDirtiness;

	// Both are marked in DestroyedStartupObjectsInternalIndicies
	// When one is destroyed both mapping will be cleared
	// Map OriginalIndex -> DestructionInfoIndex
	// Map DestructionInfoIndex -> Orignal
	TMap<uint32, uint32> DestroyedStartupObject;

	// Array used in order to track objects pending destroy/stopreplication
	TArray<FPendingDestroyInfo> PendingDestroyInfos;

	// Just an array containing data about our replicated objects
	TNetChunkedArray<FReplicatedObjectData> ReplicatedObjectData;

	// Pointers to state buffers for all replicated objects
	TNetChunkedArray<uint8*> ReplicatedObjectStateBuffers;

	// Refcounts for all tracked objects
	TNetChunkedArray<uint16> ReplicatedObjectRefCount;

	// Raw pointers to all bound instances
	TNetChunkedArray<TObjectPtr<UObject>> ReplicatedInstances;

	// Index ready to be assigned to the next replicated object
	// Index 0 is always reserved, for both static and dynamic handles
	uint64 NextStaticHandleIndex = 1;
	uint64 NextDynamicHandleIndex = 1;

	//TODO: Rename this variable its not exclusive to subobjects
	FNetDependencyData SubObjects;

	FReplicationProtocolManager& ReplicationProtocolManager;

	// Delegates
	mutable FOnMaxInternalNetRefIndexIncreased OnMaxInternalNetRefIndexIncreased;
	mutable FOnNetChunkedArrayIncrease OnNetChunkedArrayIncrease;
	mutable FOnInternalNetRefIndicesFreed OnInternalNetRefIndicesFreed;

	// Track NetBitArrays owned by the NetRefHandleManager so they can automatically be resized.
	TArray<FNetBitArray*> OwnedNetBitArrays;

	// Track previous use of internal indices that are still replicating data for the same handle to avoid having multiple internal objects replicating the same external object.
	TMap<FNetRefHandle, TArray<FInternalNetRefIndex, TInlineAllocator<2>>> InternalIndicesReplicatingNetRefHandle;

	// When we flush objects, we might need to defer sending creation info.
	TMap<FInternalNetRefIndex, TUniquePtr<const UE::Net::FNetObjectCreationHeader>> CachedCreationHeaders;

#if UE_SUPPORT_PARALLEL_IRIS
	// Protects parallel access to NextHandleId and RefHandleToInternalIndex when allocating NetRefHandles
	FTransactionallySafeCriticalSection HandleSerialCriticalSection;
#endif // UE_SUPPORT_PARALLEL_IRIS

	// We have a running counter of update for debugging purposes
	uint32 UpdatePendingDestroyCounter = 0U;
};

const FNetRefHandleManager::FReplicatedObjectData& FNetRefHandleManager::GetReplicatedObjectData(FInternalNetRefIndex InternalIndex) const
{
	check(AssignedInternalIndices.GetBit(InternalIndex));
	return GetReplicatedObjectDataNoCheck(InternalIndex);
}

FInternalNetRefIndex FNetRefHandleManager::GetInternalIndex(FNetRefHandle Handle) const
{
	const FInternalNetRefIndex* InternalIndex = RefHandleToInternalIndex.Find(Handle);
	return InternalIndex ? *InternalIndex : InvalidInternalNetRefIndex;
}

FInternalNetRefIndex FNetRefHandleManager::GetInternalIndexFromNetHandle(FNetHandle Handle) const
{
	const FInternalNetRefIndex* InternalIndex = NetHandleToInternalIndex.Find(Handle);
	return InternalIndex ? *InternalIndex : InvalidInternalNetRefIndex;
}

FNetRefHandle FNetRefHandleManager::GetNetRefHandleFromInternalIndex(FInternalNetRefIndex InternalIndex) const
{
	check(AssignedInternalIndices.GetBit(InternalIndex));
	return GetReplicatedObjectDataNoCheck(InternalIndex).RefHandle;
}

FNetRefHandle FNetRefHandleManager::GetCompleteNetRefHandle(FNetRefHandle IncompleteHandle) const
{
	if (const FInternalNetRefIndex* InternalIndex = RefHandleToInternalIndex.Find(IncompleteHandle))
	{
		return GetReplicatedObjectDataNoCheck(*InternalIndex).RefHandle;
	}
	else
	{
		return FNetRefHandle::GetInvalid();
	}
}

EIrisAsyncLoadingPriority FNetRefHandleManager::GetIrisAsyncLoadingPriority(FInternalNetRefIndex InternalIndex) const
{
	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = GetReplicatedObjectDataNoCheck(InternalIndex);
	
	// If it's a subobject, take the priority of the RootObject
	if (ObjectData.SubObjectRootIndex != InvalidInternalNetRefIndex)
	{
		return GetReplicatedObjectDataNoCheck(ObjectData.SubObjectRootIndex).IrisAsyncLoadingPriority;
	}

	return ObjectData.IrisAsyncLoadingPriority;
}

bool FNetRefHandleManager::IsNetRefHandleAssigned(FNetRefHandle Handle) const
{
	return RefHandleToInternalIndex.Contains(Handle);
}

bool FNetRefHandleManager::IsLocal(FInternalNetRefIndex InternalIndex) const
{
	return GetReplicatedObjectDataNoCheck(InternalIndex).bIsLocal;
}

bool FNetRefHandleManager::IsLocalNetRefHandle(FNetRefHandle Handle) const
{
	return IsLocal(GetInternalIndex(Handle));
}

bool FNetRefHandleManager::IsRemoteNetRefHandle(FNetRefHandle Handle) const
{
	if (const uint32 InternalIndex = GetInternalIndex(Handle))
	{
		return !IsLocal(InternalIndex);
	}

	return false;
}

bool FNetRefHandleManager::IsRootObject(FNetRefHandle Handle) const
{
	const uint32 InternalIndex = GetInternalIndex(Handle);
	return (InternalIndex != InvalidInternalNetRefIndex) ? IsRootObject(InternalIndex) : false;
}

bool FNetRefHandleManager::IsDestructionInfo(FInternalNetRefIndex InternalIndex) const
{
	return GetReplicatedObjectDataNoCheck(InternalIndex).bIsDestructionInfo;
}

uint32 FNetRefHandleManager::GetOriginalDestroyedStartupObjectIndex(FInternalNetRefIndex InternalIndex) const
{
	const uint32* FoundOriginalInternalIndex = DestroyedStartupObject.Find(InternalIndex);
	return FoundOriginalInternalIndex ? *FoundOriginalInternalIndex : 0U;
}

inline bool FNetRefHandleManager::AddSubObject(FNetRefHandle RootObjectHandle, FNetRefHandle SubObjectHandle, EAddSubObjectFlags Flags)
{
	return AddSubObject(RootObjectHandle, SubObjectHandle, FNetRefHandle::GetInvalid(), Flags);
}

TArrayView<const FInternalNetRefIndex> FNetRefHandleManager::GetSubObjects(FInternalNetRefIndex OwnerIndex) const
{
	return SubObjects.GetInternalIndexArray<FNetDependencyData::EArrayType::SubObjects>(OwnerIndex);
}

bool FNetRefHandleManager::GetChildSubObjects(FInternalNetRefIndex OwnerIndex, FChildSubObjectsInfo& OutInfo) const
{
	if (!SubObjects.GetChildSubObjects(OwnerIndex, OutInfo) || OutInfo.NumSubObjects == 0U)
	{
		return false;
	}

	OutInfo.bReplicateChildSubObjectsAfterParent = GetReplicatedObjectDataNoCheck(OwnerIndex).bReplicateChildSubObjectsAfterParent;
	return true;
}

TArrayView<const FInternalNetRefIndex> FNetRefHandleManager::GetChildSubObjects(FInternalNetRefIndex OwnerIndex) const
{
	return SubObjects.GetInternalIndexArray<FNetDependencyData::EArrayType::ChildSubObjects>(OwnerIndex);
}

TArrayView<const FInternalNetRefIndex> FNetRefHandleManager::GetDependentObjectParents(FInternalNetRefIndex DependentIndex) const
{
	return SubObjects.GetInternalIndexArray<FNetDependencyData::EArrayType::DependentParentObjects>(DependentIndex);
}
	
}
