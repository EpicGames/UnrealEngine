// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"
#include "Iris/ReplicationSystem/ObjectReferenceTypes.h"

namespace UE::Net
{
	struct FReplicationProtocol;
}

namespace UE::Net::Private
{

/** Holds important parameters needed to create a NetObject */
struct FCreateNetObjectParams
{
	FNetObjectFactoryId NetFactoryId = InvalidNetObjectFactoryId;
	EIrisAsyncLoadingPriority IrisAsyncLoadingPriority = EIrisAsyncLoadingPriority::Default;
	const UE::Net::FReplicationProtocol* ReplicationProtocol = nullptr;
};

/** An internal Iris-only enum used to serialize the reason why an object is no longer replicated to a client. */
enum class EInternalDetachReason : uint8
{
	/** Normal is simply a StopReplication call without any explicit reason given. */
	Normal = 0U,
	/** When a root object stops replicating to a client because it's no longer relevant. */
	NotRelevant,
	/** When a stable replicated object is destroyed even when the client loads it locally. */
	StaticDestroyed,
	/** An object is removed from a server but the client should keep it and maintain its internal data, expecting to receive object updates from another server later. Only supported by clients with replication systems configured to use UE::Net::EProxyType::Backend. */
	ProxyReuse,
	/** When a replicated object is torn off by the authority. */
	TornOff,
	/** Add new entries above this one. */
	Max,
};

const TCHAR* LexToString(EInternalDetachReason DetachReason);

consteval uint32 GetDetachReasonBitsNeeded()
{
	return UE::Net::GetBitsNeeded(static_cast<uint32>(EInternalDetachReason::Max) - 1U);
}


}