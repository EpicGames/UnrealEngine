// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetTokenDataStream.h"

#include "Iris/IrisConstants.h"
#include "Iris/PacketControl/PacketNotification.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Iris/ReplicationSystem/NetExports.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/Core/IrisLog.h"
#include "Net/Core/Trace/NetTrace.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetTokenDataStream)

#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#	define UE_NET_ENABLE_NETTOKEN_LOG 0
#else
#	define UE_NET_ENABLE_NETTOKEN_LOG 0
#endif 

#if UE_NET_ENABLE_NETTOKEN_LOG
#	define UE_LOG_NETTOKEN(Format, ...)  UE_LOG(LogIris, Log, Format, ##__VA_ARGS__)
#else
#	define UE_LOG_NETTOKEN(...)
#endif

#define UE_LOG_NETTOKEN_WARNING(Format, ...)  UE_LOG(LogIris, Warning, Format, ##__VA_ARGS__)

namespace UE::Net::Private
{

static bool bIrisPreExportExistingNetTokensOnConnect = false;
static FAutoConsoleVariableRef CVarIrisPreExportExistingNetTokensOnConnect(
	TEXT("net.Iris.IrisPreExportExistingNetTokensOnConnect"),
	bIrisPreExportExistingNetTokensOnConnect,
	TEXT("If true we will enqueue all existing NetTokens for pre-export when a new connection is added."
	));

}

UNetTokenDataStream::UNetTokenDataStream()
: NetTokenStore(nullptr)
, RemoteNetTokenStoreState(nullptr)
, NetExports(nullptr)
, ReplicationSystemId(UE::Net::InvalidReplicationSystemId)
, ConnectionId(~0U)
{
}

UNetTokenDataStream::~UNetTokenDataStream()
{
}

void UNetTokenDataStream::Init(const UDataStream::FInitParameters& Params)
{
	Super::Init(Params);

	using namespace UE::Net;

	ReplicationSystemId = Params.ReplicationSystemId;
	ConnectionId = Params.ConnectionId;

	UReplicationSystem* ReplicationSystem = UE::Net::GetReplicationSystem(ReplicationSystemId);
	NetTokenStore = ReplicationSystem->GetNetTokenStore();
	RemoteNetTokenStoreState = NetTokenStore->GetRemoteNetTokenStoreState(Params.ConnectionId);
	NetExports = Params.NetExports;

	// $IRIS $TODO: if we want to make this into a real feature we need to expose some sort of api to mark tokens for pre-export
	if (Private::bIrisPreExportExistingNetTokensOnConnect)
	{
		TArray<FNetToken> Tokens(NetTokenStore->GetAllNetTokens());
		for (const FNetToken& Token : Tokens)
		{
			NetTokensPendingExport.Add(Token);
		}
	}
}

void UNetTokenDataStream::AddNetTokenForExplicitExport(UE::Net::FNetToken NetToken)
{
	NetTokensPendingExport.Add(NetToken);
}

UDataStream::EWriteResult UNetTokenDataStream::BeginWrite(const FBeginWriteParameters& Params)
{
	if (!ensure(NetTokenStore) || NetTokensPendingExport.Num() == 0)
	{
		return EWriteResult::NoData;
	}

	return EWriteResult::HasMoreData;
}

UDataStream::EWriteResult UNetTokenDataStream::WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	
	if (!ensure(NetTokenStore) || NetTokensPendingExport.Num() == 0 || Writer->GetBitsLeft() < 1U)
	{
		// If we have no pending data in-flight we can trim down our storage
		if (NetTokenExports.Num() == 0)
		{
			NetTokenExports.Trim();
			NetTokensPendingExport.Trim();
		}
		
		return EWriteResult::NoData;
	}

	const FStringTokenStore* StringTokenStore = NetTokenStore->GetDataStore<const FStringTokenStore>();
	FNetExportContext* ExportContext = Context.GetExportContext();

	// Write data until we have no more data to write or it does not fit
	uint32 WrittenCount = 0;

	UE_NET_TRACE_SCOPE(NetTokenDataStream, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	// We use a substream to reserve a stop bit
	FNetBitStreamWriter SubStream = Writer->CreateSubstream(Writer->GetBitsLeft() - 1U);
	FNetSerializationContext SubContext = Context.MakeSubContext(&SubStream);

	const bool bIsNetTokenAuthority = NetTokenStore->IsAuthority();
	while (NetTokensPendingExport.Num())
	{
		FNetBitStreamRollbackScope SequenceRollback(SubStream);
		FNetExportRollbackScope ExportsRollbackScope(SubContext);

		// Peek at front index
		const FNetToken& Token = NetTokensPendingExport.GetAtIndexNoCheck(0);

		// We do not need to export tokens assigned by authority unless we are the authority.
		if (!(Token.IsAssignedByAuthority() && !bIsNetTokenAuthority) && !ExportContext->IsExported(Token))
		{
			UE_NET_TRACE_NAMED_SCOPE(ExportScope, NetTokenExport, SubStream, SubContext.GetTraceCollector(), ENetTraceVerbosity::Verbose);

			SubStream.WriteBool(true);
			NetTokenStore->WriteNetToken(SubContext, Token);
			NetTokenStore->WriteTokenData(SubContext, Token);

			if (SubStream.IsOverflown())
			{
				break;
			}
			else
			{
				UE_NET_TRACE_SET_SCOPE_NAME(ExportScope, StringTokenStore->ResolveToken(Token));

				// Mark Token as exported
				ExportContext->AddExported(Token);

				// Enqueue in our record as well for resending if we drop data
				NetTokenExports.Add(Token);
				++WrittenCount;
			}
		}

		NetTokensPendingExport.PopFrontNoCheck();
	}

	// Commit substream
	if (WrittenCount)
	{
		Writer->CommitSubstream(SubStream);
		Writer->WriteBool(false);

		// Store number of written batches in the external record pointer
		UPTRINT& OutRecordCount = *reinterpret_cast<UPTRINT*>(&OutRecord);
		OutRecordCount = WrittenCount;

		const bool bHasMoreDataToSend = NetTokensPendingExport.Num() > 0;
		return bHasMoreDataToSend ? EWriteResult::HasMoreData : EWriteResult::Ok;
	}
	else
	{
		Writer->DiscardSubstream(SubStream);

		const bool bHasMoreDataToSend = NetTokensPendingExport.Num() > 0;

		// $IRIS: $TODO: Fix me when we have addressed issues with cases where we have data to write but we did not fit anything, if over-commit is allowed we should request a new packet
		//return bHasMoreDataToSend ? EWriteResult::HasMoreData : EWriteResult::NoData;

		ensureAlways(!bHasMoreDataToSend);
		return EWriteResult::NoData;
	}
}

void UNetTokenDataStream::ReadData(UE::Net::FNetSerializationContext& Context)
{
	using namespace UE::Net;

	if (!ensure(NetTokenStore))
	{
		return;
	}

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	UE_NET_TRACE_SCOPE(NetTokenDataStream, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	while (Reader->ReadBool())
	{
		if (Reader->IsOverflown())
		{
			break;
		}

		FNetToken Token = NetTokenStore->ReadNetToken(Context);
		NetTokenStore->ReadTokenData(Context, Token, *RemoteNetTokenStoreState);
	}
}

void UNetTokenDataStream::ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record)
{
	// The Record pointer is used as storage for the number of batches to process
	UPTRINT RecordCount = reinterpret_cast<UPTRINT>(Record);

	// Acknowledgments of exported tokens is handled by the NetExports class, but in this case we want to explicitly resend the export
	if (Status == UE::Net::EPacketDeliveryStatus::Lost)
	{
		while (RecordCount)
		{
			// Could use PopFrontValue but do not want to have the RangeCheck
			NetTokensPendingExport.AddFront(NetTokenExports.GetAtIndexNoCheck(0));
			NetTokenExports.PopFrontNoCheck();
			--RecordCount;
		}
	}
	else
	{
		while (RecordCount)
		{
			NetTokenExports.PopFrontNoCheck();
			--RecordCount;
		}
	}
}

bool UNetTokenDataStream::HasAcknowledgedAllReliableData() const
{
	return NetTokensPendingExport.Num() == 0 && NetTokenExports.Num() == 0;
}
