// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/DataStream/DataStream.h"
#include "Net/Core/NetToken/NetToken.h"
#include "Containers/RingBuffer.h"

#include "NetTokenDataStream.generated.h"

namespace UE::Net
{
	class FNetToken;
	class FNetTokenStore;
	class FNetTokenStoreState;
	class FStringTokenStore;

	namespace Private
	{
		class FNetExports;
	}
}

UCLASS()
class UNetTokenDataStream final : public UDataStream
{
	GENERATED_BODY()

public:

	const UE::Net::FNetTokenStoreState* GetRemoteNetTokenStoreState() const { return RemoteNetTokenStoreState; }
	void AddNetTokenForExplicitExport(UE::Net::FNetToken NetToken);

private:
	UNetTokenDataStream();
	virtual ~UNetTokenDataStream();

	// UDataStream interface
	virtual void Init(const UDataStream::FInitParameters& Params) override;
	virtual EWriteResult BeginWrite(const FBeginWriteParameters& Params) override;
	virtual EWriteResult WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord) override;
	virtual void ReadData(UE::Net::FNetSerializationContext& Context) override;
	virtual void ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record) override;
	virtual bool HasAcknowledgedAllReliableData() const override;

private:

	// Record of in-flight NetTokens
	TRingBuffer<UE::Net::FNetToken> NetTokenExports;

	// All NetTokens enqueued for explicit export
	TRingBuffer<UE::Net::FNetToken> NetTokensPendingExport;

	// External record, simply track how many records we have in the internal record
	struct FExternalRecord : public FDataStreamRecord
	{
		uint32 Count = 0U;
	};

	UE::Net::FNetTokenStore* NetTokenStore;
	UE::Net::FNetTokenStoreState* RemoteNetTokenStoreState;
	UE::Net::Private::FNetExports* NetExports;

	uint32 ReplicationSystemId;
	uint32 ConnectionId;
};
