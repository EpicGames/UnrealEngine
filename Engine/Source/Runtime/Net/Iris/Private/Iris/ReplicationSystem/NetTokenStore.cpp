// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/NetTokenStore.h"
#include "Iris/ReplicationSystem/ObjectReferenceCacheFwd.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Net/Core/Trace/NetTrace.h"
#include "UObject/CoreNet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetTokenStore)

namespace UE::Net
{
class FNetSerializationContext;
	

class FNetTokenStoreState
{
public:
	using FNetTokenStoreKey = FNetTokenDataStore::FNetTokenStoreKey;

	// The size of the array is managed by FNetTokenDataStream
	FNetTokenStoreState()
	{
		Reset();
	}

	// Reserve space for tokens
	bool ReserveTokenCount(uint32 TypeIndex, uint32 NewCount)
	{
		if ((TypeIndex >= FNetToken::MaxTypeIdCount) || (NewCount >= FNetToken::MaxNetTokenCount))
		{
			return false;
		}

		const uint32 NewSize = FMath::Max<uint32>(TokenInfoArray[TypeIndex].Num(), NewCount);
		TokenInfoArray[TypeIndex].SetNumZeroed(NewSize);

		return true;
	}

	void Reset()
	{
		// We reserve the first token for each type as an invalid token
		for (TArray<FNetTokenStoreKey>& TokenInfos : TokenInfoArray)
		{
			TokenInfos.SetNum(1);
		}
	}

	// Map from NetTokenIndex -> NetTokenDataStoreKey (Index)
	TArray<FNetTokenDataStore::FNetTokenStoreKey> TokenInfoArray[FNetToken::MaxTypeIdCount];
};

FNetTokenDataStore::FNetTokenDataStore(FNetTokenStore& InTokenStore)
: TokenStore(InTokenStore)
, TypeId(FNetToken::InvalidTokenTypeId)
{
	// Reserve first index as invalid.
	StoredTokens.Add(FNetToken());
}

FNetTokenDataStore::~FNetTokenDataStore()
{
}

FNetTokenDataStore::FNetTokenStoreKey FNetTokenDataStore::GetTokenKey(FNetToken Token, const FNetTokenStoreState& TokenStoreState) const
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(&TokenStore));

	if (Token.GetTypeId() == GetTypeId())
	{
		const TArray<FNetTokenStoreKey>& TokenStoreKeysForType = TokenStoreState.TokenInfoArray[GetTypeId()];
		const int32 TokenIndex(Token.GetIndex());

		return TokenIndex < TokenStoreKeysForType.Num() ? TokenStoreKeysForType[TokenIndex] : FNetTokenStoreKey();
	}
	else
	{
		UE_LOGF(LogNetToken, Error, "FNetTokenDataStore::GetTokenKey Invalid tokentype %ls StoreTypeId: %u", *Token.ToString(), GetTypeId());
		return FNetTokenStoreKey();
	}
}

FNetToken FNetTokenDataStore::CreateAndStoreTokenForKey(FNetTokenStoreKey Key)
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(&TokenStore));

	FNetTokenStoreState& LocalNetTokenStoreState = *TokenStore.GetLocalNetTokenStoreState();

	// The LocalNetTokenStoreState always maps locally assigned tokens -> Key (Index)
	const uint32 NextTokenIndex = LocalNetTokenStoreState.TokenInfoArray[GetTypeId()].Num();

	if (!ensure(NextTokenIndex < FNetToken::MaxNetTokenCount))
	{
		return FNetToken();
	}

	FNetToken NewToken = FNetTokenStore::MakeNetToken(TypeId, NextTokenIndex, TokenStore.IsAuthority() ? FNetToken::ENetTokenAuthority::Authority : FNetToken::ENetTokenAuthority::None);

	// Store token info
	LocalNetTokenStoreState.TokenInfoArray[GetTypeId()].Add(Key);

	// The stored tokens array contains the current NetToken associated with the Key (Index), but it can be updated to use an authoritative token instead.
	StoredTokens[Key.GetKeyIndex()] = NewToken;

	return NewToken;
}

void FNetTokenDataStore::StoreTokenForKey(FNetTokenStoreKey Key, FNetToken NetToken)
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(&TokenStore));

	// The stored tokens array contains the current NetToken associated with the Key (Index), but it can be updated to use an authoritative token instead.
	StoredTokens[Key.GetKeyIndex()] = NetToken;
}

FNetToken FNetTokenDataStore::GetNetTokenFromKey(FNetTokenStoreKey Key) const
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(&TokenStore));
	return StoredTokens[Key.GetKeyIndex()];
}

FNetToken FNetTokenStore::InternalReadNetToken(UE::Net::FNetSerializationContext& Context, FNetToken::FTypeId TokenTypeId)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	FNetToken ReadToken;

	UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(TokenScope, FName(), *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
	const uint32 TokenIndex = ReadPackedUint32(Reader);
	if (const bool bIsValid = (TokenIndex != FNetToken::InvalidTokenIndex))
	{
		const bool bIsAssignedByAuthority = Reader->ReadBool();
		if (TokenTypeId == FNetToken::InvalidTokenTypeId)
		{
			TokenTypeId = Reader->ReadBits(FNetToken::TokenTypeIdBits);
		}

		if (!Context.HasErrorOrOverflow())
		{
			ReadToken = FNetTokenStore::MakeNetToken(TokenTypeId, TokenIndex, bIsAssignedByAuthority ? FNetToken::ENetTokenAuthority::Authority : FNetToken::ENetTokenAuthority::None);
			UE_NET_TRACE_SET_SCOPE_NAME(TokenScope, *ReadToken.ToString());

			// We expected a valid token
			if (!ReadToken.IsValid())
			{
				Context.SetError(GNetError_InvalidValue);
				UE_LOGF(LogNetToken, Error, "Expected a valid token but got an invalid one. TokenTypeId: %u TokenIndex: %u.", TokenTypeId, TokenIndex);
				ensure(false);
			}
		}
	}

	return ReadToken;
}

void FNetTokenStore::InternalWriteNetToken(UE::Net::FNetSerializationContext& Context, FNetToken Token, bool bWriteTokenType)
{
	UE_NET_TRACE_DYNAMIC_NAME_SCOPE(*Token.ToString(), *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	const uint32 TokenIndex = Token.GetIndex();
	WritePackedUint32(Writer, TokenIndex);
	if (TokenIndex != FNetToken::InvalidTokenIndex)
	{
		Writer->WriteBool(Token.IsAssignedByAuthority());
		if (bWriteTokenType)
		{
			Writer->WriteBits(Token.GetTypeId(), FNetToken::TokenTypeIdBits);
		}
	}
}

FNetToken FNetTokenStore::InternalReadNetToken(FArchive& Ar, FNetToken::FTypeId TokenTypeId)
{
	FNetToken ReadToken;

	uint32 TokenIndex = 0;
	Ar.SerializeIntPacked(TokenIndex);
	if (const bool bIsValid = (TokenIndex != FNetToken::InvalidTokenIndex))
	{
		bool bIsAssignedByAuthority = false;
		Ar.SerializeBits(&bIsAssignedByAuthority, 1);

		if (TokenTypeId == FNetToken::InvalidTokenTypeId)
		{
			uint32 TempTokenTypeId = 0;
			Ar.SerializeBits(&TempTokenTypeId, FNetToken::TokenTypeIdBits);
			TokenTypeId = TempTokenTypeId;
		}

		if (!Ar.IsError())
		{
			ReadToken = FNetTokenStore::MakeNetToken(TokenTypeId, TokenIndex, bIsAssignedByAuthority ? FNetToken::ENetTokenAuthority::Authority : FNetToken::ENetTokenAuthority::None);
		}
	}

	return ReadToken;
}

// Note: Be careful when modifying this methods to not affect replay compatibility.
void FNetTokenStore::InternalWriteNetToken(FArchive& Ar, FNetToken Token, bool bWriteTokenType)
{
	UE_NET_TRACE_DYNAMIC_NAME_SCOPE(*Token.ToString(), static_cast<FNetBitWriter&>(Ar), GetTraceCollector(static_cast<FNetBitWriter&>(Ar)), ENetTraceVerbosity::VeryVerbose);
	uint32 TokenIndex = Token.GetIndex();
	Ar.SerializeIntPacked(TokenIndex);
	if (TokenIndex != FNetToken::InvalidTokenIndex)
	{
		bool bIsAssignedByAuthority = Token.IsAssignedByAuthority();
		Ar.SerializeBits(&bIsAssignedByAuthority, 1);

		if (bWriteTokenType)
		{
			uint32 TokenTypeId = Token.GetTypeId();
			Ar.SerializeBits(&TokenTypeId, FNetToken::TokenTypeIdBits);
		}
	}
}

FNetToken FNetTokenDataStore::ReadNetToken(UE::Net::FNetSerializationContext& Context)
{
	return FNetTokenStore::InternalReadNetToken(Context, GetTypeId());
}

void FNetTokenDataStore::WriteNetToken(UE::Net::FNetSerializationContext& Context, FNetToken Token) const
{
	const bool bWriteTokenTypeId = false;
	return FNetTokenStore::InternalWriteNetToken(Context, Token, bWriteTokenTypeId);
}

FNetToken FNetTokenDataStore::ReadNetToken(FArchive& Ar)
{
	return FNetTokenStore::InternalReadNetToken(Ar, GetTypeId());
}

void FNetTokenDataStore::WriteNetToken(FArchive& Ar, FNetToken Token)
{
	const bool bWriteTokenTypeId = false;
	FNetTokenStore::InternalWriteNetToken(Ar, Token, bWriteTokenTypeId);
}

FNetTokenStore::FNetTokenStore()
: LocalNetTokenStoreState(new FNetTokenStoreState)
{
}

FNetTokenStore::~FNetTokenStore()
{
}

void FNetTokenStore::Init(FNetTokenStore::FInitParams& InParams)
{
	Params = InParams;
	RemoteNetTokenStoreStates.SetNum(InParams.MaxConnections);
}

void FNetTokenStore::InitRemoteNetTokenStoreState(uint32 ConnectionId)
{
	if (ensureMsgf((ConnectionId != InvalidConnectionId) && (ConnectionId < (uint32)RemoteNetTokenStoreStates.Num()), TEXT("Trying to init RemoteNetTokenStoreState for invalid connection %u"), ConnectionId))
	{
		if (FNetTokenStoreState* ExistingState = RemoteNetTokenStoreStates[ConnectionId].Get())
		{
			ExistingState->Reset();
		}
		else
		{
			RemoteNetTokenStoreStates[ConnectionId] = MakeUnique<FNetTokenStoreState>();
		}
	}
}

const FNetTokenStoreState* FNetTokenStore::GetRemoteNetTokenStoreState(uint32 ConnectionId) const
{
	if (!ensureMsgf((ConnectionId != InvalidConnectionId) && (ConnectionId < (uint32)RemoteNetTokenStoreStates.Num()), TEXT("Trying to access RemoteNetTokenStoreState for ConnectionID: %u"), ConnectionId))
	{
		return nullptr;
	}
	return RemoteNetTokenStoreStates[ConnectionId].Get();
}

FNetTokenStoreState* FNetTokenStore::GetRemoteNetTokenStoreState(uint32 ConnectionId)
{
	if (!ensureMsgf((ConnectionId != InvalidConnectionId) && (ConnectionId < (uint32)RemoteNetTokenStoreStates.Num()), TEXT("Trying to access non existing RemoteNetTokenStoreState for ConnectionId: %u"), ConnectionId))
	{
		return nullptr;
	}
	return RemoteNetTokenStoreStates[ConnectionId].Get();
}

const FNetTokenDataStore* FNetTokenStore::GetDataStore(FName Name) const
{
	const TTuple<FName, TUniquePtr<FNetTokenDataStore>>* Entry = TokenDataStores.FindByPredicate([Name](const TTuple<FName, TUniquePtr<FNetTokenDataStore>>& Entry){ return Name == Entry.Get<0>(); });
	return Entry ? Entry->Get<1>().Get() : nullptr;
}

 FNetTokenDataStore* FNetTokenStore::GetDataStore(FName Name)
{
	TTuple<FName, TUniquePtr<FNetTokenDataStore>>* Entry = TokenDataStores.FindByPredicate([Name](TTuple<FName, TUniquePtr<FNetTokenDataStore>>& Entry){ return Name == Entry.Get<0>(); });
	return Entry ? Entry->Get<1>().Get() : nullptr;
}

bool FNetTokenStore::RegisterDataStore(TUniquePtr<FNetTokenDataStore> DataStore, FName TokenStoreName)
{
	if (TokenDataStores.Num() >= FNetToken::MaxTypeIdCount)
	{
		return false;
	}

	if (!DataStore)
	{
		return false;
	}

	if (!ensure(GetDataStore(TokenStoreName) == nullptr))
	{
		// Already registered
		return false;
	}

	const UNetTokenTypeIdConfig* TypeIDConfig = GetDefault<UNetTokenTypeIdConfig>();
	const uint32 TypeId = TypeIDConfig->GetTypeID(TokenStoreName);
	DataStore->TypeId = TypeId;
	if (!ensureAlwaysMsgf(DataStore->TypeId != FNetToken::InvalidTokenTypeId, TEXT("No TypeID information found in [/Script/IrisCore.NetTokenTypeIdConfig] Engine.ini for %s"), *TokenStoreName.ToString()))
	{
		UE_LOGF(LogNetToken, Error, "No TypeID information found in [/Script/IrisCore.NetTokenTypeIdConfig] Engine.ini for %ls", *TokenStoreName.ToString());
		return false;
	}

	if (!ensure(TypeId != FNetToken::InvalidTokenTypeId && TypeId < FNetToken::MaxTypeIdCount))
	{
		UE_LOGF(LogNetToken, Error, "Invalid TypeID information found for %ls", *TokenStoreName.ToString());
		return false;
	}

	bool bLargeEnough = static_cast<uint32>(TokenDataStores.Num()) > DataStore->TypeId;
	bool bSomethingAlreadyExists = bLargeEnough ? TokenDataStores[DataStore->TypeId].Value.IsValid() : false;
	FName ExistingName = bLargeEnough ? TokenDataStores[DataStore->TypeId].Key : NAME_None;
	if (!ensureAlwaysMsgf(!bSomethingAlreadyExists, TEXT("An existing TokenDataStore (Name %s) has been created with that TypeId when Trying to add New TokenDataStore with name: %s"),*ExistingName.ToString(),*TokenStoreName.ToString()))
	{
		UE_LOGF(LogNetToken, Error, "An existing TokenDataStore (Name %ls) has been created with that TypeId when Trying to add New TokenDataStore with name: %ls",*ExistingName.ToString(),*TokenStoreName.ToString());
		return false;
	}

	// Need to resize to
	if (!bLargeEnough)
	{
		TokenDataStores.SetNum(DataStore->TypeId + 1);
	}

	TokenDataStores.EmplaceAt(DataStore->TypeId, TokenStoreName, MoveTemp(DataStore));

	return true;
}

bool FNetTokenStore::UnRegisterDataStore(FName TokenStoreName)
{
	if (TokenDataStores.Num() >= FNetToken::MaxTypeIdCount)
	{
		return false;
	}
	
	if (FNetTokenDataStore* DataStore = GetDataStore(TokenStoreName))
	{
		// Delete but don't resize the array
		TokenDataStores[DataStore->TypeId].Value.Reset();
		TokenDataStores[DataStore->TypeId].Key = NAME_None;
		return true;
	}

	// Failed
	return false;
}

void FNetTokenStore::WriteTokenData(FNetSerializationContext& Context, const FNetToken NetToken) const
{
	if (NetToken.IsValid())
	{
		UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(this));

		FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

		const FNetTokenStoreKey& TokenKey = LocalNetTokenStoreState->TokenInfoArray[NetToken.GetTypeId()][NetToken.GetIndex()];

		// Write token data
		TokenDataStores[NetToken.GetTypeId()].Get<1>().Get()->WriteTokenData(Context, TokenKey);
	}
}

void FNetTokenStore::WriteTokenData(FArchive& Ar, const FNetToken NetToken, UPackageMap* Map /*= nullptr*/) const
{
	if (NetToken.IsValid())
	{
		UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(this));

		const FNetToken::FTypeId TokenTypeId = NetToken.GetTypeId();

		// We cannot write token data for bad tokens
		if ((TokenTypeId >= FNetToken::MaxTypeIdCount) || (NetToken.GetIndex() >= (uint32)LocalNetTokenStoreState->TokenInfoArray[TokenTypeId].Num()))
		{
			UE_LOGF(LogNetToken, Error, "Trying to write data for unknown NetToken %ls", *NetToken.ToString());
			return;
		}

		const FNetTokenStoreKey TokenKey = LocalNetTokenStoreState->TokenInfoArray[NetToken.GetTypeId()][NetToken.GetIndex()];

		// Write token data
		TokenDataStores[NetToken.GetTypeId()].Get<1>()->WriteTokenData(Ar, TokenKey, Map);
	}
}

bool FNetTokenStore::ValidateAndStoreNetTokenData(FNetTokenDataStore& DataStore, FNetTokenStoreState& RemoteNetTokenStoreState, const FNetToken NetToken, const FNetTokenStoreKey StoreKey)
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(this));
	if (!StoreKey.IsValid() || !RemoteNetTokenStoreState.ReserveTokenCount(NetToken.GetTypeId(), NetToken.GetIndex() + 1))
	{
		return false;
	}

	TArray<FNetTokenStoreKey>& TokenStoreKeysForType = RemoteNetTokenStoreState.TokenInfoArray[NetToken.GetTypeId()];

	// Since the same tokendata might be exported multiple times we better validate that it is the same data
	const FNetTokenStoreKey& ExistingStoreKey = TokenStoreKeysForType[NetToken.GetIndex()];
	if (!ensureAlways(!ExistingStoreKey.IsValid() || StoreKey == ExistingStoreKey))
	{
		return false;
	}

	// If this is an authtoken and we are not the authority, update the stored key so that we can use the authoriative key instead of the local key
	if (NetToken.IsAssignedByAuthority() && !IsAuthority())
	{
		UE_LOGF(LogNetToken, Verbose, "FNetTokenStore::ReadTokenData - Replaced local key %ls with %ls", *DataStore.StoredTokens[StoreKey.GetKeyIndex()].ToString(), *NetToken.ToString());	

		// This way the next time we lookup this key during assignment use the imported authoritative key which we do not have to export
		DataStore.StoredTokens[StoreKey.GetKeyIndex()] = NetToken;
	}
	
	// Store
	TokenStoreKeysForType[NetToken.GetIndex()] = StoreKey;

	return true;
}

void FNetTokenStore::ReadTokenData(FNetSerializationContext& Context, const FNetToken NetToken, FNetTokenStoreState& RemoteNetTokenStoreState)
{
	if (NetToken.IsValid())
	{
		UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(this));

		FNetBitStreamReader* Reader = Context.GetBitStreamReader();

		FNetToken::FTypeId TokenTypeId = NetToken.GetTypeId();

		if (TokenTypeId >= (uint32)TokenDataStores.Num())
		{
			Context.SetError(GNetError_InvalidValue);
			UE_LOGF(LogNetToken, Error, "Failed to read ReadTokenData for %ls.", *NetToken.ToString());
			ensure(false);
			return;
		}

		FNetTokenDataStore* DataStore = TokenDataStores[TokenTypeId].Get<1>().Get();
		const FNetTokenStoreKey StoreKey = DataStore->ReadTokenData(Context, NetToken);

		if (!ValidateAndStoreNetTokenData(*DataStore, RemoteNetTokenStoreState, NetToken, StoreKey))
		{
			Context.SetError(GNetError_InvalidValue);
			UE_LOGF(LogNetToken, Error, "Failed to read ReadTokenData for %ls.", *NetToken.ToString());
			ensure(false);
			return;
		}
	}
}

void FNetTokenStore::ReadTokenData(FArchive& Ar, const FNetToken NetToken, FNetTokenStoreState& RemoteNetTokenStoreState, UPackageMap* Map /*= nullptr*/)
{
	if (NetToken.IsValid())
	{
		UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(this));

		// Read type
		FNetToken::FTypeId TokenTypeId = NetToken.GetTypeId();

		// Validate that we managed to read and verify the type
		if (Ar.IsError() || TokenTypeId >= (uint32)TokenDataStores.Num() )
		{
			Ar.SetError();
			UE_LOGF(LogNetToken, Error, "Failed to read ReadTokenData for %ls.", *NetToken.ToString());
			return;
		}

		FNetTokenDataStore* DataStore = TokenDataStores[TokenTypeId].Get<1>().Get();
		const FNetTokenStoreKey StoreKey = DataStore->ReadTokenData(Ar, NetToken, Map);

		if (!ValidateAndStoreNetTokenData(*DataStore, RemoteNetTokenStoreState, NetToken, StoreKey))
		{
			Ar.SetError();
			UE_LOGF(LogNetToken, Error, "Failed to read ReadTokenData for %ls.", *NetToken.ToString());
			ensure(false);
			return;
		}
	}
}

void FNetTokenStore::ConditionalWriteNetTokenData(FNetSerializationContext& Context, Private::FNetExportContext* ExportContext, const FNetToken NetToken) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// We should not try to export tokens received from remote
	if (IsAuthority() != NetToken.IsAssignedByAuthority())
	{
		Writer->WriteBool(false);
		return;
	}

	if (ExportContext)
	{
		if (Writer->WriteBool(!ExportContext->IsExported(NetToken)))
		{
			WriteTokenData(Context, NetToken);
			ExportContext->AddExported(NetToken);			
		}
	}
	else
	{
		Writer->WriteBool(true);
		WriteTokenData(Context, NetToken);
	}
}

void FNetTokenStore::ConditionalReadNetTokenData(FNetSerializationContext& Context, const FNetToken NetToken)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const bool bIsExportToken = Reader->ReadBool();
	if (bIsExportToken)
	{
		if (Reader->IsOverflown())
		{
			return;
		}

		FNetObjectResolveContext& ResolveContext = Context.GetInternalContext()->ResolveContext;
	
		ReadTokenData(Context, NetToken, *ResolveContext.RemoteNetTokenStoreState);
	}
}

void FNetTokenStore::AppendExport(FNetSerializationContext& Context, FNetToken NetToken)
{
	using namespace UE::Net::Private;

	FNetExportContext* ExportContext = Context.GetExportContext();
	if (ensure(ExportContext))
	{
		ExportContext->AddPendingExport(NetToken);
	}
}

TArray<FNetToken> FNetTokenStore::GetAllNetTokens() const
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(this));
	
	TArray<FNetToken> Result;

	const FNetTokenStoreState* TokenStoreState = GetLocalNetTokenStoreState();
	for (const TTuple<FName, TUniquePtr<FNetTokenDataStore>>& TokenDataStore : TokenDataStores)
	{
		const uint32 TypeId = TokenDataStore.Get<1>()->GetTypeId();
		const uint32 NumTokensForType = TokenStoreState->TokenInfoArray[TypeId].Num();
		if (NumTokensForType > 1)
		{
			Result.Append(MakeArrayView(&TokenDataStore.Get<1>()->StoredTokens[1], NumTokensForType - 1));
		}
	}

	return MoveTemp(Result);
}

FScopedNetTokenStoreAccess::FScopedNetTokenStoreAccess(const FNetTokenStore* InTokenStore)
{
#if UE_SUPPORT_PARALLEL_IRIS
	check(InTokenStore);
	InTokenStore->NetTokenStoreCS.Lock();
	TokenStore = InTokenStore;
#endif // UE_SUPPORT_PARALLEL_IRIS
}

FScopedNetTokenStoreAccess::~FScopedNetTokenStoreAccess()
{
#if UE_SUPPORT_PARALLEL_IRIS
	check(TokenStore);

	TokenStore->NetTokenStoreCS.Unlock();
#endif // UE_SUPPORT_PARALLEL_IRIS
}

} //namespace UE::Net

uint32 UNetTokenTypeIdConfig::GetTypeID(const FString& TypeName) const
{
	// Need to reevaluate every retrieval in the event of hotfixes, or ini changes due to GFP loading/unloading.
	bool bTypeIdsAppearValid = ReservedTypeIdsAppearValid();
	if (!ensureAlwaysMsgf(bTypeIdsAppearValid, TEXT("Duplicate or Invalid TypeIds detected!")))
	{
		return UE::Net::FNetToken::InvalidTokenTypeId;
	}
	for (const FNetTokenStoreTypeIdPair& TypePair : ReservedTypeIds)
	{
		if (TypePair.StoreTypeName == TypeName)
		{
			return TypePair.TypeID;
		}
	}
	checkf(false, TEXT("Unknown Token Store Type %s. StoreType MUST be declared in Engine.ini."), *TypeName);
	return UE::Net::FNetToken::InvalidTokenTypeId;
}

bool UNetTokenTypeIdConfig::ReservedTypeIdsAppearValid() const
{
	if (ReservedTypeIds.Num() == 0)
	{
		UE_LOGF(LogNetToken, Error, "No TypeID information found in [/Script/IrisCore.NetTokenTypeIdConfig] Engine.ini");
		return false;
	}
	TMap<uint32,FString> Found;
	for (const FNetTokenStoreTypeIdPair& TypeIdPair : ReservedTypeIds)
	{
		if (!ensureAlwaysMsgf(TypeIdPair.TypeID < UE::Net::FNetToken::MaxTypeIdCount, TEXT("TypeID %d found for %s that is larger than FNetToken::MaxIdCount %d"), TypeIdPair.TypeID, *TypeIdPair.StoreTypeName, UE::Net::FNetToken::MaxTypeIdCount))
		{
			UE_LOGF(LogNetToken, Error, "TypeID %d found for %ls that is larger than FNetToken::MaxIdCount %d", TypeIdPair.TypeID, *TypeIdPair.StoreTypeName, UE::Net::FNetToken::MaxTypeIdCount);
			return false;
		}
		if (!ensureAlwaysMsgf(!Found.Contains(TypeIdPair.TypeID), TEXT("Duplicate TypeIDs found: %s and %s with TypeId: %u"), *Found[TypeIdPair.TypeID], *TypeIdPair.StoreTypeName, TypeIdPair.TypeID))
		{
			UE_LOGF(LogNetToken, Error, "Duplicate TypeID information found in [/Script/IrisCore.NetTokenTypeIdConfig] Engine.ini for%ls and %ls with TypeId: %u", *Found[TypeIdPair.TypeID], *TypeIdPair.StoreTypeName, TypeIdPair.TypeID);
			return false;
		}
		Found.Add(TypeIdPair.TypeID, TypeIdPair.StoreTypeName);
	}
	return true;
}
