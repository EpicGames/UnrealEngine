// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectPollFrequencyLimiter.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Math/VectorRegister.h"
#include "Templates/AlignmentTemplates.h"

namespace UE::Net::Private
{

FObjectPollFrequencyLimiter::FObjectPollFrequencyLimiter()
{
}

void FObjectPollFrequencyLimiter::Init(FInternalNetRefIndex InMaxInternalIndex, const Private::FNetRefHandleManager* InNetRefHandleManager)
{
	NetRefHandleManager = InNetRefHandleManager;
	OnMaxInternalNetRefIndexIncreased(InMaxInternalIndex);
}

void FObjectPollFrequencyLimiter::Deinit()
{
	FramesBetweenUpdates.Empty();
	FrameCounters.Empty();
	Frequencies.Empty();
	NetRefHandleManager = nullptr;
}

void FObjectPollFrequencyLimiter::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	const uint32 StorageObjectCount = Align(NewMaxInternalIndex, 32U);

	FramesBetweenUpdates.SetNumZeroed(StorageObjectCount);
	FrameCounters.SetNumZeroed(StorageObjectCount);
	Frequencies.SetNumZeroed(StorageObjectCount);
}

void FObjectPollFrequencyLimiter::Update(const FNetBitArrayView& RelevantObjects, const FNetBitArrayView& DirtyObjects, FNetBitArrayView& OutObjectsToPoll)
{
	IRIS_PROFILER_SCOPE(ObjectPollFrequencyLimiter_Update);

	typedef FNetBitArrayView::StorageWordType WordType;
	constexpr uint32 WordBitCount = FNetBitArrayView::WordBitCount;

	if (!MaxInternalIndex)
	{
		return;
	}

	++FrameIndex;

	uint8* CountersData = FrameCounters.GetData();
	const uint8* FramesBetweenUpdatesData = FramesBetweenUpdates.GetData();

	const WordType* RelevantObjectsData = RelevantObjects.GetData();
	const WordType* DirtyObjectsData = DirtyObjects.GetData();
	WordType* ObjectsToPollData = OutObjectsToPoll.GetData();

	// Decrease counters by one and if they wrap around it's time to poll.
	// Algorithm can easily be throttled to only allow at most N objects to be polled per frame.
#if PLATFORM_ENABLE_VECTORINTRINSICS == 1 && !PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	{
		static_assert(WordBitCount == 32U, "Code assumes a NetBitArray word size of 32 bits.");

		const VectorRegisterInt AllBitsSet = _mm_cmpeq_epi32(_mm_setzero_si128(), _mm_setzero_si128());
		const VectorRegisterInt Zero = _mm_setzero_si128();
		const VectorRegisterInt One = _mm_set1_epi8(char(1));

		for (FInternalNetRefIndex ObjectIndex = 0, ObjectEndIndex = MaxInternalIndex;
			ObjectIndex <= ObjectEndIndex; 
			ObjectIndex += WordBitCount, CountersData += WordBitCount, FramesBetweenUpdatesData += WordBitCount, ++RelevantObjectsData, ++DirtyObjectsData, ++ObjectsToPollData)
		{
			// Skip ranges with no scopable objects.
			const WordType ObjectsInScopeWord = *RelevantObjectsData;
			if (!ObjectsInScopeWord)
			{
				continue;
			}

			VectorRegisterInt Values0 = _mm_loadu_si128(reinterpret_cast<const __m128i*>(CountersData + 0U));
			VectorRegisterInt Values1 = _mm_loadu_si128(reinterpret_cast<const __m128i*>(CountersData + 16U));

			const VectorRegisterInt FramesBetweenUpdates0 = _mm_loadu_si128(reinterpret_cast<const __m128i*>(FramesBetweenUpdatesData + 0U));
			const VectorRegisterInt FramesBetweenUpdates1 = _mm_loadu_si128(reinterpret_cast<const __m128i*>(FramesBetweenUpdatesData + 16U));

			Values0 = _mm_sub_epi8(Values0, One);
			const VectorRegisterInt MaskToUpdate0 = _mm_cmpeq_epi8(Values0, AllBitsSet);
			Values0 = VectorIntSelect(MaskToUpdate0, FramesBetweenUpdates0, Values0);

			Values1 = _mm_sub_epi8(Values1, One);
			const VectorRegisterInt MaskToUpdate1 = _mm_cmpeq_epi8(Values1, AllBitsSet);
			Values1 = VectorIntSelect(MaskToUpdate1, FramesBetweenUpdates1, Values1);

			const uint32 ObjectsToPoll0 = _mm_movemask_epi8(MaskToUpdate0);
			const uint32 ObjectsToPoll1 = _mm_movemask_epi8(MaskToUpdate1);

			_mm_storeu_si128(reinterpret_cast<__m128i*>(CountersData + 0U), Values0);
			_mm_storeu_si128(reinterpret_cast<__m128i*>(CountersData + 16U), Values1);

			// Poll objects that have been set to dirty or are due.
			const WordType ObjectsToPoll = (ObjectsToPoll1 << 16U) | ObjectsToPoll0;
			const WordType DirtyObjectWord = *DirtyObjectsData;
			*ObjectsToPollData = (ObjectsToPoll | DirtyObjectWord) & ObjectsInScopeWord;
		}
	}
#elif PLATFORM_ENABLE_VECTORINTRINSICS == 1 && PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	{
		const uint16x8_t AndMask = vdupq_n_u16(0x0180U);
		const uint8x16_t AllBitsSet = vdupq_n_u8(255U);
		const int16_t ShiftAmountsData[8] = { -7, -5, -3, -1, 1, 3, 5, 7 };
		const int16x8_t ShiftAmounts = vld1q_s16(ShiftAmountsData);

		for (FInternalNetRefIndex ObjectIndex = 0, ObjectEndIndex = MaxInternalIndex;
			ObjectIndex <= ObjectEndIndex;
			ObjectIndex += WordBitCount, CountersData += WordBitCount, FramesBetweenUpdatesData += WordBitCount, ++RelevantObjectsData, ++DirtyObjectsData, ++ObjectsToPollData)
		{
			// Skip ranges with no scopable objects.
			const WordType ObjectsInScopeWord = *RelevantObjectsData;
			if (!ObjectsInScopeWord)
			{
				continue;
			}

			uint8x16_t Values0 = vld1q_u8(CountersData + 0U);
			uint8x16_t Values1 = vld1q_u8(CountersData + 16U);

			const uint8x16_t FramesBetweenUpdates0 = vld1q_u8(FramesBetweenUpdatesData + 0U);
			const uint8x16_t FramesBetweenUpdates1 = vld1q_u8(FramesBetweenUpdatesData + 16U);

			uint8x16_t MaskToUpdate0 = vceqzq_u8(Values0);
			Values0 = vaddq_u8(Values0, AllBitsSet);
			Values0 = vbslq_u8(MaskToUpdate0, FramesBetweenUpdates0, Values0);

			uint8x16_t MaskToUpdate1 = vceqzq_u8(Values1);
			Values1 = vaddq_u8(Values1, AllBitsSet);
			Values1 = vbslq_u8(MaskToUpdate1, FramesBetweenUpdates1, Values1);

			// Create bitmasks via masking and horizontal add.
			uint16x8_t MiddleBits0 = vandq_u16(vreinterpretq_u16_u8(MaskToUpdate0), AndMask);
			uint16x8_t MiddleBits1 = vandq_u16(vreinterpretq_u16_u8(MaskToUpdate1), AndMask);

			uint16x8_t BitsInPlace0 = vshlq_u16(MiddleBits0, ShiftAmounts);
			uint16x8_t BitsInPlace1 = vshlq_u16(MiddleBits1, ShiftAmounts);

			const WordType ObjectsToPoll0 = vaddvq_u16(BitsInPlace0);
			const WordType ObjectsToPoll1 = vaddvq_u16(BitsInPlace1);

			vst1q_u8(CountersData + 0U, Values0);
			vst1q_u8(CountersData + 16U, Values1);

			const WordType ObjectsToPoll = ObjectsToPoll0 | (ObjectsToPoll1 << 16U);
			const WordType DirtyObjectWord = *DirtyObjectsData;
			*ObjectsToPollData = (ObjectsToPoll | DirtyObjectWord) & ObjectsInScopeWord;
		}
	}
#else
	// Slower path using scalar integer instructions.
	{
		for (FInternalNetRefIndex ObjectIndex = 0, ObjectEndIndex = MaxInternalIndex;
			ObjectIndex <= ObjectEndIndex;
			ObjectIndex += WordBitCount, CountersData += WordBitCount, FramesBetweenUpdatesData += WordBitCount, ++RelevantObjectsData, ++DirtyObjectsData, ++ObjectsToPollData)
		{
			// Skip ranges with no scopable objects.
			WordType ObjectsInScopeWord = *RelevantObjectsData;
			if (!ObjectsInScopeWord)
			{
				continue;
			}

			WordType ObjectsToPoll = 0;
			WordType ObjectsInScopeMask = ObjectsInScopeWord;
			for (SIZE_T It = 0, IndexOffset = 0; It < 8; ++It, IndexOffset += 4U, ObjectsInScopeMask >>= 4U)
			{
				if (!(ObjectsInScopeMask & 15U))
				{
					continue;
				}

				uint8 Counter0 = CountersData[IndexOffset + 0];
				uint8 Counter1 = CountersData[IndexOffset + 1];
				uint8 Counter2 = CountersData[IndexOffset + 2];
				uint8 Counter3 = CountersData[IndexOffset + 3];

				const uint8 FramesBetweenUpdates0 = FramesBetweenUpdatesData[IndexOffset + 0];
				const uint8 FramesBetweenUpdates1 = FramesBetweenUpdatesData[IndexOffset + 1];
				const uint8 FramesBetweenUpdates2 = FramesBetweenUpdatesData[IndexOffset + 2];
				const uint8 FramesBetweenUpdates3 = FramesBetweenUpdatesData[IndexOffset + 3];

				--Counter0;
				--Counter1;
				--Counter2;
				--Counter3;

				const uint32 Mask0 = Counter0 == 255 ? 1 : 0;
				const uint32 Mask1 = Counter1 == 255 ? 2 : 0;
				const uint32 Mask2 = Counter2 == 255 ? 4 : 0;
				const uint32 Mask3 = Counter3 == 255 ? 8 : 0;

				Counter0 = Mask0 ? FramesBetweenUpdates0 : Counter0;
				Counter1 = Mask1 ? FramesBetweenUpdates1 : Counter1;
				Counter2 = Mask2 ? FramesBetweenUpdates2 : Counter2;
				Counter3 = Mask3 ? FramesBetweenUpdates3 : Counter3;

				const uint32 Mask = Mask3 | Mask2 | Mask1 | Mask0;
				ObjectsToPoll |= (Mask << IndexOffset);

				CountersData[IndexOffset + 0] = Counter0;
				CountersData[IndexOffset + 1] = Counter1;
				CountersData[IndexOffset + 2] = Counter2;
				CountersData[IndexOffset + 3] = Counter3;
			}

			const WordType DirtyObjectWord = *DirtyObjectsData;
			*ObjectsToPollData = (ObjectsToPoll | DirtyObjectWord) & ObjectsInScopeWord;
		}
	}
#endif
}

void FObjectPollFrequencyLimiter::ReinitFramesBetweenUpdates()
{
	if (!NetRefHandleManager)
	{
		return;
	}

	auto UpdatePollFrequency = [this](uint32 RootObjectIndex)
	{
		const float PollFrequency = Frequencies[RootObjectIndex];
		SetPollFrequency(RootObjectIndex, PollFrequency);

		// Make sure the subobjects are polled the same frame as the root object.
		for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(RootObjectIndex))
		{
			SetPollWithObject(RootObjectIndex, SubObjectIndex);
		}
	};

	// Update poll frame periods for all root objects
	const FNetBitArrayView ReplicatedObjects = NetRefHandleManager->GetGlobalScopableInternalIndices();
	const FNetBitArrayView SubObjects = MakeNetBitArrayView(NetRefHandleManager->GetSubObjectInternalIndices());
	FNetBitArrayView::ForAllSetBits(ReplicatedObjects, SubObjects, FNetBitArrayView::AndNotOp, UpdatePollFrequency);
}

}
