// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Array.h"
#include "Net/Core/NetBitArray.h"

namespace UE::Net
{
	typedef uint32 FInternalNetRefIndex;
}
namespace UE::Net::Private
{
	class FNetRefHandleManager;
}

namespace UE::Net::Private
{

class FObjectPollFrequencyLimiter
{
public:
	FObjectPollFrequencyLimiter();

	void Init(FInternalNetRefIndex MaxInternalIndex, const Private::FNetRefHandleManager* InNetRefHandleManager = nullptr);
	void Deinit();

	void SetUpdateFrequency(float Frequency);
	float GetUpdateFrequency() const;

	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

	void SetPollFrequency(FInternalNetRefIndex InternalIndex, float PollFrequency);

	void SetPollWithObject(FInternalNetRefIndex ObjectToPollWithInternalIndex, FInternalNetRefIndex InternalIndex);

	/** 
	* Produces the list of objects that should be polled this frame.
	* This list is composed of relevant objects that are dirty or that hit their poll period this frame.
	*/
	void Update(const FNetBitArrayView& RelevantObjects, const FNetBitArrayView& DirtyObjects, FNetBitArrayView& OutObjectsToPoll);

private:
	void SetPollFramePeriod(FInternalNetRefIndex InternalIndex, uint8 PollFramePeriod);
	uint8 ConvertFrequencyToFramesBetweenUpdates(float Frequency);
	void ReinitFramesBetweenUpdates();

private:

	const Private::FNetRefHandleManager* NetRefHandleManager = nullptr;
	FInternalNetRefIndex MaxInternalIndex = 0;
	uint32 FrameIndex = 0;
	// Presumed update frequency. Used by ConvertFrequencyToFramesBetweenUpdates.
	float UpdateFrequency = 30.0f;
	// We store the number of frames between updates as a byte to be able to process 16 objects per instruction.
	// This limits polling to at least every 256th frame. At 30Hz this means every 8.5 seconds.
	uint8 FrameIndexOffsets[256] = {};
	TArray<uint8> FramesBetweenUpdates;
	TArray<uint8> FrameCounters;
	TArray<float> Frequencies;
};

inline void FObjectPollFrequencyLimiter::SetUpdateFrequency(float InUpdateFrequency)
{
	if (InUpdateFrequency <= 0.0f)
	{
		// 0 frequency seems incorrect and would cause division by zero.
		ensureMsgf(false, TEXT("ObjectPollFrequencyLimiter update frequency must be greater than zero"));
		InUpdateFrequency = 1.0f;
	}

	if (InUpdateFrequency != UpdateFrequency)
	{
		UpdateFrequency = InUpdateFrequency;
		ReinitFramesBetweenUpdates();
	}
}

inline float FObjectPollFrequencyLimiter::GetUpdateFrequency() const
{
	return UpdateFrequency;
}

inline void FObjectPollFrequencyLimiter::SetPollFrequency(FInternalNetRefIndex InternalIndex, float PollFrequency)
{
	Frequencies[InternalIndex] = PollFrequency;
	SetPollFramePeriod(InternalIndex, ConvertFrequencyToFramesBetweenUpdates(PollFrequency));
}

inline void FObjectPollFrequencyLimiter::SetPollFramePeriod(FInternalNetRefIndex InternalIndex, uint8 PollFramePeriod)
{
	MaxInternalIndex = FPlatformMath::Max(MaxInternalIndex, InternalIndex);

	FramesBetweenUpdates[InternalIndex] = PollFramePeriod;
	// Spread the polling of objects with the same frequency so that if you add lots of objects the same frame they won't be polled at the same time. The update loop decrements counters so we need to be careful with how we offset things.
	const uint8 FrameOffset = --FrameIndexOffsets[PollFramePeriod];
	FrameCounters[InternalIndex] = static_cast<uint8>(uint32(~(FrameIndex + FrameOffset)) % uint32(PollFramePeriod + 1U));
}

inline void FObjectPollFrequencyLimiter::SetPollWithObject(FInternalNetRefIndex ObjectToPollWithInternalIndex, FInternalNetRefIndex InternalIndex)
{
	MaxInternalIndex = FPlatformMath::Max(MaxInternalIndex, InternalIndex);

	// Copy state from object to poll with
	FramesBetweenUpdates[InternalIndex] = FramesBetweenUpdates[ObjectToPollWithInternalIndex];
	FrameCounters[InternalIndex] = FrameCounters[ObjectToPollWithInternalIndex];

	// This is really only somewhat useful if this was to be called for a root object rather than a subobject since we will call this method for subobjects in ReinitFramesBetweenUpdates.
	// For other root objects the SetPollWithObject would have to be called manually when the update frequency changes. This call will only guarantee that we poll as often as the object to poll with, but not necessarily on the same frame.
	Frequencies[InternalIndex] = Frequencies[ObjectToPollWithInternalIndex];
}

inline uint8 FObjectPollFrequencyLimiter::ConvertFrequencyToFramesBetweenUpdates(float Frequency)
{
	if (Frequency <= 0.0f)
	{
		return 0U;
	}

	uint32 FramesBetweenUpdatesForObject = static_cast<uint32>(UpdateFrequency/FPlatformMath::Max(0.001f, Frequency));
	if (FramesBetweenUpdatesForObject > 0)
	{
		FramesBetweenUpdatesForObject--;
	}

	return static_cast<uint8>(FMath::Clamp<uint32>(FramesBetweenUpdatesForObject, 0U, 255U));
}

}
