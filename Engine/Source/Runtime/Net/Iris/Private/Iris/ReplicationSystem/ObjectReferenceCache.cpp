// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectReferenceCache.h"

#include "HAL/IConsoleManager.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisMemoryTracker.h"

#include "Iris/IrisConstants.h"

#include "Iris/ReplicationSystem/ObjectReferenceTypes.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Iris/ReplicationSystem/PendingBatchData.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/StringTokenStore.h"

#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Iris/Serialization/QuantizedObjectReference.h"

#include "Iris/Stats/NetStatsContext.h"

#include "Misc/CoreMiscDefines.h"
#include "Misc/PackageName.h"
#include "Misc/StringBuilder.h"

#include "Net/Core/Connection/NetEnums.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Net/Core/Trace/NetTrace.h"

#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"

#ifndef UE_NET_VALIDATE_REFERENCECACHE
#	define UE_NET_VALIDATE_REFERENCECACHE 0
#endif

#ifndef UE_NET_OBJECTREFERENCECACHE_ASYNCLOAD_DEBUG
#	define UE_NET_OBJECTREFERENCECACHE_ASYNCLOAD_DEBUG !(UE_BUILD_SHIPPING)
#endif

// Don't compile verbose logs in Shipping builds
#if (UE_BUILD_SHIPPING)
#	define UE_NET_REFERENCECACHE_LOG_COMPILE_VERBOSITY Log
#else
#	define UE_NET_REFERENCECACHE_LOG_COMPILE_VERBOSITY All
#endif 

DEFINE_LOG_CATEGORY_STATIC(LogIrisReferences, Log, UE_NET_REFERENCECACHE_LOG_COMPILE_VERBOSITY);

#define UE_LOG_REFERENCECACHE(Verbosity, Format, ...)  UE_LOG(LogIrisReferences, Verbosity, Format, ##__VA_ARGS__)
#define UE_CLOG_REFERENCECACHE(Condition, Verbosity, Format, ...)  UE_CLOG(Condition, LogIrisReferences, Verbosity, Format, ##__VA_ARGS__)

namespace UE::Net::Private
{

static bool bIrisAllowAsyncLoading = true;
FAutoConsoleVariableRef CVarIrisAllowAsyncLoading(TEXT("net.iris.AllowAsyncLoading"), bIrisAllowAsyncLoading, TEXT("Flag to allow or disallow async loading when using iris replication. Note: net.allowAsyncLoading must also be enabled."), ECVF_Default);

static bool bIrisAllowAsyncReload = true;
FAutoConsoleVariableRef CVarIrisAllowAsyncReload(TEXT("net.iris.AllowAsyncReload"), bIrisAllowAsyncReload, TEXT("Flag to allow async reload of objects from packages that might be partially garbage collected. Note: Only relevant if async loading is enabled."), ECVF_Default);

#if UE_NET_ASYNCLOADING_DEBUG
static bool bForceAsyncLoading = false;
FAutoConsoleVariableRef CVarIrisForceAsyncLoading(TEXT("net.Iris.AsyncLoading.ForceEnable"), bForceAsyncLoading, TEXT("Force the async loading path even in Editor/PIE."), ECVF_Default);
#endif

#if UE_NET_OBJECTREFERENCECACHE_ASYNCLOAD_DEBUG
static FString IrisAsyncLoadingFailPackageName;
FAutoConsoleVariableRef CVarIrisAsyncLoadingFailPackageName(TEXT("net.Iris.AsyncLoading.FailPackageName"), IrisAsyncLoadingFailPackageName, TEXT("Always fail loading the specified package name. For debugging purposes."), ECVF_Default);

static bool bIrisAsyncLoadingFailNextLoad;
FAutoConsoleVariableRef CVarIrisAsyncLoadingFailNextLoad(TEXT("net.Iris.AsyncLoading.FailNextLoad"), bIrisAsyncLoadingFailNextLoad, TEXT("Simulate failing the next async loading request. For debugging purposes."), ECVF_Default);

static bool bIrisAsyncLoadingFailAllLoads;
FAutoConsoleVariableRef CVarIrisAsyncLoadingFailAllLoads(TEXT("net.Iris.AsyncLoading.FailAllLoads"), bIrisAsyncLoadingFailAllLoads, TEXT("Simulate failing all async loading requests. For debugging purposes."), ECVF_Default);
#endif

static bool bIrisVerifyOuterForStaleDynamicReferences = true;
FAutoConsoleVariableRef CVarIrisVerifyOuterForStaleDynamicReferences(TEXT("net.iris.VerifyOuterForStaleDynamicReferences"), bIrisVerifyOuterForStaleDynamicReferences, TEXT("If set, we will verify outer for when creationg stably named references of dynamic objects. Default is true."), ECVF_Default);

static bool bIrisEvictTrackedReferencesWithDynamicOuterFromCache = true;
FAutoConsoleVariableRef CVarIrisEvictTrackedReferencesWithDynamicOuterFromCache(TEXT("net.iris.EvictTrackedReferencesWithDynamicOuterFromCache"), bIrisEvictTrackedReferencesWithDynamicOuterFromCache, TEXT("If set, relative references to dynamic outer will be evicted from cache once the outer is unregistered and no longer replicates. Default is true."), ECVF_Default);

/**
 * Don't allow infinite recursion of InternalLoadObject - an attacker could
 * send malicious packets that cause a stack overflow on the server.
 */
static const int INTERNAL_READ_REF_RECURSION_LIMIT = 16;


//------------------------------------------------------------------------
// FPendingAsyncLoadRequest
//------------------------------------------------------------------------

FObjectReferenceCache::FPendingAsyncLoadRequest::FPendingAsyncLoadRequest(FNetRefHandle InNetRefHandle, double InRequestStartTime)
: RequestStartTime(InRequestStartTime)
{
	NetRefHandles.Add(InNetRefHandle);
}

void FObjectReferenceCache::FPendingAsyncLoadRequest::Merge(const FPendingAsyncLoadRequest& Other)
{
	for (FNetRefHandle OtherNetRefHandle : Other.NetRefHandles)
	{
		NetRefHandles.AddUnique(OtherNetRefHandle);
	}
}

void FObjectReferenceCache::FPendingAsyncLoadRequest::Merge(FNetRefHandle InNetRefHandle)
{
	NetRefHandles.AddUnique(InNetRefHandle);
}

//------------------------------------------------------------------------
// FObjectReferenceCache
//------------------------------------------------------------------------

FObjectReferenceCache::FObjectReferenceCache()
{
	SetAsyncLoadMode(EAsyncLoadMode::UseCVar);
}

void FObjectReferenceCache::Init(const FObjectReferenceCacheParams& Params)
{
	bIsAuthority = Params.bAuthority;
	ObjectToNetRefHandleSerial = Params.ObjectToNetRefHandleSerial;
	NetRefHandleSerialToObject = Params.NetRefHandleSerialToObject;
}

bool FObjectReferenceCache::SupportsObjectInternal(const UObject* Object) const
{
	if (Object->IsFullNameStableForNetworking())
	{
		// If object is fully net addressable, it's definitely supported
		return true;
	}

	if (Object->IsSupportedForNetworking())
	{
		// This means the server will explicitly tell the client to spawn and assign the id for this object
		return true;
	}

	return false;
}

bool FObjectReferenceCache::IsDynamicObject(const UObject* Object) const
{
	checkSlow(Object != nullptr);
	checkSlow(Object->IsSupportedForNetworking());

	// Any non net addressable object is dynamic
	return IsDynamicInternal(Object);
}

bool FObjectReferenceCache::IsDynamicInternal(const UObject* Object) const
{
	// Any non net addressable object is dynamic
	return !Object->IsFullNameStableForNetworking();
}

bool FObjectReferenceCache::CanClientLoadObjectInternal(const UObject* Object, bool bIsDynamic) const
{
	// PackageMapClient can't load maps, we must wait for the client to load the map when ready
	// These references are special references, where the reference and all child references resolve once the map has been loaded
	if (bIsDynamic)
	{
		return false;
	}

	if (Object->GetPackage()->ContainsMap())
	{
		return false;
	}

#if WITH_EDITOR
	// For objects using external package, we need to do the test on the package of their outer most object
	// (this is currently only possible in Editor)
	UObject* OutermostObject = Object->GetOutermostObject();
	if (OutermostObject && OutermostObject->GetPackage()->ContainsMap())
	{
		return false;
	}
#endif

	// We can load everything else
	return true;
}

bool FObjectReferenceCache::ShouldIgnoreWhenMissing(FNetRefHandle RefHandle) const
{
	if (RefHandle.IsDynamic())
	{
		return true;		// Ignore missing dynamic references (even on server because client may send RPC on/with object it doesn't know server destroyed)
	}

	if (IsAuthority())
	{
		return false;		// Server never ignores when missing, always warns
	}

	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));

	const FCachedNetObjectReference* CacheObject = ReferenceHandleToCachedReference.Find(RefHandle);
	if (CacheObject == nullptr)
	{
		return false;		// If we haven't been told about this static reference before, we need to warn
	}

	const FCachedNetObjectReference* OutermostCacheObject = CacheObject;

	while (OutermostCacheObject != nullptr && OutermostCacheObject->OuterNetRefHandle.IsValid())
	{
		OutermostCacheObject = ReferenceHandleToCachedReference.Find(OutermostCacheObject->OuterNetRefHandle);
	}

	if (OutermostCacheObject != nullptr)
	{
		// If our outer package is not fully loaded, then don't warn, assume it will eventually come in
		if (OutermostCacheObject->bIsPending)
		{
			// Outer is pending, don't warn
			return true;
		}
		// Sometimes, other systems async load packages, which we don't track, but still must be aware of
		if (OutermostCacheObject->Object != nullptr && !OutermostCacheObject->Object->GetPackage()->IsFullyLoaded())
		{
			return true;
		}
	}

	// Ignore warnings when we explicitly are told to
	return CacheObject->bIgnoreWhenMissing;
}

bool FObjectReferenceCache::RenamePathForPie(uint32 ConnectionId, FString& Str, bool bReading, UReplicationSystem* ReplicationSystem)
{
	if constexpr (WITH_EDITOR)
	{
		UObjectReplicationBridge* ReplicationBridge = ReplicationSystem->GetReplicationBridgeAs<UObjectReplicationBridge>();
		return ReplicationBridge->RemapPathForPIE(ConnectionId, Str, bReading);
	}
	else
	{
		return false;
	}
}

FNetObjectReference FObjectReferenceCache::GetOrCreateObjectReferenceUsingPath(const UObject* Instance, UReplicationSystem* ReplicationSystem)
{
	if (Instance && Instance->IsFullNameStableForNetworking())
	{
		FStringTokenStore* LocalStringTokenStore = ReplicationSystem->GetNetTokenStore()->GetDataStore<FStringTokenStore>();

		FString ObjectPath = Instance->GetPathName();
		const FNetToken ObjectPathToken = LocalStringTokenStore->GetOrCreateToken(ObjectPath);

		return MakeNetObjectReference(FNetRefHandle::GetInvalid(), ObjectPathToken);
	}

	return GetOrCreateObjectReference(Instance, ReplicationSystem);
}

FNetRefHandle FObjectReferenceCache::CreateObjectReferenceHandle(const UObject* Object, UReplicationSystem* ReplicationSystem)
{
	FNetObjectReference Ref;
	CreateObjectReferenceInternal(Object, Ref, ReplicationSystem);

	return Ref.GetRefHandle();
}

FNetRefHandle FObjectReferenceCache::ReassignObjectRefHandle(const UObject* Object, UReplicationSystem* ReplicationSystem)
{
	// Clear the handle already assigned to the pointer.
	const FSetElementId ObjectToNetRefId = ObjectToNetReferenceHandle.FindId(Object);
	if (ObjectToNetRefId.IsValidId())
	{
		// TODO: Should we remove old NetRefHandle -> Object map entry ?
		//		 It depends if the RepSystem will want to access the object via the old NetRefHandle later.
		//		 But since the handle is supposed to be in a Destroyed/StopReplication state its already dangerous to access  the object.
		//		 We might want to mark the entry + netrefhandle as deprecated on StopReplication directly and only access the object via an Explicit call if it is actually neeeded.
		//const FNetRefHandle DeprecatedRefHandle = ObjectToNetReferenceHandle.Get(ObjectToNetRefId);
		//ReferenceHandleToCachedReference.Remove(DeprecatedRefHandle);

		// Only remove the object->handle map for now.
		ObjectToNetReferenceHandle.Remove(ObjectToNetRefId);
	}

	FNetObjectReference ObjectRef;
	bool bSuccess = CreateObjectReferenceInternal(Object, ObjectRef, ReplicationSystem);

	return bSuccess ? ObjectRef.GetRefHandle() : FNetRefHandle::GetInvalid();
}

FNetRefHandle FObjectReferenceCache::PreRegisterObjectReferenceHandle(const UObject* Object, UReplicationSystem* ReplicationSystem)
{
	if (IsAuthority())
	{
		FNetObjectReference Ref;
		UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
		CreateObjectReferenceInternal(Object, Ref, ReplicationSystem);

		FCachedNetObjectReference* CachedRef = ReferenceHandleToCachedReference.Find(Ref.GetRefHandle());
		if (CachedRef)
		{
			CachedRef->bIsPreRegistered = true;
		}

		return Ref.GetRefHandle();
	}
	else
	{
		UE_LOG_REFERENCECACHE(Error, TEXT("FObjectReferenceCache::PreRegisterObjectReferenceHandle attempted to create new handle for object %s but this is not the reference authority (it's a client)."), *GetFullNameSafe(Object));
		ensureMsgf(false, TEXT("FObjectReferenceCache::PreRegisterObjectReferenceHandle attempted to create new handle without having reference authority."));
	}

	return FNetRefHandle::GetInvalid();
}

bool FObjectReferenceCache::IsCachedOuterChainValid(const UObject* Object, FNetRefHandle RefHandle, const FCachedNetObjectReference& CachedObject) const
{
	if (!(RefHandle.IsDynamic() && CachedObject.RelativePath.IsValid()))
	{
		return true;
	}

	const UObject* CurrentOuter = Object->GetOuter();
	FNetRefHandle CurrentCachedOuterHandle = CachedObject.OuterNetRefHandle;

	while (CurrentOuter != nullptr || CurrentCachedOuterHandle.IsValid())
	{
		const FCachedNetObjectReference* CachedOuterPtr = CurrentCachedOuterHandle.IsValid() ? ReferenceHandleToCachedReference.Find(CurrentCachedOuterHandle) : nullptr;
		if (CachedOuterPtr)
		{
			if (CachedOuterPtr->Object.Get() != CurrentOuter)
			{
				return false;
			}
			if (!(CurrentCachedOuterHandle.IsDynamic() && CachedOuterPtr->RelativePath.IsValid()))
			{
				break;
			}
			CurrentOuter = CurrentOuter ? CurrentOuter->GetOuter() : nullptr;
			CurrentCachedOuterHandle = CachedOuterPtr->OuterNetRefHandle;
		}
		else
		{
			return !CurrentCachedOuterHandle.IsValid();
		}
	}
	return true;
}

bool FObjectReferenceCache::CreateObjectReferenceInternal(const UObject* Object, FNetObjectReference& OutReference, UReplicationSystem* ReplicationSystem)
{
	if (!IsValid(Object))
	{
		return false;
	}
	
	FStringTokenStore* LocalStringTokenStore = ReplicationSystem->GetNetTokenStore()->GetDataStore<FStringTokenStore>();

	CA_ASSUME(Object != nullptr);

	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));

	// Check if we are already know about this object
	if (FNetRefHandle* RefHandlePtr = ObjectToNetReferenceHandle.Find(Object))
	{
		// Cache RefHandle as the pointer can become invalidated later.
		const FNetRefHandle RefHandle = *RefHandlePtr;

		// Verify that this is the same object and clean up if it is not
		if (FCachedNetObjectReference* CachedObjectPtr = ReferenceHandleToCachedReference.Find(RefHandle))
		{
			FCachedNetObjectReference& CachedObject = *CachedObjectPtr;
			if (CachedObject.Object.Get() == Object)
			{
				const bool bOuterChainValid = !bIrisVerifyOuterForStaleDynamicReferences || IsCachedOuterChainValid(Object, RefHandle, CachedObject);

				if (bOuterChainValid)
				{
					UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("ObjectReferenceCache::CreateObjectReferenceHandle Found existing %s for ObjectPath %s Object: %s (0x%p), OuterNetRefHandle: %s"),
					*CachedObject.NetRefHandle.ToString(), ToCStr(Object->GetPathName()), *GetNameSafe(Object), Object, *CachedObject.OuterNetRefHandle.ToString());
					OutReference = MakeNetObjectReference(CachedObject);

					return true;
				}
			}

			UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::CreateObjectReferenceHandle Removed %s from ObjectToNetReferenceHandle due to stale cache. Cached Object (0x%p).  New object %s (0x%p)"),
				*CachedObject.NetRefHandle.ToString(), CachedObject.ObjectKey, *GetNameSafe(Object), Object);

			ObjectToNetReferenceHandle.Remove(CachedObject.ObjectKey);
			ObjectToNetReferenceHandle.Remove(Object);
			RefHandlePtr = nullptr;

			if (RefHandle.IsStatic())
			{
				UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::CreateObjectReferenceHandle clearing stale cache for static handle %s. Cached Object (0x%p).  New object %s (0x%p)"),
					*CachedObject.NetRefHandle.ToString(), CachedObject.ObjectKey, *GetNameSafe(Object), Object);

				// Note we only cleanse the object reference
				// we still keep the cachedObject data around in order to be able to serialize static destruction infos
				CachedObject.ObjectKey = nullptr;
				CachedObject.Object = nullptr;

			}
			else
			{
				UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::CreateObjectReferenceHandle removing cache for handle %s. Cached Object (0x%p). New object %s (0x%p)"),
					*CachedObject.NetRefHandle.ToString(), CachedObject.ObjectKey, *GetNameSafe(Object), Object);

				ReferenceHandleToCachedReference.Remove(RefHandle);
			}
		}
		else
		{
			UE_LOG_REFERENCECACHE(Warning, TEXT("ObjectReferenceCache::CreateObjectReferenceInternal removed %s from ObjectToNetReferenceHandle due to mismatch with cache. Object %s (0x%p) "), *RefHandle.ToString(), *GetNameSafe(Object), Object);
			ObjectToNetReferenceHandle.Remove(Object);
			RefHandlePtr = nullptr;
		}
		
	}

#if WITH_EDITOR
	const UPackage* ObjectPackage = Object->GetPackage();
	if (ObjectPackage->HasAnyPackageFlags(PKG_PlayInEditor))
	{
		const int32 ReplicationSystemPIEInstanceID = ReplicationSystem->GetPIEInstanceID();
		const int32 ObjectPIEInstanceID = ObjectPackage->GetPIEInstanceID();

		if (!ensureMsgf(ReplicationSystemPIEInstanceID == ObjectPIEInstanceID, TEXT("FObjectRefereceCache::CreateObjectReferenceInternal: Object %s is not supported since its PIE InstanceID: %d differs from the one of the NetDriver's world PIE InstanceID: %d, it will replicate as an invalid reference."), *GetPathNameSafe(Object), ObjectPIEInstanceID, ReplicationSystemPIEInstanceID))
		{
			// Don't replicate references to objects owned by other PIE instances.
			return false;
		}
	}
#endif

	// Translated from GuidCache, but kept as separate variable to avoid calling the same functions multiple times
	const bool bIsFullNameStableForNetworking = Object->IsFullNameStableForNetworking();
	const bool bIsStableForNetworking = Object->IsNameStableForNetworking();
	const bool bIsDynamic = !bIsFullNameStableForNetworking;
	const bool bIsStatic = !bIsDynamic;

	// Check if we should replicate this object?
	if (!(bIsFullNameStableForNetworking || Object->IsSupportedForNetworking()))
	{
		return false;
	}

	// We currently do not support creating new references from client to server so we currently using a path based reference
	if (!IsAuthority())
	{
		// Find known outer if there is one, and express reference as a relative path to the outer
		for (const UObject* Outer = Object; Outer != nullptr; Outer = Outer->GetOuter())
		{
			if (FNetRefHandle* ReplicatedOuter = ObjectToNetReferenceHandle.Find(Outer))
			{
				FString RelativeObjectPath = Object->GetPathName(Outer);
				constexpr bool bReading = false;
				// The connection ID isn't used unless reading.
				RenamePathForPie(UE::Net::InvalidConnectionId, RelativeObjectPath, bReading, ReplicationSystem);
				OutReference = MakeNetObjectReference(*ReplicatedOuter, LocalStringTokenStore->GetOrCreateToken(RelativeObjectPath));

				return true;
			}
		}

		// For static objects we can always fall back on the full pathname
		if (bIsStatic)
		{
			FString ObjectPath = Object->GetPathName();
			constexpr bool bReading = false;
			// The connection ID isn't used unless reading.
			RenamePathForPie(UE::Net::InvalidConnectionId, ObjectPath, bReading, ReplicationSystem);
			OutReference = MakeNetObjectReference(FNetRefHandle::GetInvalid(), LocalStringTokenStore->GetOrCreateToken(ObjectPath));	
			
			return true;
		}

		// If the object is dynamic and we did not find an outer handle there's nothing we can do
		return false;
	}

	// Generate a new RefHandle
	FNetRefHandle NetRefHandle; 

	FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
	
	if (ReplicationSystem->GetProxyType() == UE::Net::EProxyType::Frontend)
	{
		TSharedPtr<FObjectReferenceCache> SharedObjectReferenceCache = FReplicationSystemFactory::GetSharedObjectReferenceCache();
		if (!ensureMsgf(SharedObjectReferenceCache, TEXT("Failed initializing and retrieving the shared object reference cache.")))
		{
			return false;
		}

		FNetRefHandle SharedNetRefHandle = SharedObjectReferenceCache->GetObjectReferenceHandleFromObject(Object); 
		if (SharedNetRefHandle.IsValid())
		{
			UE_LOGF(LogIris, Verbose, "ObjectReferenceCache::CreateObjectReferenceHandle Found a valid NetRefHandle %ls in the shared object reference cache for object %ls", *SharedNetRefHandle.ToString(), *Object->GetName());
			NetRefHandle = NetRefHandleManager.MakeNetRefHandle(SharedNetRefHandle.GetId(), ReplicationSystem->GetId());
		}
		else
		{
			UE_LOGF(LogIris, Warning, "ObjectReferenceCache::CreateObjectReferenceHandle Unable to find a NetRefHandle for object %ls in the shared object reference cache.", *Object->GetName());			
			return false;
		}
	}
	else
	{
		if (ObjectToNetRefHandleSerial.IsBound())
		{
			TObjectPtr<const UObject> ObjectPtr(Object);
			const uint64 Serial = ObjectToNetRefHandleSerial.Execute(ObjectPtr);
			NetRefHandle = NetRefHandleManager.AllocateNetRefHandleFromSerial(Serial, bIsStatic);
		}
		else
		{
			NetRefHandle = NetRefHandleManager.AllocateNetRefHandle(bIsStatic);
		}
	}

	check(NetRefHandle.IsValid());

	FNetToken PathToken;	

	// If our name is stable, or at least relative to our outer
	if (bIsFullNameStableForNetworking || Object->IsNameStableForNetworking())
	{
		FString ObjectPath = Object->GetName();
		constexpr bool bReading = false;
		// The connection ID isn't used unless reading.
		RenamePathForPie(UE::Net::InvalidConnectionId, ObjectPath, bReading, ReplicationSystem);

#if WITH_EDITOR
		ensureMsgf(!ObjectPath.IsEmpty(), TEXT("NetworkRemapPath found %s to be an invalid name for %s. This object will not replicate!"), *Object->GetName(), *GetPathNameSafe(Object));
#endif

		PathToken = LocalStringTokenStore->GetOrCreateToken(ObjectPath);
	}

	// Register cached ref
	FCachedNetObjectReference CachedObject;

	CachedObject.Object = MakeWeakObjectPtr(const_cast<UObject*>(Object));
	CachedObject.ObjectKey = Object;
	CachedObject.NetRefHandle = NetRefHandle;
	CachedObject.RelativePath.AssignToken(PathToken, { .ReplicationSystemId = ReplicationSystem->GetId(), .ConnectionId = InvalidConnectionId });
	CachedObject.OuterNetRefHandle = CreateObjectReferenceHandle(Object->GetOuter(), ReplicationSystem);
	CachedObject.bIsPackage = bIsStatic && !Object->GetOuter();
	CachedObject.bNoLoad = !CanClientLoadObjectInternal(Object, bIsDynamic); // We are probably going to deal with this as a dependency
	CachedObject.bIgnoreWhenMissing = !CachedObject.bNoLoad;
	CachedObject.bIsPending = false;
	CachedObject.bIsBroken = false;

	// Make sure it really is a package
	check(CachedObject.bIsPackage == (Cast<UPackage>(Object) != nullptr));

	ReferenceHandleToCachedReference.Add(NetRefHandle, CachedObject);
	ObjectToNetReferenceHandle.Add(Object, NetRefHandle);

#if UE_NET_TRACE_ENABLED
	if (NetRefHandle.IsValid())
	{
		TStringBuilder<512> Builder;
		Builder.Appendf(TEXT("Ref_%s"), PathToken.IsValid() ? LocalStringTokenStore->ResolveToken(PathToken) : ToCStr(Object->GetName()));
		UE_NET_TRACE_NETHANDLE_CREATED(NetRefHandle, CreatePersistentNetDebugName(Builder.ToString()), 0, IsAuthority() ? 1U : 0U);
	}
#endif
	
	UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::CreateObjectReferenceHandle Adding %s to ObjectToNetReferenceHandle and Cache for ObjectPath %s, Name: %s, Object (0x%p), OuterNetRefHandle: %s"), 
		*CachedObject.NetRefHandle.ToString(), ToCStr(Object->GetPathName()), *GetNameSafe(Object), Object, *CachedObject.OuterNetRefHandle.ToString());

	// Create reference
	OutReference = MakeNetObjectReference(CachedObject);

	const FNetRefHandle ReplicatedOuterHandle = GetFullyDynamicOuter(CachedObject);
	if (ReplicatedOuterHandle.IsValid())
	{
		HandleToDynamicOuter.Add(ReplicatedOuterHandle, NetRefHandle);
	}

	return true;
}

FNetRefHandle FObjectReferenceCache::GetObjectReferenceHandleFromObject(const UObject* Object, EGetRefHandleFlags GetRefHandleFlags) const
{
	if (Object == nullptr)
	{
		return FNetRefHandle::GetInvalid();
	}

	// Check if we already know about this object
	if (const FNetRefHandle* Reference = ObjectToNetReferenceHandle.Find(Object))
	{
		// Verify that this is the same object
		UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
		const FCachedNetObjectReference* CachedObjectPtr = ReferenceHandleToCachedReference.Find(*Reference);
		if (CachedObjectPtr == nullptr)
		{
			UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::GetObjectReferenceHandleFromObject removed %s from ObjectToNetReferenceHandle due to ReferenceHandleToCachedReference not holding the handle. For Object %s (0x%p) "), *Reference->ToString(), *GetNameSafe(Object), Object);
			const_cast<TMap<const UObject*, FNetRefHandle>&>(ObjectToNetReferenceHandle).Remove(Object);
			return FNetRefHandle::GetInvalid();
		}

		const FCachedNetObjectReference& CachedObject = *CachedObjectPtr;

		const bool bEvenIfGarbage = EnumHasAnyFlags(GetRefHandleFlags, EGetRefHandleFlags::EvenIfGarbage);
		const UObject* ExistingObject = CachedObject.Object.Get(bEvenIfGarbage);
		if (ExistingObject == Object)
		{
			UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("ObjectReferenceCache::GetObjectReferenceHandleFromObject Found existing %s for Object %s, OuterNetRefHandle: %s"), *CachedObject.NetRefHandle.ToString(), ToCStr(Object->GetPathName()), *CachedObject.OuterNetRefHandle.ToString());
			
			return *Reference;
		}
		else
		{
			UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::Found %s cached RefHandle %s for object %s (0x%p) but cache was holding an invalid object."), 
				CachedObject.Object.IsStale() ? TEXT("Stale") : TEXT("UnResolved"), *CachedObject.NetRefHandle.ToString(), *GetNameSafe(Object), Object);
		}
	}

	return FNetRefHandle::GetInvalid();
}

void FObjectReferenceCache::BindRemoteReference(FNetRefHandle RefHandle, const UObject* Object)
{
	// Only clients should get here
	check(!IsAuthority()); 
	check(IsValid(Object));
	check(RefHandle.IsCompleteHandle());

	UE_LOG_REFERENCECACHE(Verbose, TEXT("%hs: Binding Handle: %s to Object: %s (0x%p)" ), __FUNCTION__, *RefHandle.ToString(), *GetNameSafe(Object), Object);
#if UE_NET_VALIDATE_REFERENCECACHE
	// Dynamic object might have been referenced before it gets replicated, then it will then already be registered (with a path relative to its outer)
	if (FNetRefHandle* ExistingHandle = ObjectToNetReferenceHandle.Find(Object))
	{
		if (FCachedNetObjectReference* ExistingCachedObject = ReferenceHandleToCachedReference.Find(*ExistingHandle))
		{
			if (ExistingCachedObject->NetRefHandle != RefHandle)
			{
				UE_LOG_REFERENCECACHE(Warning, TEXT("%hs: Detected conflicting Ref %s: Received: %s, Object: %s (0x%p)"), __FUNCTION__, *ExistingCachedObject->NetRefHandle.ToString(),
									  *RefHandle.ToString(), *GetNameSafe(Object), Object);
				ensure(false);
			}
		}
	}
#endif

	// The object could already be known as it has been exported as a reference prior to being replicated to this connection
	if (FCachedNetObjectReference* CacheObjectPtr = ReferenceHandleToCachedReference.Find(RefHandle))
	{
		UE_LOG_REFERENCECACHE(Verbose, TEXT("%hs: Updating handle:%s to Object: %s (0x%p). Previous ptr: (0x%p)"), __FUNCTION__, *RefHandle.ToString(), *GetNameSafe(Object), Object, CacheObjectPtr->ObjectKey);

		// Patch-up object pointers
		CacheObjectPtr->Object = MakeWeakObjectPtr(const_cast<UObject*>(Object));
		CacheObjectPtr->ObjectKey = Object;

	}
	else
	{
		LLM_SCOPE_BYTAG(Iris);

		FCachedNetObjectReference CachedObject;
	
		CachedObject.Object = MakeWeakObjectPtr(const_cast<UObject*>(Object));
		CachedObject.ObjectKey = Object;
		CachedObject.NetRefHandle = RefHandle;
		CachedObject.OuterNetRefHandle = FNetRefHandle::GetInvalid();

		UE_LOG_REFERENCECACHE(Verbose, TEXT("%hs: Adding new cache entry for: %s with Object: %s (0x%p)."), __FUNCTION__, *RefHandle.ToString(), *GetNameSafe(Object), Object);

		ReferenceHandleToCachedReference.Add(RefHandle, CachedObject);
	}

	ObjectToNetReferenceHandle.Add(Object, RefHandle);
}

void FObjectReferenceCache::AddPreRegisteredReference(FNetRefHandle RefHandle, const UObject* Object, UReplicationSystem* ReplicationSystem)
{
	if (!IsAuthority())
	{
		// Promote to full handle
		FNetRefHandle CompleteRefHandle = FNetRefHandleManager::MakeNetRefHandle(RefHandle.GetId(), ReplicationSystem->GetId());

		BindRemoteReference(CompleteRefHandle, Object);
		if (FCachedNetObjectReference* CacheObjectPtr = ReferenceHandleToCachedReference.Find(CompleteRefHandle))
		{
			CacheObjectPtr->bIsPreRegistered = true;
		}
	}
	else
	{
		UE_LOG_REFERENCECACHE(Error, TEXT("FObjectReferenceCache::AddPreRegisteredReference attempted to associate handle %s with object %s on the reference authority (server)."), *RefHandle.ToString(), *GetFullNameSafe(Object));
		ensureMsgf(false, TEXT("FObjectReferenceCache::AddPreRegisteredReference attempted to associate a handle with an object on the reference authority."));
	}
}

void FObjectReferenceCache::RemoveReference(FNetRefHandle RefHandle, const UObject* Object, bool& bOutInvalidatedTrackedSubObjectHandles)
{
	bOutInvalidatedTrackedSubObjectHandles = false;

	// Cleanup cached data for dynamic instance
	const bool bIsDynamic = RefHandle.IsDynamic();
	if (!bIsDynamic)
	{
		ensureMsgf(!bIsDynamic, TEXT("%hs not allowed on static handles: %s - %s"), __FUNCTION__, *RefHandle.ToString(), *GetNameSafe(Object));
		return;
	}

	// Util to safely remove object validating that it is the same
	auto SafeRemoveObjectToRefHandleMapping = [this](const UObject* ObjectKeyToRemove, FNetRefHandle ExpectedRefHandle)
	{
		if (ObjectKeyToRemove)
		{
			// We create an iterator to avoid having to find the key twice
			TMap<const UObject*, FNetRefHandle>::TKeyIterator ObjectIt = ObjectToNetReferenceHandle.CreateKeyIterator(ObjectKeyToRemove);
			if (ObjectIt)
			{
				const FNetRefHandle ExistingRefHandle = ObjectIt.Value();
				if (ExistingRefHandle == ExpectedRefHandle)
				{
					UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::RemoveReference: %s removed from ObjectToNetReferenceHandle Object: (0x%p)."), 
						*ExpectedRefHandle.ToString(), ObjectKeyToRemove);
					ObjectIt.RemoveCurrent();
				}
				else
				{
					// If we have an existing handle associated with the Object that differs from the one stored in the cache it means that the client might have destroyed the instance without notifying the replication system
					// and reused the instance, if that is the case we just leave the association as it is
					UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::RemoveReference: ExistingReference %s != RefHandle being removed: %s, Object: %s (0x%p) "), 
						*(ExistingRefHandle.ToString()), *ExpectedRefHandle.ToString(), *GetNameSafe(ObjectKeyToRemove), ObjectKeyToRemove);
				}
			}
			else
			{
				UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::RemoveReference: %s, Object: %s (0x%p). Skipped removing since ObjectToNetReferenceHandle had no entry"), 
					*ExpectedRefHandle.ToString(), *GetNameSafe(ObjectKeyToRemove), ObjectKeyToRemove);
			}
		}
	};

	UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::RemoveReference: %s, Object: %s (0x%p)"), *RefHandle.ToString(), *GetNameSafe(Object), Object);

	// We create an iterator to avoid having to find the key twice
	TMap<FNetRefHandle, FCachedNetObjectReference>::TKeyIterator It = ReferenceHandleToCachedReference.CreateKeyIterator(RefHandle);
	if (It)
	{
		FCachedNetObjectReference& CachedItemToRemove = It.Value();
		if (Object == nullptr)
		{
			// Object might have been GC'd but we still want to cleanup 
			Object = CachedItemToRemove.ObjectKey;
		}
		else
		{
			// RemoveReference can be called multiple times for the same NetRefHandle. The first call will clear out the object references.
			ensureMsgf(CachedItemToRemove.ObjectKey == nullptr || Object == CachedItemToRemove.ObjectKey, TEXT("ObjectReferenceCache::RemoveReference: %s, Object 0x%p doesn't match cached reference 0x%p."), *RefHandle.ToString(), Object, CachedItemToRemove.ObjectKey);
		}

		// Invalidate subobject references that referenced the removed dynamic root
		// We keep cache entries intact for relative references so remapping can work later
		// Note: We do NOT remove from HandleToDynamicOuter here.
		// This is because we don't know if the root is being destroyed or just filtered out, 
		// and we need the tracking to persist for UpdateUnresolvableReferenceTracking to be able to remap references if the object becomes relevant again
		TArray<FNetRefHandle, TInlineAllocator<16>> SubReferences;
		HandleToDynamicOuter.MultiFind(RefHandle, SubReferences);
		if (SubReferences.Num())
		{
			for (FNetRefHandle SubRefHandle : SubReferences)
			{
				if (FCachedNetObjectReference* SubObjectRef = ReferenceHandleToCachedReference.Find(SubRefHandle))
				{
					UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::RemoveReference: %s invalidating subref %s. Object: %s Pointer: %p"),
						*RefHandle.ToString(), *SubRefHandle.ToString(),
						SubObjectRef->Object.Get() ? *SubObjectRef->Object->GetName() : TEXT("NULL"), SubObjectRef->ObjectKey);

					// Remove from ObjectToNetReferenceHandle so the object can be remapped later
					SafeRemoveObjectToRefHandleMapping(SubObjectRef->ObjectKey, SubRefHandle);

					// Invalidate object pointers but keep cache entry for remapping
					SubObjectRef->ObjectKey = nullptr;
					SubObjectRef->Object.Reset();
				}
			}
			bOutInvalidatedTrackedSubObjectHandles = true;
		}

		// Remove if RelativePath is invalid
		if (!CachedItemToRemove.RelativePath.IsValid())
		{
			UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::RemoveReference: %s, Object: (0x%p). Destroying cache since Path is invalid (%s)"),
				*RefHandle.ToString(), Object, *CachedItemToRemove.RelativePath.ToString());
			It.RemoveCurrent();
		}
		else
		{
			UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::RemoveReference: %s, Object: (0x%p). Clearing cache reference to (0x%p)"),
				*RefHandle.ToString(), Object, CachedItemToRemove.ObjectKey);
			CachedItemToRemove.ObjectKey = nullptr;
			CachedItemToRemove.Object.Reset();
		}
	}

	SafeRemoveObjectToRefHandleMapping(Object, RefHandle);
}

UObject* FObjectReferenceCache::GetObjectFromReferenceHandle(FNetRefHandle RefHandle)
{
	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	FCachedNetObjectReference* CachedObject = ReferenceHandleToCachedReference.Find(RefHandle);

	UObject* ResolvedObject = CachedObject ? CachedObject->Object.Get() : nullptr;

	return ResolvedObject;
}

UObject* FObjectReferenceCache::GetPreRegisteredObjectFromReferenceHandle(FNetRefHandle RefHandle)
{
	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	FCachedNetObjectReference* CachedObject = ReferenceHandleToCachedReference.Find(RefHandle);

	UObject* ResolvedObject = (CachedObject && CachedObject->bIsPreRegistered) ? CachedObject->Object.Get() : nullptr;

	return ResolvedObject;
}

bool FObjectReferenceCache::IsNetRefHandleBroken(FNetRefHandle RefHandle, bool bMustBeRegistered) const
{
	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	const FCachedNetObjectReference* CacheObjectPtr = ReferenceHandleToCachedReference.Find(RefHandle);	
	return CacheObjectPtr ? CacheObjectPtr->bIsBroken : bMustBeRegistered;
}

bool FObjectReferenceCache::IsNetRefHandlePreRegistered(FNetRefHandle RefHandle) const
{
	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	const FCachedNetObjectReference* CacheObjectPtr = ReferenceHandleToCachedReference.Find(RefHandle);
	return CacheObjectPtr ? CacheObjectPtr->bIsPreRegistered : false;
}

bool FObjectReferenceCache::IsNetRefHandlePending(FNetRefHandle NetRefHandle, const FPendingBatchHolder& PendingBatchHolder) const
{
	// Check Outer chain
	while (NetRefHandle.IsValid())
	{
		// Need lambda to figure this one out
		if (PendingBatchHolder.Contains(NetRefHandle))
		{
			return true;
		}

		UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
		const FCachedNetObjectReference* CacheObjectPtr = ReferenceHandleToCachedReference.Find(NetRefHandle);

		if (!CacheObjectPtr)
		{
			return false;
		}

		if (CacheObjectPtr->bIsPending)
		{
			return true;
		}

		NetRefHandle = CacheObjectPtr->OuterNetRefHandle;
	}

	return false;
}

UObject* FObjectReferenceCache::ResolveObjectReferenceHandleInternal(FNetRefHandle RefHandle, const FNetObjectResolveContext& ResolveContext, bool& bOutMustBeMapped)
{
	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	FCachedNetObjectReference* CacheObjectPtr = ReferenceHandleToCachedReference.Find(RefHandle);
	
	if (!CacheObjectPtr)
	{
		bOutMustBeMapped = false;
		return nullptr;
	}
	
	UObject* Object = CacheObjectPtr->Object.Get();

	bOutMustBeMapped = !CacheObjectPtr->bNoLoad;

	if (Object != nullptr)
	{
		// Either the name should match, or this is dynamic, or we're on the server
		// $IRIS: $TODO: Consider using FNames over string tokens, or store FNames in cache and fixup when resolving?
#if UE_NET_VALIDATE_REFERENCECACHE
		{
			//FString ObjectPath(StringTokenStore->ResolveRemoteToken(CacheObjectPtr->RelativePath, *ResolveContext.RemoteNetTokenStoreState));
			//constexpr bool bReading = true;
			//RenamePathForPie(ResolveContext.ConnectionId, ObjectPath, bReading, ResolveContext.ReplicationSystem);
			//check(ObjectPath == Object->GetFName().ToString() || RefHandle.IsDynamic() || IsAuthority());

			FNetRefHandle* ExistingRefHandle = ObjectToNetReferenceHandle.Find(Object);
			if (ExistingRefHandle && *ExistingRefHandle != RefHandle)
			{
				UE_LOG_REFERENCECACHE(Warning, TEXT("ObjectReferenceCache::ResolveObjectRefererenceHandle detected potential conflicting references: %s and %s"), *RefHandle.ToString(), *(ExistingRefHandle->ToString()));
			}
		}
#endif
		UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("ObjectReferenceCache::ResolveObjectRefererenceHandle Found existing object for %s for Object: %s Pointer: 0x%p"), *RefHandle.ToString(), ToCStr(Object->GetPathName()), Object);
		return Object;
	}

	if (CacheObjectPtr->bIsBroken)
	{
		// This object is broken, we know it won't load
		// At this stage, any warnings should have already been logged, so we just need to ignore from this point forward
		return nullptr;
	}

	if (CacheObjectPtr->bIsPending)
	{
		// We're not done loading yet (and no error has been reported yet)
		return nullptr;
	}

	// If we do not have a path and our handle is invalid, we cannot do much
	if (!CacheObjectPtr->RelativePath.IsValid())
	{
		// If we don't have a path, assume this is a dynamic handle
		ensure(RefHandle.IsDynamic());
		
		return nullptr;
	}

	// First, resolve the outer
	UObject* ObjOuter = nullptr;

	if (CacheObjectPtr->OuterNetRefHandle.IsValid())
	{
		// If we get here, we depend on an outer to fully load, don't go further until we know we have a fully loaded outer
		FCachedNetObjectReference* OuterCacheObject = ReferenceHandleToCachedReference.Find(CacheObjectPtr->OuterNetRefHandle);
		if (OuterCacheObject == nullptr)
		{
			// Shouldn't be possible, but just in case...
			if (CacheObjectPtr->OuterNetRefHandle.IsStatic())
			{
				UE_LOGF(LogIris, Error, "GetObjectFromRefHandle: Static outer not registered. FullPath: %ls", ToCStr(FullPath(RefHandle, ResolveContext)))
				CacheObjectPtr->bIsBroken = 1;	// Set this to 1 so that we don't keep spamming
			}
			return nullptr;
		}

		// If outer is broken, we will never load, set ourselves to broken as well and bail
		if (OuterCacheObject->bIsBroken)
		{
			UE_LOGF(LogIris, Error, "GetObjectFromRefHandle:  Outer is broken. FullPath: %ls", ToCStr(FullPath(RefHandle, ResolveContext)));
			CacheObjectPtr->bIsBroken = 1;	// Set this to 1 so that we don't keep spamming

			return nullptr;
		}

		// Try to resolve the outer
		bool bOuterMustBeMapped;
		ObjOuter = ResolveObjectReferenceHandleInternal(CacheObjectPtr->OuterNetRefHandle, ResolveContext, bOuterMustBeMapped);

		// If we can't resolve the outer, we cannot do more, we are probably waiting for a replicated object or we are waiting for a stringtoken
		if (ObjOuter == nullptr)
		{
			return nullptr;
		}
	}

	// At this point, we either have an outer, or we are a package
	const uint32 TreatAsLoadedFlags = EPackageFlags::PKG_CompiledIn | EPackageFlags::PKG_PlayInEditor;
	check(!CacheObjectPtr->bIsPending);

	if (!ensure(ObjOuter == nullptr || ObjOuter->GetPackage()->IsFullyLoaded() || ObjOuter->GetPackage()->HasAnyPackageFlags(TreatAsLoadedFlags)))
	{
		UE_LOGF(LogIris, Error, "GetObjectFromRefHandle: Outer is null or package is not fully loaded. FullPath: %ls Outer: %ls", ToCStr(FullPath(RefHandle, ResolveContext)), *GetFullNameSafe(ObjOuter));
	}

	const TCHAR* ResolvedToken = CacheObjectPtr->RelativePath.ResolveToken();

	if (ResolvedToken == nullptr)
	{
		return nullptr;
	}

	FString ObjectPath(ResolvedToken);
	constexpr bool bReading = true;
	RenamePathForPie(ResolveContext.ConnectionId, ObjectPath, bReading, ResolveContext.ReplicationSystem);

#if WITH_EDITOR
	ensureMsgf(!ObjectPath.IsEmpty(), TEXT("NetworkRemapPath found %s to be an invalid name. This object will not be binded and replicated!"), ResolvedToken);
#endif

	const FName ObjectPathName(ObjectPath);

	// See if this object is in memory
	Object = FindObjectFast<UObject>(ObjOuter, ObjectPathName);
#if WITH_EDITOR
	// Object must be null if the package is a dynamic PIE package with pending external objects still loading, as it would normally while object is async loading
	if (Object && Object->GetPackage()->IsDynamicPIEPackagePending())
	{
		Object = nullptr;
	}
#endif

	// Assume this is a package if the outer is invalid and this is a static reference
	const bool bIsPackage = CacheObjectPtr->bIsPackage;

	if (Object == nullptr && !CacheObjectPtr->bNoLoad)
	{
		if (IsAuthority())
		{
			// Log when the server needs to re-load an object, it's probably due to a GC after initially loading as default reference
			UE_LOGF(LogIris, Error, "GetObjectFromRefHandle: Server re-loading object (might have been GC'd). FullPath: %ls", ToCStr(FullPath(RefHandle, ResolveContext)));
		}

		if (bIsPackage)
		{
			// Async load the package if:
			//	1. We are actually a package
			//	2. We aren't already pending
			//	3. We're actually suppose to load (levels don't load here for example)
			//		(Refer to CanClientLoadObject, which is where we protect clients from trying to load levels)

			if (ShouldAsyncLoad() && !ResolveContext.bForceSyncLoad)
			{
				if (!PendingAsyncLoadRequests.Contains(ObjectPathName))
				{
					CacheObjectPtr->bIsPending = true;

					StartAsyncLoadingPackage(*CacheObjectPtr, ObjectPathName, ResolveContext.AsyncLoadingPriority, RefHandle, false, ResolveContext.ReplicationSystem);
					UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("GetObjectFromRefHandle: Async loading package. Path: %s, %s"), *ObjectPath, *RefHandle.ToString());
				}
				else
				{
					ValidateAsyncLoadingPackage(*CacheObjectPtr, ObjectPathName, RefHandle);
				}

				// There is nothing else to do except wait on the delegate to tell us this package is done loading
				return nullptr;
			}
			else
			{
				// Async loading disabled
				Object = LoadPackage(nullptr, *ObjectPath, LOAD_None);
			}
		}
		else
		{
			// If we have a package, but for some reason didn't find the object we try to do a partial async load of the specific object or a blocking load as a last attempt
			// This can happen for a few reasons:
			//	1. The object was GC'd, but the package wasn't, so we need to reload
			//	2. Someone else started async loading the outer package, and it's not fully loaded yet

			if (ShouldAsyncLoad() && !ResolveContext.bForceSyncLoad)
			{
				if (bIrisAllowAsyncReload)
				{
					// Construct full object path:
					const bool bUseSubObjectDelimiter = ObjOuter->GetClass() != UPackage::StaticClass() && ObjOuter->GetOuter()->GetClass() == UPackage::StaticClass();
					FSoftObjectPath SoftPath(FString::Printf(TEXT("%s%c%s"), *ObjOuter->GetPathName(), bUseSubObjectDelimiter ? SUBOBJECT_DELIMITER_CHAR : TEXT('.'), *ObjectPath));

					if (!PendingAsyncObjectLoadRequests.Contains(SoftPath))
					{
						CacheObjectPtr->bIsPending = true;
						StartAsyncLoadingObject(*CacheObjectPtr, SoftPath, ResolveContext.AsyncLoadingPriority, RefHandle, ResolveContext.ReplicationSystem);
						UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("GetObjectFromRefHandle: Async loading object. Path: %s, %s"), *SoftPath.ToString(), *RefHandle.ToString());
					}
					else
					{
						// Merge into existing request
						PendingAsyncObjectLoadRequests[SoftPath].Merge(RefHandle);
						CacheObjectPtr->bIsPending = true;
						UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("GetObjectFromRefHandle: Already async loading object. Path: %s, %s"), *SoftPath.ToString(), *RefHandle.ToString());
					}
					return nullptr;
				}
				else
				{
					UE_LOGF(LogIris, Error, "GetObjectFromRefHandle: Forced blocking load. Path: %ls, %ls", *ObjectPath, *RefHandle.ToString());
				}
			}

			Object = StaticLoadObject(UObject::StaticClass(), ObjOuter, *ObjectPath, nullptr, LOAD_NoWarn);
		}
	}

	if (Object == nullptr)
	{
		if (!CacheObjectPtr->bIgnoreWhenMissing)
		{
			CacheObjectPtr->bIsBroken = 1;	// Set this to 1 so that we don't keep spamming
			UE_LOGF(LogIris, Error, "GetObjectFromRefHandle: Failed to resolve. FullPath: %ls", ToCStr(FullPath(RefHandle, ResolveContext)));
		}

		return nullptr;
	}

	if (bIsPackage)
	{
		UPackage* Package = Cast<UPackage>(Object);

		if (Package == nullptr)
		{
			// This isn't really a package but it should be
			CacheObjectPtr->bIsBroken = true;
			UE_LOGF(LogIris, Error, "GetObjectFromRefHandle: Object is not a package but should be! Path: %ls, NetRefHandle: %ls", *ObjectPath, *RefHandle.ToString() );
			return nullptr;
		}

		if (!Package->IsFullyLoaded() 
			&& !Package->HasAnyPackageFlags(TreatAsLoadedFlags)) //TODO: dependencies of CompiledIn could still be loaded asynchronously. Are they necessary at this point??
		{
			if (ShouldAsyncLoad() && !ResolveContext.bForceSyncLoad && Package->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
			{
				// Something else is already async loading this package, calling load again will add our callback to the existing load request
				StartAsyncLoadingPackage(*CacheObjectPtr, ObjectPathName, ResolveContext.AsyncLoadingPriority, RefHandle, true, ResolveContext.ReplicationSystem);
				UE_LOGF(LogIrisReferences, Verbose, "GetObjectFromRefHandle: Listening to existing async load. Path: %ls, NetRefHandle: %ls", *ObjectPath, *RefHandle.ToString());
			}
			else if (/*!GbNetCheckNoLoadPackages ||*/ !CacheObjectPtr->bNoLoad)
			{
				// If package isn't fully loaded, load it now
				UE_LOG_REFERENCECACHE(Warning, TEXT("ObjectReferenceCache::GetObjectFromRefHandle Blocking load of %s, %s"), ToCStr(ObjectPath), *RefHandle.ToString());
				Object = LoadPackage(nullptr, ToCStr(ObjectPath), LOAD_None);
			}
			else
			{
				return nullptr;
			}
		}
	}

	UObjectReplicationBridge* ReplicationBridge = ResolveContext.ReplicationSystem->GetReplicationBridgeAs<UObjectReplicationBridge>();
	if (Object && !ReplicationBridge->ObjectContainerHasFinishedLoading(Object))
	{
		UE_LOG_REFERENCECACHE(Verbose, TEXT("GetObjectFromRefHandle: Forcing object to NULL since container is not loaded yet. Object: %s" ), *Object->GetFullName());

		return nullptr;
	}

	// Assign the resolved object to this reference
	if (!Object)
	{
		if (!CacheObjectPtr->bIgnoreWhenMissing)
		{
			CacheObjectPtr->bIsBroken = 1;	// Set this to 1 so that we don't keep spamming
			UE_LOGF(LogIris, Error, "GetObjectFromRefHandle: Failed to resolve. FullPath: %ls non existing", ToCStr(FullPath(RefHandle, ResolveContext)));
		}
		return nullptr;
	}

	if (!IsValid(Object))
	{
		UE_LOG_REFERENCECACHE(Verbose, TEXT("GetObjectFromRefHandle: Resolved object Pointer: %p that is not valid? FullPath: %s"), Object, ToCStr(FullPath(RefHandle, ResolveContext)));
	}

	CacheObjectPtr->Object = MakeWeakObjectPtr(Object);
	CacheObjectPtr->ObjectKey = Object;

	// Promote to full handle
	FNetRefHandle CompleteRefHandle = FNetRefHandleManager::MakeNetRefHandle(RefHandle.GetId(), ResolveContext.ReplicationSystem->GetId());

	// If we already have an existing handle associated with this object, we need do do a few additional checks.
	// Due to level streaming (streaming in and out the same level) we might have multiple path based references 
	// to the same object which might potentially might resolve to the same object due to how streaming works
	// This is fine, but we can only allow one of them to map from object to RefHandle
	// The Current resolution strategy is to rely on the fact that we assign Static handles in an ever increasing manner
	// and simply use the highest index as the correct reference	

	bool bShouldAssign = true;
	FNetRefHandle* ExistingHandle = ObjectToNetReferenceHandle.Find(Object);
	if (ExistingHandle && (*ExistingHandle != CompleteRefHandle))
	{		
		if (FCachedNetObjectReference* ExistingCachedObjectPtr = ReferenceHandleToCachedReference.Find(*ExistingHandle))
		{
			FCachedNetObjectReference& ExistingCachedObject = *ExistingCachedObjectPtr;
			if (ExistingCachedObject.Object.IsStale())
			{
				UE_LOG_REFERENCECACHE(Verbose, TEXT("Invalidating object for Stale Ref %s with %s as the object has been reused"), *ExistingCachedObject.NetRefHandle.ToString(), *CompleteRefHandle.ToString());
			}
			else if (ExistingCachedObject.Object.HasSameIndexAndSerialNumber(CacheObjectPtr->Object))
			{
				bShouldAssign = CompleteRefHandle.GetId() > (*ExistingHandle).GetId();
				if (bShouldAssign)
				{
					UE_LOG_REFERENCECACHE(Verbose, TEXT("Invalidating object for conflicting Ref %s with %s"), *ExistingCachedObject.NetRefHandle.ToString(), *CompleteRefHandle.ToString());
				}
			}
		}
		else
		{
			UE_LOG_REFERENCECACHE(Warning, TEXT("ObjectReferenceCache::ResolveObjectReferenceHandleInternal removed %s from ObjectToNetReferenceHandle due to mismatch with cache. Object %s (0x%p) "), *ExistingHandle->ToString(), *GetNameSafe(Object), Object);
			ObjectToNetReferenceHandle.Remove(Object);
			ExistingHandle = nullptr;
		}
	}

	LLM_SCOPE_BYTAG(Iris);

	if (bShouldAssign)
	{
		ObjectToNetReferenceHandle.Add(Object, CompleteRefHandle);
		UE_NET_TRACE_NETHANDLE_CREATED(CompleteRefHandle, CreatePersistentNetDebugName(ResolvedToken), 0, IsAuthority() ? 1U : 0U);

		UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::ResolveObjectReferenceHandleInternal Adding %s Object %s (0x%p) to ObjectToNetReferenceHandle"),
			*CompleteRefHandle.ToString(), *GetNameSafe(Object), Object);
	}
	
	UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("ObjectReferenceCache::ResolveObjectRefererenceHandle %s for ObjectPath: %s Object %s (0x%p)"), *RefHandle.ToString(), ToCStr(Object->GetPathName()), *GetNameSafe(Object), Object);

	// Update our QueuedObjectReference if one exists.
	UpdateTrackedQueuedBatchObjectReference(CompleteRefHandle, Object);

	return Object;
}

ENetObjectReferenceResolveResult FObjectReferenceCache::ResolveObjectReference(const FNetObjectReference& Reference, const FNetObjectResolveContext& ResolveContext, UObject*& OutResolvedObject)
{
	UObject* ResolvedObject = nullptr;
	bool bMustBeMapped = false;
	
	if (Reference.PathToken.IsValid())
	{
		FStringTokenStore* LocalStringTokenStore = ResolveContext.ReplicationSystem->GetNetTokenStore()->GetDataStore<FStringTokenStore>();

		// This path is only used by Client to Server references
		const TCHAR* ResolvedToken = LocalStringTokenStore->ResolveRemoteToken(Reference.PathToken, *ResolveContext.RemoteNetTokenStoreState);
		if (ResolvedToken)
		{
			FString ObjectPath(ResolvedToken);
			constexpr bool bReading = true;
			RenamePathForPie(ResolveContext.ConnectionId, ObjectPath, bReading, ResolveContext.ReplicationSystem);

			// If both path and handle is valid this is a reference to an object with a relative path.
			if (Reference.GetRefHandle().IsValid())
			{
				UObject* ReplicatedOuter = nullptr;

				if (ReplicatedOuter == nullptr)
				{
					bool bLocalMustBeMapped = false;
					ReplicatedOuter = ResolveObjectReferenceHandleInternal(Reference.GetRefHandle(), ResolveContext, bLocalMustBeMapped);
				}
				
				if (ReplicatedOuter == nullptr && NetRefHandleSerialToObject.IsBound())
				{
					const uint64 Serial = Reference.GetRefHandle().GetSerial();
					ReplicatedOuter = NetRefHandleSerialToObject.Execute(Serial);
				}

				ResolvedObject = ReplicatedOuter ? StaticFindObject(UObject::StaticClass(), ReplicatedOuter, ToCStr(ObjectPath), EFindObjectFlags::None) : nullptr;

				if (ResolvedObject == nullptr)
				{
					UE_LOG_REFERENCECACHE(Warning, TEXT("ResolveObjectReference Failed to resolve clientassigned ref: %s due to not finding object with relative path %s."), *Reference.ToString(), *ObjectPath);	
				}
			}
			else
			{
				ResolvedObject = StaticFindObject(UObject::StaticClass(), nullptr, *ObjectPath, EFindObjectFlags::None);

				// Try to load package if it wasn't found. Note load package fails if the package is already loaded.
				if (ResolvedObject == nullptr)
				{
					FPackagePath Path = FPackagePath::FromPackageNameChecked(FPackageName::ObjectPathToPackageName(ObjectPath));
					if (UPackage* LoadedPackage = LoadPackage(nullptr, Path, LOAD_None))
					{
						ResolvedObject = StaticFindObject(UObject::StaticClass(), nullptr, *ObjectPath, EFindObjectFlags::None);
					}
				}

				if (ResolvedObject == nullptr)
				{
					UE_LOG_REFERENCECACHE(Warning, TEXT("ResolveObjectReference Failed to resolve clientassigned ref: due to not finding object with path %s."), *ObjectPath);	
				}
			}
		}
		else
		{
			UE_LOG_REFERENCECACHE(Warning, TEXT("ResolveObjectReference Failed to resolve clientassigned ref: %s due to missing token:"), *Reference.ToString());	
		}
	}
	else
	{
		ResolvedObject = ResolveObjectReferenceHandleInternal(Reference.GetRefHandle(), ResolveContext, bMustBeMapped);

		if (ResolvedObject == nullptr && Reference.GetRefHandle().IsValid() && NetRefHandleSerialToObject.IsBound())
		{
			const uint64 Serial = Reference.GetRefHandle().GetSerial();
			ResolvedObject = NetRefHandleSerialToObject.Execute(Reference.GetRefHandle().GetSerial());
		}
	}
	
	ENetObjectReferenceResolveResult Result = ENetObjectReferenceResolveResult::None;
	if (!ResolvedObject && Reference.IsValid())
	{
		Result |= ENetObjectReferenceResolveResult::HasUnresolvedReferences;
		Result |= bMustBeMapped ? ENetObjectReferenceResolveResult::HasUnresolvedMustBeMappedReferences : ENetObjectReferenceResolveResult::None;		
	}

	OutResolvedObject = ResolvedObject;

	return Result;
}

// Find replicated outer
FNetObjectReference FObjectReferenceCache::GetReplicatedOuter(const FNetObjectReference& Reference, UReplicationSystem* ReplicationSystem) const
{
	if (!Reference.IsValid())
	{
		return FNetObjectReference();
	}

	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	const FCachedNetObjectReference* CachedObject = ReferenceHandleToCachedReference.Find(Reference.GetRefHandle());
	FNetRefHandle OuterNetRefHandle = CachedObject ? CachedObject->OuterNetRefHandle : FNetRefHandle::GetInvalid();

	while (OuterNetRefHandle.IsValid())
	{
		FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
		if (NetRefHandleManager.GetInternalIndex(OuterNetRefHandle))
		{
			return FNetObjectReference(OuterNetRefHandle);
		}

		CachedObject = ReferenceHandleToCachedReference.Find(OuterNetRefHandle);
		OuterNetRefHandle = CachedObject ? CachedObject->OuterNetRefHandle : FNetRefHandle::GetInvalid();
	};

	return FNetObjectReference();
}

FNetRefHandle FObjectReferenceCache::GetFullyDynamicOuter(const FCachedNetObjectReference& Reference) const
{
	if (!Reference.NetRefHandle.IsDynamic() || !Reference.RelativePath.IsValid())
	{
		return FNetRefHandle::GetInvalid();
	}

	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	const FCachedNetObjectReference* CachedObject = &Reference;
	while (CachedObject && CachedObject->NetRefHandle.IsDynamic())
	{
		if (CachedObject->RelativePath.IsValid())
		{
			CachedObject = ReferenceHandleToCachedReference.Find(CachedObject->OuterNetRefHandle);
		}
		else
		{
			return CachedObject->NetRefHandle;
		}
	}

	return FNetRefHandle::GetInvalid();
}

uint32 FObjectReferenceCache::GetNumObjectsWithTrackedSubObjectHandles() const
{
	return (uint32)HandleToDynamicOuter.Num();
}

void FObjectReferenceCache::GetTrackedSubObjectHandles(FNetRefHandle ReplicatedObjectHandle, TArray<FNetRefHandle>& OutSubObjectHandles) const
{
	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	HandleToDynamicOuter.MultiFind(ReplicatedObjectHandle, OutSubObjectHandles);
}

void FObjectReferenceCache::MarkTrackedSubObjectHandlesForEviction(FInternalNetRefIndex ObjectIndex, FNetRefHandle ReplicatedObjectHandle)
{
	if (!bIrisEvictTrackedReferencesWithDynamicOuterFromCache)
	{
		return;
	}

	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	DynamicHandlesToEvictFromCache.Add(ObjectIndex, ReplicatedObjectHandle);
}

void FObjectReferenceCache::OnInternalNetRefIndicesFreed(const TConstArrayView<FInternalNetRefIndex>& FreedIndices)
{
	if (!bIrisEvictTrackedReferencesWithDynamicOuterFromCache)
	{
		return;
	}

	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	for (const FInternalNetRefIndex ObjectIndex : FreedIndices)
	{
		FNetRefHandle HandleWithDynamicReferences;
		if (DynamicHandlesToEvictFromCache.RemoveAndCopyValue(ObjectIndex, HandleWithDynamicReferences))
		{
			// Evict tracked dynamice relative references
			TArray<FNetRefHandle, TInlineAllocator<16>> SubReferences;
			HandleToDynamicOuter.MultiFind(HandleWithDynamicReferences, SubReferences);
			if (SubReferences.Num())
			{
				for (FNetRefHandle SubRefHandle : SubReferences)
				{
					ReferenceHandleToCachedReference.Remove(SubRefHandle);
					UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::OnInternalNetRefIndicesFreed: Evicting invalidating subref %s.from cache"), *SubRefHandle.ToString());
				}
				HandleToDynamicOuter.Remove(HandleWithDynamicReferences);
			}
		}
	}
}

FNetObjectReference FObjectReferenceCache::GetOrCreateObjectReference(const UObject* Instance, UReplicationSystem* ReplicationSystem)
{
	FNetObjectReference ObjectReference;
	CreateObjectReferenceInternal(Instance, ObjectReference, ReplicationSystem);

	return ObjectReference;
}

FNetObjectReference FObjectReferenceCache::GetOrCreateObjectReference(const FString& ObjectPath, const UObject* Outer, UReplicationSystem* ReplicationSystem)
{
	// Only allow creation of path based references on server
	if (!IsAuthority())
	{
		return FNetObjectReference();
	}
	
	// Generate a new Handle, we assume this to be a static handle
	FNetObjectReference OutReference;

	const bool bIsStatic = true;
	FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
	FNetRefHandle RefHandle = NetRefHandleManager.AllocateNetRefHandle(bIsStatic);

	if (RefHandle.IsValid())
	{
		LLM_SCOPE_BYTAG(Iris);

		FStringTokenStore* LocalStringTokenStore = ReplicationSystem->GetNetTokenStore()->GetDataStore<FStringTokenStore>();

		// Register cached ref
		FCachedNetObjectReference CachedObject;

		CachedObject.ObjectKey = nullptr;
		CachedObject.NetRefHandle = RefHandle;
		CachedObject.RelativePath.AssignToken(LocalStringTokenStore->GetOrCreateToken(ObjectPath), { .ReplicationSystemId = ReplicationSystem->GetId(), .ConnectionId = InvalidConnectionId });
		CachedObject.OuterNetRefHandle = CreateObjectReferenceHandle(Outer, ReplicationSystem);
		CachedObject.bIsPackage = 0U;
		CachedObject.bNoLoad = 1U;
		CachedObject.bIgnoreWhenMissing = 1U;
		CachedObject.bIsPending = 0U;
		CachedObject.bIsBroken = 0U;

		UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
		ReferenceHandleToCachedReference.Add(RefHandle, CachedObject);
	
		UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::GetOrCreateObjectReference(FromPath) Added cache for %s for Object: %s, OuterHandle %s OuterObject: %s (0x%p)"), 
			*CachedObject.NetRefHandle.ToString(), ToCStr(ObjectPath), *CachedObject.OuterNetRefHandle.ToString(), *GetNameSafe(Outer), Outer);
	}

	OutReference = FNetObjectReference(RefHandle);

	return OutReference;
}

void FObjectReferenceCache::WriteFullReferenceInternal(FNetSerializationContext& Context, const FNetObjectReference& Ref) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	const FNetRefHandle RefHandle = Ref.GetRefHandle();

	UE_NET_TRACE_OBJECT_SCOPE(RefHandle, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));
	
	// This is unfortunate, but as we defer writing of references, we can end up with stale references
	const FCachedNetObjectReference* CachedObject = RefHandle.IsValid() ? ReferenceHandleToCachedReference.Find(RefHandle) : nullptr;
	// Write invalid handle
	if (!CachedObject)
	{
		UE_CLOGF(RefHandle.IsValid(), LogIrisReferences, Log, "ObjectReferenceCache::WriteFullReference Trying to write Stale handle, %ls", *RefHandle.ToString());

		WriteNetRefHandle(Context, FNetRefHandle::GetInvalid());
		return;
	}

	// Write handle
	WriteNetRefHandle(Context, RefHandle);

	const bool bHasPath = CachedObject->RelativePath.IsValid();

	FNetExportContext* ExportContext = Context.GetExportContext();

	// We always export if we do not have a ExportContext
	const bool bMustExport = bHasPath && IsAuthority() && (!ExportContext || !ExportContext->IsExported(RefHandle));

	UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("ObjectReferenceCache::WriteFullReferenceInternal Auth: %u %s, Outer: %s Export: %u"), IsAuthority() ? 1U : 0U, *CachedObject->NetRefHandle.ToString(), *CachedObject->OuterNetRefHandle.ToString(), bMustExport ? 1U : 0U);

	if (Writer->WriteBool(bMustExport))
	{	
		// Write the data, bNoLoad is implicit true for dynamic objects
		Writer->WriteBits(CachedObject->bNoLoad, 1U);

		if (Writer->WriteBool(bHasPath))
		{
			// Note: As we use the StringTokenStore to write the token we also need to read it using the stringtokenstore.
			CachedObject->RelativePath.WriteNetToken(Context);
			CachedObject->RelativePath.ConditionalWriteNetTokenData(Context, ExportContext);
			WriteFullReferenceInternal(Context, FNetObjectReference(CachedObject->OuterNetRefHandle));
		}
	
		if (ExportContext)
		{
			ExportContext->AddExported(RefHandle);
		}
	}
}

void FObjectReferenceCache::ReadFullReferenceInternal(FNetSerializationContext& Context, FNetObjectReference& OutObjectRef, uint32 RecursionCount)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	if (RecursionCount > INTERNAL_READ_REF_RECURSION_LIMIT) 
	{
		UE_LOG_REFERENCECACHE(Warning, TEXT("ReadFullReferenceInternal: Hit recursion limit."));
		Context.SetError(GNetError_InternalError);
		OutObjectRef = FNetObjectReference();

		return;
	}

	UE_NET_TRACE_NAMED_OBJECT_SCOPE(ReferenceScope, FNetRefHandle::GetInvalid(), *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

	const FNetRefHandle NetRefHandle = ReadNetRefHandle(Context);

	if (!NetRefHandle.IsValid())
	{
		OutObjectRef = FNetObjectReference();
		return;
	}

	const bool bIsExported = Reader->ReadBool();
	if (Reader->IsOverflown())
	{
		UE_LOG_REFERENCECACHE(Error, TEXT("ReadFullReferenceInternal: Bitstream overflow"));
		Context.SetError(GNetError_BitStreamOverflow);
		OutObjectRef = FNetObjectReference();
		return;
	}

	UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("ObjectReferenceCache::ReadFullReferenceInternal Auth: %u %s, IsExported: %u"), IsAuthority() ? 1U : 0U, *NetRefHandle.ToString(), bIsExported ? 1U : 0U);

	// For dynamic we assume no load, otherwise we will get it through export
	uint8 bNoLoad = NetRefHandle.IsDynamic();

	// Do we have a path
	FNetObjectReference OuterRef;
	FNetToken RelativePath;

	// Exported object 
	if (bIsExported)
	{
		// Can we load this object?
		bNoLoad = Reader->ReadBits(1U) & 1;
		if (Reader->IsOverflown())
		{
			UE_LOG_REFERENCECACHE(Error, TEXT("ReadFullReferenceInternal: Bitstream overflow"));
			Context.SetError(GNetError_BitStreamOverflow);
			OutObjectRef = FNetObjectReference();
			return;
		}

		if (Reader->ReadBool())
		{
			FStringTokenStore* LocalStringTokenStore = Context.GetInternalContext()->ReplicationSystem->GetNetTokenStore()->GetDataStore<FStringTokenStore>();
			RelativePath = LocalStringTokenStore->ReadNetToken(Context);
			FNetTokenStore* LocalNetTokenStore = Context.GetInternalContext()->ReplicationSystem->GetNetTokenStore();
			LocalNetTokenStore->ConditionalReadNetTokenData(Context, RelativePath);
			ReadFullReferenceInternal(Context, OuterRef, RecursionCount + 1U);
		}
	}

	if (Reader->IsOverflown())
	{
		UE_LOG_REFERENCECACHE(Error, TEXT("ReadFullReferenceInternal: Bitstream overflow"));
		Context.SetError(GNetError_BitStreamOverflow);
		OutObjectRef = FNetObjectReference();
		return;
	}

	UE_NET_TRACE_SET_SCOPE_OBJECTID(ReferenceScope, NetRefHandle);

	// Already known? Validate that it is correct!
	if (const FCachedNetObjectReference* CachedObject = ReferenceHandleToCachedReference.Find(NetRefHandle))
	{
		FNetTokenResolveContext ReceivedTokenContext = Context.GetInternalContext()->ReplicationSystem->GetNetTokenResolveContext(Context.GetLocalConnectionId());

		// Dynamic objects can be referenced by either NetRefHandle or parent NetRefHandle and path combo
		check(!bIsExported || ((CachedObject->NetRefHandle == NetRefHandle && CachedObject->RelativePath.IsEqualToToken(RelativePath, ReceivedTokenContext) && CachedObject->OuterNetRefHandle == OuterRef.GetRefHandle()) ||
				(NetRefHandle.IsDynamic() && (NetRefHandle == CachedObject->NetRefHandle))));

		OutObjectRef = FNetObjectReference(NetRefHandle);

		return;
	}
	else
	{
		if (!(bIsExported || NetRefHandle.IsDynamic()))
		{
			UE_LOG_REFERENCECACHE(Error, TEXT("ObjectReferenceCache::ReadFullReferenceInternal Fail reading %s IsExported:%u"), *NetRefHandle.ToString(), bIsExported ? 1U : 0U);
			Context.SetError(GNetError_InvalidNetHandle);
			return;
		}
	}

	// Register in cache
	if (!IsAuthority())
	{
		// Register cached ref
		FCachedNetObjectReference CachedObject;

		FSharedStringNetToken::FTokenSource TokenSource =
		{
			.ReplicationSystemId = Context.GetInternalContext()->ReplicationSystem->GetId(),
			.ConnectionId = Context.GetLocalConnectionId()
		};

		CachedObject.ObjectKey = nullptr;
		CachedObject.NetRefHandle = NetRefHandle;
		CachedObject.RelativePath.AssignToken(RelativePath, TokenSource);
		CachedObject.OuterNetRefHandle = OuterRef.GetRefHandle();
		CachedObject.bIsPackage = NetRefHandle.IsStatic() && !OuterRef.GetRefHandle().IsValid();
		CachedObject.bNoLoad = bNoLoad;
		CachedObject.bIsBroken = false;
		CachedObject.bIsPending = false;
		CachedObject.bIgnoreWhenMissing = bNoLoad;

		ReferenceHandleToCachedReference.Add(NetRefHandle, CachedObject);

		UE_LOG_REFERENCECACHE(Verbose, TEXT("ObjectReferenceCache::ReadFullReferenceInternal Added cache for %s for Object: %s, OuterHandle %s"),
			*CachedObject.NetRefHandle.ToString(), *RelativePath.ToString(), *OuterRef.GetRefHandle().ToString());

		// We want to track references with a dynamic root so that we can purge them when the dynamic root is removed
		const FNetRefHandle DynamicOuterHandle = GetFullyDynamicOuter(CachedObject);
		if (DynamicOuterHandle.IsValid())
		{
			HandleToDynamicOuter.Add(DynamicOuterHandle, NetRefHandle);
		}
	}

	OutObjectRef = FNetObjectReference(NetRefHandle);
}

void FObjectReferenceCache::ReadFullReference(FNetSerializationContext& Context, FNetObjectReference& OutRef)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	UE_NET_TRACE_SCOPE(FullNetObjectReference, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	LLM_SCOPE_BYTAG(Iris);

	// Handle client assigned reference as they never should take the export path.
	if (const bool bIsClientAssignedReference = Reader->ReadBool())
	{
		UE_NET_TRACE_SCOPE(ClientAssigned, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

		FNetTokenStore* LocalNetTokenStore = Context.GetInternalContext()->ReplicationSystem->GetNetTokenStore();
		FStringTokenStore* LocalStringTokenStore = Context.GetInternalContext()->ReplicationSystem->GetNetTokenStore()->GetDataStore<FStringTokenStore>();

		const FNetRefHandle NetRefHandle = ReadNetRefHandle(Context);
		FNetToken RelativePath = LocalStringTokenStore->ReadNetToken(Context);
		LocalNetTokenStore->ConditionalReadNetTokenData(Context, RelativePath);

		// We expect a valid RelativePath
		if (!RelativePath.IsValid())
		{
			UE_LOG_REFERENCECACHE(Error, TEXT("ReadFullReference: Invalid relative path"));
			Context.SetError(GNetError_InvalidValue);
		}

		if (Context.HasErrorOrOverflow())
		{
			OutRef = FNetObjectReference();
		}
		else
		{
			OutRef.RefHandle = NetRefHandle;
			OutRef.PathToken = RelativePath;
		}

		return;
	}

	// Normally all references are imported, unless we are doing this when reading exports
	if (Context.GetInternalContext()->bInlineObjectReferenceExports == 0U)
	{
		ReadReference(Context, OutRef);
		return;
	}

	FNetObjectReference ObjectRef;

	UE_NET_TRACE_NAMED_OBJECT_SCOPE(ReferenceScope, FNetRefHandle::GetInvalid(), *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
	ReadFullReferenceInternal(Context, ObjectRef, 0U);

	UE_NET_TRACE_SET_SCOPE_OBJECTID(ReferenceScope, ObjectRef.GetRefHandle());

	OutRef = ObjectRef;
}

void FObjectReferenceCache::WriteFullReference(FNetSerializationContext& Context, FNetObjectReference Ref) const
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	UE_NET_TRACE_SCOPE(FullNetObjectReference, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	// Client assigned references are always written outside of the export path
	const bool bIsClientAssignedReference = Ref.PathToken.IsValid();
	if (Writer->WriteBool(bIsClientAssignedReference))
	{
		UE_NET_TRACE_SCOPE(ClientAssigned, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

		FNetTokenStore* LocalNetTokenStore = Context.GetInternalContext()->ReplicationSystem->GetNetTokenStore();
		FStringTokenStore* LocalStringTokenStore = Context.GetInternalContext()->ReplicationSystem->GetNetTokenStore()->GetDataStore<FStringTokenStore>();

		WriteNetRefHandle(Context, Ref.GetRefHandle());
		// Note: As we use the StringTokenStore to write the token we also need to read it using the stringtokenstore.
		LocalStringTokenStore->WriteNetToken(Context, Ref.PathToken);
		LocalNetTokenStore->ConditionalWriteNetTokenData(Context, Context.GetExportContext(), Ref.PathToken);
		return;
	}

	// If we do not inline reference exports we just write the reference
	if (Context.GetInternalContext()->bInlineObjectReferenceExports == 0U)
	{
		if (FNetExportContext* ExportContext = Context.GetExportContext())
		{
			AddPendingExport(*ExportContext, Ref);
		}
		WriteReference(Context, Ref);
		return;
	}

	WriteFullReferenceInternal(Context, Ref);
}

void FObjectReferenceCache::WriteReference(FNetSerializationContext& Context, FNetObjectReference Ref) const
{
	UE_NET_TRACE_SCOPE(NetObjectReference, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	UE_NET_TRACE_OBJECT_SCOPE(Ref.GetRefHandle(), *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	const FNetRefHandle RefHandle = Ref.GetRefHandle();

	// Only client assigned references contains a pathtoken
	const bool bIsClientAssignedReference = Ref.PathToken.IsValid();
	if (Writer->WriteBool(bIsClientAssignedReference))
	{
		FStringTokenStore* LocalStringTokenStore = Context.GetInternalContext()->ReplicationSystem->GetNetTokenStore()->GetDataStore<FStringTokenStore>();

		WriteNetRefHandle(Context, Ref.GetRefHandle());
		// Note: As we use the StringTokenStore to write the token we also need to read it using the stringtokenstore.
		LocalStringTokenStore->WriteNetToken(Context, Ref.PathToken);

		return;
	}

	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));

	// This is unfortunate, but as we defer writing of references, we can end up with stale references so we need to do this lookup
	const FCachedNetObjectReference* CachedObject = RefHandle.IsValid() ? ReferenceHandleToCachedReference.Find(RefHandle) : nullptr;

	// Write invalid handle
	if (!CachedObject)
	{
		UE_CLOGF(RefHandle.IsValid(), LogIrisReferences, Verbose, "ObjectReferenceCache::WriteReference Trying to write Stale handle, %ls", *RefHandle.ToString());

		WriteNetRefHandle(Context, FNetRefHandle::GetInvalid());

		return;
	}

	UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("ObjectReferenceCache::WriteReference Auth: %u %s"), IsAuthority() ? 1U : 0U, *RefHandle.ToString());

	checkSlow(Ref.CanBeExported() || RefHandle.IsDynamic());

	// Write handle
	WriteNetRefHandle(Context, RefHandle);
}

void FObjectReferenceCache::ReadReference(FNetSerializationContext& Context, FNetObjectReference& OutRef)
{
	UE_NET_TRACE_SCOPE(NetObjectReference, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	UE_NET_TRACE_NAMED_OBJECT_SCOPE(ReferenceScope, FNetRefHandle::GetInvalid(), *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	if (const bool bIsClientAssignedReference = Reader->ReadBool())
	{
		FStringTokenStore* LocalStringTokenStore = Context.GetInternalContext()->ReplicationSystem->GetNetTokenStore()->GetDataStore<FStringTokenStore>();

		const FNetRefHandle NetRefHandle = ReadNetRefHandle(Context);
		FNetToken RelativePath = LocalStringTokenStore->ReadNetToken(Context);

		// We expect a valid RelativePath
		if (!RelativePath.IsValid())
		{
			UE_LOG_REFERENCECACHE(Error, TEXT("ReadReference: Invalid relative path"));
			Context.SetError(GNetError_InvalidValue);
		}

		if (Context.HasErrorOrOverflow())
		{
			OutRef = FNetObjectReference();
		}
		else
		{
			OutRef.RefHandle = NetRefHandle;
			OutRef.PathToken = RelativePath;
		}

		return;
	}

	const FNetRefHandle NetRefHandle = ReadNetRefHandle(Context);

	if (Reader->IsOverflown())
	{
		UE_LOG_REFERENCECACHE(Error, TEXT("ReadReference: Bitstream overflow"));
		Context.SetError(GNetError_BitStreamOverflow);
		OutRef = FNetObjectReference();
		return;
	}

	if (!NetRefHandle.IsValid())
	{
		OutRef = FNetObjectReference();
		return;
	}

	UE_NET_TRACE_SET_SCOPE_OBJECTID(ReferenceScope, NetRefHandle);

	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));

	// The handle must exist in the cache or it must be a dynamic handle that will be explicitly registered
	if (ensureMsgf(NetRefHandle.IsDynamic() || ReferenceHandleToCachedReference.Find(NetRefHandle), TEXT("FObjectReferenceCache::ReadReference: Handle %s must already be known or be dynamic"), *NetRefHandle.ToString()))
	{
		UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("ObjectReferenceCache::ReadReference Auth: %u %s"), IsAuthority() ? 1U : 0U, *NetRefHandle.ToString());
		OutRef = FNetObjectReference(NetRefHandle);
	}
	else
	{
		OutRef = FNetObjectReference();
	}
}

void FObjectReferenceCache::AddPendingExport(FNetExportContext& ExportContext, const FNetObjectReference& Reference) const
{
	const bool bIsClientAssigned = Reference.PathToken.IsValid();
	const FNetRefHandle Handle = Reference.GetRefHandle();

	if (bIsClientAssigned || (IsAuthority() && Reference.CanBeExported()))
	{
		LLM_SCOPE_BYTAG(Iris);
		ExportContext.AddPendingExport(Reference);
	}
}

void FObjectReferenceCache::AddPendingExports(FNetSerializationContext& Context, TArrayView<const FNetObjectReference> ExportsView) const
{
	if (FNetExportContext* ExportContext = Context.GetExportContext())
	{
		for (const FNetObjectReference& Reference : ExportsView)
		{
			AddPendingExport(*ExportContext, Reference);
		}
	}
}

void FObjectReferenceCache::AddPendingExports(FNetSerializationContext& Context, TArrayView<const FObjectNetSerializerQuantizedReferenceStorage> ExportsView) const
{
	if (FNetExportContext* ExportContext = Context.GetExportContext())
	{
		for (const FObjectNetSerializerQuantizedReferenceStorage& Reference : ExportsView)
		{
			const FQuantizedObjectReference& InternalReference = *reinterpret_cast<const FQuantizedObjectReference*>(&Reference.Storage);

			if (InternalReference.IsNetReference())
			{
				AddPendingExport(*ExportContext, InternalReference.NetReference);
			}
		}
	}
}

FObjectReferenceCache::EWriteExportsResult FObjectReferenceCache::WritePendingExports(FNetSerializationContext& Context, FInternalNetRefIndex ObjectIndex)
{	
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	FNetExportContext* ExportContext = Context.GetExportContext();
	if (!ExportContext || !ExportContext->GetBatchExports().HasPendingExports())
	{
		return EWriteExportsResult::NoExports;
	}

	UE_NET_TRACE_SCOPE(Exports, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
	FNetBitStreamRollbackScope Rollback(*Context.GetBitStreamWriter());
	const uint32 ExportsStartBitPos = Writer.GetPosBits();

#if UE_NET_IRIS_CSV_STATS
		uint32 CountExports = 0;
#endif

	// Export NetObjectReference pending export
	TArrayView<const FNetObjectReference> NetObjectReferenceExportsView = MakeArrayView(ExportContext->GetBatchExports().ReferencesPendingExportInCurrentBatch);

	// Force exports to be written
	{
		FForceInlineExportScope ForceInlineExportScope(Context.GetInternalContext());

		// For now we export NetTokens here
		TArrayView<const FNetToken> NetTokenExportsView = MakeArrayView(ExportContext->GetBatchExports().NetTokensPendingExportInCurrentBatch);
		{
			UE_NET_TRACE_SCOPE(NetTokenExports, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
			FNetTokenStore* LocalNetTokenStore = Context.GetInternalContext()->ReplicationSystem->GetNetTokenStore();
			const bool bIsNetTokenAuthority = LocalNetTokenStore->IsAuthority();

			for (const FNetToken& NetToken : NetTokenExportsView)
			{
				// We do not need to export tokens assigned by authority unless we are the authority.
				if (!(NetToken.IsAssignedByAuthority() && !bIsNetTokenAuthority) && !ExportContext->IsExported(NetToken))
				{
					Writer.WriteBool(true);

					LocalNetTokenStore->WriteNetToken(Context, NetToken);
					LocalNetTokenStore->WriteTokenData(Context, NetToken);

					ExportContext->AddExported(NetToken);
				}
			}
		}
		// Write stop bit
		Writer.WriteBool(false);

		for (const FNetObjectReference& Reference : NetObjectReferenceExportsView)
		{
			const bool bIsClientAssigned = Reference.PathToken.IsValid();
			const FNetRefHandle Handle = Reference.GetRefHandle();
			if ((bIsClientAssigned && !ExportContext->IsExported(Reference.GetPathToken())) || (IsAuthority() && Reference.CanBeExported() && !ExportContext->IsExported(Handle)))
			{
				Writer.WriteBool(true);
				WriteFullReference(Context, Reference);

				//UE_LOGF(LogIrisReferences, Verbose, "WritePendingExports for %ls : Export[%d] %ls", *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), CountExports, *NetRefHandleManager->PrintObjectFromNetRefHandle(Handle));

#if UE_NET_IRIS_CSV_STATS
				++CountExports;
#endif
			}
		}

		UE_NET_IRIS_STATS_ADD_COUNT_FOR_OBJECT(Context.GetNetStatsContext(), WriteExports, ObjectIndex, CountExports);

		// Write stop bit
		Writer.WriteBool(false);
	}

	// We also write any must be mapped exports
	WriteMustBeMappedExports(Context, ObjectIndex, NetObjectReferenceExportsView);

	// Reset state of pending exports
	ExportContext->ClearPendingExports();

	if (Writer.IsOverflown())
	{
		return EWriteExportsResult::BitStreamOverflow;
	}
	else if ((Writer.GetPosBits() - ExportsStartBitPos) > 2U)
	{
		return EWriteExportsResult::WroteExports;
	}
	
	Rollback.Rollback();

	return EWriteExportsResult::NoExports;
}

bool FObjectReferenceCache::ReadExports(const FNetRefHandle& NetObjectHandle, FNetSerializationContext& Context, TArray<FNetRefHandle>* MustBeMappedExports, EIrisAsyncLoadingPriority& OutIrisAsyncLoadingPriority)
{
	if (!MustBeMappedExports)
	{
		Context.SetError(GNetError_InvalidValue);
		UE_LOG_REFERENCECACHE(Error, TEXT("ReadExports: Null MustBeMappedExports"));
		ensure(false);
		return false;
	}

	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	// Read NetToken exports
	{
		bool bHasExportsToRead = Reader.ReadBool(); 
		UE_NET_TRACE_SCOPE(NetTokenExports, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

		// Force inlined exports
		{
			FNetObjectResolveContext& ResolveContext = Context.GetInternalContext()->ResolveContext;

			FForceInlineExportScope ForceInlineExportScope(Context.GetInternalContext());
			while (bHasExportsToRead && !Context.HasErrorOrOverflow())
			{
				FNetTokenStore* LocalNetTokenStore = Context.GetInternalContext()->ReplicationSystem->GetNetTokenStore();

				FNetToken ImportedNetToken = LocalNetTokenStore->ReadNetToken(Context);
				LocalNetTokenStore->ReadTokenData(Context, ImportedNetToken, *ResolveContext.RemoteNetTokenStoreState);
				bHasExportsToRead = Reader.ReadBool();
			}
		}
	}

	// Read NetObjectReference exports
	{
		bool bHasExportsToRead = Reader.ReadBool(); 
		UE_NET_TRACE_SCOPE(NetObjectReferenceExports, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

		// Force inlined exports
		{
			FForceInlineExportScope ForceInlineExportScope(Context.GetInternalContext());
			while (bHasExportsToRead && !Context.HasErrorOrOverflow())
			{
				FNetObjectReference Import;
				ReadFullReference(Context, Import);
				bHasExportsToRead = Reader.ReadBool();
			}
		}
	}

	if (!Context.HasErrorOrOverflow())
	{
		ReadMustBeMappedExports(NetObjectHandle, Context, MustBeMappedExports, OutIrisAsyncLoadingPriority);
	}

	return !Context.HasErrorOrOverflow();
}

bool FObjectReferenceCache::WriteMustBeMappedExports(FNetSerializationContext& Context, FInternalNetRefIndex ObjectIndex, TArrayView<const FNetObjectReference> ExportsView) const
{
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();
	FNetExportContext* ExportContext = Context.GetExportContext();

	if (IsAuthority() && ShouldAsyncLoad() && ExportContext)
	{
		UE_NET_TRACE_SCOPE(MustBeMappedExports, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

#if UE_NET_IRIS_CSV_STATS
		uint32 CountExports = 0;
#endif

		UReplicationSystem* ReplicationSystem = Context.GetInternalContext()->ReplicationSystem;
		FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
		const EIrisAsyncLoadingPriority IrisAsyncLoadingPriority = NetRefHandleManager.GetIrisAsyncLoadingPriority(ObjectIndex);

		// Only write the loading priority before the first reference
		bool bHasWrittenPriority = false;

		UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));

		for (const FNetObjectReference& Reference : ExportsView)
		{
			const FNetRefHandle Handle = Reference.GetRefHandle();
			const FCachedNetObjectReference* CachedObject = Reference.CanBeExported() ? ReferenceHandleToCachedReference.Find(Handle) : nullptr;
			if (CachedObject && !CachedObject->bNoLoad)
			{
				UE_NET_TRACE_OBJECT_SCOPE(Handle, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
				Writer.WriteBool(true);

				if (!bHasWrittenPriority)
				{
					bHasWrittenPriority = true;
					Writer.WriteBits((uint32)IrisAsyncLoadingPriority, GetIrisAsyncLoadingPriorityBits());
				}

				WriteNetRefHandle(Context, Handle);

#if UE_NET_IRIS_CSV_STATS
				UE_LOGF(LogIrisReferences, VeryVerbose, "WriteMustBeMappedExports for %ls. MustBeMapped[%d] %ls (%ls)", *NetRefHandleManager.PrintObjectFromIndex(ObjectIndex), CountExports, *GetNameSafe(CachedObject->Object.Get()), *Handle.ToString());
#endif

#if UE_NET_IRIS_CSV_STATS
				++CountExports;
#endif
			}
		}

#if UE_NET_IRIS_CSV_STATS
		UE_CLOGF(CountExports>0, LogIrisReferences, VeryVerbose, "WriteMustBeMappedExports for %ls. Wrote %d exports with priority: %ls", *NetRefHandleManager.PrintObjectFromIndex(ObjectIndex), CountExports, LexToString(IrisAsyncLoadingPriority));
#endif

		// $TODO: This should be a separate state, as MustBeMapped is not really an export, just a list of important exports
		UE_NET_IRIS_STATS_ADD_COUNT_FOR_OBJECT(Context.GetNetStatsContext(), WriteExports, ObjectIndex, CountExports);
	}

	// Write stop bit
	Writer.WriteBool(false);

	return !Writer.IsOverflown();
}

void FObjectReferenceCache::ReadMustBeMappedExports(const FNetRefHandle& NetObjectHandle, FNetSerializationContext& Context, TArray<FNetRefHandle>* MustBeMappedExports, EIrisAsyncLoadingPriority& OutIrisAsyncLoadingPriority)
{
	if (!MustBeMappedExports)
	{
		Context.SetError(GNetError_InvalidValue);
		UE_LOG_REFERENCECACHE(Error, TEXT("ReadMustBeMappedExports: Null MustBeMappedExports"));
		ensure(false);
		return;
	}

	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	bool bHasExportsToRead = Reader.ReadBool(); 
	UE_NET_TRACE_SCOPE(MustBeMappedExports, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

	if (bHasExportsToRead)
	{	
		OutIrisAsyncLoadingPriority = static_cast<EIrisAsyncLoadingPriority>(Reader.ReadBits(GetIrisAsyncLoadingPriorityBits()));
	}

	while (bHasExportsToRead && !Context.HasErrorOrOverflow())
	{
		UE_NET_TRACE_NAMED_OBJECT_SCOPE(ReferenceScope, FNetRefHandle::GetInvalid(), *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		const FNetRefHandle MustBeMappedHandle = ReadNetRefHandle(Context);
		UE_NET_TRACE_SET_SCOPE_OBJECTID(ReferenceScope, MustBeMappedHandle);	

		bHasExportsToRead = Reader.ReadBool();

		UE_LOGF(LogIrisReferences, VeryVerbose, "ReadMustBeMappedExports for %ls. MustBeMapped[%d] %ls", *NetObjectHandle.ToString(), MustBeMappedExports->Num(), *MustBeMappedHandle.ToString());

		MustBeMappedExports->Add(MustBeMappedHandle);
	}

	UE_CLOGF(MustBeMappedExports->Num() > 0, LogIrisReferences, VeryVerbose, "ReadMustBeMappedExports for %ls. Read %d references with priority: %ls", *NetObjectHandle.ToString(), MustBeMappedExports->Num(), LexToString(OutIrisAsyncLoadingPriority));
}

FString FObjectReferenceCache::DescribeObjectReference(const FNetObjectReference Ref, const FNetObjectResolveContext& ResolveContext)
{
	FString FullPath;

	GenerateFullPath_r(Ref.GetRefHandle(), ResolveContext, FullPath);

	// Only client assigned FNetObjectReferences has a path stored directly
	if (Ref.PathToken.IsValid())
	{
		FStringTokenStore* LocalStringTokenStore = ResolveContext.ReplicationSystem->GetNetTokenStore()->GetDataStore<FStringTokenStore>();

		// This path is only used by Client to Server references
		const TCHAR* ResolvedToken = LocalStringTokenStore->ResolveRemoteToken(Ref.PathToken, *ResolveContext.RemoteNetTokenStoreState);
		if (ResolvedToken)
		{
			if (!FullPath.IsEmpty())
			{
				FullPath += TEXT(".");
			}

			FString ObjectPath(ResolvedToken);
			constexpr bool bReading = true;
			RenamePathForPie(ResolveContext.ConnectionId, ObjectPath, bReading, ResolveContext.ReplicationSystem);
			FullPath += FString::Printf(TEXT("%s"), ResolvedToken);
		}
	}

	return FullPath;
}

FString FObjectReferenceCache::FullPath(FNetRefHandle RefHandle, const FNetObjectResolveContext& ResolveContext) const
{
	FString FullPath;

	GenerateFullPath_r(RefHandle, ResolveContext, FullPath);

	return FullPath;
}

void FObjectReferenceCache::GenerateFullPath_r(FNetRefHandle RefHandle, const FNetObjectResolveContext& ResolveContext, FString& FullPath) const
{
	if (!RefHandle.IsValid())
	{
		// This is the end of the outer chain, we're done
		return;
	}

	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));

	const FCachedNetObjectReference* CacheObject = ReferenceHandleToCachedReference.Find(RefHandle);

	if (CacheObject == nullptr)
	{
		// Doh, this shouldn't be possible, but if this happens, we can't continue
		// So warn, and return
		FullPath += FString::Printf(TEXT("[%s](NOT_IN_CACHE)" ), *RefHandle.ToString());
		return;
	}

	GenerateFullPath_r(CacheObject->OuterNetRefHandle, ResolveContext, FullPath);

	if (!FullPath.IsEmpty())
	{
		FullPath += TEXT(".");
	}

	// Prefer the object name first, since non stable named objects don't store the path
	if (CacheObject->Object.IsValid())
	{
		// Sanity check that the names match if the path was stored
		// $IRIS: $TODO: 
		// Consider resolving -> FName rather than using NetTokens
		//if (CacheObject->RelativePath.IsValid() && CacheObject->Object->GetName() != StringTokenStore->ResolveRemoteToken(CacheObject->RelativePath, ResolveContext.RemoteNetTokenStoreState))
		//{
		//	UE_LOG_REFERENCECACHE_WARNING(TEXT( "GenerateFullPath_r: Name mismatch! %s != %s" ), *CacheObject->PathName.ToString(), *CacheObject->Object->GetName() );
		//}

		FullPath += FString::Printf(TEXT("[%s](%s)"), *RefHandle.ToString(), *CacheObject->Object->GetName());
	}
	else
	{
		if (!CacheObject->RelativePath.IsValid())
		{
			// This can happen when a non stably named object is nullptr
			FullPath += FString::Printf(TEXT("[%s](EMPTY)" ), *RefHandle.ToString());
		}
		else
		{
			const TCHAR* ResolvedPath = CacheObject->RelativePath.ResolveToken();
			FullPath += FString::Printf(TEXT("[%s](%s)"), *RefHandle.ToString(), ResolvedPath ? ResolvedPath : TEXT("EMPTY"));
		}
	}
}


void FObjectReferenceCache::ValidateAsyncLoadingPackage(FCachedNetObjectReference& CacheObject, FName PackagePath, const FNetRefHandle RefHandle)
{
	// With level streaming support we may end up trying to load the same package with a different
	// RefHandle during replay fast-forwarding. This is because if a package was unloaded, and later
	// re-loaded, it will likely be assigned a new RefHandle (since the TWeakObjectPtr to the old package
	// in the cache object would have gone stale). During replay fast-forward, it's possible
	// to see the new RefHandle before the previous one has finished loading, so here we fix up
	// PendingAsyncLoadRequests to refer to the new NetRefHandle. Also keep track of all the NetRefHandles referring
	// to the same package so their CacheObjects can be properly updated later.
	FPendingAsyncLoadRequest& PendingLoadRequest = PendingAsyncLoadRequests[PackagePath];

	PendingLoadRequest.Merge(RefHandle);
	CacheObject.bIsPending = true;

	if (PendingLoadRequest.NetRefHandles.Last() != RefHandle)
	{
		UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("ValidateAsyncLoadingPackage: Already async loading package with a different NetRefHandle. Path: %s, original RefHandle: %s, new RefHandle: %s"),
			*PackagePath.ToString(), *PendingLoadRequest.NetRefHandles.Last().ToString(), *RefHandle.ToString());
	}
	else
	{
		UE_LOG_REFERENCECACHE(VeryVerbose, TEXT("ValidateAsyncLoadingPackage: Already async loading package. Path: %s, RefHandle: %s"), *PackagePath.ToString(), *RefHandle.ToString());
	}
	
//#if CSV_PROFILER
//	PendingLoadRequest.bWasRequestedByOwnerOrPawn |= IsTrackingOwnerOrPawn();
//#endif
}

void FObjectReferenceCache::StartAsyncLoadingPackage(FCachedNetObjectReference& CacheObject, FName PackagePath, TAsyncLoadPriority AsyncLoadingPriority, const FNetRefHandle RefHandle, const bool bWasAlreadyAsyncLoading, UReplicationSystem* ReplicationSystem)
{
	LLM_SCOPE_BYTAG(Iris);

	const bool bIsPrioritySet = AsyncLoadingPriority != INDEX_NONE;
	if (!bIsPrioritySet)
	{
		ensureMsgf(false, TEXT("AsyncLoadingPriorty wasn't set in the ResolveContext. Using default value as fallback"));
		AsyncLoadingPriority = ConvertAsyncLoadingPriority(EIrisAsyncLoadingPriority::Default);
	}

	FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();

	UE_LOGF(LogIrisReferences, Verbose, "StartAsyncLoadingPackage for %ls: Package: %ls (Async loading priority: %d)", *NetRefHandleManager.PrintObjectFromNetRefHandle(RefHandle), *PackagePath.ToString(), AsyncLoadingPriority);

	// Need timer?
	FPendingAsyncLoadRequest LoadRequest(RefHandle, ReplicationSystem->GetElapsedTime());
	
//#if CSV_PROFILER
//	LoadRequest.bWasRequestedByOwnerOrPawn = IsTrackingOwnerOrPawn();
//#endif

	CacheObject.bIsPending = true;

	FPendingAsyncLoadRequest* ExistingRequest = PendingAsyncLoadRequests.Find(PackagePath);
	if (ExistingRequest)
	{
		// Same package name but a possibly different NetRefHandle. Note down the NetRefHandle and wait for the async load completion callback
		ExistingRequest->Merge(LoadRequest);
		return;
	}

	PendingAsyncLoadRequests.Emplace(PackagePath, MoveTemp(LoadRequest));

	//DelinquentAsyncLoads.MaxConcurrentAsyncLoads = FMath::Max<uint32>(DelinquentAsyncLoads.MaxConcurrentAsyncLoads, PendingAsyncLoadRequests.Num());

#if UE_NET_OBJECTREFERENCECACHE_ASYNCLOAD_DEBUG
	if (bIrisAsyncLoadingFailAllLoads || bIrisAsyncLoadingFailNextLoad || (IrisAsyncLoadingFailPackageName.Len() > 0 && PackagePath.ToString().Find(IrisAsyncLoadingFailPackageName) != INDEX_NONE))
	{
		bIrisAsyncLoadingFailNextLoad = false;
		// Not thread safe. We don't want to load the package so we call our fail callback immediately.
		AsyncPackageForcedFailCallback(PackagePath, nullptr, EAsyncLoadingResult::Failed);
	}
	else
#endif
	{
		LoadPackageAsync(
			PackagePath.ToString(), 
			FLoadPackageAsyncDelegate::CreateWeakLambda(ReplicationSystem, 
				[this](const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
				{
					AsyncPackageCallback(PackageName, Package, Result);
				}),
			AsyncLoadingPriority
		);
	}
}

void FObjectReferenceCache::AsyncPackageCallback(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
{
	// $TODO: $IRIS
	// We probably want to deffer this to a safe time to process, we do not want any callbacks when we do not expect them
	// Probably want to put this in a queue that we can guard to be processed when we are ready.
	
	LLM_SCOPE_BYTAG(Iris);

	check(Package == nullptr || Package->IsFullyLoaded());

	if (FPendingAsyncLoadRequest const* const PendingLoadRequest = PendingAsyncLoadRequests.Find(PackageName))
	{
		const bool bIsBroken = (Package == nullptr);

		UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));

		for (FNetRefHandle RefHandleToProcess : PendingLoadRequest->NetRefHandles)
		{
			if (FCachedNetObjectReference* CacheObject = ReferenceHandleToCachedReference.Find(RefHandleToProcess))
			{
				if (!CacheObject->bIsPending)
				{
					UE_LOG_REFERENCECACHE(Error, TEXT("AsyncPackageCallback: Package wasn't pending. Path: %s, RefHandle: %s"), *PackageName.ToString(), *RefHandleToProcess.ToString());
				}

				CacheObject->bIsPending = false;

				if (bIsBroken)
				{
					CacheObject->bIsBroken = true;
					UE_LOG_REFERENCECACHE(Error, TEXT("AsyncPackageCallback: Package FAILED to load. Path: %s, RefHandle: %s"), *PackageName.ToString(), *RefHandleToProcess.ToString());
				}

				// $TODO: Jira: UE-370863: Investigate if it is safe to resolve now to update CacheObject and mapping, or if it is best to wait until it is done as part of dispatching data.
				// This does not really do anything since CachedObject->Object is not updated until the next resolve of the reference
				if (UObject* Object = CacheObject->Object.Get())
				{
					// Something is loaded, we can now resolve queued data
					UpdateTrackedQueuedBatchObjectReference(RefHandleToProcess, Object);
				}
			}
			else
			{
				UE_LOG_REFERENCECACHE(Error, TEXT("AsyncPackageCallback: Could not find NetRefHandle. Path: %s, RefHandle: %s"), *PackageName.ToString(), *RefHandleToProcess.ToString());
			}
		}

		// This won't be the exact amount of time that we spent loading the package, but should
		// give us a close enough estimate (within a frame time).
		//const double LoadTime = (ReplicationSystem->GetElapsedTime() - PendingLoadRequest->RequestStartTime);
		//if (GGuidCacheTrackAsyncLoadingGUIDThreshold > 0.f &&
		//	LoadTime >= GGuidCacheTrackAsyncLoadingGUIDThreshold)
		//{
		//	DelinquentAsyncLoads.DelinquentAsyncLoads.Emplace(PackageName, LoadTime);
		//}

//#if CSV_PROFILER
//		if (PendingLoadRequest->bWasRequestedByOwnerOrPawn &&
//			GGuidCacheTrackAsyncLoadingGUIDThresholdOwner > 0.f &&
//			LoadTime >= GGuidCacheTrackAsyncLoadingGUIDThresholdOwner &&
//			Driver->ServerConnection)
//		{
//			CSV_EVENT(PackageMap, TEXT("Owner Net Stall Async Load (Package=%s|LoadTime=%.2f)"), *PackageName.ToString(), LoadTime);
//		}
//#endif

		PendingAsyncLoadRequests.Remove(PackageName);
	}
	else
	{
		UE_LOG_REFERENCECACHE(Error, TEXT( "AsyncPackageCallback: Could not find package. Path: %s" ), *PackageName.ToString());
	}
}

void FObjectReferenceCache::AsyncPackageForcedFailCallback(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
{
#if UE_NET_OBJECTREFERENCECACHE_ASYNCLOAD_DEBUG
	UE_LOG_REFERENCECACHE(Warning, TEXT("AsyncPackageCallback: Force fail async package load. Path: %s"), ToCStr(PackageName.ToString()));

	if (FPendingAsyncLoadRequest const* const PendingLoadRequest = PendingAsyncLoadRequests.Find(PackageName))
	{
		UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));

		for (FNetRefHandle RefHandleToProcess : PendingLoadRequest->NetRefHandles)
		{
			if (FCachedNetObjectReference* CacheObject = ReferenceHandleToCachedReference.Find(RefHandleToProcess))
			{
				CacheObject->bIsPending = false;
				CacheObject->bIsBroken = true;

				if (UObject* Object = CacheObject->Object.Get())
				{
					// Something is loaded, we can now resolve queued data
					UpdateTrackedQueuedBatchObjectReference(RefHandleToProcess, Object);

				}
			}
		}

		PendingAsyncLoadRequests.Remove(PackageName);
	}
#endif
}

void FObjectReferenceCache::StartAsyncLoadingObject(FCachedNetObjectReference& CacheObject, const FSoftObjectPath& ObjectPath, TAsyncLoadPriority AsyncLoadingPriority, const FNetRefHandle RefHandle, UReplicationSystem* ReplicationSystem)
{
	LLM_SCOPE_BYTAG(Iris);

	const bool bIsPrioritySet = AsyncLoadingPriority != INDEX_NONE;
	if (!bIsPrioritySet)
	{
		ensureMsgf(false, TEXT("AsyncLoadingPriority wasn't set in the ResolveContext. Using default value as fallback"));
		AsyncLoadingPriority = ConvertAsyncLoadingPriority(EIrisAsyncLoadingPriority::Default);
	}

	FNetRefHandleManager& NetRefHandleManager = ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();

	UE_LOGF(LogIrisReferences, Verbose, "StartAsyncLoadingObject for %ls: Object: %ls (Async loading priority: %d)", *NetRefHandleManager.PrintObjectFromNetRefHandle(RefHandle), *ObjectPath.ToString(), AsyncLoadingPriority);

	FPendingAsyncObjectLoadRequest LoadRequest(RefHandle, ReplicationSystem->GetElapsedTime());

	CacheObject.bIsPending = true;

	FPendingAsyncObjectLoadRequest* ExistingRequest = PendingAsyncObjectLoadRequests.Find(ObjectPath);
	if (ExistingRequest)
	{
		// Same object path but a possibly different NetRefHandle. Note down the NetRefHandle and wait for the async load completion callback
		ExistingRequest->Merge(LoadRequest);
		return;
	}

	PendingAsyncObjectLoadRequests.Emplace(ObjectPath, MoveTemp(LoadRequest));

#if UE_NET_OBJECTREFERENCECACHE_ASYNCLOAD_DEBUG
	FName PackageName = ObjectPath.GetAssetPath().GetPackageName();
	if (bIrisAsyncLoadingFailAllLoads || bIrisAsyncLoadingFailNextLoad || (IrisAsyncLoadingFailPackageName.Len() > 0 && PackageName.ToString().Find(IrisAsyncLoadingFailPackageName) != INDEX_NONE))
	{
		bIrisAsyncLoadingFailNextLoad = false;
		// Not thread safe. We don't want to load the object so we call our callback immediately with nullptr.
		AsyncObjectCallback(ObjectPath, nullptr);
	}
	else
#endif
	{
		FLoadAssetAsyncOptionalParams OptionalParams;
		OptionalParams.PackagePriority = AsyncLoadingPriority;

		ObjectPath.LoadAsync(
			FLoadSoftObjectPathAsyncDelegate::CreateWeakLambda(ReplicationSystem,
				[this, ObjectPath](const FSoftObjectPath& InObjectPath, UObject* LoadedObject)
				{
					AsyncObjectCallback(InObjectPath, LoadedObject);
				}),
			OptionalParams
		);
	}
}

void FObjectReferenceCache::AsyncObjectCallback(const FSoftObjectPath& ObjectPath, UObject* LoadedObject)
{
	LLM_SCOPE_BYTAG(Iris);

	if (FPendingAsyncObjectLoadRequest const* const PendingLoadRequest = PendingAsyncObjectLoadRequests.Find(ObjectPath))
	{
		const bool bIsBroken = (LoadedObject == nullptr);

		UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));

		for (FNetRefHandle RefHandleToProcess : PendingLoadRequest->NetRefHandles)
		{
			if (FCachedNetObjectReference* CacheObject = ReferenceHandleToCachedReference.Find(RefHandleToProcess))
			{
				if (!CacheObject->bIsPending)
				{
					UE_LOG_REFERENCECACHE(Error, TEXT("AsyncObjectCallback: Object wasn't pending. Path: %s, RefHandle: %s"), *ObjectPath.ToString(), *RefHandleToProcess.ToString());
				}

				CacheObject->bIsPending = false;

				if (bIsBroken)
				{
					CacheObject->bIsBroken = true;
					UE_LOG_REFERENCECACHE(Error, TEXT("AsyncObjectCallback: Object FAILED to load. Path: %s, RefHandle: %s"), *ObjectPath.ToString(), *RefHandleToProcess.ToString());
				}

				// $TODO: Jira: UE-370863: Investigate if it is safe to resolve now to update CacheObject and mapping, or if it is best to wait until it is done as part of dispatching data.

				if (UObject* Object = LoadedObject)
				{
					// Something is loaded, we can now resolve queued data
					UpdateTrackedQueuedBatchObjectReference(RefHandleToProcess, Object);
				}
			}
			else
			{
				UE_LOG_REFERENCECACHE(Error, TEXT("AsyncObjectCallback: Could not find net ref handle. Path: %s, RefHandle: %s"), *ObjectPath.ToString(), *RefHandleToProcess.ToString());
			}
		}

		PendingAsyncObjectLoadRequests.Remove(ObjectPath);
	}
	else
	{
		UE_LOG_REFERENCECACHE(Error, TEXT("AsyncObjectCallback: Could not find object. Path: %s"), *ObjectPath.ToString());
	}
}

void FObjectReferenceCache::SetAsyncLoadMode(const EAsyncLoadMode NewMode)
{
	AsyncLoadMode = NewMode;
	if (const IConsoleVariable* CVarAllowAsyncLoading = IConsoleManager::Get().FindConsoleVariable(TEXT("net.AllowAsyncLoading"), false /* bTrackFrequentCalls */))
	{
		bCachedCVarAllowAsyncLoading = CVarAllowAsyncLoading->GetInt() > 0;
	}
}

bool FObjectReferenceCache::ShouldAsyncLoad() const
{
#if UE_NET_ASYNCLOADING_DEBUG
	if (bForceAsyncLoading)
	{
		return true;
	}
#endif

	if (!bIrisAllowAsyncLoading)
	{
		return false;
	}

	switch (AsyncLoadMode)
	{
		case EAsyncLoadMode::UseCVar:		return bCachedCVarAllowAsyncLoading;
		case EAsyncLoadMode::ForceDisable:	return false;
		case EAsyncLoadMode::ForceEnable:	return true;
		default: ensureMsgf( false, TEXT( "Invalid AsyncLoadMode: %i" ), (int32)AsyncLoadMode ); return false;
	}
}
void FObjectReferenceCache::AddReferencedObjects(FReferenceCollector& ReferenceCollector, UReplicationSystem* ReplicationSystem)
{
	for (auto& It : QueuedBatchObjectReferences)
	{
		FQueuedBatchObjectReference& Ref = It.Value;

		if (Ref.Object)
		{
			// AddReferencedObject will set our reference to nullptr if the object is pending kill.
			ReferenceCollector.AddReferencedObject(Ref.Object, ReplicationSystem);

			if (!Ref.Object)
			{
				UE_LOG_REFERENCECACHE(Warning, TEXT("FObjectReferenceCache::CollectReferences: QueuedBatchObjectReference was killed by GC. %s"), *(It.Key.ToString()));
			}			
		}
	}
}

void FObjectReferenceCache::AddTrackedQueuedBatchObjectReference(const FNetRefHandle InHandle, const UObject* InObject)
{
	FQueuedBatchObjectReference& Ref = QueuedBatchObjectReferences.FindOrAdd(InHandle);
	Ref.Object = InObject;
	++Ref.RefCount;
}

void FObjectReferenceCache::UpdateTrackedQueuedBatchObjectReference(const FNetRefHandle InHandle, const UObject* NewObject)
{
	if (FQueuedBatchObjectReference* Ref = QueuedBatchObjectReferences.Find(InHandle))
	{
		Ref->Object = NewObject;
	}
}

void FObjectReferenceCache::RemoveTrackedQueuedBatchObjectReference(const FNetRefHandle InHandle)
{
	if (auto It = QueuedBatchObjectReferences.CreateKeyIterator(InHandle))
	{
		FQueuedBatchObjectReference& Ref = It.Value();
		check(Ref.RefCount > 0);
		if (--Ref.RefCount == 0)
		{
			It.RemoveCurrent();
		}
	}
}

const TCHAR* FObjectReferenceCache::GetObjectRelativePath(FNetRefHandle NetRefHandle, UReplicationSystem* ReplicationSystem) const
{
	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock InternalLock(ObjectReferenceCacheInternalCS));

	if (const FCachedNetObjectReference* CacheObject = ReferenceHandleToCachedReference.Find(NetRefHandle))
	{
		const TCHAR* ResolvedToken = CacheObject->RelativePath.ResolveToken();
		return ResolvedToken;
	}

	return nullptr;
}

void FObjectReferenceCache::ReplicationSystemDestroyed(UReplicationSystem* RepSystem)
{
	// Invalidate FNetTokens that came from the RepSystem in our FCachedNetObjectReferences.
	for (TPair<FNetRefHandle, FCachedNetObjectReference>& ReferencePair : ReferenceHandleToCachedReference)
	{
		FSharedStringNetToken& RelativePath = ReferencePair.Value.RelativePath;
		if (RelativePath.GetSourceReplicationSystemId() == RepSystem->GetId())
		{
			RelativePath.MarkStale();
		}
	}
}

void FObjectReferenceCache::RemoveConnection(UReplicationSystem* RepSystem, uint32 ConnectionId)
{
	// Invalidate FNetTokens that came from the RepSystem and connection in our FCachedNetObjectReferences.
	for (TPair<FNetRefHandle, FCachedNetObjectReference>& ReferencePair : ReferenceHandleToCachedReference)
	{
		FSharedStringNetToken& RelativePath = ReferencePair.Value.RelativePath;
		if (RelativePath.GetSourceReplicationSystemId() == RepSystem->GetId() &&
			RelativePath.GetSourceConnectionId() == ConnectionId)
		{
			RelativePath.MarkStale();
		}
	}
}

} // end namespace UE::Net::Private
