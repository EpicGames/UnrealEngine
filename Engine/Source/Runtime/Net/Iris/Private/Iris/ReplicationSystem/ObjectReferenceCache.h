// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/SharedStringNetToken.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "Containers/Map.h"
#include "ObjectReferenceCacheFwd.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"

enum class EIrisAsyncLoadingPriority : uint8;

namespace UE::Net
{
	class FStringTokenStore;
	class FNetTokenStore;
	class FNetSerializationContext;
	
	struct FObjectNetSerializerQuantizedReferenceStorage;

	typedef uint32 FInternalNetRefIndex;

	namespace Private
	{
		class FNetRefHandleManager;
		class FNetExportContext;
		struct FPendingBatchHolder;
	}
}

namespace UE::Net::Private
{

struct FObjectReferenceCacheParams
{
	bool bAuthority = false;
	FObjectToNetRefHandleSerial ObjectToNetRefHandleSerial;
	FNetRefHandleSerialToObject NetRefHandleSerialToObject;
};

// A lot of code in this class is extracted from the re-factored GUIDCache/ClientPackageMap and must be kept up to sync once they are submitted
// Hopefully we can merge parts of this back together later on

class FObjectReferenceCache
{
	template<typename T> friend struct FObjectNetSerializerBase;

public:

	FObjectReferenceCache();

	void Init(const FObjectReferenceCacheParams& Params);

	// Determine if the object is dynamic
	bool IsDynamicObject(const UObject* Object) const;

	// Are we allowed to create new NetHandles to reference objects?
	bool IsAuthority() const
	{
		return bIsAuthority;
	}

	// Create and assign a new NetHandle to the object
	FNetRefHandle CreateObjectReferenceHandle(const UObject* Object, UReplicationSystem* ReplicationSystem);

	// Remove any previous Object->NetRefHandle maps and ensures a new NetRefHandle is assigned to the object.
	FNetRefHandle ReassignObjectRefHandle(const UObject* Object, UReplicationSystem* ReplicationSystem);

	// Create and assign a new pre-registered NetHandle to the object
	FNetRefHandle PreRegisterObjectReferenceHandle(const UObject* Object, UReplicationSystem* ReplicationSystem);

	// Get existing handle for object
	FNetRefHandle GetObjectReferenceHandleFromObject(const UObject* Object, EGetRefHandleFlags GetRefHandleFlags = EGetRefHandleFlags::None) const;

	// Get object from handle, only if the object is in the cache.
	UObject* GetObjectFromReferenceHandle(FNetRefHandle RefHandle);

	// Get object from handle, only if the object is in the cache and only if it was registered via PreregisterObjectReferenceHandle or AddPreregisteredReference.
	UObject* GetPreRegisteredObjectFromReferenceHandle(FNetRefHandle RefHandle);

	// Try to resolve the object reference and try to load it if the object cannot be found
	UObject* ResolveObjectReferenceHandle(FNetRefHandle RefHandle, const FNetObjectResolveContext& ResolveContext);

	UObject* ResolveObjectReference(const FNetObjectReference& ObjectRef, const FNetObjectResolveContext& ResolveContext);
	ENetObjectReferenceResolveResult ResolveObjectReference(const FNetObjectReference& ObjectRef, const FNetObjectResolveContext& ResolveContext, UObject*& OutResolvedObject);

	// Returns true of this NetRefHandle is marked as broken
	bool IsNetRefHandleBroken(FNetRefHandle Handle, bool bMustBeRegistered) const;

	// Returns true of the provided NetRefHandle or one of its outers is pending async loading.
	bool IsNetRefHandlePending(FNetRefHandle NetRefHandle, const FPendingBatchHolder& PendingBatchHolder) const;

	// Returns true of the provided NetRefHandle was assigned via PreRegisterObjectReferenceHandle or AddPreregisteredReference.
	bool IsNetRefHandlePreRegistered(FNetRefHandle NetRefHandle) const;

	// Find replicated outer
	FNetObjectReference GetReplicatedOuter(const FNetObjectReference& Reference, UReplicationSystem* ReplicationSystem) const;

	// Get or create a NetObjectReference from the object
	FNetObjectReference GetOrCreateObjectReference(const UObject* Instance, UReplicationSystem* ReplicationSystem);

	// Get or create a NetObjectReference from the object identifed by path relative to outer
	FNetObjectReference GetOrCreateObjectReference(const FString& ObjectPath, const UObject* Outer, UReplicationSystem* ReplicationSystem);

	// Bind a nethandle and the object reference cache on the client
	void BindRemoteReference(FNetRefHandle RefHandle, const UObject* Object);
	
	// Add reference for pre-registered object (client)
	void AddPreRegisteredReference(FNetRefHandle RefHandle, const UObject* Object, UReplicationSystem* ReplicationSystem);

	// Remove references to dynamic objects, sets bOutInvalidatedTrackedSubObjectHandles to true if removed refhandle had tracked subobjecthandles that were invalidated
	void RemoveReference(FNetRefHandle RefHandle, const UObject* Object, bool& bOutInvalidatedTrackedSubObjectHandles);

	// Write full chain of object references for RefHandle
	void WriteFullReference(FNetSerializationContext& Context, FNetObjectReference Ref) const;

	// Read/load full reference data, this will populate the cache on the receiving end, but will not try to resolve the actual objects
	void ReadFullReference(FNetSerializationContext& Context, FNetObjectReference& OutRef);

	// Write reference, the reference must already be exported
	void WriteReference(FNetSerializationContext& Context, FNetObjectReference Ref) const;
	
	// Read reference, as written by WriteReference
	void ReadReference(FNetSerializationContext& Context, FNetObjectReference& OutRef);

	// Add exports to the set of pending exports for the current batch being written
	void AddPendingExports(FNetSerializationContext& Context, TArrayView<const FNetObjectReference> ExportsView) const;

	// Add exports to the set of pending exports for the current batch being written
	void AddPendingExports(FNetSerializationContext& Context, TArrayView<const FObjectNetSerializerQuantizedReferenceStorage> ExportsView) const;

	// Add export to the set of pending exports for the current batch being written
	void AddPendingExport(FNetExportContext& ExportContext, const FNetObjectReference& Reference) const;

	enum class EWriteExportsResult : unsigned
	{
		// We did write exports
		WroteExports,

		// BitStream overflow.
		BitStreamOverflow,

		// Some error occurred while serializing the object.
		NoExports,
	};

	// Exports are expected to be part of the written state, so if the result is a BitStreamOverflow
	// it is up to the caller to roll back written data and pending exports
	EWriteExportsResult WritePendingExports(FNetSerializationContext& Context, FInternalNetRefIndex ObjectIndex);

	bool ReadExports(const FNetRefHandle& NetObjectHandle, FNetSerializationContext& Context, TArray<FNetRefHandle>* MustBeMappedExports, EIrisAsyncLoadingPriority& OutIrisAsyncLoadingPriority);

	static FNetObjectReference MakeNetObjectReference(FNetRefHandle Handle);

	// Async interface, kept as close to possible to FNetGuidCache/PackageMapClient
	enum class EAsyncLoadMode : uint8
	{
		UseCVar			= 0,		// Use CVar (net.AllowAsyncLoading) to determine if we should async load
		ForceDisable	= 1,		// Disable async loading
		ForceEnable		= 2,		// Force enable async loading
	};

	void SetAsyncLoadMode(const EAsyncLoadMode NewMode);
	bool ShouldAsyncLoad() const;


	// While async loading of pending must be mapped references we need to maintain references to already resolved objects as there will be no instance referencing them
	void AddReferencedObjects(FReferenceCollector& ReferenceCollector, UReplicationSystem* ReplicationSystem);
	void AddTrackedQueuedBatchObjectReference(const FNetRefHandle InHandle, const UObject* InObject);
	void UpdateTrackedQueuedBatchObjectReference(const FNetRefHandle InHandle, const UObject* NewObject);
	void RemoveTrackedQueuedBatchObjectReference(const FNetRefHandle InHandle);

	FString DescribeObjectReference(const FNetObjectReference Ref, const FNetObjectResolveContext& ResolveContext);

	/** Return the stored relative path of a replicated object if it's in the cache. */
	const TCHAR* GetObjectRelativePath(FNetRefHandle NetRefHandle, UReplicationSystem* ReplicationSystem) const;

	/** Called on the shared ObjectReferenceCache to invalidate cached data owned by the RepSystem (like FNetTokens) */
	void ReplicationSystemDestroyed(UReplicationSystem* RepSystem);

	/** Called on the shared ObjectReferenceCache to invalidate cached data owned by the connection (like remote FNetTokens) */
	void RemoveConnection(UReplicationSystem* RepSystem, uint32 ConnectionId);

	/** Get number of tracked objects that have registered references to subobjects with relative path */
	uint32 GetNumObjectsWithTrackedSubObjectHandles() const;

	/** Get all subobject handles tracked for a given replicated object. Used for updating unresolvable reference tracking. */
	void GetTrackedSubObjectHandles(FNetRefHandle ReplicatedObjectHandle, TArray<FNetRefHandle>& OutSubObjectHandles) const;

	/** When we know that an object is being destroyed, we can evict its references from the cache once it has stopped replicating */
	void MarkTrackedSubObjectHandlesForEviction(FInternalNetRefIndex ObjectIndex, FNetRefHandle ReplicatedObjectHandle);
	
	/** Called when when one or more NetRefInternalIndices have been freed and can be re-assigned to new objects. */
	void OnInternalNetRefIndicesFreed(const TConstArrayView<FInternalNetRefIndex>& FreedIndices);

private:
	void AsyncPackageCallback(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result);
	void AsyncPackageForcedFailCallback(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result);

	struct FCachedNetObjectReference
	{
		TWeakObjectPtr<UObject> Object;
		const UObject* ObjectKey = nullptr;

		// NetRefHandle
		FNetRefHandle NetRefHandle;

		// RelativePath to outer
		FSharedStringNetToken RelativePath;

		// Ref to outer
		FNetRefHandle OuterNetRefHandle;

		// Flags
		uint8 bNoLoad : 1 = false;				// Don't load this, only do a find
		uint8 bIgnoreWhenMissing : 1 = false;
		uint8 bIsPackage : 1 = false;
		uint8 bIsBroken : 1 = false;
		uint8 bIsPending : 1 = false;
		uint8 bIsPreRegistered : 1 = false;		// True if assigned via PreregisterObjectReferenceHandle or AddPreregisteredReference
	};

	struct FQueuedBatchObjectReference
	{
		TObjectPtr<const UObject> Object = nullptr;
		uint32 RefCount = 0U;
	};

	struct FPendingAsyncLoadRequest
	{
		FPendingAsyncLoadRequest(FNetRefHandle InNetRefHandle, double InRequestStartTime);
		void Merge(const FPendingAsyncLoadRequest& Other);
		void Merge(FNetRefHandle InNetRefHandle);

		// NetRefHandles that requested loading for the same UPackage/UObject
		TArray<FNetRefHandle, TInlineAllocator<4>> NetRefHandles;
		double RequestStartTime;
	};

	using FPendingAsyncObjectLoadRequest = FPendingAsyncLoadRequest;

	bool CreateObjectReferenceInternal(const UObject* Object, FNetObjectReference& OutReference, UReplicationSystem* ReplicationSystem);


	void ReadFullReferenceInternal(FNetSerializationContext& Context, FNetObjectReference& OutRef, uint32 RecursionCount);
	void WriteFullReferenceInternal(FNetSerializationContext& Context, const FNetObjectReference& Ref) const;

	UObject* ResolveObjectReferenceHandleInternal(FNetRefHandle RefHandle, const FNetObjectResolveContext& ResolveContext, bool& bOutMustBeMapped);
	bool IsDynamicInternal(const UObject* Object) const;
	bool SupportsObjectInternal(const UObject* Object) const;
	bool CanClientLoadObjectInternal(const UObject* Object, bool bIsDynamic) const;
	bool ShouldIgnoreWhenMissing(FNetRefHandle RefHandle) const;
	bool RenamePathForPie(uint32 ConnectionId, FString& Str, bool bReading, UReplicationSystem* ReplicationSystem);
	
	// Get or create a NetObjectReference from an object: always using the object path if static, and a NetRefHandle if dynamic.
	FNetObjectReference GetOrCreateObjectReferenceUsingPath(const UObject* Instance, UReplicationSystem* ReplicationSystem);

	// Get the string path of RefHandle
	FString FullPath(FNetRefHandle RefHandle, const FNetObjectResolveContext& ResolveContext) const;
	void GenerateFullPath_r(FNetRefHandle RefHandle, const FNetObjectResolveContext& ResolveContext, FString& OutFullPath) const;

	// Find outer that is not expressed relative to another reference, returns invalid if there is no outer or if it is not dynamic.
	FNetRefHandle GetFullyDynamicOuter(const FCachedNetObjectReference& Reference) const;

	static FNetObjectReference MakeNetObjectReference(FNetRefHandle RefHandle, FNetToken RelativePath);
	static FNetObjectReference MakeNetObjectReference(const FCachedNetObjectReference& CachedReference);
	bool IsCachedOuterChainValid(const UObject* Object, FNetRefHandle RefHandle, const FCachedNetObjectReference& CachedObject) const;

	// Must be mapped exports are written for each batch that serializes object references, if async loading is enabled the client
	// will defer application of data contained in the batch until the must be mapped exports are resolvable.
	bool WriteMustBeMappedExports(FNetSerializationContext& Context, FInternalNetRefIndex ObjectIndex, TArrayView<const FNetObjectReference> ExportsView) const;
	void ReadMustBeMappedExports(const FNetRefHandle& NetObjectHandle, FNetSerializationContext& Context, TArray<FNetRefHandle>* MustBeMappedExports, EIrisAsyncLoadingPriority& OutIrisAsyncLoadingPriority);

	void StartAsyncLoadingPackage(FCachedNetObjectReference& Object, FName PackagePath, TAsyncLoadPriority AsyncLoadingPriority, const FNetRefHandle RefHandle, const bool bWasAlreadyAsyncLoading, UReplicationSystem* ReplicationSystem);
	void ValidateAsyncLoadingPackage(FCachedNetObjectReference& Object, FName PackagePath, const FNetRefHandle RefHandle);

	void StartAsyncLoadingObject(FCachedNetObjectReference& CacheObject, const FSoftObjectPath& ObjectPath, TAsyncLoadPriority AsyncLoadingPriority, const FNetRefHandle RefHandle, UReplicationSystem* ReplicationSystem);
	void AsyncObjectCallback(const FSoftObjectPath& ObjectPath, UObject* LoadedObject);

private:

#if UE_SUPPORT_PARALLEL_IRIS
	// Protects access to internal state such as the reference maps. Does not protect access to TokenStores or NetRefHandleManager
	// Lock must be held whilst accessing pointers to FCachedNetObjectReference in ReferenceHandleToCachedReference
	mutable FTransactionallySafeCriticalSection ObjectReferenceCacheInternalCS;
#endif // UE_SUPPORT_PARALLEL_IRIS

	// Map raw UObject pointer -> Handle
	// To verify that the reference is valid we need to check the weakpointer stored in the cache
	TMap<const UObject*, FNetRefHandle> ObjectToNetReferenceHandle;

	// Map ReferenceHandle -> CachedReference
	TMap<FNetRefHandle, FCachedNetObjectReference> ReferenceHandleToCachedReference;

	// To properly clean up stale references referencing dynamic objects we need to track them, and also deffer eviction from cache.
	TMultiMap<FNetRefHandle, FNetRefHandle> HandleToDynamicOuter;
	TMap<FInternalNetRefIndex, FNetRefHandle> DynamicHandlesToEvictFromCache;
	
	/**
	 * Set of all current Objects that we've been requested to be referenced while we are doing async loading.
	 * This is used to prevent objects (especially async load objects,
	 * which may have no other references) from being GC'd while a the object is waiting for more
	 * pending references
	 */
	TMap<FNetRefHandle, FQueuedBatchObjectReference> QueuedBatchObjectReferences;

	EAsyncLoadMode AsyncLoadMode = EAsyncLoadMode::UseCVar;
	bool bCachedCVarAllowAsyncLoading = false;
	
	// Do we have authority to create references?
	bool bIsAuthority = false;

	/** Set of packages that are currently pending async loads, referenced by package name. */
	TMap<FName, FPendingAsyncLoadRequest> PendingAsyncLoadRequests;

	/** Set of objects (non-package) that are currently pending async loads, referenced by soft object path. */
	TMap<FSoftObjectPath, FPendingAsyncObjectLoadRequest> PendingAsyncObjectLoadRequests;

	/** Optional delegates that can be used to override the UObject<->NetRefHandle id assignments */
	FObjectToNetRefHandleSerial ObjectToNetRefHandleSerial;
	FNetRefHandleSerialToObject NetRefHandleSerialToObject;

	// $TODO: $IRIS: Stats support
#if 0
	/** Store all GUIDs that caused the sync loading of a package, for debugging & logging with LogNetSyncLoads */
	//TArray<FNetRefHandle> SyncLoadedGUIDs;
	//FNetAsyncLoadDelinquencyAnalytics DelinquentAsyncLoads;
	//void ConsumeAsyncLoadDelinquencyAnalytics(FNetAsyncLoadDelinquencyAnalytics& Out);
	//const FNetAsyncLoadDelinquencyAnalytics& GetAsyncLoadDelinquencyAnalytics() const;
	//void ResetAsyncLoadDelinquencyAnalytics();	
	//bool WasGUIDSyncLoaded(FNetworkGUID NetGUID) const { return SyncLoadedGUIDs.Contains(NetGUID); }
	//void ClearSyncLoadedGUID(FNetworkGUID NetGUID) { SyncLoadedGUIDs.Remove(NetGUID); }
	/**
	 * If LogNetSyncLoads is enabled, log all objects that caused a sync load that haven't been otherwise reported
	 * by the package map yet, and clear that list.
	 */
	//void ReportSyncLoadedGUIDs();
#endif

};

inline UObject* FObjectReferenceCache::ResolveObjectReferenceHandle(FNetRefHandle RefHandle, const FNetObjectResolveContext& ResolveContext)
{
	bool bOutMustBeMapped;
	return ResolveObjectReferenceHandleInternal(RefHandle, ResolveContext, bOutMustBeMapped);
}

inline UObject* FObjectReferenceCache::ResolveObjectReference(const FNetObjectReference& ObjectRef, const FNetObjectResolveContext& ResolveContext)
{
	UObject* ResolvedObject = nullptr;
	ResolveObjectReference(ObjectRef, ResolveContext, ResolvedObject);
	return ResolvedObject;
}

inline FNetObjectReference FObjectReferenceCache::MakeNetObjectReference(FNetRefHandle Handle)
{
	return FNetObjectReference(Handle);
}

inline FNetObjectReference FObjectReferenceCache::MakeNetObjectReference(FNetRefHandle RefHandle, FNetToken RelativePath)
{
	const ENetObjectReferenceTraits Traits = RelativePath.IsValid() ? ENetObjectReferenceTraits::CanBeExported : ENetObjectReferenceTraits::None;
	return FNetObjectReference(RefHandle, RelativePath, Traits);
}

inline FNetObjectReference FObjectReferenceCache::MakeNetObjectReference(const FCachedNetObjectReference& CachedReference)
{
	const ENetObjectReferenceTraits Traits = CachedReference.RelativePath.IsValid() ? ENetObjectReferenceTraits::CanBeExported : ENetObjectReferenceTraits::None;
	return FNetObjectReference(CachedReference.NetRefHandle, FNetToken(), Traits);
}

}
