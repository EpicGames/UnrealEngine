// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/IrisConstants.h"

class UReplicationSystem;
class UObjectReplicationBridge;

// From UObjectGlobals.h
typedef int32 TAsyncLoadPriority;

// Forward declarations
namespace UE::Net
{
	class FNetTokenStoreState;
	class FStringTokenStore;
}

namespace UE::Net::Private
{
	class FNetExportContext;
	class FNetPendingBatches;
}

// Definitions
namespace UE::Net
{
	struct FNetObjectResolveContext
	{
		FNetTokenStoreState* RemoteNetTokenStoreState = nullptr;
		uint32 ConnectionId = InvalidConnectionId;
		TAsyncLoadPriority AsyncLoadingPriority = INDEX_NONE; // default to -1 to trap codepaths that are missing the assignment
		bool bForceSyncLoad = false;
		UReplicationSystem* ReplicationSystem = nullptr;
	};
}
