// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ObjectReferenceTypes.h"

#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectReferenceTypes)

namespace UE::Net::Private
{
	static int32 IrisDefaultAsyncLoadingPriority = 200;
	static FAutoConsoleVariableRef CVarIrisDefaultAsyncLoadingPriority(
		TEXT("net.iris.AsyncLoading.DefaultPriority"),
		IrisDefaultAsyncLoadingPriority,
		TEXT("The default priority to use when async loading packages referenced by replicated properties.")
		);

	static int32 IrisHighAsyncLoadingPriority = 225;
	static FAutoConsoleVariableRef CVarIrisHighAsyncLoadingPriority(
		TEXT("net.iris.AsyncLoading.HighPriority"),
		IrisHighAsyncLoadingPriority,
		TEXT("The high priority value to use when async loading packages referenced by replicated properties.")
		);

	static int32 IrisVeryHighAsyncLoadingPriority = 250;
	static FAutoConsoleVariableRef CVarIrisVeryHighAsyncLoadingPriority(
		TEXT("net.iris.AsyncLoading.VeryHighPriority"),
		IrisVeryHighAsyncLoadingPriority,
		TEXT("The very high priority value to use when async loading packages referenced by replicated properties.")
		);

}

TAsyncLoadPriority ConvertAsyncLoadingPriority(EIrisAsyncLoadingPriority IrisPriority)
{
	switch(IrisPriority)
	{
		case EIrisAsyncLoadingPriority::High:
		{
			return UE::Net::Private::IrisHighAsyncLoadingPriority;
		}

		case EIrisAsyncLoadingPriority::VeryHigh:
		{
			return UE::Net::Private::IrisVeryHighAsyncLoadingPriority;
		}

		case EIrisAsyncLoadingPriority::Invalid:
		{
			ensureMsgf(false, TEXT("Invalid config used, should not happen."));
			return UE::Net::Private::IrisDefaultAsyncLoadingPriority;
		}

		default:
		{
			return UE::Net::Private::IrisDefaultAsyncLoadingPriority;
		}
	}
}
