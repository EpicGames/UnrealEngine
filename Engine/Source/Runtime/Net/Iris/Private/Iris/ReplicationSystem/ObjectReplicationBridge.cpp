// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"

#include "Misc/ScopeExit.h"
#include "HAL/IConsoleManager.h"
#include "UObject/CoreNetTypes.h"

#include "Iris/IrisConfigInternal.h"

#include "Iris/Core/IrisCsv.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisMemoryTracker.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisDebugging.h"
#include "Iris/Core/IrisDelegates.h"

#include "Net/Core/NetBitArrayPrinter.h"
#include "Net/Core/NetHandle/NetHandleManager.h"
#include "Net/Core/Connection/NetEnums.h"
#include "Net/Core/PropertyConditions/PropertyConditionsDelegates.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/Core/Trace/NetDebugName.h"

#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"

#include "Iris/ReplicationSystem/IrisCreationFlowLog.h"
#include "Iris/ReplicationSystem/LegacyPushModel.h"
#include "Iris/ReplicationSystem/NetObjectFactory.h"
#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"
#include "Iris/ReplicationSystem/NetRefHandleManagerTypes.h"
#include "Iris/ReplicationSystem/ObjectPollFrequencyLimiter.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridgeConfig.h"
#include "Iris/ReplicationSystem/Polling/ObjectPoller.h"
#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizer.h"
#include "Iris/ReplicationSystem/ReplicationConnections.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/ReplicationFragmentInternal.h"
#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationSystem/ReplicationProtocolManager.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"
#include "Iris/ReplicationSystem/ReplicationWriter.h"
#include "Iris/ReplicationSystem/RepTag.h"

#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectReplicationBridge)

namespace UE::Net::Private::ObjectBridgeDebugging
{
	extern void RemoteProtocolMismatchDetected(TMap<FObjectKey, bool>& ArchetypesAlreadyPrinted, UReplicationSystem* , uint32, const FReplicationFragments&, const UObject*, const UObject*, FDebugCDOState* OutCDOState = nullptr);
}

#ifndef UE_IRIS_VALIDATE_PROTOCOLS
#	define UE_IRIS_VALIDATE_PROTOCOLS !UE_BUILD_SHIPPING
#endif

DEFINE_LOG_CATEGORY(LogIrisFilterConfig)

#define UE_LOG_OBJECTREPLICATIONBRIDGE(Category, Format, ...)  UE_LOG(LogIris, Category, TEXT("RepBridge(%u)::") Format, GetReplicationSystem()->GetId(), ##__VA_ARGS__)

static bool bUseFrequencyBasedPolling = true;
static FAutoConsoleVariableRef CVarUseFrequencyBasedPolling(
		TEXT("net.Iris.UseFrequencyBasedPolling"),
		bUseFrequencyBasedPolling,
		TEXT("Whether to use frequency based polling or not. Default is true.")
		);

static bool bUseDormancyToFilterPolling = true;
static FAutoConsoleVariableRef CVarUseDormancyToFilterPolling(
		TEXT("net.Iris.UseDormancyToFilterPolling"),
		bUseDormancyToFilterPolling,
		TEXT("Whether we should use dormancy to filter out objects that we should not poll. Default is true.")
		);

static bool bAllowPollPeriodOverrides = true;
static FAutoConsoleVariableRef CVarAllowPollPeriodOverrides(
		TEXT("net.Iris.AllowPollPeriodOverrides"),
		bAllowPollPeriodOverrides,
		TEXT("Whether we allow poll period overrides set in ObjectReplicationBridgeConfig. Default is true.")
		);

static bool bEnableFilterMappings = true;
static FAutoConsoleVariableRef CVarEnableFilterMappings(
		TEXT("net.Iris.EnableFilterMappings"),
		bEnableFilterMappings,
		TEXT("Whether we honor filter mappings set in ObjectReplicationBridgeConfig. If filter mappings are enabled then objects may also be assigned the default spatial filter even if there aren't any specific mappings. Default is true.")
		);

static bool bEnableForceNetUpdate = false;
static FAutoConsoleVariableRef CVarEnableForceNetUpdate(
	TEXT("net.Iris.EnableForceNetUpdate"),
	bEnableForceNetUpdate,
	TEXT("When true the system only allows ForceNetUpdate to skip the poll frequency of objects. When false any MarkDirty object will be immediately polled.")
	);

static FString IgnoreProtocolMismatchForClasses;
static FAutoConsoleVariableRef CVarIgnoreProtocolMismatchClasses(
	TEXT("net.Iris.Protocols.IgnoreMismatchForClasses"),
	IgnoreProtocolMismatchForClasses,
	TEXT("Array of class names that when wildcard matched will disable the protocol validation between the server and client. Should be only used temporarely to unblock replication while trying to fix the source of the protocol mismatch.")
);

static bool bSendCDOStateOnProtocolMismatch = true;
static FAutoConsoleVariableRef CVarSendCDOStateOnProtocolMismatch(
	TEXT("net.Iris.SendCDOStateOnProtocolMismatch"),
	bSendCDOStateOnProtocolMismatch,
	TEXT("When true, clients send their structured CDO state to the server on protocol mismatch so the server can log a diff.")
);

static int32 MaxCDOStateBytesOnProtocolMismatch = 64 * 1024;
static FAutoConsoleVariableRef CVarMaxCDOStateBytesOnProtocolMismatch(
	TEXT("net.Iris.MaxCDOStateBytesOnProtocolMismatch"),
	MaxCDOStateBytesOnProtocolMismatch,
	TEXT("Max byte size of the client-sent CDO state payload on protocol mismatch. Payloads above this fall back to logging locally + sending the plain mismatch message.")
);

static bool bRemoveDestructionInfoIfRestartingReplication = true;
static FAutoConsoleVariableRef CVarRemoveDestructionInfoIfRestartingReplication(
	TEXT("net.Iris.RemoveDestructionInfoIfRestartingReplication"),
	bRemoveDestructionInfoIfRestartingReplication,
	TEXT("When true the we remove all static destructioninfos when starting replication for a given object.")
	);

#if UE_BUILD_SHIPPING
static int32 GInvalidLocationWarnings = 0; // don't check OOB locations in Shipping
#else
static int32 GInvalidLocationWarnings = 1; // check OOB locations and warn once per class 
#endif
static FAutoConsoleVariableRef CVarInvalidLocationWarnings(
	TEXT("net.Iris.InvalidLocationWarnings"),
	GInvalidLocationWarnings,
	TEXT("Chooses how to warn when we detect an invalid world location on a rootobject. 0=never warn. 1=warn once per class. 2=warn once per object. 3=warn on every update")
);

#if UE_BUILD_SHIPPING
static bool bUseVerboseIrisCsvStats = false;
#else
static bool bUseVerboseIrisCsvStats = true;
#endif
static FAutoConsoleVariableRef CVarUseVerboseCsvStats(
	TEXT("net.Iris.UseVerboseIrisCsvStats"),
	bUseVerboseIrisCsvStats,
	TEXT("Whether to use verbose per-class csv stats. Default is false in Shipping, otherwise True.")
);

#if !UE_BUILD_SHIPPING
static bool bUseVeryVerboseIrisCsvStats = false;
static FAutoConsoleVariableRef CVarUseVeryVerboseCsvStats(
	TEXT("net.Iris.UseVeryVerboseIrisCsvStats"),
	bUseVeryVerboseIrisCsvStats,
	TEXT("Whether to use very verbose per-class csv stats. This causes all untracked classes to get their own CSV stats. Will incur significant CSV bloat.")
);
#endif

#if DO_ENSURE
static bool bLogBlockedByAsyncLoadingError = true;
static FAutoConsoleVariableRef CVarLogBlockedByAsyncLoadingErrorr(
	TEXT("net.Iris.Error.LogBlockedByAsyncLoading"),
	bLogBlockedByAsyncLoadingError,
	TEXT("Log and ensure on client reports of objects blocked by must be mapped references stuck in a async loading queue.")
);
#endif

#if DO_ENSURE
static bool bLogMissingParentsError = true;
static FAutoConsoleVariableRef CVarLogMissingParentsError(
	TEXT("net.Iris.Error.LogMissingParentsError"),
	bLogMissingParentsError,
	TEXT("Log and ensure on client reports of objects not spawning due to missing creation dependent parents.")
);
#endif

namespace UE::Net::Private
{
	void CallRegisterReplicationFragments(UObject* Object, FFragmentRegistrationContext& Context, EFragmentRegistrationFlags RegistrationFlags)
	{
		// Try the user implementation first
		Object->RegisterReplicationFragments(Context, RegistrationFlags);

		// If no user implementations exist, create fragments automatically
		if (!Context.WasRegistered())
		{
			FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(Object, Context, RegistrationFlags);
			ensure(Context.WasRegistered());
		}
	}

	struct FReplicationInstanceProtocolDeleter
	{
		void operator()(FReplicationInstanceProtocol* InstanceProtocol)
		{ 
			if (InstanceProtocol != nullptr)
			{
				FReplicationProtocolManager::DestroyInstanceProtocol(InstanceProtocol);
			}
		}
	};
	typedef TUniquePtr<FReplicationInstanceProtocol, FReplicationInstanceProtocolDeleter> FReplicationInstanceProtocolPtr;

	FOnRootObjectPostInit::RegistrationType& FObjectReplicationBridgeDelegates::GetRootObjectPostInitDelegate(UObjectReplicationBridge* Bridge)
	{
		return Bridge->OnRootObjectPostInit;
	}

	FOnRootObjectDetached::RegistrationType& FObjectReplicationBridgeDelegates::GetRootObjectDetachedDelegate(UObjectReplicationBridge* Bridge)
	{
		return Bridge->OnRootObjectDetached;
	}

	FOnSubObjectPostInit::RegistrationType& FObjectReplicationBridgeDelegates::GetSubObjectPostInitDelegate(UObjectReplicationBridge* Bridge)
	{
		return Bridge->OnSubObjectPostInit;
	}

	FOnSubObjectDetached::RegistrationType& FObjectReplicationBridgeDelegates::GetSubObjectDetachedDelegate(UObjectReplicationBridge* Bridge)
	{
		return Bridge->OnSubObjectDetached;
	}
}

UObjectReplicationBridge::UObjectReplicationBridge()
: Super()
, PollFrequencyLimiter(new UE::Net::Private::FObjectPollFrequencyLimiter())
, DefaultSpatialFilterHandle(UE::Net::InvalidNetObjectFilterHandle)
{
	SetShouldUseDefaultSpatialFilterFunction([](const UClass*){ return false; });
	SetShouldSubclassUseSameFilterFunction([](const UClass*,const UClass*){ return true; });
}

UObjectReplicationBridge::~UObjectReplicationBridge()
{
	delete PollFrequencyLimiter;
}

bool UObjectReplicationBridge::ShouldUseVerboseCsvStats() const
{
	return bUseVerboseIrisCsvStats;
}

void UObjectReplicationBridge::Initialize(UReplicationSystem* InReplicationSystem)
{
	Super::Initialize(InReplicationSystem);

	CreateDestructionProtocol();

	const uint32 CurrentMaxInternalIndex = NetRefHandleManager->GetCurrentMaxInternalNetRefIndex();
	PollFrequencyLimiter->Init(CurrentMaxInternalIndex, NetRefHandleManager);
	ObjectsWithObjectReferences.Init(CurrentMaxInternalIndex);
	GarbageCollectionAffectedObjects.Init(CurrentMaxInternalIndex);

	NetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().AddUObject(this, &UObjectReplicationBridge::OnMaxInternalNetRefIndexIncreased);

	LoadConfig();

	IgnoreProtocolMismatchForClasses.RemoveSpacesInline();
	IgnoreProtocolMismatchForClasses.ParseIntoArray(ClassNamesToIgnoreProtocolMismatch, TEXT(","));

	InitConditionalPropertyDelegates();
	InitNetObjectFactories();
}

void UObjectReplicationBridge::Deinitialize()
{
	NetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().RemoveAll(this);

	DeinitNetObjectFactories();

	UE::Net::Private::FPropertyConditionDelegates::GetOnPropertyCustomConditionChangedDelegate().Remove(OnCustomConditionChangedHandle);
	UE::Net::Private::FPropertyConditionDelegates::GetOnPropertyDynamicConditionChangedDelegate().Remove(OnDynamicConditionChangedHandle);
	OnCustomConditionChangedHandle.Reset();
	OnDynamicConditionChangedHandle.Reset();
	PollFrequencyLimiter->Deinit();

	// Release destructioninfo protocol.
	if (DestructionInfoProtocol)
	{
		DestructionInfoProtocol->Release();
		DestructionInfoProtocol = nullptr;
	}

	Super::Deinitialize();
}

void UObjectReplicationBridge::CreateDestructionProtocol()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// Create destruction info protocol
	const FReplicationFragments RegisteredFragments;
	const FReplicationProtocolIdentifier ProtocolId = FReplicationProtocolManager::CalculateProtocolIdentifier(RegisteredFragments);
	const FCreateReplicationProtocolParameters CreateProtocolParams
	{
		.bValidateProtocolId = false,
		.bHasTemplateKey = false, // Destruction info protocol doesn't need a TemplateKey
		.TypeStatsIndex = GetReplicationSystem()->GetReplicationSystemInternal()->GetNetTypeStats().GetOrCreateTypeStats(FName("DestructionInfo")),
	};
	DestructionInfoProtocol = ReplicationProtocolManager->CreateReplicationProtocol(ProtocolId, RegisteredFragments, TEXT("InternalDestructionInfo"), CreateProtocolParams);
	// Explicit refcount
	DestructionInfoProtocol->AddRef();
}

void UObjectReplicationBridge::InitNetObjectFactories()
{
	using namespace UE::Net;
	const TConstArrayView<FNetObjectFactoryRegistry::FFactoryData> RegisteredFactories = FNetObjectFactoryRegistry::GetRegisteredFactories();

	check(NetObjectFactories.IsEmpty());

	for(int32 Index=0; Index < RegisteredFactories.Num(); ++Index )
	{
		const FNetObjectFactoryRegistry::FFactoryData& Data = RegisteredFactories[Index];
		if (Data.Id != InvalidNetObjectFactoryId)
		{
			check(Data.Id == (FNetObjectFactoryId)Index);
			check(Data.NetFactoryClass.Get());

			UNetObjectFactory* Factory = NewObject<UNetObjectFactory>(this, Data.NetFactoryClass.Get());
			NetObjectFactories.Add(Factory);

			Factory->Init(Data.Id, this);
		}
	}
}

void UObjectReplicationBridge::DeinitNetObjectFactories()
{
	for (UNetObjectFactory* Factory : NetObjectFactories)
	{
		Factory->Deinit();
	}

	NetObjectFactories.Empty();
}

UNetObjectFactory* UObjectReplicationBridge::GetNetFactory(UE::Net::FNetObjectFactoryId FactoryId) const
{
	if (ensureMsgf(FactoryId < NetObjectFactories.Num() && FactoryId != UE::Net::InvalidNetObjectFactoryId, TEXT("GetNetFactory invalid factory ID %u, num factories: %d"), FactoryId, NetObjectFactories.Num()))
	{
		return NetObjectFactories[FactoryId];
	}

	return nullptr;
}

void UObjectReplicationBridge::OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex)
{
	PollFrequencyLimiter->OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
	ObjectsWithObjectReferences.SetNumBits(NewMaxInternalIndex);
	GarbageCollectionAffectedObjects.SetNumBits(NewMaxInternalIndex);
}

UObject* UObjectReplicationBridge::GetObjectFromReferenceHandle(FNetRefHandle RefHandle) const
{
	return GetObjectReferenceCache()->GetObjectFromReferenceHandle(RefHandle);
}

UObject* UObjectReplicationBridge::ResolveObjectReference(const UE::Net::FNetObjectReference& Reference, const UE::Net::FNetObjectResolveContext& ResolveContext)
{
	return GetObjectReferenceCache()->ResolveObjectReference(Reference, ResolveContext);
}

FString UObjectReplicationBridge::DescribeObjectReference(const UE::Net::FNetObjectReference& Reference, const UE::Net::FNetObjectResolveContext& ResolveContext)
{
	return GetObjectReferenceCache()->DescribeObjectReference(Reference, ResolveContext);
}

UE::Net::FNetObjectReference UObjectReplicationBridge::GetOrCreateObjectReference(const UObject* Instance) const
{
	return GetObjectReferenceCache()->GetOrCreateObjectReference(Instance, GetReplicationSystem());
}

UE::Net::FNetObjectReference UObjectReplicationBridge::GetOrCreateObjectReference(const FString& Path, const UObject* Outer) const
{
	return GetObjectReferenceCache()->GetOrCreateObjectReference(Path, Outer, GetReplicationSystem());
}

void UObjectReplicationBridge::AddStaticDestructionInfo(const FString& ObjectPath, const UObject* Outer, const UE::Net::FDestructionParameters& Parameters)
{
	UE::Net::FNetObjectReference ObjectRef = GetOrCreateObjectReference(ObjectPath, Outer);
	if (ObjectRef.IsValid())
	{
		StoreDestructionInfo(ObjectRef.GetRefHandle(), Parameters);
	}
}

UE::Net::FNetRefHandle UObjectReplicationBridge::StoreDestructionInfo(FNetRefHandle Handle, const UE::Net::FDestructionParameters& Parameters)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (!FNetObjectFactoryRegistry::IsValidFactoryId(Parameters.NetFactoryId))
	{
		ensureMsgf(false, TEXT("StoreDestructionInfo received invalid NetFactoryId: %u for %s"), Parameters.NetFactoryId, *PrintObjectFromNetRefHandle(Handle));
		UE_LOGF(LogIris, Error, "StoreDestructionInfo received invalid NetFactoryID: %u for %ls", Parameters.NetFactoryId, *PrintObjectFromNetRefHandle(Handle));

		return FNetRefHandle::GetInvalid();
	}

	// Not allowed to create destruction infos at the moment
	if (!CanCreateDestructionInfo())
	{
		return FNetRefHandle::GetInvalid();
	}

	FReplicationProtocolManager* ProtocolManager = GetReplicationProtocolManager();

	const FCreateNetObjectParams CreateNetObjectParams
	{
		.NetFactoryId = Parameters.NetFactoryId,
		.ReplicationProtocol = DestructionInfoProtocol,
	};

	// Create handle for DestructionInfo to be able to scope destruction infos
	const FNetObjectGroupHandle ContainerGroupHandle = GetContainerGroup(Parameters.Container);
	FNetRefHandle DestructionInfoHandle = NetRefHandleManager->CreateHandleForDestructionInfo(Handle, CreateNetObjectParams);

	if (ContainerGroupHandle.IsValid())
	{
		GetReplicationSystem()->AddToGroup(ContainerGroupHandle, DestructionInfoHandle);
	}

	const FInternalNetRefIndex InternalReplicationIndex = NetRefHandleManager->GetInternalIndex(DestructionInfoHandle);

	// We also need to store the actual data we send to destroy static references when they are scoped
	FStaticDestructionInfo PersistentDestructionInfo
	{
		.StaticRef = FObjectReferenceCache::MakeNetObjectReference(Handle),
		.ContainerGroupHandle = ContainerGroupHandle,
		.NetFactoryId = Parameters.NetFactoryId,
		.InternalReplicationIndex = InternalReplicationIndex
	};

	StaticObjectsPendingDestroy.Add(DestructionInfoHandle, PersistentDestructionInfo);
	DestroyedStaticObjectToDestructionInfoHandleMap.Add(Handle, DestructionInfoHandle);

	// If we use distance based Prioritization for destruction infos we need to populate the quantized state used for prioritization.
	const bool bUseDynamicPrioritization = Parameters.bUseDistanceBasedPrioritization;
	if (bUseDynamicPrioritization)
	{
		// Use WorldLocations to feed the location of the destruction info so that it can be prioritized properly.
		FWorldLocations& WorldLocations = GetReplicationSystem()->GetReplicationSystemInternal()->GetWorldLocations();

		// Check the position lies within world boundaries.
		ensureMsgf(Parameters.Location.X >= WorldLocations.GetWorldMinPos().X && Parameters.Location.X <= WorldLocations.GetWorldMaxPos().X && 
				   Parameters.Location.Y >= WorldLocations.GetWorldMinPos().Y && Parameters.Location.Y <= WorldLocations.GetWorldMaxPos().Y, TEXT("Object %s with position %s lies outside configured world boundary."), ToCStr(NetRefHandleManager->PrintObjectFromIndex((InternalReplicationIndex))), *Parameters.Location.ToString());

		WorldLocations.InitObjectInfoCache(InternalReplicationIndex);
		WorldLocations.SetObjectInfo(InternalReplicationIndex, Parameters.Location, 0.0f);

		GetReplicationSystem()->SetPrioritizer(DestructionInfoHandle, DefaultSpatialNetObjectPrioritizerHandle);
	}

	UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("StoreDestructionInfo %s (InternalIndex: %u ) for %s GroupIndex: %u"), *DestructionInfoHandle.ToString(), InternalReplicationIndex,  *PrintObjectFromNetRefHandle(Handle), ContainerGroupHandle.GetGroupIndex());
	
	return DestructionInfoHandle;
}

void UObjectReplicationBridge::RemoveDestructionInfo(FNetRefHandle Handle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (FNetRefHandle* DestructionInfoHandlePtr = DestroyedStaticObjectToDestructionInfoHandleMap.Find(Handle))
	{
		const FNetRefHandle DestructionInfoHandle = *DestructionInfoHandlePtr;
		if (const UE::Net::Private::FStaticDestructionInfo* Info = StaticObjectsPendingDestroy.Find(DestructionInfoHandle))
		{
			const FInternalNetRefIndex InternalObjectIndex = Info->InternalReplicationIndex;

			if (NetRefHandleManager->IsDestroyedStartupObject(InternalObjectIndex))
			{
				// Remove destruction info tracking and info.
				NetRefHandleManager->ResetDestroyedStartupObject(InternalObjectIndex);
				RemoveDestructionInfo(InternalObjectIndex);

				UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("RemoveDestructionInfo %s (InternalIndex: %u ) for handle %s"), *DestructionInfoHandle.ToString(), InternalObjectIndex, *Handle.ToString());
			}
		}
	}
}

void UObjectReplicationBridge::RemoveDestructionInfosForGroup(UE::Net::FNetObjectGroupHandle GroupHandle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FWorldLocations& WorldLocations = GetReplicationSystem()->GetReplicationSystemInternal()->GetWorldLocations();

	if (GroupHandle.IsValid())
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("RemoveDestructionInfosForGroup GroupIndex: %u"), GroupHandle.GetGroupIndex());

		FNetObjectGroup* Group = Groups->GetGroup(GroupHandle);
		check(Group);

		TArray<FInternalNetRefIndex, TInlineAllocator<384>> ObjectIndicesToRemove;	
		for (FInternalNetRefIndex InternalObjectIndex : MakeArrayView(Group->Members))
		{
			if (NetRefHandleManager->IsDestroyedStartupObject(InternalObjectIndex))
			{
				const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);
				FStaticDestructionInfo RemovedDestructionInfo;
				if (StaticObjectsPendingDestroy.RemoveAndCopyValue(ObjectData.RefHandle, RemovedDestructionInfo))
				{
					DestroyedStaticObjectToDestructionInfoHandleMap.Remove(RemovedDestructionInfo.StaticRef.GetRefHandle());
					ObjectIndicesToRemove.Add(InternalObjectIndex);
				}
			}
		}

		// Destroy NetObjects and remove from WorldLocations
		for (FInternalNetRefIndex InternalObjectIndex : MakeArrayView(ObjectIndicesToRemove))
		{
			NetRefHandleManager->DestroyNetObjectByIndex(InternalObjectIndex);
			WorldLocations.RemoveObjectInfoCache(InternalObjectIndex);
		}
	}
	else
	{
		// Destroy NetObjects and remove from WorldLocations
		for (const TPair<FNetRefHandle, UE::Net::Private::FStaticDestructionInfo>& It : StaticObjectsPendingDestroy)
		{
			const UE::Net::FInternalNetRefIndex InternalObjectIndex = It.Value.InternalReplicationIndex;
			NetRefHandleManager->DestroyNetObjectByIndex(InternalObjectIndex);
			WorldLocations.RemoveObjectInfoCache(InternalObjectIndex);
		}

		StaticObjectsPendingDestroy.Empty();
		DestroyedStaticObjectToDestructionInfoHandleMap.Empty();
	}
}

void UObjectReplicationBridge::RemoveDestructionInfo(UE::Net::FInternalNetRefIndex InternalObjectIndex)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (InternalObjectIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);
	FStaticDestructionInfo RemovedDestructionInfo;
	if (StaticObjectsPendingDestroy.RemoveAndCopyValue(ObjectData.RefHandle, RemovedDestructionInfo))
	{
		DestroyedStaticObjectToDestructionInfoHandleMap.Remove(RemovedDestructionInfo.StaticRef.GetRefHandle());

		FWorldLocations& WorldLocations = GetReplicationSystem()->GetReplicationSystemInternal()->GetWorldLocations();

		NetRefHandleManager->DestroyNetObjectByIndex(InternalObjectIndex);
		WorldLocations.RemoveObjectInfoCache(InternalObjectIndex);
	}
}

void UObjectReplicationBridge::ReadAndExecuteDestructionInfoFromRemote(FReplicationBridgeSerializationContext& Context)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	check(Context.bIsDestructionInfo);

	UE_NET_TRACE_SCOPE(DestructionInfo, *Context.SerializationContext.GetBitStreamReader(), Context.SerializationContext.GetTraceCollector(), ENetTraceVerbosity::Trace);

	//TODO: We need to support destruction messages that work on any stable objects, not just those with static path names

	// Read the static path referencing the object
	FNetObjectReference ReferenceToDestroy;
	ReadFullNetObjectReference(Context.SerializationContext, ReferenceToDestroy);

	// Read the NetFactoryId
	const FNetObjectFactoryId NetFactoryId = (FNetObjectFactoryId)Context.SerializationContext.GetBitStreamReader()->ReadBits(FNetObjectFactoryRegistry::GetMaxBits());

	// Find the object referenced by the path name
	const UObject* Instance = ObjectReferenceCache->ResolveObjectReference(ReferenceToDestroy, Context.SerializationContext.GetInternalContext()->ResolveContext);
	if (!Instance)
	{
		// Object does not exist, nothing to destroy
		return;
	}
	
	// Look if this object has an internal index assigned. If it doesn't have one it's because it was never replicated to the client before
	const FInternalNetRefIndex InternalReplicationIndex = NetRefHandleManager->GetInternalIndex(ReferenceToDestroy.GetRefHandle());
	if (InternalReplicationIndex != InvalidInternalNetRefIndex)
	{
		NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalReplicationIndex).bPendingEndReplication = 1U;
	}

	UE_LOGF(LogIris, VeryVerbose, "ReadAndExecuteDestructionInfoFromRemote on %ls | %ls | NetFactoryId: %u", *ReferenceToDestroy.ToString(), *ReferenceToDestroy.GetRefHandle().ToString(), NetFactoryId);

	constexpr EDetachReason DetachReason = EDetachReason::StaticDestroyed;
	DetachRootObjectFromRemote(ReferenceToDestroy.GetRefHandle(), DetachReason, NetFactoryId);
}

UObject* UObjectReplicationBridge::GetReplicatedObject(FNetRefHandle Handle) const
{
	return IsReplicatedHandle(Handle) ? GetObjectFromReferenceHandle(Handle) : nullptr;
};

UObject* UObjectReplicationBridge::GetPreRegisteredObject(FNetRefHandle Handle) const
{
	return GetObjectReferenceCache()->GetPreRegisteredObjectFromReferenceHandle(Handle);
};

UE::Net::FNetRefHandle UObjectReplicationBridge::GetReplicatedRefHandle(const UObject* Object, EGetRefHandleFlags GetRefHandleFlags) const
{
	const FNetRefHandle Handle = GetObjectReferenceCache()->GetObjectReferenceHandleFromObject(Object, GetRefHandleFlags);
	return IsReplicatedHandle(Handle) ? Handle : FNetRefHandle::GetInvalid();
}

UE::Net::FNetRefHandle UObjectReplicationBridge::GetReplicatedRefHandle(FNetHandle Handle) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// If the object is replicated by the owning ReplicationSystem the internal handle should be valid.
	FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager->GetInternalIndexFromNetHandle(Handle);
	if (ObjectInternalIndex == InvalidInternalNetRefIndex)
	{
		return FNetRefHandle::GetInvalid();
	}

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectInternalIndex);

	// We do not consider objects pending stop due to flush or tearoff to be replicated to prevent unwanted API calls.
	const bool bPendingStopDueToFlushOrTearOff = ObjectData.bTearOff || ObjectData.bFlush;
	if (bPendingStopDueToFlushOrTearOff)
	{
		return FNetRefHandle::GetInvalid();
	}

	return ObjectData.RefHandle;
}

bool UObjectReplicationBridge::IsNetRefHandlePreRegistered(FNetRefHandle Handle)
{
	return ObjectReferenceCache->IsNetRefHandlePreRegistered(Handle);
}

UE::Net::FNetRefHandle UObjectReplicationBridge::PreRegisterNewObjectReferenceHandle(UObject* Instance)
{
	return ObjectReferenceCache->PreRegisterObjectReferenceHandle(Instance, GetReplicationSystem());
}

void UObjectReplicationBridge::PreRegisterObjectWithReferenceHandle(const UObject* Instance, FNetRefHandle Handle)
{
	ObjectReferenceCache->AddPreRegisteredReference(Handle, Instance, ReplicationSystem);
}

bool UObjectReplicationBridge::IsReplicatingTransactionally() const
{
	return GetReplicationSystem()->IsReplicatingTransactionally();
}

UE::Net::FNetRefHandle UObjectReplicationBridge::StartReplicatingNetObject(UObject* Instance, UE::Net::EReplicationFragmentTraits Traits, UE::Net::FNetObjectFactoryId NetFactoryId, bool bForceNewHandle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandle AllocatedRefHandle = ObjectReferenceCache->CreateObjectReferenceHandle(Instance, ReplicationSystem);

	// If we failed to assign a handle
	if (!AllocatedRefHandle.IsValid())
	{
		return FNetRefHandle::GetInvalid();
	}

	// The instance is already replicating, just return the handle
	if (IsReplicatedHandle(AllocatedRefHandle) && RestartOrStopObjectPendingAsyncStopReplication(AllocatedRefHandle, bForceNewHandle))
	{
		ensureMsgf(bForceNewHandle == false, TEXT("bForceNewHandle was requested for %s but it is already replicating. The object must stop replicating first or use async stop replication."), *PrintObjectFromNetRefHandle(AllocatedRefHandle));
		return AllocatedRefHandle;
	}

	if (bRemoveDestructionInfoIfRestartingReplication)
	{
		RemoveDestructionInfo(AllocatedRefHandle);
	}

	//TODO: We could decide to automatically assign a new handle to objects that changed from Stable to Dynamic
	//		Or objects that were renamed or changed their Outer since the last time they were replicating.
	//		We could even do it for objects that can change protocols.
	if (bForceNewHandle)
	{
		AllocatedRefHandle = ObjectReferenceCache->ReassignObjectRefHandle(Instance, ReplicationSystem);

		if (!AllocatedRefHandle.IsValid())
		{
			ensureMsgf(AllocatedRefHandle.IsValid(), TEXT("Reassignment for %s failed to return a valid handle"), *GetNameSafe(Instance));
			return FNetRefHandle::GetInvalid();
		}
	}

	IRIS_PROFILER_SCOPE(StartReplicatingNetObject);
	
	FFragmentRegistrationContext FragmentRegistrationContext(GetReplicationStateDescriptorRegistry(), ReplicationSystem, Traits, Instance);

	// For everything derived from UObject we can call the virtual function RegisterReplicationFragments	
	CallRegisterReplicationFragments(Instance, FragmentRegistrationContext, EFragmentRegistrationFlags::None);

	const FReplicationFragments& RegisteredFragments = FFragmentRegistrationContextPrivateAccessor::GetReplicationFragments(FragmentRegistrationContext);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (RegisteredFragments.IsEmpty() && !FragmentRegistrationContext.IsFragmentlessNetObject())
	{
		// Look if the class registered replicated properties
		TArray<FLifetimeProperty> ReplicatedProps;
		Instance->GetLifetimeReplicatedProps(ReplicatedProps);
		
		if (ReplicatedProps.IsEmpty())
		{
			ensureMsgf(false, TEXT("NetObject %s (class %s) registered no fragments. Call SetIsFragmentlessNetObject if this is intentional."), *GetNameSafe(Instance), *GetNameSafe(Instance->GetClass()));
		}
		else
		{
			ensureMsgf(false, TEXT("NetObject %s (class %s) registered no fragments but GetLifetimeReplicatedProps returned %d variables. Make sure to call CreateAndRegisterFragmentsForObject in RegisterReplicationFragments"),
				*GetNameSafe(Instance), *GetNameSafe(Instance->GetClass()), ReplicatedProps.Num());
		}
	}
	else if (FragmentRegistrationContext.IsFragmentlessNetObject() && !RegisteredFragments.IsEmpty())
	{
		ensureMsgf(false, TEXT("NetObject %s (class %s) says that it is Fragment-less but still created %d Fragments."), *GetNameSafe(Instance), *GetNameSafe(Instance->GetClass()), RegisteredFragments.Num());
	}
#endif

	// Some fragments may be bound to a default source (e.g. template or archetype or cdo)
	// Find the state source used for this instance and use it to bind the protocol to.
    const UObject* TemplateKey = FFragmentRegistrationContextPrivateAccessor::GetDefaultStateSource(FragmentRegistrationContext);
	
	// Fallback to using the archetype or cdo here if the main object register any state source. This only happens for special unit tests
	if (!TemplateKey)
	{
		TemplateKey = Instance->GetArchetype();
	}

	// Create Protocols
	FReplicationProtocolManager* ProtocolManager = GetReplicationProtocolManager();

	FReplicationInstanceProtocolPtr InstanceProtocol(ProtocolManager->CreateInstanceProtocol(RegisteredFragments, FragmentRegistrationContext.GetFragmentTraits()));
		
	FReplicationProtocolIdentifier ProtocolIdentifier = FReplicationProtocolManager::CalculateProtocolIdentifier(RegisteredFragments);
	const FReplicationProtocol* ReplicationProtocol = ProtocolManager->GetReplicationProtocol(ProtocolIdentifier, TemplateKey);
	if (!ReplicationProtocol)
	{
		const FCreateReplicationProtocolParameters CreateProtocolParams
		{
			.TemplateKey = TemplateKey,
			.TypeStatsIndex = GetTypeStatsIndex(Instance->GetClass()) 
		};
		ReplicationProtocol = ProtocolManager->CreateReplicationProtocol(ProtocolIdentifier, RegisteredFragments, *Instance->GetClass()->GetName(), CreateProtocolParams);
	}
#if UE_IRIS_VALIDATE_PROTOCOLS
	else
	{
		const bool bIsValidProtocol = FReplicationProtocolManager::ValidateReplicationProtocol(ReplicationProtocol, RegisteredFragments);
		if (!bIsValidProtocol)
		{
			UE_LOG_OBJECTREPLICATIONBRIDGE(Error, TEXT("StartReplicatingNetObject Found invalid protocol ProtocolId:0x%x for Object named %s"), ReplicationProtocol->ProtocolIdentifier, *Instance->GetName());
			return FNetRefHandle::GetInvalid();
		}
	}
#endif
		
	if (!ReplicationProtocol)
	{
		UE_LOGF(LogIris, Error, "StartReplicatingNetObject - Failed to create protocol for %ls", ToCStr(Instance->GetPathName()));
		return FNetRefHandle::GetInvalid();
	}

	IRIS_PROFILER_PROTOCOL_NAME(ReplicationProtocol->DebugName->Name);

	const FCreateNetObjectParams CreateNetObjectParams
	{
		.NetFactoryId = NetFactoryId,
		.IrisAsyncLoadingPriority = GetClassIrisAsyncLoadingPriority(Instance->GetClass()),
		.ReplicationProtocol = ReplicationProtocol,
	};

	// Create NetHandle and bind instance
	FNetHandle NetHandle = FNetHandleManager::GetOrCreateNetHandle(Instance);
	FNetRefHandle RefHandle = InternalCreateNetObject(AllocatedRefHandle, NetHandle, CreateNetObjectParams);

	if (!RefHandle.IsValid())
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Warning, TEXT("StartReplicatingNetObject Failed to create NetRefHandle with ProtocolId:0x%x for Object named %s"), (ReplicationProtocol != nullptr ? ReplicationProtocol->ProtocolIdentifier : FReplicationProtocolIdentifier(0)), *Instance->GetName());
		return FNetRefHandle::GetInvalid();
	}

	UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("StartReplicatingNetObject Created %s [handle:%s|stable:%s] with ProtocolId:0x%x for Object named %s"), 
		*RefHandle.ToString(), 
		RefHandle.IsStatic()?TEXT("Static"):TEXT("Dynamic"),
		Instance->IsNameStableForNetworking()?TEXT("yes"):TEXT("no"),
		ReplicationProtocol->ProtocolIdentifier, 
		*Instance->GetName()
	);

	// Attach the instance and bind the instance protocol to dirty tracking
	constexpr bool bBindInstanceProtocol = true;
	InternalAttachInstanceToNetRefHandle(RefHandle, bBindInstanceProtocol, InstanceProtocol.Get(), Instance, NetHandle);
#if WITH_PUSH_MODEL
	SetNetPushIdOnInstance(InstanceProtocol.Get(), NetHandle);
#endif

	const FInternalNetRefIndex InternalReplicationIndex = NetRefHandleManager->GetInternalIndex(RefHandle);

	check(NetFactoryId != InvalidNetObjectFactoryId);
	NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalReplicationIndex).NetFactoryId = NetFactoryId;

	// Initialize conditionals
	FReplicationSystemInternal* ReplicationSystemInternal = ReplicationSystem->GetReplicationSystemInternal();
	ReplicationSystemInternal->GetConditionals().InitPropertyCustomConditions(InternalReplicationIndex);

	// Keep track of handles with object references for garbage collection's sake.
	ObjectsWithObjectReferences.SetBitValue(InternalReplicationIndex, EnumHasAnyFlags(InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::HasObjectReference));

	if (ShouldClassBeDeltaCompressed(Instance->GetClass()))
	{
		ReplicationSystem->SetDeltaCompressionStatus(RefHandle, ENetObjectDeltaCompressionStatus::Allow);
	}

	// Release instance protocol from the UniquePtr as it is now successfully bound to the handle
	(void)InstanceProtocol.Release();

	return RefHandle;
}

UE::Net::FNetRefHandle UObjectReplicationBridge::InternalStartReplicatingRootObject(UObject* Instance, const FRootObjectReplicationParams& Params, UE::Net::FNetObjectFactoryId NetFactoryId)
{
	LLM_SCOPE_BYTAG(IrisState);

	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (bBlockStartRootObjectReplication)
	{
		ensureMsgf(false, TEXT("StartReplicatingRootObject is not allowed during this operation. %s will not be replicated"), *GetNameSafe(Instance));
		return FNetRefHandle::GetInvalid();
	}

	if (!ensureMsgf(!Instance->IsTemplate(), TEXT("Object %s cannot be replicated because it's an %s"), 
		*GetNameSafe(Instance), Instance->HasAnyFlags(RF_ArchetypeObject)?TEXT("Archetype"):TEXT("DefaultObject")))
	{
		return FNetRefHandle::GetInvalid();
	}

	EReplicationFragmentTraits Traits = EReplicationFragmentTraits::CanReplicate;
	Traits |= Params.bNeedsPreUpdate ? EReplicationFragmentTraits::NeedsPreSendUpdate : EReplicationFragmentTraits::None;

	// Register the object
	FNetRefHandle RefHandle = StartReplicatingNetObject(Instance, Traits, NetFactoryId, Params.bForceNewHandle);

	if (!RefHandle.IsValid())
	{
		return FNetRefHandle::GetInvalid();;
	}
	
	// Now configure the RootObject
	const FInternalNetRefIndex ObjectIndex = NetRefHandleManager->GetInternalIndex(RefHandle);

	// Set poll frame period
	float PollFrequency = Params.PollFrequency;
	FindOrCachePollFrequency(Instance->GetClass(), PollFrequency);			
	PollFrequencyLimiter->SetPollFrequency(ObjectIndex, PollFrequency);

	if (Params.bNeedsWorldLocationUpdate)
	{
		FWorldLocations& WorldLocations = ReplicationSystem->GetReplicationSystemInternal()->GetWorldLocations();
		WorldLocations.InitObjectInfoCache(ObjectIndex);
		ForceUpdateWorldLocation(RefHandle, ObjectIndex);
	}

	// Set prioritizer
	const bool bRequireForceEnabled = Params.StaticPriority > 0.0f;
	const FNetObjectPrioritizerHandle PrioritizerHandle = GetPrioritizer(Instance->GetClass(), bRequireForceEnabled);

	// Set static priority if valid unless we have a force enabled prioritizer.
	if (Params.StaticPriority > 0.0f && PrioritizerHandle == InvalidNetObjectPrioritizerHandle)
	{
		ReplicationSystem->SetStaticPriority(RefHandle, Params.StaticPriority);
	}
	else
	{
		if (PrioritizerHandle != InvalidNetObjectPrioritizerHandle)
		{
			ReplicationSystem->SetPrioritizer(RefHandle, PrioritizerHandle);
		}
		else if (Params.bNeedsWorldLocationUpdate || HasRepTag(ReplicationSystem->GetReplicationProtocol(RefHandle), RepTag_WorldLocation))
		{
			ReplicationSystem->SetPrioritizer(RefHandle, DefaultSpatialNetObjectPrioritizerHandle);
		}
	}

	// Set dynamic filter
	AssignDynamicFilter(Instance, Params, RefHandle);

	// Spatially filtered non-dormant objects requires frequent world location updates. Expecting a better solution that instead of us polling will inform us when locations change, UE-193004.
	if (Params.bNeedsWorldLocationUpdate && !Params.bIsDormant)
	{
		OptionallySetObjectRequiresFrequentWorldLocationUpdate(RefHandle, true);
	}

	return RefHandle;
}

UE::Net::FNetRefHandle UObjectReplicationBridge::StartReplicatingRootObject(UObject* Instance, const FRootObjectReplicationParams& Params, UE::Net::FNetObjectFactoryId NetFactoryId)
{
	return InternalStartReplicatingRootObject(Instance, Params, NetFactoryId);
}

UE::Net::FNetRefHandle UObjectReplicationBridge::StartReplicatingRootObject(UObject* Instance, UE::Net::FNetObjectFactoryId NetFactoryId)
{
	using namespace UE::Net;

	LLM_SCOPE_BYTAG(IrisState);
	
	if (UNetObjectFactory* NetObjectFactory = GetNetFactory(NetFactoryId))
	{
		FRootObjectReplicationParamsContext ParamsContext;
		FRootObjectReplicationParams Params;
		ParamsContext.Object = Instance;
		ParamsContext.Reason = ERootObjectReplicationParamsReason::StartReplication;
		NetObjectFactory->FillRootObjectReplicationParams(ParamsContext, Params);

		return InternalStartReplicatingRootObject(Instance, Params, NetFactoryId);
	}
	else
	{
		UE_LOG_BRIDGEID(LogIris, Warning, TEXT("%hs: Unknown NetObjectFactory (Id: %u)"), __FUNCTION__, NetFactoryId); 
		return FNetRefHandle::GetInvalid();
	}
}

void UObjectReplicationBridge::AssignDynamicFilter(UObject* Instance, const FRootObjectReplicationParams& Params, FNetRefHandle RefHandle)
{
	using namespace UE::Net;

	if (!bEnableFilterMappings)
	{
		return;
	}

	FNetObjectFilterHandle FilterHandle = InvalidNetObjectFilterHandle;
	FName FilterConfigProfile;

	if (Params.bUseExplicitDynamicFilter)
	{
		if (Params.ExplicitDynamicFilterName != NAME_None)
		{
			FilterHandle = ReplicationSystem->GetFilterHandle(Params.ExplicitDynamicFilterName);
			
			UE_CLOGF(FilterHandle == InvalidNetObjectFilterHandle, LogIris, Error, "Could not assign explicit dynamic filter to %ls. No filters named %ls exist", *GetPathNameSafe(Instance), *Params.ExplicitDynamicFilterName.ToString() );
			ensure(FilterHandle != InvalidNetObjectFilterHandle);
		}
	}
	else if (Params.bUseClassConfigDynamicFilter)
	{
		constexpr bool bRequireForceEnabled = false;
		FilterHandle = GetDynamicFilter(Instance->GetClass(), bRequireForceEnabled, FilterConfigProfile);
	}

	if (FilterHandle != InvalidNetObjectFilterHandle)
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("StartReplicatingRootObject Filter: %s will be used for %s. (FilterProfile: %s)"), *(ReplicationSystem->GetFilterName(FilterHandle).ToString()), *NetRefHandleManager->PrintObjectFromNetRefHandle(RefHandle), *FilterConfigProfile.ToString());
		ReplicationSystem->SetFilter(RefHandle, FilterHandle, FilterConfigProfile);
	}
}

UE::Net::FNetRefHandle UObjectReplicationBridge::StartReplicatingSubObject(UObject* Instance, const FSubObjectReplicationParams& Params, UE::Net::FNetObjectFactoryId NetFactoryId)
{
	LLM_SCOPE_BYTAG(IrisState);

	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	const FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();

	const FInternalNetRefIndex RootObjectIndex = LocalNetRefHandleManager.GetInternalIndex(Params.RootObjectHandle);

	if (!ensureMsgf(!Instance->IsTemplate(), TEXT("Iris cannot replicate subobject %s with root %s because it's an %s"), 
		*GetNameSafe(Instance), *GetNameSafe(LocalNetRefHandleManager.GetReplicatedObjectInstance(RootObjectIndex)), Instance->HasAnyFlags(RF_ArchetypeObject)?TEXT("Archetype"):TEXT("DefaultObject")))
	{
		return FNetRefHandle::GetInvalid();
	}

	if (!ensureMsgf(IsReplicatedHandle(Params.RootObjectHandle), TEXT("Root %s (%s) must be replicated for subobject %s to replicate."), 
		*GetNameSafe(LocalNetRefHandleManager.GetReplicatedObjectInstance(RootObjectIndex)), *Params.RootObjectHandle.ToString(), *GetNameSafe(Instance)))
	{
		return FNetRefHandle::GetInvalid();
	}

	if (!LocalNetRefHandleManager.IsScopableIndex(RootObjectIndex))
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Warning, TEXT("Root %s (%s) must be in scope for subobject %s to replicate."), 
			*GetNameSafe(LocalNetRefHandleManager.GetReplicatedObjectInstance(RootObjectIndex)), *Params.RootObjectHandle.ToString(), *GetNameSafe(Instance))
		return FNetRefHandle::GetInvalid();
	}

	// Verify that ParentSubObjectHandle is valid if specified
	if (Params.ParentSubObjectHandle.IsValid() && (Params.ParentSubObjectHandle != Params.RootObjectHandle))
	{		
		if (!ensureMsgf(NetRefHandleManager->GetRootObjectOfSubObject(Params.ParentSubObjectHandle) == Params.RootObjectHandle, TEXT("ParentSubObjectHandle %s must be a replicated subobject with same RootObjectHandle %s. Trying to add subobject: %s"), 
			*LocalNetRefHandleManager.PrintObjectFromNetRefHandle(Params.ParentSubObjectHandle), *LocalNetRefHandleManager.PrintObjectFromNetRefHandle(Params.RootObjectHandle), *GetNameSafe(Instance)))
		{
			return FNetRefHandle::GetInvalid();
		}
	}

	// Pre-registering subobjects is not supported
	{
		const FNetRefHandle PotentiallyPreRegisteredHandle = GetObjectReferenceCache()->GetObjectReferenceHandleFromObject(Instance);
		if (PotentiallyPreRegisteredHandle.IsValid() && IsNetRefHandlePreRegistered(PotentiallyPreRegisteredHandle))
		{
			UE_LOG_OBJECTREPLICATIONBRIDGE(Error, TEXT("StartReplicatingSubObject attempted to replicate pre-registered object %s."), *GetFullNameSafe(Instance));
			ensureMsgf(false, TEXT("Attempted to replicate a pre-registered subobject"));
			return FNetRefHandle::GetInvalid();
		}
	}

	FNetRefHandle SubObjectRefHandle = GetReplicatedRefHandle(Instance);
	if (SubObjectRefHandle.IsValid() && RestartOrStopObjectPendingAsyncStopReplication(SubObjectRefHandle, Params.bForceNewHandle))
	{
		// Verify that the existing object is a subobject of the owner
		const FNetRefHandle ExistingRootObjectHandle = LocalNetRefHandleManager.GetRootObjectOfSubObject(SubObjectRefHandle);
		if (Params.RootObjectHandle != ExistingRootObjectHandle)
		{
			UE_LOG_OBJECTREPLICATIONBRIDGE(Error, TEXT("StartReplicatingSubObject attempted to replicate already replicating subobject (%s) with a different rootobject (%s)."),
				*LocalNetRefHandleManager.PrintObjectFromNetRefHandle(SubObjectRefHandle), *LocalNetRefHandleManager.PrintObjectFromNetRefHandle(Params.RootObjectHandle));
			ensureMsgf(false, TEXT("Attempted to replicate already replicating subobject with a different root"));
		}
		return SubObjectRefHandle;
	}

	const EReplicationFragmentTraits Traits = EReplicationFragmentTraits::CanReplicate;
	SubObjectRefHandle = StartReplicatingNetObject(Instance, Traits, NetFactoryId, Params.bForceNewHandle);

	if (!SubObjectRefHandle.IsValid())
	{
		return FNetRefHandle::GetInvalid();
	}

	// Add subobject
	{
		if (Params.ParentSubObjectHandle.IsValid() && Params.ParentSubObjectHandle != Params.RootObjectHandle)
		{
			InternalAddSubObject(Params.ParentSubObjectHandle, SubObjectRefHandle, Params.InsertRelativeToSubObjectHandle, Params.InsertionOrder, Params.ChildSubObjectsReplicationOrder);
			UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("StartReplicatingChildSubObject Added %s RelativeToSubObjectHandle %s"), *PrintObjectFromNetRefHandle(SubObjectRefHandle), *PrintObjectFromNetRefHandle(Params.InsertRelativeToSubObjectHandle));
		}
		else
		{
			InternalAddSubObject(Params.RootObjectHandle, SubObjectRefHandle, Params.InsertRelativeToSubObjectHandle, Params.InsertionOrder, Params.ChildSubObjectsReplicationOrder);
			UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("StartReplicatingSubObject Added %s RelativeToSubObjectHandle %s"), *PrintObjectFromNetRefHandle(SubObjectRefHandle), *PrintObjectFromNetRefHandle(Params.InsertRelativeToSubObjectHandle));
		}
	}

	// SubObjects should always poll with owner
	SetPollWithObject(Params.RootObjectHandle, SubObjectRefHandle);

	// Copy pending dormancy from owner
	SetSubObjectDormancyStatus(SubObjectRefHandle, Params.RootObjectHandle);
	
	return SubObjectRefHandle;
}

UE::Net::EDetachReason ConvertDetachReasonFromInternalToPublic(UE::Net::Private::EInternalDetachReason InternalDetachReason)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	switch (InternalDetachReason)
	{
		case EInternalDetachReason::Normal:
		{
			return EDetachReason::Stopped;
		}

		case EInternalDetachReason::NotRelevant:
		{
			return EDetachReason::NoLongerRelevant;
		}

		case EInternalDetachReason::StaticDestroyed:
		{
			return EDetachReason::StaticDestroyed;
		}

		case EInternalDetachReason::ProxyReuse:
		{
			// ProxyReuse isn't currently exposed to factories
			ensureMsgf(false, TEXT("ProxyReuse isn't a public DetachReason"));
		} break;

		case EInternalDetachReason::TornOff:
		{
			return EDetachReason::TornOff;
		}

		default:
		{
			ensureMsgf(false, TEXT("Missing entry!"));
		} break;
	}

	return EDetachReason::Stopped;
}

void UObjectReplicationBridge::DestroyNetObjectFromRemote(UE::Net::FInternalNetRefIndex InternalObjectIndex, UE::Net::Private::EInternalDetachReason InternalDetachReason)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (InternalObjectIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	ensure(InternalDetachReason != EInternalDetachReason::Max);

	FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);

	ObjectData.bTearOff = (InternalDetachReason == EInternalDetachReason::TornOff);
	ObjectData.InternalDetachReason = InternalDetachReason;

	//TODO: Find why this is needed here? Is it to block client StopReplicatingNetRefHandle calls ?
	ObjectData.bPendingEndReplication = 1U;

	// Reused objects need to keep the Handle->Pointer binding alive since proxies reuse pointers instead of recreating them. This won't be required in the future if proxies simply do not instantiate a local instance and use protocol data directly.
	if (InternalDetachReason != EInternalDetachReason::ProxyReuse)
	{
		// Convert from internal to public reason
		EDetachReason PublicDetachReason = ConvertDetachReasonFromInternalToPublic(InternalDetachReason);

		UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("DestroyNetObjectFromRemote for %s | InternalReason: %s | PublicReason: %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalObjectIndex), LexToString(InternalDetachReason), LexToString(PublicDetachReason));

		const FNetRefHandle DestroyedObjectHandle = ObjectData.RefHandle;
		ensureMsgf(DestroyedObjectHandle.IsValid(), TEXT("Object with InternalIndex assigned an invalid handle: %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalObjectIndex));
		if (!DestroyedObjectHandle.IsValid())
		{
			return;
		}

		if (ObjectData.IsRootObject())
		{
			//TODO: Test how Iris behaves if a subobject is destroyed with a different reason (tearoff, etc.) from it's root object and we receive both the destruction messages at the same or different time.
			//It feels like the root object reason migh always have priority over everything else ?
			DetachRootObjectFromRemote(DestroyedObjectHandle, PublicDetachReason, ObjectData.NetFactoryId);
		}
		else
		{
			DetachSubObjectFromRemote(DestroyedObjectHandle, PublicDetachReason, ObjectData.NetFactoryId);
		}
	}

	// Detach instance protocol
	InternalDetachInstanceFromNetRefHandle(InternalObjectIndex);

	// Destroy the NetRefHandle
	InternalDestroyNetObject(InternalObjectIndex);
}

void UObjectReplicationBridge::SetSubObjectDormancyStatus(FNetRefHandle SubObjectRefHandle, FNetRefHandle OwnerRefHandle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FNetRefHandleManager& LocalNetRefHandleManager = *NetRefHandleManager;
	const FInternalNetRefIndex SubObjectInternalIndex = LocalNetRefHandleManager.GetInternalIndex(SubObjectRefHandle);
	const FInternalNetRefIndex OwnerInternalIndex = LocalNetRefHandleManager.GetInternalIndex(OwnerRefHandle);
	
	FNetBitArrayView DormantObjects = LocalNetRefHandleManager.GetWantToBeDormantInternalIndices();
	DormantObjects.SetBitValue(SubObjectInternalIndex, DormantObjects.GetBit(OwnerInternalIndex));
}

void UObjectReplicationBridge::SetSubObjectNetCondition(FNetRefHandle SubObjectRefHandle, ELifetimeCondition Condition)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// We assume that we can store the condition in an int8;
	static_assert(ELifetimeCondition::COND_Max <= INT8_MAX);

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();

	const FInternalNetRefIndex SubObjectInternalIndex = LocalNetRefHandleManager.GetInternalIndex(SubObjectRefHandle);
	bool bWasModified = false;
	
	if (LocalNetRefHandleManager.SetSubObjectNetCondition(SubObjectInternalIndex, (int8)Condition, bWasModified))
	{
		if (bWasModified)
		{
			UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("SetSubObjectNetCondition for SubObject %s Condition %s"), *PrintObjectFromNetRefHandle(SubObjectRefHandle), *UEnum::GetValueAsString<ELifetimeCondition>(Condition));
			MarkNetObjectStateDirty(ReplicationSystem->GetId(), SubObjectInternalIndex);
		}
	}
	else
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Warning, TEXT("Failed to Set SubObjectNetCondition for SubObject %s Condition %s"), *PrintObjectFromNetRefHandle(SubObjectRefHandle), *UEnum::GetValueAsString<ELifetimeCondition>(Condition));
	}
}

bool UObjectReplicationBridge::IsRootObject(FNetRefHandle RootObjectHandle) const
{
	return GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager().IsRootObject(RootObjectHandle);
}

UE::Net::FNetRefHandle UObjectReplicationBridge::GetRootObjectOfAnyObject(FNetRefHandle NetRefHandle) const
{
	return GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager().GetRootObjectOfAnyObject(NetRefHandle);
}

UE::Net::FNetRefHandle UObjectReplicationBridge::GetRootObjectOfSubObject(FNetRefHandle SubObjectHandle) const
{
	return GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager().GetRootObjectOfSubObject(SubObjectHandle);
}


void UObjectReplicationBridge::AddDependentObject(FNetRefHandle ParentHandle, FNetRefHandle DependentHandle, UE::Net::EDependentObjectSchedulingHint SchedulingHint)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();

	if (LocalNetRefHandleManager.AddDependentObject(ParentHandle, DependentHandle, SchedulingHint))
	{
		FReplicationFiltering& Filtering = ReplicationSystemInternal->GetFiltering();
		const FInternalNetRefIndex DependentInternalIndex = LocalNetRefHandleManager.GetInternalIndex(DependentHandle);
		Filtering.NotifyAddedDependentObject(DependentInternalIndex);

		UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("AddDependentObject Added dependent object %s to parent %s"), *PrintObjectFromNetRefHandle(DependentHandle), *PrintObjectFromNetRefHandle(ParentHandle));
	}
	else
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Warning , TEXT("AddDependentObject Failed to add dependent object %s to parent %s"), *PrintObjectFromNetRefHandle(DependentHandle), *PrintObjectFromNetRefHandle(ParentHandle));
	}
}

void UObjectReplicationBridge::RemoveDependentObject(FNetRefHandle ParentHandle, FNetRefHandle DependentHandle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("RemoveDependentObject Removing dependent object %s from parent %s"), *DependentHandle.ToString(), *ParentHandle.ToString());

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();

	// Remove dependent object
	FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();
	LocalNetRefHandleManager.RemoveDependentObject(ParentHandle, DependentHandle);

	const FInternalNetRefIndex DependentInternalIndex = LocalNetRefHandleManager.GetInternalIndex(DependentHandle);
	if (DependentInternalIndex != InvalidInternalNetRefIndex)
	{
		FReplicationFiltering& Filtering = ReplicationSystemInternal->GetFiltering();
		Filtering.NotifyRemovedDependentObject(DependentInternalIndex);
	}
}

void UObjectReplicationBridge::AddCreationDependencyLink(FNetRefHandle Parent, FNetRefHandle Child)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();

    const bool bSuccess = LocalNetRefHandleManager.AddCreationDependency(Parent, Child);
    
    if (bSuccess)
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("AddCreationDependencyLink Added creation dependency between child: %s and parent: %s"), *PrintObjectFromNetRefHandle(Child), *PrintObjectFromNetRefHandle(Parent));
	}
	else
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Warning , TEXT("AddCreationDependencyLink Failed to add creation dependent between child: %s and parent: %s"), *PrintObjectFromNetRefHandle(Child), *PrintObjectFromNetRefHandle(Parent));
	}
}

void UObjectReplicationBridge::RemoveCreationDependencyLink(FNetRefHandle Parent, FNetRefHandle Child)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();

	LocalNetRefHandleManager.RemoveCreationDependency(Parent, Child);
}

bool UObjectReplicationBridge::WriteNetRefHandleCreationInfo(FReplicationBridgeSerializationContext& Context, FNetRefHandle Handle, UE::Net::FInternalNetRefIndex InternalObjectIndex)
{
	UE_IRIS_PARALLEL_EXPR(FScopeLock ScopeLock(&WriteNetRefHandleCreationCS));
	using namespace UE::Net;
	using namespace UE::Net::Private;

	check(!Context.bIsDestructionInfo);

	// Validate that this is a legit object that we can write creation info for
	if (!NetRefHandleManager->IsLocal(InternalObjectIndex))
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Error, TEXT("WriteNetRefHandleCreationInfo: Cannot write creationinfo for %s ( InternalIndex: %u ), since it is no longer a valid object."), *Handle.ToString(), InternalObjectIndex);
		ensureMsgf(false, TEXT("Cannot write creationinfo for %s ( InternalIndex: %u ), since it is no longer a valid object."), *Handle.ToString(), InternalObjectIndex);
		return false;
	}

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);

	// If there is a cached header for this handle, use it!
	if (const FNetObjectCreationHeader* Header = ObjectData.bHasCachedCreationInfo ? NetRefHandleManager->GetCachedCreationInfo(InternalObjectIndex) : nullptr)
	{
		//TODO: Right now cached creation data is only for destroyed or torn-off objects so the data can be considered const
		//      If we eventually use it to store actively replicated objects, some creation data will want to be updated
		//	    before being sent.  We should add a trait and a NetObjectFactory::UpdateCachedData to do so.

		UNetObjectFactory* Factory = GetNetFactory(Header->GetNetFactoryId());
		UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("WriteNetRefHandleCreationInfo (cached) for %s : Factory: %s (id:%u) Header: %s"), *Handle.ToString(), *GetNameSafe(Factory), Header->GetNetFactoryId(), *Header->ToString());
		return Factory->WriteHeader(Handle, Context.SerializationContext, Header);
	}

	// Find the ProtocolId
	const UE::Net::FReplicationProtocol* Protocol = ObjectData.Protocol;
	if (!Protocol)
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Error, TEXT("WriteNetRefHandleCreationInfo: Cannot write creationinfo for %s, since protocol has been detached"), *Handle.ToString());
		ensureMsgf(Protocol, TEXT("WriteNetRefHandleCreationInfo: Cannot write creationinfo for %s, since protocol has been detached"), *Handle.ToString());
		return false;
	}

	const FNetObjectFactoryId FactoryId = ObjectData.NetFactoryId;

	UNetObjectFactory* Factory = GetNetFactory(FactoryId);
	check(Factory);
	TUniquePtr<FNetObjectCreationHeader> Header = Factory->CreateHeader(Handle, Protocol->ProtocolIdentifier);

	if (!Header.IsValid())
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("WriteNetRefHandleCreationInfo: CreateHeader via %s failed for %s"), *GetNameSafe(Factory), *Handle.ToString());
		return false;
	}

	UE_LOG_BRIDGEID(LogIris, VeryVerbose, TEXT("WriteNetRefHandleCreationInfo (new) for %s : Factory: %s (id:%u) Header: %s"), *Handle.ToString(), *GetNameSafe(Factory), FactoryId, *Header->ToString());

	return Factory->WriteHeader(Handle, Context.SerializationContext, Header.Get());
}

bool UObjectReplicationBridge::CacheNetRefHandleCreationInfo(FNetRefHandle Handle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FReplicationProtocol* Protocol = GetReplicationSystem()->GetReplicationProtocol(Handle);	
	if (!Protocol)
	{
		return false;
	}

	const FInternalNetRefIndex ObjectIndex = NetRefHandleManager->GetInternalIndex(Handle);
	if (ObjectIndex == InvalidInternalNetRefIndex)
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("CacheNetRefHandleCreationInfo cannot cache creation info for %s: no internal index assigned to it"), *Handle.ToString());
		ensureMsgf(false, TEXT("CacheNetRefHandleCreationInfo cannot cache creation info for %s: no internal index assigned to it"), *Handle.ToString());
		return false;
	}

	const FNetObjectFactoryId FactoryId = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex).NetFactoryId;

	UNetObjectFactory* Factory = GetNetFactory(FactoryId);
	check(Factory);
	TUniquePtr<FNetObjectCreationHeader> Header = Factory->CreateHeader(Handle, Protocol->ProtocolIdentifier);

	if (!Header.IsValid())
	{
		return false;
	}
	
	NetRefHandleManager->StoreCachedCreationInfo(ObjectIndex, MoveTemp(Header));
	return true;
}

void UObjectReplicationBridge::StopReplicatingNetObject(UObject* Instance, EEndReplicationFlags EndReplicationFlags)
{
	const FNetRefHandle RefHandle = GetReplicatedRefHandle(Instance, EGetRefHandleFlags::EvenIfGarbage);
	if (RefHandle.IsValid())
	{
		ensureMsgf(IsValid(Instance), TEXT("Calling StopReplicatingNetObject for Invalid Object: %s."), *PrintObjectFromNetRefHandle(RefHandle));
		StopReplicatingNetRefHandle(RefHandle, EndReplicationFlags);
	}
}

void UObjectReplicationBridge::DetachRootObjectFromRemote(FNetRefHandle RootObjectHandle, UE::Net::EDetachReason DetachReason, UE::Net::FNetObjectFactoryId NetFactoryId)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("%hs: %s Reason: %s"), __FUNCTION__, *PrintObjectFromNetRefHandle(RootObjectHandle), LexToString(DetachReason));

	if (!FNetObjectFactoryRegistry::IsValidFactoryId(NetFactoryId))
	{
		ensureMsgf(false, TEXT("Received invalid netfactory: %u for %s"), NetFactoryId,  *PrintObjectFromNetRefHandle(RootObjectHandle));
		return;
	}

	UNetObjectFactory* RootObjectFactory = GetNetFactory(NetFactoryId);
	
	const FInternalNetRefIndex DetachedInternalIndex = NetRefHandleManager->GetInternalIndex(RootObjectHandle);

	// The instance may be null even if the object was still replicating. This mostly happens when the local destroy paths call StopReplication early before waiting on the server to do it.
	UObject* RootObjectInstance = GetObjectFromReferenceHandle(RootObjectHandle);

	const UNetObjectFactory::FSubObjectDetachContext SubObjectDetachContext
	{
		.RootObject = RootObjectInstance,
		.RootObjectHandle = RootObjectHandle,
		.RootObjectDetachReason = DetachReason,
	};

	// Detach all the subobjects
	for (const FInternalNetRefIndex SubObjectInternalIndex : NetRefHandleManager->GetChildSubObjects(DetachedInternalIndex))
	{
		UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("%hs: Subobject %s | Reason: %s"), __FUNCTION__, *NetRefHandleManager->PrintObjectFromIndex(SubObjectInternalIndex), LexToString(DetachReason));

		FNetRefHandleManager::FReplicatedObjectData& SubObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(SubObjectInternalIndex);

		SubObjectData.bTearOff = DetachReason == EDetachReason::TornOff;
		SubObjectData.bPendingEndReplication = 1U;

		const FNetRefHandle SubObjectHandle = SubObjectData.RefHandle;

		const UNetObjectFactory::FDetachContext DetachContext
		{
			.DetachedInstance = GetObjectFromReferenceHandle(SubObjectHandle),
			.DetachedObjectHandle = SubObjectHandle,
			.Reason = DetachReason,
		};

		if (OnSubObjectDetached.IsBound())
		{
			OnSubObjectDetached.Broadcast(SubObjectHandle, DetachContext.DetachedInstance, RootObjectHandle, RootObjectInstance, DetachReason);
		}

		// Tell the root object he will lose this subobject
		RootObjectFactory->SubObjectDetachedFromReplication(DetachContext, SubObjectDetachContext);

		// Now ask the subobject's factory to handle the detachment
		if (UNetObjectFactory* SubObjectFactory = GetNetFactory(SubObjectData.NetFactoryId))
		{
			SubObjectFactory->DetachedFromReplication(DetachContext, SubObjectDetachContext);
		}

		UnregisterInstance(SubObjectInternalIndex);
	}

	// Now that subobjects have been detached, handle the root object itself

	if (OnRootObjectDetached.IsBound())
	{
		OnRootObjectDetached.Broadcast(RootObjectHandle, RootObjectInstance, DetachReason);
	}
	
	const UNetObjectFactory::FDetachContext DetachContext
	{
		.DetachedInstance = RootObjectInstance,
		.DetachedObjectHandle = RootObjectHandle,
		.Reason = DetachReason,
	};

	RootObjectFactory->DetachedFromReplication(DetachContext, {});

	UnregisterInstance(DetachedInternalIndex);
}

void UObjectReplicationBridge::DetachSubObjectFromRemote(FNetRefHandle DetachedSubObjectHandle, UE::Net::EDetachReason DetachReason, UE::Net::FNetObjectFactoryId NetFactoryId)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("%hs: %s Reason: %s"), __FUNCTION__, *PrintObjectFromNetRefHandle(DetachedSubObjectHandle), LexToString(DetachReason));

	if (!FNetObjectFactoryRegistry::IsValidFactoryId(NetFactoryId))
	{
		ensureMsgf(false, TEXT("Received invalid netfactory: %u for %s"), NetFactoryId,  *PrintObjectFromNetRefHandle(DetachedSubObjectHandle));
		return;
	}

	UNetObjectFactory* SubObjectFactory = GetNetFactory(NetFactoryId);

	const FInternalNetRefIndex SubObjectInternalIndex = NetRefHandleManager->GetInternalIndex(DetachedSubObjectHandle);

	const FNetRefHandleManager::FReplicatedObjectData& DetachedSubObjectData = NetRefHandleManager->GetReplicatedObjectData(SubObjectInternalIndex);

	if (!DetachedSubObjectData.IsSubObject())
	{
		ensureMsgf(false, TEXT("Detached obbject %s is not a subobject."), *PrintObjectFromNetRefHandle(DetachedSubObjectHandle));
		return;
	}

	// The instance may be null even if the object was still replicating. This mostly happens when the root object decides to stop replication before the server requested it.
	UObject* DetachedSubObjectInstance = GetObjectFromReferenceHandle(DetachedSubObjectHandle);
	
	const FInternalNetRefIndex RootObjectIndex = DetachedSubObjectData.SubObjectRootIndex;
	const FNetRefHandle RootObjectHandle = NetRefHandleManager->GetNetRefHandleFromInternalIndex(RootObjectIndex);
	UObject* RootObjectInstance = GetObjectFromReferenceHandle(RootObjectHandle);
	const FNetRefHandleManager::FReplicatedObjectData& RootObjectData = NetRefHandleManager->GetReplicatedObjectData(RootObjectIndex);

	if (OnSubObjectDetached.IsBound())
	{
		OnSubObjectDetached.Broadcast(DetachedSubObjectHandle, DetachedSubObjectInstance, RootObjectHandle, RootObjectInstance, DetachReason);
	}

	const UNetObjectFactory::FDetachContext DetachContext
	{
		.DetachedInstance = DetachedSubObjectInstance,
		.DetachedObjectHandle = DetachedSubObjectHandle,
		.Reason = DetachReason,
	};

	const UNetObjectFactory::FSubObjectDetachContext SubObjectDetachContext
	{
		.RootObject = RootObjectInstance,
		.RootObjectHandle = RootObjectHandle,
		.RootObjectDetachReason = EDetachReason::RootObjectStillRelevant,
	};

	// Start by telling the RootObject that one of it's subobject is no longer replicated.
	if (UNetObjectFactory* RootObjectFactory = GetNetFactory(RootObjectData.NetFactoryId))
	{
		RootObjectFactory->SubObjectDetachedFromReplication(DetachContext, SubObjectDetachContext);
	}

	SubObjectFactory->DetachedFromReplication(DetachContext, SubObjectDetachContext);

	UnregisterInstance(SubObjectInternalIndex);
}

void UObjectReplicationBridge::RegisterRemoteInstance(FNetRefHandle RefHandle, UObject* Instance, const UE::Net::FReplicationProtocol* Protocol, UE::Net::FReplicationInstanceProtocol* InstanceProtocol, uint32 ConnectionId)
{
	// Attach the instance protocol and instance to the handle
	constexpr bool bBindInstanceProtocol = false;
	InternalAttachInstanceToNetRefHandle(RefHandle, bBindInstanceProtocol, InstanceProtocol, Instance, FNetHandle());

	UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("RegisterRemoteInstance %s %s with ProtocolId:0x%x"), *PrintObjectFromNetRefHandle(RefHandle), ToCStr(Instance->GetName()), Protocol->ProtocolIdentifier);
}

FReplicationBridgeCreateNetRefHandleResult UObjectReplicationBridge::CreateNetRefHandleFromRemote(FNetRefHandle RootObjectOfSubObject, FNetRefHandle WantedNetHandle, FReplicationBridgeSerializationContext& Context)
{
	LLM_SCOPE_BYTAG(IrisState);

	using namespace UE::Net;
	using namespace UE::Net::Private;

	check(!Context.bIsDestructionInfo);

	FNetBitStreamReader* Reader = Context.SerializationContext.GetBitStreamReader();

	const FNetObjectFactoryId FactoryId = (FNetObjectFactoryId)Reader->ReadBits(FNetObjectFactoryRegistry::GetMaxBits());
	UNetObjectFactory* Factory = GetNetFactory(FactoryId);
	if (Factory == nullptr)
	{
		// There must be a valid factory. We're reading a creation header so a bitstream error seems likely and we should disconnect.
		Context.SerializationContext.SetError(GNetError_InvalidValue);
		return FReplicationBridgeCreateNetRefHandleResult();
	}

	TUniquePtr<FNetObjectCreationHeader> Header = Factory->ReadHeader(WantedNetHandle, Context.SerializationContext);
		
	if (!Header.IsValid() || Context.SerializationContext.HasErrorOrOverflow())
	{
		return FReplicationBridgeCreateNetRefHandleResult();
	}

	const FReplicationProtocolIdentifier ReceivedProtocolId = Header->GetProtocolId();

	// Tag the error context with the protocol id in case of errors
	Context.SerializationContext.SetErrorProtocolId(ReceivedProtocolId);

	if (UE_LOG_ACTIVE(LogIris, Verbose))
	{
		if (RootObjectOfSubObject.IsValid())
		{
			UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("CreateNetRefHandleFromRemote: SubObject: %s of RootObject: %s using header: %s"), *WantedNetHandle.ToString(), *RootObjectOfSubObject.ToString(), ToCStr(Header->ToString()));
		}
		else
		{
			UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("CreateNetRefHandleFromRemote: RootObject: %s using header: %s"), *WantedNetHandle.ToString(), ToCStr(Header->ToString()));
		}
	}
	
	// Currently we need to always instantiate remote objects, moving forward we want to make this optional so that can be deferred until it is time to apply received state data.
	// https://jira.it.epicgames.com/browse/UE-127369	

	UObject* InstancePtr = nullptr;
	
	// Instantiate the replicated object
	const UNetObjectFactory::FInstantiateContext InstantiateContext(WantedNetHandle, Context.SerializationContext.GetInternalContext()->ResolveContext, RootObjectOfSubObject);
	
	UNetObjectFactory::FInstantiateResult Result;

	if (GetReplicationSystem()->GetProxyType() == UE::Net::EProxyType::Backend)
	{
		const FNetRefHandle CompleteLookupNetHandle = NetRefHandleManager->MakeNetRefHandle(WantedNetHandle.GetId(), GetReplicationSystem()->GetId());
		if (UObject* ExistingObject = GetObjectReferenceCache()->GetObjectFromReferenceHandle(CompleteLookupNetHandle))
		{
			UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("CreateNetRefHandleFromRemote: Reusing existing object found in the object reference cache: Object: %s NetHandle: %s"), *GetNameSafe(ExistingObject), *CompleteLookupNetHandle.ToString());

			Result = { .Instance = ExistingObject };
		}
	}
	
	if (!Result.Instance)
	{
		Result = Factory->InstantiateReplicatedObjectFromHeader(InstantiateContext, Header.Get());
	}

	// Failed ?
	if (!Result.Instance)
	{
		if (UE_LOG_ACTIVE(LogIris, Warning) && !bSuppressCreateInstanceFailedEnsure)
		{
			if (RootObjectOfSubObject.IsValid())
			{
				UE_LOG_BRIDGEID(LogIris, Warning, TEXT("CreateNetRefHandleFromRemote: Failed to instantiate SubObject NetHandle: %s of %s using header: %s"), *WantedNetHandle.ToString(), *PrintObjectFromNetRefHandle(RootObjectOfSubObject), ToCStr(Header->ToString()));
			}
			else
			{
				UE_LOG_BRIDGEID(LogIris, Warning, TEXT("CreateNetRefHandleFromRemote: Failed to instantiate RootObject NetHandle: %s using header: %s"), *WantedNetHandle.ToString(), ToCStr(Header->ToString()));
			}
		}

		if (Result.FailureDiagnosticMessage.IsEmpty())
		{
			Context.SerializationContext.SetErrorDiagnostic(FString::Printf(TEXT("Factory %s failed to instantiate. No diagnostic message returned."), *Factory->GetName()));
		}
		else
		{
			Context.SerializationContext.SetErrorDiagnostic(FString::Printf(TEXT("Factory %s failed to instantiate. Error: %s"), *Factory->GetName(), *Result.FailureDiagnosticMessage));
		}
		
		ensureMsgf(bSuppressCreateInstanceFailedEnsure, TEXT("Failed to instantiate Handle: %s"), *WantedNetHandle.ToString());
		return FReplicationBridgeCreateNetRefHandleResult();
	}

	InstancePtr = Result.Instance;

	// Currently remote objects can only receive replicated data
	FFragmentRegistrationContext FragmentRegistrationContext(GetReplicationStateDescriptorRegistry(), GetReplicationSystem(), EReplicationFragmentTraits::CanReceive, InstancePtr);

	// Set the optional template provided by the Factory
	FFragmentRegistrationContextPrivateAccessor::SetTemplate(FragmentRegistrationContext, Result.Template);

	// Register all fragments
	CallRegisterReplicationFragments(InstancePtr, FragmentRegistrationContext, EFragmentRegistrationFlags::None);

	// Find the final template assigned to the instance during fragment creation
	const UObject* TemplateKey = FFragmentRegistrationContextPrivateAccessor::GetDefaultStateSource(FragmentRegistrationContext);

	// If no templates were registered
	if (!TemplateKey)
	{
		// Use the template provided by the factory or the object's archetype
		TemplateKey = Result.Template ? Result.Template : InstancePtr->GetArchetype();
	}

	FReplicationProtocolManager* ProtocolManager = GetReplicationProtocolManager();
	const FReplicationFragments& RegisteredFragments = FFragmentRegistrationContextPrivateAccessor::GetReplicationFragments(FragmentRegistrationContext);

	// Create Protocol instance
	FReplicationInstanceProtocolPtr InstanceProtocol(ProtocolManager->CreateInstanceProtocol(RegisteredFragments, FragmentRegistrationContext.GetFragmentTraits()));

	// See if the protocol is known
	const FReplicationProtocol* ReplicationProtocol = ProtocolManager->GetReplicationProtocol(ReceivedProtocolId, TemplateKey);
	if (!ReplicationProtocol)
	{
		// Make sure the local protocol hash matches the one sent by the server
		bool bValidateProtocol = true;

		const FString ClassName = InstancePtr->GetClass()->GetName();

		if (ClassNamesToIgnoreProtocolMismatch.Num())
		{
			for (const FString& ClassNameToIgnore : ClassNamesToIgnoreProtocolMismatch)
			{
				if (ClassName.Contains(ClassNameToIgnore))
				{
					// This class wants to keep working even if a protocol mismatch is detected.
					bValidateProtocol = false;
					break;
				}
			}
		}


		// Create the protocol and compare the received hash ID to make sure it is identical to the one on the server
		const FCreateReplicationProtocolParameters CreateProtocolParams 
		{
			.TemplateKey = TemplateKey, 
			.bValidateProtocolId = bValidateProtocol,
		};
		ReplicationProtocol = ProtocolManager->CreateReplicationProtocol(ReceivedProtocolId, RegisteredFragments, *ClassName, CreateProtocolParams);
	}
	else
	{
		// Don't log errors because it would spam for every individual object of the same class. 
		constexpr bool bDoNotLogErrors = false;

		// Confirm that the found protocol works with the fragments
		const bool bIsValid = FReplicationProtocolManager::ValidateReplicationProtocol(ReplicationProtocol, RegisteredFragments, bDoNotLogErrors);
		if (!bIsValid)
		{
			ReplicationProtocol = nullptr;
		}
	}

	if (!ReplicationProtocol)
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Protocol mismatch prevents binding %s to instanced object %s (Template: %s)."), *WantedNetHandle.ToString(), *GetNameSafe(InstancePtr), *GetPathNameSafe(TemplateKey));

		TArray<uint8> CDOStateBytes;
		bool bLogCDOState = !bSendCDOStateOnProtocolMismatch;

		if (UE_LOG_ACTIVE(LogIris, Error))
		{
			if (bSendCDOStateOnProtocolMismatch)
			{
				UE::Net::Private::FDebugCDOState CDOState;
				UE::Net::Private::ObjectBridgeDebugging::RemoteProtocolMismatchDetected(ArchetypesAlreadyPrinted, ReplicationSystem, Context.ConnectionId, RegisteredFragments, TemplateKey, InstancePtr, &CDOState);

				if (CDOState.Fragments.Num() > 0)
				{
					CDOStateBytes = FReplicationInstanceOperations::SerializeInternalDefaultState(CDOState);
					if (CDOStateBytes.Num() > MaxCDOStateBytesOnProtocolMismatch)
					{
						UE_LOGF(LogIris, Warning, "Client CDO state for %ls is %d bytes, exceeds %d byte limit. Logging state locally and sending plain protocol mismatch instead.",
							*GetPathNameSafe(TemplateKey), CDOStateBytes.Num(), MaxCDOStateBytesOnProtocolMismatch);
						CDOStateBytes.Reset();
						bLogCDOState = true;
					}
				}
			}

			if (bLogCDOState)
			{
				if (!bSendCDOStateOnProtocolMismatch)
				{
					// Call without OutCDOState. Will log CDO if archetype hasn't printed yet.
					UE::Net::Private::ObjectBridgeDebugging::RemoteProtocolMismatchDetected(ArchetypesAlreadyPrinted, ReplicationSystem, Context.ConnectionId, RegisteredFragments, TemplateKey, InstancePtr);
				}
				else
				{
					// We already called RemoteProtocolMismatchDetected. If we're in this branch, the archetype hadn't been printed yet, so we should print it here.
					TStringBuilder<4096> StringBuilder;
					FReplicationInstanceOperations::OutputInternalDefaultStateMemberHashesToString(ReplicationSystem, StringBuilder, RegisteredFragments);
					UE_LOGF(LogIris, Error, "Printing replication state of CDO %ls used for %ls:\n%ls",
						*GetNameSafe(TemplateKey), *GetNameSafe(InstancePtr), StringBuilder.ToString());
				}
			}
		}

		FIrisDelegates::GetCriticalErrorDetectedDelegate().Broadcast(ReplicationSystem);

		OnProtocolMismatchDetected(WantedNetHandle, CDOStateBytes);
		return FReplicationBridgeCreateNetRefHandleResult();
	}
	
	// Create NetHandle
	const FCreateNetObjectParams CreateNetObjectParams
	{
		.NetFactoryId = FactoryId,
		.ReplicationProtocol = ReplicationProtocol,
	};

	const FNetRefHandle CreatedNetRefHandle = InternalCreateNetObjectFromRemote(WantedNetHandle, CreateNetObjectParams);

	if (!CreatedNetRefHandle.IsValid())
	{
		return FReplicationBridgeCreateNetRefHandleResult();
	}

	UE_CLOGF_IRISCREATIONFLOW(!RootObjectOfSubObject.IsValid(), UE::Net::CreationFlowLog::IsTracedClass(InstancePtr->GetClass()), Verbose, "[CreateRef] Handle=%ls Class=%ls", *CreatedNetRefHandle.ToString(), *GetNameSafe(InstancePtr->GetClass()));

	// Finalize the creation process
	const FReplicationBridgeCreateNetRefHandleResult CreateResult
	{
		.NetRefHandle = CreatedNetRefHandle,
		.Flags = Result.Flags,
	};

	RegisterRemoteInstance(CreateResult.NetRefHandle, InstancePtr, ReplicationProtocol, InstanceProtocol.Get(), Context.ConnectionId);

	// Release instance protocol from the uniquePtr as it is now successfully bound to the handle
	(void)InstanceProtocol.Release();

	// Dynamic objects or static objects that request it need to update the object cache
	if (CreateResult.NetRefHandle.IsDynamic() || EnumHasAnyFlags(CreateResult.Flags, EReplicationBridgeCreateNetRefHandleResultFlags::BindStaticObjectInReferenceCache))
	{
		GetObjectReferenceCache()->BindRemoteReference(CreateResult.NetRefHandle, InstancePtr);
	}

	// Trigger the PostInstantiation callback
	{
		const UNetObjectFactory::FPostInstantiationContext PostContext
		{
			.Instance = InstancePtr,
			.Header = Header.Get(),
			.ConnectionId = Context.ConnectionId
		};
		Factory->PostInstantiation(PostContext);
	}

	// If it's a subobjects add the new handle to the rootobject list
	if (RootObjectOfSubObject.IsValid())
	{
		NetRefHandleManager->AddSubObject(RootObjectOfSubObject, CreateResult.NetRefHandle);
	}

	return CreateResult;
}

void UObjectReplicationBridge::SubObjectCreatedFromReplication(UE::Net::FInternalNetRefIndex RootObjectIndex, FNetRefHandle SubObjectCreated)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FNetObjectFactoryId FactoryId = NetRefHandleManager->GetReplicatedObjectDataNoCheck(RootObjectIndex).NetFactoryId;
	UNetObjectFactory* Factory = GetNetFactory(FactoryId);
	check(Factory);
	Factory->SubObjectCreatedFromReplication(NetRefHandleManager->GetNetRefHandleFromInternalIndex(RootObjectIndex), SubObjectCreated);
}

void UObjectReplicationBridge::PostApplyInitialState(UE::Net::FInternalNetRefIndex InternalObjectIndex)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);
	const FNetObjectFactoryId FactoryId = ObjectData.NetFactoryId;
	UNetObjectFactory* Factory = GetNetFactory(FactoryId);
	check(Factory);
	const UNetObjectFactory::FPostInitContext Context 
	{ 
		.Instance = NetRefHandleManager->GetReplicatedObjectInstance(InternalObjectIndex),
		.Handle = NetRefHandleManager->GetNetRefHandleFromInternalIndex(InternalObjectIndex)
	};
	Factory->PostInit(Context);

	UE_CLOGF_IRISCREATIONFLOW(ObjectData.IsRootObject() && Context.Instance != nullptr, UE::Net::CreationFlowLog::IsTracedClass(Context.Instance->GetClass()), Verbose, "[PostInit] Handle=%ls Class=%ls Instance=%ls", *Context.Handle.ToString(), *GetNameSafe(Context.Instance->GetClass()), *GetNameSafe(Context.Instance));

	if (ObjectData.IsRootObject())
	{
		if (OnRootObjectPostInit.IsBound())
		{
			OnRootObjectPostInit.Broadcast(Context.Handle, Context.Instance, FactoryId);
		}
	}
	else
	{
		if (OnSubObjectPostInit.IsBound())
		{
			const FNetRefHandle RootObjectHandle = NetRefHandleManager->GetNetRefHandleFromInternalIndex(ObjectData.SubObjectRootIndex);
			UObject* RootObject = NetRefHandleManager->GetReplicatedObjectInstance(ObjectData.SubObjectRootIndex);
			OnSubObjectPostInit.Broadcast(Context.Handle, Context.Instance, RootObjectHandle, RootObject, FactoryId);
		}
	}
}

void UObjectReplicationBridge::PreSendUpdateSingleHandle(FNetRefHandle RefHandle)
{
	ForcePollObject(RefHandle);
}

void UObjectReplicationBridge::OnStartPreSendUpdate()
{
	// During SendUpdate it is not supported to start replication of new root objects.
	bBlockStartRootObjectReplication = true;
}

void UObjectReplicationBridge::PreSendUpdate()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager& LocalNetRefHandleManager = GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();

	{
		const uint32 InitialNumBits = LocalNetRefHandleManager.GetPolledObjectsInternalIndices().GetNumBits();
		
		FNetBitArray InitialPollList(InitialNumBits);
		FNetBitArrayView InitialPollListView(MakeNetBitArrayView(InitialPollList));
		BuildPollList(InitialPollListView);

		PreUpdate(InitialPollListView);
		
		// PreUpdate is allowed to generate new objects, so the netrefhandlemanager bitarrays may have grown.
		// Limit the size of the NetHandleManager list to the pre-grown array.
		FNetBitArrayView CachedPollList(LocalNetRefHandleManager.GetPolledObjectsInternalIndices().GetData(), InitialNumBits);

		CachedPollList.Copy(InitialPollList);
	}

	FinalizeDirtyObjects();

	{
		FNetBitArrayView ObjectsConsideredForPolling = LocalNetRefHandleManager.GetPolledObjectsInternalIndices();
		
		ReconcileNewSubObjects(ObjectsConsideredForPolling);

		PollAndCopy(ObjectsConsideredForPolling);
	}
}

void UObjectReplicationBridge::OnPostSendUpdate()
{
	bBlockStartRootObjectReplication = false;
}

void UObjectReplicationBridge::OnPostReceiveUpdate()
{
	for (UNetObjectFactory* NetFactory : NetObjectFactories)
	{
		NetFactory->PostReceiveUpdate();
	}
}

void UObjectReplicationBridge::PruneStaleObjects()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	IRIS_PROFILER_SCOPE(UObjectReplicationBridge_PruneStaleObjects);

	// Mark all objects with object references as potentially affected by GC
	GarbageCollectionAffectedObjects = ObjectsWithObjectReferences;

	FNetRefHandleManager& LocalNetRefHandleManager = GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();
	const TNetChunkedArray<TObjectPtr<UObject>>& ReplicatedInstances = LocalNetRefHandleManager.GetReplicatedInstances();

	TArray<FNetRefHandle> StaleObjects;

	// Detect stale references and try to kill/report them
	auto DetectStaleObjectsFunc = [&LocalNetRefHandleManager, &StaleObjects, &ReplicatedInstances, this](uint32 InternalNetHandleIndex)
	{
		if (!IsValid(ReplicatedInstances[InternalNetHandleIndex]))
		{
			const FNetRefHandleManager::FReplicatedObjectData& ObjectData = LocalNetRefHandleManager.GetReplicatedObjectDataNoCheck(InternalNetHandleIndex);
			if (ObjectData.InstanceProtocol)
			{
				const FReplicationProtocol* Protocol = ObjectData.Protocol;
				const FNetDebugName* DebugName = Protocol ? Protocol->DebugName : nullptr;

				// Only log warning if pruned object is not marked as async stopping.
				UE_CLOGF(!ObjectData.bIsAsyncStopping, LogIris, Warning, "UObjectReplicationBridge::PruneStaleObjects ObjectInstance replicated as: %ls of Type named:%ls has been destroyed without notifying the ReplicationSystem %u", *ObjectData.RefHandle.ToString(), ToCStr(DebugName), ObjectData.RefHandle.GetReplicationSystemId());						

				// If instance protocol is bound, we detach and destroy instance protocol without touching the instance
				if (EnumHasAnyFlags(ObjectData.InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::IsBound))
				{
					// Ensure, to bring extra attention if we have missed to call StopReplication on bound object.
					ensureMsgf(ObjectData.bIsAsyncStopping, TEXT("UObjectReplicationBridge::PruneStaleObjects Bound ObjectInstance replicated as: %s has been destroyed without notifying the ReplicationSystem."), *ObjectData.RefHandle.ToString());

					// Destroy instance protocol without modifying it to avoid potentially touching bad objects.
					const bool bUnbindInstanceProtocol = false;
					InternalDetachInstanceFromNetRefHandle(InternalNetHandleIndex, bUnbindInstanceProtocol);
				}

				StaleObjects.Push(ObjectData.RefHandle);
			}
		}
	};

	// Iterate over assigned indices and detect if any of the replicated instances has been garbagecollected (excluding DestroyedStartupObjectInternalIndices) as they never have an instance
	FNetBitArrayView::ForAllSetBits(LocalNetRefHandleManager.GetAssignedInternalIndicesView(), LocalNetRefHandleManager.GetDestroyedStartupObjectInternalIndices(), FNetBitArrayView::AndNotOp, DetectStaleObjectsFunc);

	// EndReplication/detach stale instances
	for (FNetRefHandle Handle : MakeArrayView(StaleObjects.GetData(), StaleObjects.Num()))
	{
		StopReplicatingNetRefHandle(Handle, EEndReplicationFlags::Destroy);
	}

	// Mark poll override info as dirty
	bHasDirtyClassesInPollPeriodOverrides = ClassHierarchyPollPeriodOverrides.Num() > 0;
}

void UObjectReplicationBridge::SetInstancePreUpdateFunction(FInstancePreUpdateFunction InPreUpdateFunction)
{
	PreUpdateInstanceFunction = InPreUpdateFunction;
}

UObjectReplicationBridge::FInstancePreUpdateFunction UObjectReplicationBridge::GetInstancePreUpdateFunction() const
{
	return PreUpdateInstanceFunction;
}

void UObjectReplicationBridge::ForcePollObject(FNetRefHandle Handle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (!Handle.IsValid())
	{
		return;
	}

	IRIS_PROFILER_SCOPE(UObjectReplicationBridge_ForcePollAndCopyObject);

	const FNetRefHandleManager& LocalNetRefHandleManager = GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();

	const FInternalNetRefIndex ObjectIndex = LocalNetRefHandleManager.GetInternalIndex(Handle);
	if (ObjectIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return;
	}

	if (PreUpdateInstanceFunction)
	{
		UObject* Instance = LocalNetRefHandleManager.GetReplicatedInstances()[ObjectIndex];
		if (Instance && LocalNetRefHandleManager.GetObjectsWithPreUpdate().GetBit(ObjectIndex))
		{
			PreUpdateInstanceFunction(MakeArrayView<UObject*>(&Instance, 1U), this);
		}
	}

	FObjectPoller::FInitParams PollerInitParams;
	PollerInitParams.ObjectReplicationBridge = this;
	PollerInitParams.ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	FObjectPoller Poller(PollerInitParams);

	Poller.PollAndCopySingleObject(ObjectIndex);
}

void UObjectReplicationBridge::BuildPollList(UE::Net::FNetBitArrayView ObjectsConsideredForPolling)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	IRIS_PROFILER_SCOPE(UObjectReplicationBridge_BuildPollList);

	// Update every relevant objects from here
	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	FDirtyNetObjectTracker& DirtyNetObjectTracker = ReplicationSystemInternal->GetDirtyNetObjectTracker();

	const FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();
	const FNetBitArrayView RelevantObjects = LocalNetRefHandleManager.GetRelevantObjectsInternalIndices();
	const FNetBitArrayView WantToBeDormantObjects = LocalNetRefHandleManager.GetWantToBeDormantInternalIndices();

	if (bUseFrequencyBasedPolling)
	{
		if (bEnableForceNetUpdate)
		{
			// Find objects ready to be polled and add objects that called ForceNetupdate
			const FNetBitArrayView ForceNetUpdateObjects = DirtyNetObjectTracker.GetForceNetUpdateObjects();
			PollFrequencyLimiter->Update(RelevantObjects, ForceNetUpdateObjects, ObjectsConsideredForPolling);
		}
		else
		{
			// Find objects ready to be polled and add objects that were flagged Dirty.
			const FNetBitArrayView AccumulatedDirtyObjects = DirtyNetObjectTracker.GetAccumulatedDirtyNetObjects();
			PollFrequencyLimiter->Update(RelevantObjects, AccumulatedDirtyObjects, ObjectsConsideredForPolling);
		}
	}
	else
	{
		ObjectsConsideredForPolling.Copy(RelevantObjects);
	}

	// Mask off objects pending dormancy as we do not want to poll/pre-update them unless they are marked for flush
	if (bUseDormancyToFilterPolling)
	{
		IRIS_PROFILER_SCOPE(BuildPollList_Dormancy);

		// Mask off all dormant objects
		ObjectsConsideredForPolling.Combine(WantToBeDormantObjects, FNetBitArrayView::AndNotOp);

		// Force a poll on objects that requested a FlushNet
		FNetBitArrayView DormantObjectsPendingFlushNet = LocalNetRefHandleManager.GetDormantObjectsPendingFlushNet();
		FNetBitArrayView ForceNetUpdateObjects = ReplicationSystemInternal->GetDirtyNetObjectTracker().GetForceNetUpdateObjects();
		
		uint32 FlushNetCount = 0;		
		auto FlushNetDormancyForRelevantObjects = [&](uint32 DormantObjectIndex)
		{
			// Poll the dormant object this frame and treat it as a forcenetupdate in order to also schedule subobjects correctly
			ObjectsConsideredForPolling.SetBit(DormantObjectIndex);
			ForceNetUpdateObjects.SetBit(DormantObjectIndex);
			++FlushNetCount;
		};

		FNetBitArrayView::ForAllSetBits(DormantObjectsPendingFlushNet, RelevantObjects, FNetBitArrayView::AndOp, FlushNetDormancyForRelevantObjects);
		
		// Remove FlushNetDormancy request for the objects that were relevant
		DormantObjectsPendingFlushNet.Combine(RelevantObjects, FNetBitArrayView::AndNotOp);

		UE_NET_TRACE_FRAME_STATSCOUNTER(ReplicationSystem->GetId(), FlushDormancyObjectCount, FlushNetCount, ENetTraceVerbosity::Trace);
	}
	

	/**
	* Make sure to propagate polling for owners to subobjects and vice versa. If an actor is not due to update due to
	* polling frequency it can still be force net update or a dormant object marked for flush and polled for that reason. In order to make sure all recent state updates
	* are replicated atomically this polling propagation is required.
	*/
	{
		auto PropagateSubObjectDirtinessToOwner = [&LocalNetRefHandleManager, &ObjectsConsideredForPolling](uint32 InternalObjectIndex)
		{
			const FNetRefHandleManager::FReplicatedObjectData& ObjectData = LocalNetRefHandleManager.GetReplicatedObjectDataNoCheck(InternalObjectIndex);
			ObjectsConsideredForPolling.SetBit(ObjectData.SubObjectRootIndex);
		};

		auto PropagateOwnerDirtinessToSubObjects = [&LocalNetRefHandleManager, &ObjectsConsideredForPolling](uint32 InternalObjectIndex)
		{
			const FNetRefHandleManager::FReplicatedObjectData& ObjectData = LocalNetRefHandleManager.GetReplicatedObjectDataNoCheck(InternalObjectIndex);
			for (const FInternalNetRefIndex SubObjectInternalIndex : LocalNetRefHandleManager.GetSubObjects(InternalObjectIndex))
			{
				ObjectsConsideredForPolling.SetBit(SubObjectInternalIndex);
			}
		};

		IRIS_PROFILER_SCOPE(BuildPollList_PropagatePolling);

		// Update subobjects' owner first and owners' subobjects second. It's the only way to properly mark all groups of objects in two passes.
		const FNetBitArrayView SubObjects = MakeNetBitArrayView(LocalNetRefHandleManager.GetSubObjectInternalIndices());
		const FNetBitArrayView ForceNetUpdateObjects = DirtyNetObjectTracker.GetForceNetUpdateObjects();

		if (bEnableForceNetUpdate)
		{
			// Make a list of objects which forced an update and are also relevant
			FNetBitArray ForceNetUpdateAndRelevantObjects(RelevantObjects.GetNumBits(), FNetBitArray::NoResetNoValidate);
			FNetBitArrayView ForceNetUpdateAndRelevantObjectsView = MakeNetBitArrayView(ForceNetUpdateAndRelevantObjects, FNetBitArrayView::NoResetNoValidate);
			
			ForceNetUpdateAndRelevantObjectsView.Set(RelevantObjects, FNetBitArray::AndOp, ForceNetUpdateObjects);

			FNetBitArrayView::ForAllSetBits(ForceNetUpdateAndRelevantObjectsView, SubObjects, FNetBitArray::AndOp, PropagateSubObjectDirtinessToOwner);
		}
		else
		{
			// Make the list of objects which are dirty or forced an update, and are also relevant
			FNetBitArray DirtyAndRelevantObjects(RelevantObjects.GetNumBits(), FNetBitArray::NoResetNoValidate);
			FNetBitArrayView DirtyAndRelevantObjectsView = MakeNetBitArrayView(DirtyAndRelevantObjects, FNetBitArrayView::NoResetNoValidate);

			{
				// Use the accumulated set of dirty objects to ensure correctness for objects dirtied when not relevant to any connection.
				const FNetBitArrayView AccumulatedDirtyNetObject = DirtyNetObjectTracker.GetAccumulatedDirtyNetObjects();
				DirtyAndRelevantObjectsView.Set(AccumulatedDirtyNetObject, FNetBitArray::OrOp, ForceNetUpdateObjects);
				DirtyAndRelevantObjectsView.Combine(RelevantObjects, FNetBitArray::AndOp);
			}

			FNetBitArrayView::ForAllSetBits(DirtyAndRelevantObjectsView, SubObjects, FNetBitArray::AndOp, PropagateSubObjectDirtinessToOwner);
		}
			
		// If an object with dependents is about to be polled, force it's dependents to poll at the same time.
		{
			IRIS_PROFILER_SCOPE(BuildPollList_PatchDependentObjects);

			FNetBitArray TempObjectsConsideredForPolling;
			TempObjectsConsideredForPolling.InitAndCopy(ObjectsConsideredForPolling);
			FNetBitArrayView::ForAllSetBits(MakeNetBitArrayView(TempObjectsConsideredForPolling), NetRefHandleManager->GetObjectsWithDependentObjectsInternalIndices(), FNetBitArray::AndOp,
				[&LocalNetRefHandleManager, &ObjectsConsideredForPolling](FInternalNetRefIndex ObjectIndex) 
				{
					LocalNetRefHandleManager.ForAllDependentObjectsRecursive(ObjectIndex, [&ObjectsConsideredForPolling, ObjectIndex](FInternalNetRefIndex DependentObjectIndex)
					{ 
						ObjectsConsideredForPolling.SetBit(DependentObjectIndex); 
					});
				}
			);
		}

		// Currently we must enforce polling SubObjects with owner
		FNetBitArrayView::ForAllSetBits(ObjectsConsideredForPolling, SubObjects, FNetBitArray::AndNotOp, PropagateOwnerDirtinessToSubObjects);
	}
}
	
void UObjectReplicationBridge::PreUpdate(const UE::Net::FNetBitArrayView ObjectsConsideredForPolling)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationBridge_PreUpdate);
	IRIS_PROFILER_SCOPE_VERBOSE(PreUpdatePass);

	if (!PreUpdateInstanceFunction)
	{
		return;
	}

	const FNetRefHandleManager& LocalNetRefHandleManager = GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();

	const TNetChunkedArray<TObjectPtr<UObject>>& ReplicatedInstances = LocalNetRefHandleManager.GetReplicatedInstances();

	uint32 PreUpdatedObjectCount = 0;

	constexpr uint32 PreUpdateBatchCount = 128U;
	UObject* BatchedObjects[PreUpdateBatchCount];
	uint32 BatchedObjectCount = 0U;

	auto BatchedPreUpdate = [&](FInternalNetRefIndex Objectindex)
	{
		// Flush if needed
		if (BatchedObjectCount == PreUpdateBatchCount)
		{
			PreUpdateInstanceFunction(MakeArrayView<UObject*>(BatchedObjects, BatchedObjectCount), this);
			PreUpdatedObjectCount += BatchedObjectCount;
			BatchedObjectCount = 0U;
		}

		UObject* Instance = ReplicatedInstances[Objectindex];
		BatchedObjects[BatchedObjectCount] = Instance;
		BatchedObjectCount += Instance ? 1U : 0U;
	};

	// Make a copy of the list we'll iterate on since the PreUpdate callbacks can create new objects and realloc the NetRefHandleManager bitarrays
	FNetBitArray ObjectsWithPreUpdate;
	ObjectsWithPreUpdate.InitAndCopy(LocalNetRefHandleManager.GetObjectsWithPreUpdate());

	FNetBitArrayView::ForAllSetBits(ObjectsConsideredForPolling, MakeNetBitArrayView(ObjectsWithPreUpdate), FNetBitArrayView::AndOp, BatchedPreUpdate);

	// Flush last batch
	if (BatchedObjectCount > 0)
	{
		PreUpdateInstanceFunction(MakeArrayView(BatchedObjects, BatchedObjectCount), this);
		PreUpdatedObjectCount += BatchedObjectCount;
	}

	UE_NET_TRACE_FRAME_STATSCOUNTER(ReplicationSystem->GetId(), ReplicationSystem.PreUpdatedObjectCount, PreUpdatedObjectCount, ENetTraceVerbosity::Trace);
}

void UObjectReplicationBridge::PollAndCopy(const UE::Net::FNetBitArrayView ObjectsConsideredForPolling)
{
	using namespace UE::Net::Private;

	IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationBridge_PollAndCopy);

	FObjectPoller::FInitParams PollerInitParams;
	PollerInitParams.ObjectReplicationBridge = this;
	PollerInitParams.ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();

	FObjectPoller Poller(PollerInitParams);
	Poller.PollAndCopyObjects(ObjectsConsideredForPolling);
	
	FObjectPoller::FPreUpdateAndPollStats Stats = Poller.GetPollStats();

	// Report stats
	UE_NET_TRACE_FRAME_STATSCOUNTER(ReplicationSystem->GetId(), ReplicationSystem.PolledObjectCount, Stats.PolledObjectCount, ENetTraceVerbosity::Trace);
	UE_NET_TRACE_FRAME_STATSCOUNTER(ReplicationSystem->GetId(), ReplicationSystem.PolledReferencesObjectCount, Stats.PolledReferencesObjectCount, ENetTraceVerbosity::Trace);
}

void UObjectReplicationBridge::FinalizeDirtyObjects()
{
	using namespace UE::Net::Private;

	IRIS_PROFILER_SCOPE(UObjectReplicationBridge_FinalizeDirtyObjects);

	// Look for new dirty pushmodel objects and then prevent future modifications to it.
	GetReplicationSystem()->GetReplicationSystemInternal()->GetDirtyNetObjectTracker().UpdateAndLockDirtyNetObjects();
}

void UObjectReplicationBridge::ReconcileNewSubObjects(UE::Net::FNetBitArrayView ObjectsConsideredForPolling)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	IRIS_PROFILER_SCOPE(UObjectReplicationBridge_ReconcileNewSubObjects);

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	const FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();

	const FNetBitArrayView SubObjectList = LocalNetRefHandleManager.GetSubObjectInternalIndicesView();
	FReplicationConnections& Connections = ReplicationSystemInternal->GetConnections();
	FReplicationFiltering& Filtering = ReplicationSystemInternal->GetFiltering();

	auto HandleNewSubObject = [&](FInternalNetRefIndex SubObjectIndex)
	{
		const bool bIsSubObject = SubObjectList.IsBitSet(SubObjectIndex);
		checkf(bIsSubObject, TEXT("Found a root object %s (Index:%u) that was created after the start of PreSendUpdate(). This is not supported"), *GetNameSafe(LocalNetRefHandleManager.GetReplicatedObjectInstance(SubObjectIndex)), SubObjectIndex);
		if (UNLIKELY(!bIsSubObject))
		{
			return;
		}

		const FInternalNetRefIndex RootObjectIndex = LocalNetRefHandleManager.GetRootObjectInternalIndexOfSubObject(SubObjectIndex);
		if (UNLIKELY(RootObjectIndex == InvalidInternalNetRefIndex))
		{
			ensureMsgf(RootObjectIndex != InvalidInternalNetRefIndex, TEXT("SubObject %s (Index:%u) had invalid RootObjectIndex"), *GetNameSafe(LocalNetRefHandleManager.GetReplicatedObjectInstance(SubObjectIndex)), SubObjectIndex);
			return;
		}

		// Add the new subobject to the Poll list
		ObjectsConsideredForPolling.SetBit(SubObjectIndex);

		// Iterate over all connections and add the subobject if the root object is relevant to the connection
		auto UpdateConnectionScope = [&Filtering, &Connections, RootObjectIndex, SubObjectIndex](uint32 ConnectionId)
		{
			FReplicationConnection* Conn = Connections.GetConnection(ConnectionId);
			FNetBitArrayView ObjectsInScope = Filtering.GetRelevantObjectsInScope(ConnectionId);

			if (ObjectsInScope.IsBitSet(RootObjectIndex))
			{
				ObjectsInScope.SetBit(SubObjectIndex);
			}
		};

		const FNetBitArray& ValidConnections = Connections.GetValidConnections();
		ValidConnections.ForAllSetBits(UpdateConnectionScope);
	};

	// Find any objects that got added since the start of the PreSendUpdate
	const FNetBitArrayView GlobalScopeList = LocalNetRefHandleManager.GetGlobalScopableInternalIndices();
	const FNetBitArrayView CurrentFrameScopeList = LocalNetRefHandleManager.GetCurrentFrameScopableInternalIndices();
	FNetBitArrayView::ForAllSetBits(GlobalScopeList, CurrentFrameScopeList, FNetBitArrayView::AndNotOp, HandleNewSubObject);
}

void UObjectReplicationBridge::ForceUpdateWorldLocation(FNetRefHandle NetRefHandle, UE::Net::FInternalNetRefIndex InternalObjectIndex)
{
	using namespace UE::Net;

	FWorldLocations& WorldLocations = GetReplicationSystem()->GetReplicationSystemInternal()->GetWorldLocations();

	if (!WorldLocations.HasInfoForObject(InternalObjectIndex))
	{
		ensureMsgf(false, TEXT("UObjectReplicationBridge::ForceUpdateWorldLocation was called on object that is not registered in the WorldLocations cache: %s"), *PrintObjectFromNetRefHandle(NetRefHandle));
		return;
	}

	const FUpdateWorldInfoContext UpdateContext
	{
		.WorldLocations = WorldLocations,
		.ObjectIndex = InternalObjectIndex,
		.NetRefHandle = NetRefHandle,
	};
	UpdateRootObjectWorldInfo(UpdateContext);
}

void UObjectReplicationBridge::UpdateRootObjectWorldInfo(const FUpdateWorldInfoContext& UpdateContext)
{
	using namespace UE::Net::Private;
	using FWorldInfoData = UNetObjectFactory::FWorldInfoData;

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(UpdateContext.ObjectIndex);

	// Ask the NetFactory for the object's current world info
	UNetObjectFactory* NetFactory = GetNetFactory(ObjectData.NetFactoryId);

	const UNetObjectFactory::FWorldInfoContext WorldInfoContext
	{
		.Instance = NetRefHandleManager->GetReplicatedObjectInstance(UpdateContext.ObjectIndex),
		.Handle = UpdateContext.NetRefHandle,
		.InfoRequested = UNetObjectFactory::EWorldInfoRequested::All,
	};

	TOptional<FWorldInfoData> OptWorldInfo = NetFactory->GetWorldInfo(WorldInfoContext);
	if (!OptWorldInfo)
	{
		// OK: Locally-destroyed-but-not-replicated objects no longer have location updates.
		return;
	}

	const FWorldInfoData& WorldInfo = OptWorldInfo.GetValue();

	// Check if we need to warn about invalid world locations.
	// Note that FWorldLocations will store the invalid location but only return a valid clamped position from it's API
	if (UE_LOG_ACTIVE(LogIris, Warning) && GInvalidLocationWarnings != 0)
	{
		const bool bIsValidLocation = UpdateContext.WorldLocations.IsValidLocation(WorldInfo.WorldLocation);
		
		if (!bIsValidLocation)
		{
			FObjectKey ClassOrObject;
			
			// Warn once per class type
			if (GInvalidLocationWarnings == 1)
			{
				ClassOrObject = WorldInfoContext.Instance ? WorldInfoContext.Instance->GetClass() : nullptr;
			}
			// Warn once per object
			else if (GInvalidLocationWarnings >= 2)
			{
				ClassOrObject = WorldInfoContext.Instance;
			}

			bool bAlreadyWarned(false);
			WorldLocationOOBWarnings.FindOrAdd(ClassOrObject, &bAlreadyWarned);

			// If it's the first warning or it's set to warn on every update
			if (!bAlreadyWarned || GInvalidLocationWarnings == 3)
			{
				UE_LOG_BRIDGEID(LogIris, Warning, TEXT("RootObject %s has a location %s that is out of bounds."), 
					ToCStr(NetRefHandleManager->PrintObjectFromIndex(UpdateContext.ObjectIndex)), *WorldInfo.WorldLocation.ToCompactString());
				bAlreadyWarned = true;
			}
		}
	}

	UpdateContext.WorldLocations.SetObjectInfo(UpdateContext.ObjectIndex, WorldInfo.WorldLocation, WorldInfo.CullDistance);
}

void UObjectReplicationBridge::UpdateInstancesWorldLocation()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	const FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();
	FWorldLocations& WorldLocations = ReplicationSystemInternal->GetWorldLocations();

	FUpdateWorldInfoContext UpdateContext
	{
		.WorldLocations = WorldLocations
	};

	// Retrieve the world location for instances that supports it. Only dirty objects are considered.
	auto UpdateInstanceWorldLocation = [this, &LocalNetRefHandleManager, &UpdateContext](uint32 InternalObjectIndex)
	{
		if (UpdateContext.WorldLocations.HasInfoForObject(InternalObjectIndex))
		{
			UpdateContext.ObjectIndex = InternalObjectIndex;
			UpdateContext.NetRefHandle = LocalNetRefHandleManager.GetNetRefHandleFromInternalIndex(InternalObjectIndex);
			UpdateRootObjectWorldInfo(UpdateContext);
		}
	};

	// Objects marked as dirty or that have requested frequent world location updates will be updated. Failing to do either when the location has changed will result in replication issues when using spatial filters such as the NetObjectGridFilter.
	FDirtyObjectsAccessor DirtyObjectsAccessor(ReplicationSystemInternal->GetDirtyNetObjectTracker());
	const FNetBitArrayView DirtyObjectsThisFrame = DirtyObjectsAccessor.GetDirtyNetObjects();
	const FNetBitArrayView ObjectsRequiringFrequentUpdates = WorldLocations.GetObjectsRequiringFrequentWorldLocationUpdate();
	FNetBitArrayView::ForAllSetBits(DirtyObjectsThisFrame, ObjectsRequiringFrequentUpdates, FNetBitArrayBase::OrOp, UpdateInstanceWorldLocation);
}

void UObjectReplicationBridge::SetPollWithObject(FNetRefHandle ObjectToPollWithHandle, FNetRefHandle ObjectHandle)
{
	const UE::Net::FInternalNetRefIndex PollWithInternalReplicationIndex = NetRefHandleManager->GetInternalIndex(ObjectToPollWithHandle);
	const UE::Net::FInternalNetRefIndex InternalReplicationIndex = NetRefHandleManager->GetInternalIndex(ObjectHandle);
	PollFrequencyLimiter->SetPollWithObject(PollWithInternalReplicationIndex, InternalReplicationIndex);
}

bool UObjectReplicationBridge::GetObjectWantsToBeDormant(FNetRefHandle Handle) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	const FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();

	if (const FInternalNetRefIndex InternalObjectIndex = LocalNetRefHandleManager.GetInternalIndex(Handle))
	{
		return LocalNetRefHandleManager.GetWantToBeDormantInternalIndices().GetBit(InternalObjectIndex);
	}

	return false;
}

void UObjectReplicationBridge::SetObjectWantsToBeDormant(FNetRefHandle Handle, bool bWantsToBeDormant)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager& LocalNetRefHandleManager = GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();

	const FInternalNetRefIndex InternalObjectIndex = LocalNetRefHandleManager.GetInternalIndex(Handle);
	if (InternalObjectIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("SetObjectWantsToBeDormant: %s wants to be %s "), *LocalNetRefHandleManager.PrintObjectFromIndex(InternalObjectIndex), bWantsToBeDormant ? TEXT("Dormant"):TEXT("Awake"));

	// Only RootObjects can trigger dormancy changes
	ensureMsgf(!LocalNetRefHandleManager.GetSubObjectInternalIndices().IsBitSet(InternalObjectIndex), TEXT("Only root objects can become dormant: %s "), *LocalNetRefHandleManager.PrintObjectFromIndex(InternalObjectIndex));
	
	FNetBitArrayView WantToBeDormantObjects = LocalNetRefHandleManager.GetWantToBeDormantInternalIndices();

	// Update pending dormancy status
	WantToBeDormantObjects.SetBitValue(InternalObjectIndex, bWantsToBeDormant);

	// If we want to be dormant we want to make sure we poll the object immediately
	LocalNetRefHandleManager.GetDormantObjectsPendingFlushNet().SetBitValue(InternalObjectIndex, bWantsToBeDormant);

	// Since we use this as a mask when updating objects we must include subobjects as well
	// Subobjects added later will copy status from owner when they are added
	for (const FInternalNetRefIndex SubObjectInternalIndex : LocalNetRefHandleManager.GetSubObjects(InternalObjectIndex))
	{
		WantToBeDormantObjects.SetBitValue(SubObjectInternalIndex, bWantsToBeDormant);
	}

	// Request frequent world location updates for non-dormant spatially filtered objects.
	OptionallySetObjectRequiresFrequentWorldLocationUpdate(Handle, !bWantsToBeDormant);
}

void UObjectReplicationBridge::NetFlushDormantObject(FNetRefHandle Handle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager& LocalNetRefHandleManager = GetReplicationSystem()->GetReplicationSystemInternal()->GetNetRefHandleManager();

	const FInternalNetRefIndex InternalObjectIndex = LocalNetRefHandleManager.GetInternalIndex(Handle);
	if (InternalObjectIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	UE_LOG_OBJECTREPLICATIONBRIDGE(Verbose, TEXT("NetFlushDormantObject: %s"), *LocalNetRefHandleManager.PrintObjectFromIndex(InternalObjectIndex));

	LocalNetRefHandleManager.GetDormantObjectsPendingFlushNet().SetBit(InternalObjectIndex);

	// Mark the object dirty in order to trigger an update of its WorldLocation and to accumulate dirty flags for when he comes back out of dormancy.
	ReplicationSystem->MarkDirty(Handle);
}

void UObjectReplicationBridge::SetNetPushIdOnInstance(UE::Net::FReplicationInstanceProtocol* InstanceProtocol, FNetHandle NetHandle)
{
#if WITH_PUSH_MODEL
	using namespace UE::Net;

	// Set push ID only if any state supports it. If no state supports it then we might crash if setting the ID.
	if (EnumHasAnyFlags(InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::HasPushBasedDirtiness))
	{
		const Private::FNetPushObjectHandle PushHandle(NetHandle);
		TArrayView<const FReplicationFragment* const> Fragments(InstanceProtocol->Fragments, InstanceProtocol->FragmentCount);
		SetNetPushIdOnFragments(Fragments, PushHandle);
	}
#endif
}

bool UObjectReplicationBridge::GetClassPollFrequency(const UClass* Class, float& OutPollFrequency) const
{
	if (!(bAllowPollPeriodOverrides & bHasPollOverrides))
	{
		return false;
	}

	const FName ClassName = Class->GetFName();
	if (const FPollInfo* PollInfo = ClassesWithPollPeriodOverride.Find(ClassName))
	{
		OutPollFrequency = PollInfo->PollFrequency;
		return true;
	}

	if (ClassesWithoutPollPeriodOverride.Find(ClassName))
	{
		return false;
	}

	bool bFoundOverride = false;
	const UClass* SuperclassWithPollInfo = nullptr;
	FPollInfo SuperclassPollInfo;
	for (const auto& ClassNameAndPollInfo : ClassHierarchyPollPeriodOverrides)
	{
		const UClass* ClassWithPollInfo = ClassNameAndPollInfo.Value.Class.Get();
		if (ClassWithPollInfo == nullptr)
		{
			continue;
		}

		if (Class->IsChildOf(ClassWithPollInfo))
		{
			// If we've already found a superclass with a config, see which one is closer in the hierarchy.
			if (SuperclassWithPollInfo != nullptr)
			{
				if (ClassWithPollInfo->IsChildOf(SuperclassWithPollInfo))
				{
					bFoundOverride = true;
					SuperclassWithPollInfo = ClassWithPollInfo;
					SuperclassPollInfo = ClassNameAndPollInfo.Value;
				}
			}
			else
			{
				bFoundOverride = true;
				SuperclassWithPollInfo = ClassWithPollInfo;
				SuperclassPollInfo = ClassNameAndPollInfo.Value;
			}
		}
	}

	if (bFoundOverride)
	{
		OutPollFrequency = SuperclassPollInfo.PollFrequency;
		return true;
	}

	return false;
}

bool UObjectReplicationBridge::FindOrCachePollFrequency(const UClass* Class, float& OutPollFrequency)
{
	if (!(bAllowPollPeriodOverrides & bHasPollOverrides))
	{
		return false;
	}

	const FName ClassName = Class->GetFName();
	if (const FPollInfo* PollInfo = ClassesWithPollPeriodOverride.Find(ClassName))
	{
		OutPollFrequency = PollInfo->PollFrequency;
		return true;
	}

	if (ClassesWithoutPollPeriodOverride.Find(ClassName))
	{
		return false;
	}

	/**
	 * Only if there are poll period overrides for class hierarchies does it make sense to add
	 * more class names to the exact match containers.
	 */
	if (ClassHierarchyPollPeriodOverrides.Num() == 0)
	{
		return false;
	}

	// We have not encountered this class before. Let's add it to the appropriate container for faster lookup next time.
	if (bHasDirtyClassesInPollPeriodOverrides)
	{
		FindClassesInPollPeriodOverrides();
	}

	const UClass* SuperclassWithPollInfo = nullptr;
	FPollInfo SuperclassPollInfo;
	for (const auto& ClassNameAndPollInfo : ClassHierarchyPollPeriodOverrides)
	{
		const UClass* ClassWithPollInfo = ClassNameAndPollInfo.Value.Class.Get();
		if (ClassWithPollInfo == nullptr)
		{
			continue;
		}

		if (Class->IsChildOf(ClassWithPollInfo))
		{
			// If we've already found a superclass with a config, see which one is closer in the hierarchy.
			if (SuperclassWithPollInfo != nullptr)
			{
				if (ClassWithPollInfo->IsChildOf(SuperclassWithPollInfo))
				{
					SuperclassWithPollInfo = ClassWithPollInfo;
					SuperclassPollInfo = ClassNameAndPollInfo.Value;
				}
			}
			else
			{
				SuperclassWithPollInfo = ClassWithPollInfo;
				SuperclassPollInfo = ClassNameAndPollInfo.Value;
			}
		}
	}

	if (SuperclassWithPollInfo != nullptr)
	{
		/*
		 * Reset class weak pointer as it's not used for exact class matches
		 * and we have no interest in maintaining a valid weak pointer for this case.
		 */
		SuperclassPollInfo.Class.Reset();

		// Add the class hiearchy to our set of classes with overrides.
		for (const UClass* ClassToAdd = Class; ClassToAdd != nullptr; ClassToAdd = ClassToAdd->GetSuperClass())
		{
			const FName ClassToAddName = ClassToAdd->GetFName();
			ClassesWithPollPeriodOverride.FindOrAdd(ClassToAddName, SuperclassPollInfo);
		}

		OutPollFrequency = SuperclassPollInfo.PollFrequency;
		return true;
	}
	else
	{
		// Add the class hierarchy to our set of classes without overrides.
		for (const UClass* ClassToAdd = Class; ClassToAdd != nullptr; ClassToAdd = ClassToAdd->GetSuperClass())
		{
			// We avoid adding classes that are in the exact match container, even though it's not strictly necessary.
			// It makes it easier to reason about things as a class will only be found in exactly one exact cast container.
			const FName ClassToAddName = ClassToAdd->GetFName();
			if (ClassesWithPollPeriodOverride.Contains(ClassToAddName))
			{
				continue;
			}

			ClassesWithoutPollPeriodOverride.Add(ClassToAddName);
		}
	}

	return false;
}

EIrisAsyncLoadingPriority UObjectReplicationBridge::GetClassIrisAsyncLoadingPriority(const UClass* Class)
{
	const FName ClassName = GetConfigClassPathName(Class);
	
	//$IRIS TODO: Add a passive cache of classes we looked up here
	for (const UClass* ClassToFind = Class; ClassToFind != nullptr; ClassToFind = ClassToFind->GetSuperClass())
	{
		const FName ClassToFindName = GetConfigClassPathName(ClassToFind);
		if (const EIrisAsyncLoadingPriority* AsyncLoadingPriority = ClassesIrisAsyncLoadingPriority.Find(ClassToFindName))
		{
			return *AsyncLoadingPriority;
		}
	}

	return EIrisAsyncLoadingPriority::Default;
}

FName UObjectReplicationBridge::GetConfigClassPathName(const UClass* Class)
{
	if (FName* CachedPathName = ConfigClassPathNameCache.Find(Class))
	{
		return *CachedPathName;
	}

	const FName ClassPathName(Class->GetPathName());

	ConfigClassPathNameCache.Add(Class, ClassPathName);

	return ClassPathName;
}

bool UObjectReplicationBridge::ShouldClassBeDeltaCompressed(const UClass* Class)
{
	using namespace UE::Net;

	if (ClassesWithDeltaCompression.Num() > 0)
	{
		for (; Class != nullptr; Class = Class->GetSuperClass())
		{
			if (bool* bShouldBeDeltaCompressed = ClassesWithDeltaCompression.Find(GetConfigClassPathName(Class)))
			{
				return *bShouldBeDeltaCompressed;
			}
		}
	}

	return false;
}

bool UObjectReplicationBridge::IsClassCritical(const UClass* Class)
{
	const UObjectReplicationBridgeConfig* BridgeConfig = UObjectReplicationBridgeConfig::GetConfig();
	if (BridgeConfig->AreAllClassesCritical())
	{
		return true;
	}

	if (ClassesFlaggedCritical.Num() > 0)
	{
		for (; Class != nullptr; Class = Class->GetSuperClass())
		{
			if (bool* bIsClassCritical = ClassesFlaggedCritical.Find(GetConfigClassPathName(Class)))
			{
				return *bIsClassCritical;
			}
		}
	}

	return false;
}

bool UObjectReplicationBridge::IsClassReplicatedByDefault(const UClass* Class) const
{
	return Class != nullptr;
}

FString UObjectReplicationBridge::PrintConnectionInfo(uint32 ConnectionId) const
{
	return FString::Printf(TEXT("ConnectionId:%u"), ConnectionId);
}

void UObjectReplicationBridge::SetMaxTickRate(float InMaxTickRate)
{
	MaxTickRate = InMaxTickRate;
	PollFrequencyLimiter->SetUpdateFrequency(InMaxTickRate);
}

void UObjectReplicationBridge::OptionallySetObjectRequiresFrequentWorldLocationUpdate(FNetRefHandle RefHandle, bool bDesiresFrequentWorldLocationUpdate)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	const FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();
	const FInternalNetRefIndex InternalObjectIndex = LocalNetRefHandleManager.GetInternalIndex(RefHandle);
	FWorldLocations& WorldLocations = ReplicationSystemInternal->GetWorldLocations();
	// When this function is called due to dormancy changes we don't know that the object requires world location updates at all. Checking if it has world location info is how we find that out.
	if (WorldLocations.HasInfoForObject(InternalObjectIndex))
	{
		const FReplicationFiltering& Filtering = ReplicationSystemInternal->GetFiltering();
		const bool bRequireFrequentWorldLocationUpdates = bDesiresFrequentWorldLocationUpdate && Filtering.IsUsingSpatialFilter(InternalObjectIndex);
		WorldLocations.SetObjectRequiresFrequentWorldLocationUpdate(InternalObjectIndex, bRequireFrequentWorldLocationUpdates);
	}
}

int32 UObjectReplicationBridge::GetTypeStatsIndex(const UClass* Class)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const UClass* const CachedClass = Class;

	if (ClassesWithTypeStats.Num() > 0)
	{
		FNetTypeStats& TypeStats = GetReplicationSystem()->GetReplicationSystemInternal()->GetNetTypeStats();
		for (; Class != nullptr; Class = Class->GetSuperClass())
		{
			if (FName* TypeStatsName = ClassesWithTypeStats.Find(GetConfigClassPathName(Class)))
			{
				return TypeStats.GetOrCreateTypeStats(*TypeStatsName);
			}
		}
	}

#if !UE_BUILD_SHIPPING
	if (bUseVeryVerboseIrisCsvStats)
	{
		FNetTypeStats& TypeStats = GetReplicationSystem()->GetReplicationSystemInternal()->GetNetTypeStats();
		if (CachedClass)
		{
			const FName ClassName = CachedClass->GetFName();
			return TypeStats.GetOrCreateTypeStats(ClassName);
		}
	}
#endif // !UE_BUILD_SHIPPING

	return FNetTypeStats::DefaultTypeStatsIndex;
}

void UObjectReplicationBridge::SetClassTypeStatsConfig(FName ClassPathName, FName TypeStatsName)
{
	if (ClassPathName.IsNone())
	{
		return;
	}

	ClassesWithTypeStats.Add(ClassPathName, TypeStatsName);

}

void UObjectReplicationBridge::SetClassTypeStatsConfig(const FString& ClassPathName, const FString& TypeStatsName)
{
	SetClassTypeStatsConfig(FName(ClassPathName), FName(TypeStatsName));
}

void UObjectReplicationBridge::SetClassDynamicFilterConfig(FName ClassPathName, const UE::Net::FNetObjectFilterHandle FilterHandle, FName FilterProfile)
{
	if (ClassPathName.IsNone())
	{
		return;
	}

	if (UE_LOG_ACTIVE(LogIrisFilterConfig, Log))
	{
		if (const FClassFilterInfo* OldFilterInfo = ClassesWithDynamicFilter.Find(ClassPathName))
		{
			if (OldFilterInfo->FilterHandle != FilterHandle)
			{
				UE_LOG_BRIDGEID(LogIrisFilterConfig, Log, TEXT("SetClassDynamicFilterConfig assigned %s to use filter %s (Profile %s). Previously using filter %s."),
					*ClassPathName.ToString(), *(ReplicationSystem->GetFilterName(FilterHandle).ToString()), *FilterProfile.ToString(), *(ReplicationSystem->GetFilterName(OldFilterInfo->FilterHandle).ToString()));
			}
			else
			{
				UE_LOG_BRIDGEID(LogIrisFilterConfig, Log, TEXT("SetClassDynamicFilterConfig assigned %s to use filter %s but the class was already assigned to this filter."), *ClassPathName.ToString(), *(ReplicationSystem->GetFilterName(FilterHandle).ToString()));
			}
		}
		else
		{
			UE_LOG_BRIDGEID(LogIrisFilterConfig, Log, TEXT("SetClassDynamicFilterConfig assigned %s to use filter %s (Profile %s)."), *ClassPathName.ToString(), *(ReplicationSystem->GetFilterName(FilterHandle).ToString()), *FilterProfile.ToString());
		}
	}

	FClassFilterInfo FilterInfo;
	FilterInfo.FilterHandle = FilterHandle;
	FilterInfo.FilterProfile = FilterProfile;
	FilterInfo.bForceEnable = false;
	ClassesWithDynamicFilter.Add(ClassPathName, FilterInfo);
}

void UObjectReplicationBridge::SetClassDynamicFilterConfig(FName ClassPathName, FName FilterName, FName FilterProfile)
{
	using namespace UE::Net;

	if (ClassPathName.IsNone())
	{
		return;
	}

	if (FilterName != NAME_None)
	{
		FNetObjectFilterHandle FilterHandle = GetReplicationSystem()->GetFilterHandle(FilterName);

		if (ensureMsgf(FilterHandle != InvalidNetObjectFilterHandle, TEXT("SetClassDynamicFilterConfig for %s received invalid filter named %s"), *ClassPathName.ToString(), *FilterName.ToString()))
		{
			SetClassDynamicFilterConfig(ClassPathName, FilterHandle, FilterProfile);
		}
	}
	else
	{
		// Reset the filter so the class does not get assigned a dynamic filter anymore.
		SetClassDynamicFilterConfig(ClassPathName, InvalidNetObjectFilterHandle, FilterProfile);
	}
}

UE::Net::FNetObjectFilterHandle UObjectReplicationBridge::GetDynamicFilter(const UClass* Class, bool bRequireForceEnabled, FName& OutFilterProfile)
{
	using namespace UE::Net;

	if (ClassesWithDynamicFilter.IsEmpty())
	{
		/**
		 * For the cases when there are no configured filter mappings we just check whether to use a spatial filter or not.
		 * We don't add anything to the filter mapping.
		 */
		const FNetObjectFilterHandle FilterHandle = ShouldUseDefaultSpatialFilterFunction(Class) ? DefaultSpatialFilterHandle : InvalidNetObjectFilterHandle;
		return FilterHandle;
	}

	const FName ClassName = GetConfigClassPathName(Class);

	// Try exact match first.
	if (FClassFilterInfo* FilterInfoPtr = ClassesWithDynamicFilter.Find(ClassName))
	{
		const bool bUseFilter = !bRequireForceEnabled || FilterInfoPtr->bForceEnable;
		if (bUseFilter)
		{
			OutFilterProfile = FilterInfoPtr->FilterProfile;
			return FilterInfoPtr->FilterHandle;
		}
		else
		{
			return InvalidNetObjectFilterHandle;
		}
	}

	/**
	* Try to find superclass. If we find it and the classes are considered equal we copy the filter setting.
	* If it's not equal we check whether it can be spatialized or not and use the result of that.
	* In all cases we add the result to the mapping for faster lookup next time.
	*/
	for (const UClass* SuperClass = Class->GetSuperClass(); SuperClass != nullptr; SuperClass = SuperClass->GetSuperClass())
	{
		const FName SuperClassName = GetConfigClassPathName(SuperClass);

		// Try to get exact match first.
		if (FClassFilterInfo* FilterInfoPtr = ClassesWithDynamicFilter.Find(SuperClassName))
		{
			if (ShouldSubclassUseSameFilterFunction(SuperClass, Class))
			{
				FClassFilterInfo FilterInfo = *FilterInfoPtr;
				ClassesWithDynamicFilter.Add(ClassName, FilterInfo);

				const bool bUseFilter = !bRequireForceEnabled || FilterInfo.bForceEnable;

				if (bUseFilter)
				{
					OutFilterProfile = FilterInfo.FilterProfile;
					return FilterInfo.FilterHandle;
				}
				else
				{
					return InvalidNetObjectFilterHandle;
				}
					
			}

			// Here's a good place to put a line of code and set a breakpoint to debug inheritance issues.

			break;
		}
	}

	// Either super class wasn't found or it wasn't considered equal. Let's add a new filter mapping.
	FClassFilterInfo FilterInfo;
	FilterInfo.FilterHandle = ShouldUseDefaultSpatialFilterFunction(Class) ? DefaultSpatialFilterHandle : InvalidNetObjectFilterHandle;
	FilterInfo.bForceEnable = false;
	ClassesWithDynamicFilter.Add(ClassName, FilterInfo);
	return FilterInfo.FilterHandle;
}

UE::Net::FNetObjectPrioritizerHandle UObjectReplicationBridge::GetPrioritizer(const UClass* Class, bool bRequireForceEnabled)
{
	using namespace UE::Net;

	if (ClassesWithPrioritizer.Num() > 0)
	{
		const FName ClassName = GetConfigClassPathName(Class);

		// Try exact match first.
		if (const FClassPrioritizerInfo* PrioritizerInfo = ClassesWithPrioritizer.Find(ClassName))
		{
			const bool bUsePrioritizer = !bRequireForceEnabled || PrioritizerInfo->bForceEnable;
			return bUsePrioritizer ? PrioritizerInfo->PrioritizerHandle : InvalidNetObjectPrioritizerHandle;
		}

		// Try to find superclass with prioritizer config. If we find it we copy the config and add the result to the mapping for faster lookup next time.
		for (const UClass* SuperClass = Class->GetSuperClass(); SuperClass != nullptr; SuperClass = SuperClass->GetSuperClass())
		{
			const FName SuperClassName = GetConfigClassPathName(SuperClass);

			if (const FClassPrioritizerInfo* PrioritizerInfoPtr = ClassesWithPrioritizer.Find(SuperClassName))
			{
				// Copy info to this class
				FClassPrioritizerInfo PrioritizerInfo = *PrioritizerInfoPtr;
				ClassesWithPrioritizer.Add(ClassName, PrioritizerInfo);

				const bool bUsePrioritizer = !bRequireForceEnabled || PrioritizerInfo.bForceEnable;
				return bUsePrioritizer ? PrioritizerInfo.PrioritizerHandle : InvalidNetObjectPrioritizerHandle;
			}
		}
	}

	// No prioritizer has been configured for this class.
	return InvalidNetObjectPrioritizerHandle;
}

void UObjectReplicationBridge::ReloadConfig()
{
	LoadConfig();
}

void UObjectReplicationBridge::LoadConfig()
{
	// Clear everything related to the config.
	bHasPollOverrides = false;
	bHasDirtyClassesInPollPeriodOverrides = false;
	ClassHierarchyPollPeriodOverrides.Empty();
	ClassesWithPollPeriodOverride.Empty();
	ClassesWithoutPollPeriodOverride.Empty();
	ClassesWithDynamicFilter.Empty();
	ClassesWithPrioritizer.Empty();
	ClassesWithDeltaCompression.Empty();
	ClassesFlaggedCritical.Empty();
	ClassesIrisAsyncLoadingPriority.Empty();

	// Reset PathNameCache
	ConfigClassPathNameCache.Empty();

	const UObjectReplicationBridgeConfig* BridgeConfig = UObjectReplicationBridgeConfig::GetConfig();

	// Load poll configs
	
	// These classes are forbidden to override due to being too generic and could cause memory and performance issues.
	// If there's need for a global poll period override it should be implemented separately and not via class overrides.
	const FName ForbiddenNames[] = {NAME_Object, NAME_Actor};
	const TConstArrayView<FName> ForbiddenNamesArray = MakeArrayView(ForbiddenNames);

	for (const FObjectReplicationBridgePollConfig& PollOverride : BridgeConfig->GetPollConfigs())
	{
		if (!ensure(!ForbiddenNamesArray.Contains(PollOverride.ClassName)))
		{
			continue;
		}

		bHasPollOverrides = true;

		FPollInfo PollInfo;
		PollInfo.PollFrequency = FPlatformMath::Max(PollOverride.PollFrequency, 0.0f);
		if (PollOverride.bIncludeSubclasses)
		{
			bHasDirtyClassesInPollPeriodOverrides = true;
			ClassHierarchyPollPeriodOverrides.Add(PollOverride.ClassName, PollInfo);
		}
		else
		{
			ClassesWithPollPeriodOverride.Add(PollOverride.ClassName, PollInfo);
		}
	}

	if (bHasDirtyClassesInPollPeriodOverrides)
	{
		FindClassesInPollPeriodOverrides();
	}

	// Filter mappings.
	DefaultSpatialFilterName = BridgeConfig->GetDefaultSpatialFilterName();
	DefaultSpatialFilterHandle = ReplicationSystem->GetFilterHandle(DefaultSpatialFilterName);

	{
		for (const FObjectReplicationBridgeFilterConfig& FilterConfig : BridgeConfig->GetFilterConfigs())
		{
			FClassFilterInfo FilterInfo;
			FilterInfo.FilterHandle = ReplicationSystem->GetFilterHandle(FilterConfig.DynamicFilterName);
			FilterInfo.FilterProfile = FilterConfig.FilterProfile;
			FilterInfo.bForceEnable = FilterConfig.bForceEnableOnAllInstances;
			ClassesWithDynamicFilter.Add(FilterConfig.ClassName, FilterInfo);
		}
	}

	// Prioritizer mappings
	{
		for (const FObjectReplicationBridgePrioritizerConfig& PrioritizerConfig : BridgeConfig->GetPrioritizerConfigs())
		{
			FClassPrioritizerInfo PrioInfo;
			PrioInfo.PrioritizerHandle = ReplicationSystem->GetPrioritizerHandle(PrioritizerConfig.PrioritizerName);
			PrioInfo.bForceEnable = PrioritizerConfig.bForceEnableOnAllInstances;
			ClassesWithPrioritizer.Add(PrioritizerConfig.ClassName, PrioInfo);
		}
	}

	// Load delta compression settings
	{
		for (const FObjectReplicationBridgeDeltaCompressionConfig& DCConfig : BridgeConfig->GetDeltaCompressionConfigs())
		{
			if (!ensure(!ForbiddenNamesArray.Contains(DCConfig.ClassName)))
			{
				continue;
			}

			ClassesWithDeltaCompression.Add(DCConfig.ClassName, DCConfig.bEnableDeltaCompression);
		}
	}

	// Critical classes
	if (!BridgeConfig->AreAllClassesCritical())
	{
		for (const FObjectReplicatedBridgeCriticalClassConfig& CriticalClassConfig : BridgeConfig->GetCriticalClassConfigs())
		{
			if (!ensure(!ForbiddenNamesArray.Contains(CriticalClassConfig.ClassName)))
			{
				continue;
			}

			ClassesFlaggedCritical.Add(CriticalClassConfig.ClassName, CriticalClassConfig.bDisconnectOnProtocolMismatch);
		}
	}

	// Load TypeStats settings
	{
		for (const FObjectReplicationBridgeTypeStatsConfig& TypeStatsConfig : BridgeConfig->GetTypeStatsConfigs())
		{
			// Skip all non shipping TypeStats unless CVar is set
			if (!bUseVerboseIrisCsvStats && !TypeStatsConfig.bIncludeInMinimalCSVStats)
			{
				continue;
			}
			
			ClassesWithTypeStats.Add(TypeStatsConfig.ClassName, TypeStatsConfig.TypeStatsName);
		}
	}

	// Async loading priority
	{
		for (const FObjectReplicationBridgeAsyncLoadingClassConfig& AsyncLoadingConfig : BridgeConfig->GetAsyncLoadingClassConfigs())
		{
			ClassesIrisAsyncLoadingPriority.Add(AsyncLoadingConfig.ClassName, AsyncLoadingConfig.IrisAsyncLoadingPriority);
		}
	}
}

void UObjectReplicationBridge::FindClassesInPollPeriodOverrides()
{
	bool bFailedToFindClass = false;
	for (auto& ClassNameAndPollInfo : ClassHierarchyPollPeriodOverrides)
	{
		FPollInfo& PollInfo = ClassNameAndPollInfo.Value;
		if (!PollInfo.Class.IsValid())
		{
			constexpr UObject* ClassOuter = nullptr;
			const UClass* Class = CastChecked<UClass>(StaticFindObject(UClass::StaticClass(), ClassOuter, ToCStr(ClassNameAndPollInfo.Key.ToString()), EFindObjectFlags::ExactClass));
			bFailedToFindClass |= (Class == nullptr);
			PollInfo.Class = Class;
		}
	}

	bHasDirtyClassesInPollPeriodOverrides = bFailedToFindClass;
}

void UObjectReplicationBridge::SetShouldUseDefaultSpatialFilterFunction(TFunction<bool(const UClass*)> InShouldUseDefaultSpatialFilterFunction)
{
	if (!ensureMsgf((bool)InShouldUseDefaultSpatialFilterFunction, TEXT("%s"), TEXT("A valid function must be provided for SetShouldUseDefaultSpatialFilterFunction.")))
	{
		return;
	}

	ShouldUseDefaultSpatialFilterFunction = InShouldUseDefaultSpatialFilterFunction;
}

void UObjectReplicationBridge::SetShouldSubclassUseSameFilterFunction(TFunction<bool(const UClass* Class, const UClass* Subclass)> InShouldSubclassUseSameFilterFunction)
{
	if (!ensureMsgf((bool)InShouldSubclassUseSameFilterFunction, TEXT("%s"), TEXT("A valid function must be provided for SetShouldSubclassUseSameFilterFunction.")))
	{
		return;
	}

	ShouldSubclassUseSameFilterFunction = InShouldSubclassUseSameFilterFunction;
}

void UObjectReplicationBridge::InitConditionalPropertyDelegates()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// Hookup delegate for when a property custom condition is changed
	OnCustomConditionChangedHandle = UE::Net::Private::FPropertyConditionDelegates::GetOnPropertyCustomConditionChangedDelegate().AddLambda([this](const UObject* Owner, uint16 RepIndex, bool bEnable)
	{
		const FNetRefHandle RefHandle = this->GetReplicatedRefHandle(Owner);
		if (RefHandle.IsValid())
		{
			FReplicationSystemInternal* ReplicationSystemInternal = this->GetReplicationSystem()->GetReplicationSystemInternal();
			const FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();
			FReplicationConditionals& Conditionals = ReplicationSystemInternal->GetConditionals();

			Conditionals.SetPropertyCustomCondition(LocalNetRefHandleManager.GetInternalIndex(RefHandle), Owner, RepIndex, bEnable);
		}
	});

	// Hookup delegate for when a property dynamic condition is changed
	OnDynamicConditionChangedHandle = UE::Net::Private::FPropertyConditionDelegates::GetOnPropertyDynamicConditionChangedDelegate().AddLambda([this](const UObject* Owner, uint16 RepIndex, ELifetimeCondition Condition)
	{
		const FNetRefHandle RefHandle = this->GetReplicatedRefHandle(Owner);
		if (RefHandle.IsValid())
		{
			FReplicationSystemInternal* ReplicationSystemInternal = this->GetReplicationSystem()->GetReplicationSystemInternal();
			const FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();
			FReplicationConditionals& Conditionals = ReplicationSystemInternal->GetConditionals();

			Conditionals.SetPropertyDynamicCondition(LocalNetRefHandleManager.GetInternalIndex(RefHandle), Owner, RepIndex, Condition);
		}
	});
}

void UObjectReplicationBridge::ReinitPollFrequency()
{
}

void UObjectReplicationBridge::SetPollFrequency(FNetRefHandle RefHandle, float PollFrequency)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	const FNetRefHandleManager& LocalNetRefHandleManager = ReplicationSystemInternal->GetNetRefHandleManager();

	FInternalNetRefIndex RootObjectIndex = LocalNetRefHandleManager.GetInternalIndex(RefHandle);
	if (RootObjectIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	PollFrequencyLimiter->SetPollFrequency(RootObjectIndex, PollFrequency);

	// Make sure the subobjects are polled the same frame as the root object.
	for (const FInternalNetRefIndex SubObjectIndex : LocalNetRefHandleManager.GetSubObjects(RootObjectIndex))
	{
		PollFrequencyLimiter->SetPollWithObject(RootObjectIndex, SubObjectIndex);
	}
}

void UObjectReplicationBridge::OnProtocolMismatchReported(FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<uint8>& ClientCDOStateBytes)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// Ensure at the end so the log contains all the relevant information
	ON_SCOPE_EXIT
	{
		ensureMsgf(false, TEXT("Protocol mismatch detected from %s. Check the server log for CDO state differences."), *PrintConnectionInfo(ConnectionId));
	};


	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager->GetInternalIndex(RefHandle);
	if (ObjectInternalIndex == InvalidInternalNetRefIndex)
	{
		UE_LOG_BRIDGEID(LogIris, Warning, TEXT("OnProtocolMismatchReported from Connection:%s for %s. But object has no InternalIndex."), *PrintConnectionInfo(ConnectionId), *RefHandle.ToString());
		return;
	}

	UObject* ObjInstance = NetRefHandleManager->GetReplicatedObjectInstance(ObjectInternalIndex);
	UObject* ObjArchetype = ObjInstance ? ObjInstance->GetArchetype() : nullptr;

	UE_LOG_BRIDGEID(LogIris, Error, TEXT("OnProtocolMismatchReported from client:%s when instancing %s. CDO:%s ReplicatedObject:%s NetObject:%s"), *PrintConnectionInfo(ConnectionId), *RefHandle.ToString(), *GetPathNameSafe(ObjArchetype), *GetNameSafe(ObjInstance), *NetRefHandleManager->PrintObjectFromIndex(ObjectInternalIndex));

	if (UE_LOG_ACTIVE(LogIris, Error))
	{
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectInternalIndex);
		const FReplicationInstanceProtocol* InstanceProtocol = ObjectData.InstanceProtocol;
		if (!InstanceProtocol)
		{
			UE_LOG_BRIDGEID(LogIris, Warning, TEXT("OnProtocolMismatchReported from Connection:%s for %s. But object %s has no InstanceProtocol."), *PrintConnectionInfo(ConnectionId), *RefHandle.ToString(), *GetNameSafe(ObjInstance));
			return;
		}

		const FReplicationProtocol* Protocol = ObjectData.Protocol;
		if (!Protocol)
		{
			UE_LOG_BRIDGEID(LogIris, Warning, TEXT("OnProtocolMismatchReported from Connection:%s for %s. But object %s has no Protocol."), *PrintConnectionInfo(ConnectionId), *RefHandle.ToString(), *GetNameSafe(ObjInstance));
			return;
		}

		check(Protocol->ReplicationStateCount == InstanceProtocol->FragmentCount);

		// Build the list of fragments of this object
		FReplicationFragments Fragments;
		for (uint16 FragmentIndex=0; FragmentIndex < InstanceProtocol->FragmentCount; ++FragmentIndex)
		{
			FReplicationFragmentInfo FragmentInfo;
			FragmentInfo.Fragment = InstanceProtocol->Fragments[FragmentIndex];
			FragmentInfo.Descriptor = Protocol->ReplicationStateDescriptors[FragmentIndex];

			Fragments.Emplace(MoveTemp(FragmentInfo));
		}

		// Only build the structured ServerCDOState when the client sent its CDO state, so we can diff them.
		// Otherwise fall back to the legacy text dump inside RemoteProtocolMismatchDetected.
		const bool bWantDiff = ClientCDOStateBytes.Num() > 0;
		UE::Net::Private::FDebugCDOState ServerCDOState;
		UE::Net::Private::ObjectBridgeDebugging::RemoteProtocolMismatchDetected(ArchetypesAlreadyPrinted, ReplicationSystem, ConnectionId, Fragments, ObjArchetype, ObjInstance, bWantDiff ? &ServerCDOState : nullptr);

		if (bWantDiff)
		{
			UE::Net::Private::FDebugCDOState ClientCDOState;
			if (FReplicationInstanceOperations::DeserializeInternalDefaultState(ClientCDOStateBytes, ClientCDOState))
			{
				FReplicationInstanceOperations::DiffAndLogInternalDefaultState(ClientCDOState, ServerCDOState);
			}
			else
			{
				// Emit the text dump directly here as a fallback when deserialization fails.
				UE_LOGF(LogIris, Error, "Client CDO state bytes were malformed, falling back to server-side text dump.");
				TStringBuilder<4096> StringBuilder;
				FReplicationInstanceOperations::OutputInternalDefaultStateMemberHashesToString(ReplicationSystem, StringBuilder, Fragments);
				UE_LOGF(LogIris, Error, "Printing replication state of CDO %ls used for %ls:\n%ls",
					*GetNameSafe(ObjArchetype), *GetNameSafe(ObjInstance), StringBuilder.ToString());
			}
		}
	}

	//TODO: Set object as broken in RepWriter
}

void UObjectReplicationBridge::OnErrorWithNetRefHandleReported(UE::Net::ENetRefHandleError ErrorType, FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<FNetRefHandle>& ExtraHandles, const FString& DiagnosticMessage)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	switch(ErrorType)
	{
		case ENetRefHandleError::BitstreamCorrupted:
		{
			OnErrorBitstreamCorrupted(RefHandle, ConnectionId);
		} break;
		
		case ENetRefHandleError::ReplicationDisabled:
		{
			OnErrorCannotReplicateObject(RefHandle, ConnectionId, DiagnosticMessage);
		} break;
		
		case ENetRefHandleError::BlockedByMustBeMapped:
		{
			OnErrorBlockedByAsyncLoading(RefHandle, ConnectionId, ExtraHandles);
		} break;
		
		case ENetRefHandleError::BlockedByCreationParents:
		{
			OnErrorBlockedByParents(RefHandle, ConnectionId, ExtraHandles);
		} break;
		
		default:
		{
			ensureMsgf(false, TEXT("Not implemented"));
		} break;
	}

	// Print all internal info we have on the blocked object
	//PrintDebugInfoForNetRefHandle(RefHandle, ConnectionId, UObjectReplicationBridge::EPrintDebugInfoTraits::NoProtocolState);
}

void UObjectReplicationBridge::OnErrorBitstreamCorrupted(FNetRefHandle RefHandle, uint32 ConnectionId)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager->GetInternalIndex(RefHandle);
	if (ObjectInternalIndex == InvalidInternalNetRefIndex)
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Bitstream corruption from unmapped handle: %s. Reported by client: %s."), *RefHandle.ToString(), *PrintConnectionInfo(ConnectionId));
	}
	else
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Bitstream corruption from object: %s. Reported by client: %s"), *NetRefHandleManager->PrintObjectFromIndex(ObjectInternalIndex), *PrintConnectionInfo(ConnectionId));
	}

	ensureMsgf(false, TEXT("Bitstream corruption reported with %s. Look at the log for important information on the object tied to the handle."), *NetRefHandleManager->PrintObjectFromNetRefHandle(RefHandle));
}

void UObjectReplicationBridge::OnErrorCannotReplicateObject(FNetRefHandle RefHandle, uint32 ConnectionId, const FString& ClientDiagnosticMessage)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager->GetInternalIndex(RefHandle);
	if (ObjectInternalIndex == InvalidInternalNetRefIndex)
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Client cannot replicate unmapped handle: %s. Reported by client: %s."), *RefHandle.ToString(), *PrintConnectionInfo(ConnectionId));
	}
	else
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Client cannot replicate object: %s. Reported by client: %s"), *NetRefHandleManager->PrintObjectFromIndex(ObjectInternalIndex), *PrintConnectionInfo(ConnectionId));

		// TODO: Tell RepWriter the creation failed
	}

	if (!ClientDiagnosticMessage.IsEmpty())
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Client could not instantiate %s. Reason: %s"), *NetRefHandleManager->PrintObjectFromNetRefHandle(RefHandle), *ClientDiagnosticMessage);
	}

	ensureMsgf(false, TEXT("Client could not instantiate %s, replication disabled. %s"), *NetRefHandleManager->PrintObjectFromNetRefHandle(RefHandle), ClientDiagnosticMessage.IsEmpty() ? TEXT("Reason unknown.") : TEXT("Look at the log for the client diagnostic message."));
}

void UObjectReplicationBridge::OnErrorBlockedByAsyncLoading(FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<FNetRefHandle>& MustBeMappedRefs)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

#if DO_ENSURE

	if (!bLogBlockedByAsyncLoadingError)
	{
		return;
	}

	// A client cannot process incoming data for the Object because it has been waiting on the MustBeMappedRefs to be async loaded for too long (30secs by default)

	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager->GetInternalIndex(RefHandle);

	if (ObjectInternalIndex == InvalidInternalNetRefIndex)
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Replication Blocked by AsyncLoading for unmapped handle: %s. Client: %s"), *RefHandle.ToString(), *PrintConnectionInfo(ConnectionId));
	}
	else
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Replication Blocked by AsyncLoading for: %s. Client: %s"), *NetRefHandleManager->PrintObjectFromIndex(ObjectInternalIndex), *PrintConnectionInfo(ConnectionId));
	}
	
	for (FNetRefHandle MustBeMappedRef : MustBeMappedRefs)
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("AsyncLoading reference: %s | PathName: %s"), *NetRefHandleManager->PrintObjectFromNetRefHandle(MustBeMappedRef), GetObjectReferenceCache()->GetObjectRelativePath(MustBeMappedRef, ReplicationSystem));
	}
		
	ensureMsgf(false, TEXT("Blocked by AsyncLoading error: Replication blocked for %s due to %d unmapped references."), *NetRefHandleManager->PrintObjectFromNetRefHandle(RefHandle), MustBeMappedRefs.Num());

#endif // DO_ENSURE
}


void UObjectReplicationBridge::OnErrorBlockedByParents(FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<FNetRefHandle>& BlockedParents)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

#if DO_ENSURE

	if (!bLogMissingParentsError)
	{
		return;
	}

	// A client cannot instantiate the Object because it has been waiting on the creation parents to be spawned for too long (default is 30secs)
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager->GetInternalIndex(RefHandle);
	
	if (ObjectInternalIndex == InvalidInternalNetRefIndex)
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Client cannot spawn unmapped object: %s. Reported by client: %s"), *RefHandle.ToString(), *PrintConnectionInfo(ConnectionId));
	}
	else
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Client cannot spawn object: %s. Reported by client: %s"), *NetRefHandleManager->PrintObjectFromIndex(ObjectInternalIndex), *PrintConnectionInfo(ConnectionId));
	}

	for (FNetRefHandle BlockedParent : BlockedParents)
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Missing Creation Dependency Parent: %s"), *NetRefHandleManager->PrintObjectFromNetRefHandle(BlockedParent));
	}
		
	ensureMsgf(false, TEXT("Client cannot spawn: %s. Blocked by %d missing Parents."), *NetRefHandleManager->PrintObjectFromNetRefHandle(RefHandle), BlockedParents.Num());

#endif // DO_ENSURE
}

TArray<uint32> UObjectReplicationBridge::FindConnectionsFromArgs(const TArray<FString>& Args) const
{
	using namespace UE::Net::Private;

	TArray<uint32> ConnectionList;

	// If ConnectionId=XX was specified
	if (const FString* ArgConnectionIds = Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("ConnectionId=")); }))
	{
		constexpr bool bIgnoreSeperators = false;

		// Find all the Ids passed in the argument
		FString StrConnectionIds;
		if (FParse::Value(**ArgConnectionIds, TEXT("ConnectionId="), StrConnectionIds, bIgnoreSeperators))
		{
			TArray<FString> StrIds;
			StrConnectionIds.ParseIntoArray(StrIds, TEXT(","));

			for (const FString& StrId : StrIds)
			{
				int32 Id = INDEX_NONE;
				LexFromString(Id, *StrId);
				if (Id > INDEX_NONE)
				{
					ConnectionList.AddUnique((uint32)Id);
				}
			}
		}
	}

	return ConnectionList;
}
