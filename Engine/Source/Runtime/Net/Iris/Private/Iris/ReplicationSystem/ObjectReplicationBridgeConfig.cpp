// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ObjectReplicationBridgeConfig.h"
#include "Containers/ArrayView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectReplicationBridgeConfig)

UObjectReplicationBridgeConfig::UObjectReplicationBridgeConfig()
: Super()
{
}

const UObjectReplicationBridgeConfig* UObjectReplicationBridgeConfig::GetConfig()
{
	return GetDefault<UObjectReplicationBridgeConfig>();
}

TConstArrayView<FObjectReplicationBridgePollConfig> UObjectReplicationBridgeConfig::GetPollConfigs() const
{
	return MakeArrayView(PollConfigs);
}

TConstArrayView<FObjectReplicationBridgeFilterConfig> UObjectReplicationBridgeConfig::GetFilterConfigs() const
{
	return MakeArrayView(FilterConfigs);
}

TConstArrayView<FObjectReplicationBridgePrioritizerConfig> UObjectReplicationBridgeConfig::GetPrioritizerConfigs() const
{
	return MakeArrayView(PrioritizerConfigs);
}

TConstArrayView<FObjectReplicationBridgeDeltaCompressionConfig> UObjectReplicationBridgeConfig::GetDeltaCompressionConfigs() const
{
	return MakeArrayView(DeltaCompressionConfigs);
}

TConstArrayView<FObjectReplicatedBridgeCriticalClassConfig> UObjectReplicationBridgeConfig::GetCriticalClassConfigs() const
{
	return MakeArrayView(CriticalClassConfigs);
}

TConstArrayView<FObjectReplicationBridgeTypeStatsConfig> UObjectReplicationBridgeConfig::GetTypeStatsConfigs() const
{
	return MakeArrayView(TypeStatsConfigs);
}

TConstArrayView<FObjectReplicationBridgeAsyncLoadingClassConfig> UObjectReplicationBridgeConfig::GetAsyncLoadingClassConfigs() const
{
	return MakeArrayView(AsyncLoadingClassConfigs);
}
