// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"

#include "HAL/IConsoleManager.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisDebugging.h"

#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationReader.h"
#include "Iris/ReplicationSystem/ReplicationWriter.h"

#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineManager.h"

#include "Iris/ReplicationSystem/Filtering/NetObjectFilterDefinitions.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFiltering.h"

#include "Iris/ReplicationSystem/Prioritization/ReplicationPrioritization.h"

#include "Iris/ReplicationSystem/ObjectReplicationBridgeConfig.h"

#include "Net/Core/NetBitArrayPrinter.h"
#include "Net/Core/NetHandle/NetHandleManager.h"
#include "Net/Core/Trace/NetDebugName.h"

#include "Templates/UnrealTemplate.h"
#include "UObject/ClassTree.h"
#include "UObject/CoreNet.h"

/**
 * This class contains misc console commands that log the state of different Iris systems.
 * 
 * Most cmds support common optional parameters that are listed here:
 *		RepSystemId=X => Execute the cmd on a specific ReplicationSystem. Useful in PIE
 *		WithSubObjects => Print the subobjects attached to each RootObject
 *		SortByClass => Log the rootobjects alphabetically by ClassName (usually the default)
 *		SortByNetRefHandle => Log the rootobjects by their NetRefHandle Id starting with static objects (odd Id) then dynamic objects (even Id)
 */

namespace UE::Net::Private::ObjectBridgeDebugging
{

/** 
 * Generic traits to control what and how we log debug information
 */
enum class EPrintTraits : uint32
{
	Default					= 0x0000,
	LogSubObjects			= 0x0001, // log the subobjects of each rootobject
	LogTraits				= EPrintTraits::LogSubObjects,

	SortByClass				= 0x0100, // log objects sorted by their class name
	SortByNetRefHandle		= 0x0200, // log objects sorted by netrefhandle (odd (static) first, even (dynamic) second)
	SortTraits				= EPrintTraits::SortByNetRefHandle | EPrintTraits::SortByClass
};
ENUM_CLASS_FLAGS(EPrintTraits);

/** Extracts the print traits from console arguments */
EPrintTraits FindPrintTraitsFromArgs(const TArray<FString>& Args)
{
	EPrintTraits Traits = EPrintTraits::Default;

	if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("WithSubObjects")); }))
	{
		Traits = Traits | EPrintTraits::LogSubObjects;
	}

	if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("SortByClass")); }) )
	{
		Traits = Traits | EPrintTraits::SortByClass;
	}
	else if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("SortByNetRefHandle")); }))
	{
		Traits = Traits | EPrintTraits::SortByNetRefHandle;
	}

	return Traits;
}

/**
 * Print traits to control the dynamic filter debug commands
 */
enum class EDynamicFilterPrintTraits : uint32
{
	Default							= 0x0000,
	// Print classes and what filter they will set. Includes configured classes as well as all replicated classes that have been automatically assigned a filter so far. 
	Config							= 0x0001,
	// Detect issues like non-existing classes being configured or child classes not being configured and ending up with a different filter
	IssueDetection					= 0x0002,
	// Ignore blueprint classes if filter config IssueDetection is enabled.
	SkipBPIssueDetection			= 0x0004,
	// Ignore issues related to classes whose nearest configured super class is Actor
	SkipActorChildIssueDetection	= 0x0008,

};
ENUM_CLASS_FLAGS(EDynamicFilterPrintTraits);

/** Extract dynamic filter print traits from console arguments */
EDynamicFilterPrintTraits FindDynamicFilterPrintTraitsFromArgs(const TArray<FString>& Args)
{
	EDynamicFilterPrintTraits Traits = EDynamicFilterPrintTraits::Default;

	if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("Config")); }))
	{
		Traits = Traits | EDynamicFilterPrintTraits::Config;
	}

	if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("IssueDetection")); }))
	{
		Traits = Traits | EDynamicFilterPrintTraits::IssueDetection;
	}

	if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("SkipBPIssueDetection")); }) )
	{
		Traits = Traits | EDynamicFilterPrintTraits::SkipBPIssueDetection;
	}

	if (Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("SkipActorChildIssueDetection")); }) )
	{
		Traits = Traits | EDynamicFilterPrintTraits::SkipActorChildIssueDetection;
	}

	return Traits;
}

/** Holds information about root objects sortable by class name */
struct FRootObjectData
{
	FInternalNetRefIndex ObjectIndex = 0;
	FNetRefHandle NetHandle;
	UObject* Instance = nullptr;
	UClass* Class = nullptr;
};

// Transform a bit array of root object indexes into an array of RootObjectData struct
void FillRootObjectArrayFromBitArray(TArray<FRootObjectData>& OutRootObjects, const FNetBitArrayView RootObjectList, FNetRefHandleManager* NetRefHandleManager)
{
	RootObjectList.ForAllSetBits([&](uint32 RootObjectIndex)
	{
		FRootObjectData Data;
		Data.ObjectIndex = RootObjectIndex;
		Data.NetHandle = NetRefHandleManager->GetNetRefHandleFromInternalIndex(RootObjectIndex);
		Data.Instance = NetRefHandleManager->GetReplicatedObjectInstance(RootObjectIndex);
		Data.Class = Data.Instance ? Data.Instance->GetClass() : nullptr;

		OutRootObjects.Emplace(MoveTemp(Data));
	});
}

void SortByClassName(TArray<FRootObjectData>& OutArray)
{
	Algo::Sort(OutArray, [](const FRootObjectData& lhs, const FRootObjectData& rhs)
	{
		if (lhs.Class == rhs.Class) { return false; }
		if (!lhs.Class) { return false; }
		if (!rhs.Class) { return true; }
		return lhs.Class->GetName() < rhs.Class->GetName();
	});
}

void SortByNetRefHandle(TArray<FRootObjectData>& OutArray)
{
	// Sort static objects first (odds) then dynamic ones second (evens)
	Algo::Sort(OutArray, [](const FRootObjectData& lhs, const FRootObjectData& rhs)
	{
		if (lhs.NetHandle == rhs.NetHandle) { return false; }
		if (!lhs.NetHandle.IsValid()) { return false; }
		if (!rhs.NetHandle.IsValid()) { return true; }
		if (lhs.NetHandle.IsStatic() && rhs.NetHandle.IsDynamic()) { return false; }
		if (lhs.NetHandle.IsDynamic() && rhs.NetHandle.IsStatic()) { return true; }
		return lhs.NetHandle < rhs.NetHandle;
	});
}

/** Sort the array with the selected trait. If no traits were selected, sort via the default one */
void SortViaTrait(TArray<FRootObjectData>& OutArray, EPrintTraits ArgTraits, EPrintTraits DefaultTraits)
{
	EPrintTraits SelectedTrait = ArgTraits & EPrintTraits::SortTraits;
	if (SelectedTrait == EPrintTraits::Default)
	{
		SelectedTrait = DefaultTraits;
	}

	switch(SelectedTrait)
	{
		case EPrintTraits::SortByClass: SortByClassName(OutArray);  break;
		case EPrintTraits::SortByNetRefHandle: SortByNetRefHandle(OutArray); break;
	}
}

/** Print all the protocols of the default state of an object (so it's CDO/Archetype baseline) */
void PrintDefaultNetObjectState(UReplicationSystem* ReplicationSystem, uint32 ConnectionId, const FReplicationFragments& RegisteredFragments, FStringBuilderBase& StringBuilder)
{
	FReplicationSystemInternal* ReplicationSystemInternal = ReplicationSystem->GetReplicationSystemInternal();

	// Setup Context
	FInternalNetSerializationContext InternalContext;
	FInternalNetSerializationContext::FInitParameters InternalContextInitParams;
	InternalContextInitParams.ReplicationSystem = ReplicationSystem;
	InternalContextInitParams.PackageMap = ReplicationSystemInternal->GetIrisObjectReferencePackageMap();
	InternalContextInitParams.ObjectResolveContext.RemoteNetTokenStoreState = ReplicationSystem->GetNetTokenStore()->GetRemoteNetTokenStoreState(ConnectionId);
	InternalContextInitParams.ObjectResolveContext.ConnectionId = ConnectionId;
	InternalContext.Init(InternalContextInitParams);

	FNetSerializationContext NetSerializationContext;
	NetSerializationContext.SetInternalContext(&InternalContext);
	NetSerializationContext.SetLocalConnectionId(ConnectionId);

	FReplicationInstanceOperations::OutputInternalDefaultStateToString(NetSerializationContext, StringBuilder, RegisteredFragments);
	FReplicationInstanceOperations::OutputInternalDefaultStateMemberHashesToString(ReplicationSystem, StringBuilder, RegisteredFragments);
}

void RemoteProtocolMismatchDetected(TMap<FObjectKey, bool>& ArchetypesAlreadyPrinted, UReplicationSystem* ReplicationSystem, uint32 ConnectionId, const FReplicationFragments& RegisteredFragments, const UObject* ArchetypeOrCDOKey, const UObject* InstancePtr, FDebugCDOState* OutCDOState)
{
	if (UE_LOG_ACTIVE(LogIris, Error))
	{
		// Only print the CDO state once
		if (ArchetypesAlreadyPrinted.Find(FObjectKey(ArchetypeOrCDOKey)) == nullptr)
		{
			ArchetypesAlreadyPrinted.Add(FObjectKey(ArchetypeOrCDOKey), true);

			if (OutCDOState)
			{
				// Caller will diff the structured state
				FReplicationInstanceOperations::BuildInternalDefaultState(ReplicationSystem, RegisteredFragments, *OutCDOState);
			}
			else
			{
				TStringBuilder<4096> StringBuilder;
				PrintDefaultNetObjectState(ReplicationSystem, ConnectionId, RegisteredFragments, StringBuilder);
				UE_LOGF(LogIris, Error, "Printing replication state of CDO %ls used for %ls:\n%ls", *GetNameSafe(ArchetypeOrCDOKey), *GetNameSafe(InstancePtr), StringBuilder.ToString());
			}
		}
	}
}
/**
 * Find the replication system from console arguments: 'RepSystemId=0...'
 */
UReplicationSystem* FindReplicationSystemFromArg(const TArray<FString>& Args)
{
	uint32 RepSystemId = 0;

	// If the ReplicationSystemId was specified
	if (const FString* ArgRepSystemId = Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("RepSystemId=")); }))
	{
		FParse::Value(**ArgRepSystemId, TEXT("RepSystemId="), RepSystemId);
	}

	return UE::Net::GetReplicationSystem(RepSystemId);
}


FString PrintNetObject(FNetRefHandleManager* NetRefHandleManager, FInternalNetRefIndex ObjectIndex)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandle NetRefHandle = NetRefHandleManager->GetNetRefHandleFromInternalIndex(ObjectIndex);
	const FNetRefHandleManager::FReplicatedObjectData& NetObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
	UObject* ObjectPtr = NetRefHandleManager->GetReplicatedObjectInstance(ObjectIndex);

	return FString::Printf(TEXT("%s %s (InternalIndex: %u) (%s)"), 
		(NetObjectData.SubObjectRootIndex == InvalidInternalNetRefIndex) ? TEXT("RootObject"):TEXT("SubObject"),
		*GetNameSafe(ObjectPtr), ObjectIndex, *NetRefHandle.ToString()
	);
}

struct FLogContext
{
	// Mandatory parameters
	FNetRefHandleManager* NetRefHandleManager = nullptr;
	const TArray<FRootObjectData>& RootObjectArray;

	// Optional parameters
	TFunction<FString(FInternalNetRefIndex ObjectIndex)> OptionalObjectPrint;

	// Stats
	uint32 NumRootObjects = 0;
	uint32 NumSubObjects = 0;
};

void LogRootObjectList(FLogContext& LogContext, bool bLogSubObjects)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager* NetRefHandleManager = LogContext.NetRefHandleManager;

	for (const FRootObjectData& RootObject : LogContext.RootObjectArray)
	{
		UE_LOGF(LogIris, Display, "%ls %ls", *PrintNetObject(NetRefHandleManager, RootObject.ObjectIndex), LogContext.OptionalObjectPrint ? *LogContext.OptionalObjectPrint(RootObject.ObjectIndex) : TEXT(""));

		LogContext.NumRootObjects++;

		if (bLogSubObjects)
		{
			TArrayView<const FInternalNetRefIndex> SubObjects = NetRefHandleManager->GetSubObjects(RootObject.ObjectIndex);
			for (FInternalNetRefIndex SubObjectIndex : SubObjects)
			{
				UE_LOGF(LogIris, Display, "\t%ls %ls", *PrintNetObject(NetRefHandleManager, SubObjectIndex), LogContext.OptionalObjectPrint ? *LogContext.OptionalObjectPrint(SubObjectIndex) : TEXT(""));

				LogContext.NumSubObjects++;
			}
		}
	};
}

void LogViaTrait(FLogContext& LogContext, EPrintTraits ArgTraits, EPrintTraits DefaultTraits)
{
	EPrintTraits SelectedTrait = ArgTraits & EPrintTraits::LogTraits;
	if (SelectedTrait == EPrintTraits::Default)
	{
		SelectedTrait = DefaultTraits & EPrintTraits::LogTraits;
	}

	const bool bLogSubObjects = (SelectedTrait & EPrintTraits::LogSubObjects) != EPrintTraits::Default;
	LogRootObjectList(LogContext, bLogSubObjects);	
}

/** 
 * Returns a list of NetRefHandles of replicated objects from console arguments
 */
TArray<FNetRefHandle> FindNetRefHandlesFromArg(UReplicationSystem* RepSystem, const TArray<FString>& Args)
{
	//TODO: Add more ways to find objects FindByInternalIndex, FindByPtr, etc.

	TArray<FNetRefHandle> NetRefHandles;
	 
	const FNetRefHandleManager&  NetRefHandleManager = RepSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();

	if (const FString* FindByClass = Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("FindByClass=")); }))
	{
		FString ClassNameToFind;
		FParse::Value(**FindByClass, TEXT("FindByClass="), ClassNameToFind);

		for (const auto& MapIt : NetRefHandleManager.GetReplicatedHandles())
		{
			const FInternalNetRefIndex InternalIndex = NetRefHandleManager.GetInternalIndex(MapIt.Key);
			UObject* ReplicatedObject = NetRefHandleManager.GetReplicatedObjectInstance(InternalIndex);
			if (IsValid(ReplicatedObject))
			{
				for (const UClass* Class = ReplicatedObject->GetClass(); Class; Class = Class->GetSuperClass())
				{
					const FString ClassName = Class->GetPathName();
					if (ClassName.Contains(ClassNameToFind))
					{
						// Found it
						NetRefHandles.Emplace(MapIt.Key);
						break;
					}
				}
			}
		}
	}

	if (const FString* FindByName = Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("FindByName=")); }))
	{
		FString ObjectNameToFind;
		FParse::Value(**FindByName, TEXT("FindByName="), ObjectNameToFind);

		TArray<UObject*> FoundObjects;

		for (FThreadSafeObjectIterator It; It; ++It)
		{
			UObject* Object = *It;

			if (!IsValid(Object) || Object->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
			{
				continue;
			}

			const FString FullName = Object->GetName();
			if (FullName.Contains(ObjectNameToFind))
			{
				FoundObjects.Emplace(Object);
			}
		}

		// Add the NetRefHandles of replicated objects matching the name
		for (UObject* FoundObject : FoundObjects)
		{
			const FNetRefHandle NetHandle = RepSystem->GetReplicationBridge()->GetReplicatedRefHandle(FoundObject);
			if (NetHandle.IsValid())
			{
				NetRefHandles.Emplace(NetHandle);
			}
			else
			{
				// TODO: add a flag to export objects that are not replicated but name match
			}
		}
	}
	
	if (const FString* FindById = Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("FindById=")); }))
	{
		FString IdSubString;
		constexpr bool bShouldStopOnSeperator = false;
		FParse::Value(**FindById, TEXT("FindById="), IdSubString, bShouldStopOnSeperator);

		TArray<FString> StringIds;
		IdSubString.ParseIntoArray(StringIds, TEXT(","));

		for (FString& StringId : StringIds)
		{
			StringId.TrimStartAndEndInline();
			const int64 HandleId = FCString::Atoi64(*StringId);
			const FNetRefHandle Handle = FNetRefHandleManager::MakeNetRefHandle(HandleId, RepSystem->GetId());
			NetRefHandles.Emplace(Handle);
		}
	}

	return NetRefHandles;
}

} // end namespace UE::Net::Private::ObjectBridgeDebugging

// --------------------------------------------------------------------------------------------------------------------------------------------
// Debug commands
// --------------------------------------------------------------------------------------------------------------------------------------------

//-----------------------------------------------
FAutoConsoleCommand ObjectBridgePrintDynamicFilterConfig(
	TEXT("Net.Iris.PrintDynamicFilterClassConfig"), 
	TEXT("Prints the dynamic filter configured to be assigned to specific classes."), 
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	UReplicationSystem* RepSystem = FindReplicationSystemFromArg(Args);
	if (!RepSystem)
	{
		UE_LOGF(LogIris, Error, "Could not find ReplicationSystem.");
		return;
	}

	UObjectReplicationBridge* ObjectBridge = RepSystem->GetReplicationBridge();
	if (!ObjectBridge)
	{
		UE_LOGF(LogIris, Error, "Could not find ObjectReplicationBridge.");
		return;
	}

	const EDynamicFilterPrintTraits PrintTraits = EDynamicFilterPrintTraits::Config | FindDynamicFilterPrintTraitsFromArgs(Args);
	ObjectBridge->PrintDynamicFilterClassConfig(static_cast<uint32>(PrintTraits));
}));

FAutoConsoleCommand ObjectBridgePrintDynamicFilterConfigIssues(
	TEXT("Net.Iris.PrintDynamicFilterClassConfigIssues"), 
	TEXT("Prints potential issues with the class filter config. Optional argument SkipBPIssueDetection will ignore blueprint classes. Optional argument SkipActorChildIssueDetection will ignore issues related to classes whose nearest configured super class is Actor."), 
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	UReplicationSystem* RepSystem = FindReplicationSystemFromArg(Args);
	if (!RepSystem)
	{
		UE_LOGF(LogIris, Error, "Could not find ReplicationSystem.");
		return;
	}

	UObjectReplicationBridge* ObjectBridge = RepSystem->GetReplicationBridge();
	if (!ObjectBridge)
	{
		UE_LOGF(LogIris, Error, "Could not find ObjectReplicationBridge.");
		return;
	}

	const EDynamicFilterPrintTraits PrintTraits = EDynamicFilterPrintTraits::IssueDetection | FindDynamicFilterPrintTraitsFromArgs(Args);
	ObjectBridge->PrintDynamicFilterClassConfig(static_cast<uint32>(PrintTraits));
}));

void UObjectReplicationBridge::PrintDynamicFilterClassConfig(uint32 ArgTraits)
{
	using namespace UE::Net;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	const UReplicationSystem* RepSystem = GetReplicationSystem();

	UE_LOGF(LogIrisFilterConfig, Display, "");

	const EDynamicFilterPrintTraits PrintTraits = static_cast<EDynamicFilterPrintTraits>(ArgTraits);
	if (EnumHasAnyFlags(PrintTraits, EDynamicFilterPrintTraits::Config))
	{
		UE_LOGF(LogIrisFilterConfig, Display, "### Begin Default Dynamic Filter Class Config ###");
		{
			TMap<FName, FClassFilterInfo> SortedClassConfig = ClassesWithDynamicFilter;

			SortedClassConfig.KeyStableSort([](FName lhs, FName rhs) { return lhs.Compare(rhs) < 0; });
			for (auto MapIt = SortedClassConfig.CreateConstIterator(); MapIt; ++MapIt)
			{
				const FName ClassName = MapIt.Key();
				const FClassFilterInfo FilterInfo = MapIt.Value();

				UE_LOGF(LogIrisFilterConfig, Display, "\t%ls -> %ls", *ClassName.ToString(), *RepSystem->GetFilterName(FilterInfo.FilterHandle).ToString());
			}
		}
		UE_LOGF(LogIrisFilterConfig, Display, "### End Default Dynamic Filter Class Config ###");
	}

	if (EnumHasAnyFlags(PrintTraits, EDynamicFilterPrintTraits::IssueDetection))
	{
		UE_LOGF(LogIrisFilterConfig, Display, "### Begin Dynamic Filter Class Config Issue Detection ###");
		{
			const UObjectReplicationBridgeConfig* BridgeConfig = UObjectReplicationBridgeConfig::GetConfig();

			// Guard ClassesWithDynamicFilter. We want to use regular functions to validate the setup
			TGuardValue<TMap<FName, FClassFilterInfo>> GuardClassesWithDynamicFilter(ClassesWithDynamicFilter, TMap<FName, FClassFilterInfo>());

			// Populate ClassesWithDynamicFilter with the user config
			for (const FObjectReplicationBridgeFilterConfig& FilterConfig : BridgeConfig->GetFilterConfigs())
			{
				FClassFilterInfo FilterInfo;
				FilterInfo.FilterHandle = ReplicationSystem->GetFilterHandle(FilterConfig.DynamicFilterName);
				FilterInfo.FilterProfile = FilterConfig.FilterProfile;
				FilterInfo.bForceEnable = FilterConfig.bForceEnableOnAllInstances;
				ClassesWithDynamicFilter.Add(FilterConfig.ClassName, FilterInfo);
			}

			// Populate a class tree in order to be able to more easily traverse classes top down and stop validating on subclasses with specific filter configs.
			FClassTree FilterConfigClassTree(UObject::StaticClass());
			if (!ClassesWithDynamicFilter.IsEmpty())
			{
				TArray<UObject*> Classes;
				Classes.Reserve(128*1024);
				constexpr bool bIncludeDerivedClasses = true;
				GetObjectsOfClass(UClass::StaticClass(), Classes, bIncludeDerivedClasses, RF_ClassDefaultObject, GetObjectIteratorDefaultInternalExclusionFlags(EInternalObjectFlags::None));
				if (EnumHasAnyFlags(PrintTraits, EDynamicFilterPrintTraits::SkipBPIssueDetection))
				{
					for (UObject* Object : Classes)
					{
						UClass* Class = static_cast<UClass*>(Object);
						if (Class->HasAnyClassFlags(CLASS_Native))
						{
							FilterConfigClassTree.AddClass(Class);
						}
					}
				}
				else
				{
					for (UObject* Object : Classes)
					{
						UClass* Class = static_cast<UClass*>(Object);
						FilterConfigClassTree.AddClass(Class);
					}
				}
			}

			// Iterate over class filter configs and see if we can find issues with it.
			TSet<UClass*> ClassesWithConfig;
			TSet<UClass*> ValidatedClasses;
			TArray<const FClassTree*> ClassesToValidate;
			TArray<const FClassTree*> ClassesToValidateNext;
			const bool bSkipActorChildIssues = EnumHasAnyFlags(PrintTraits, EDynamicFilterPrintTraits::SkipActorChildIssueDetection);

			// Store all classes with config for easy validation termination of subclass trees
			for (const FObjectReplicationBridgeFilterConfig& FilterConfig : BridgeConfig->GetFilterConfigs())
			{
				const FString& ClassName = FilterConfig.ClassName.ToString();

				constexpr UObject* ClassOuter = nullptr;
				UClass* Class = Cast<UClass>(StaticFindObject(UClass::StaticClass(), ClassOuter, ToCStr(ClassName), EFindObjectFlags::ExactClass));
				if (Class)
				{
					ClassesWithConfig.Add(Class);
				}
				else
				{
					UE_LOGF(LogIrisFilterConfig, Warning, "\tCan't find class %ls. Check spelling.", ToCStr(ClassName));
				}
			}

			// Validate classes in top down order
			TArray<UClass*> RootClassProspects;
			bool bAlreadyValidated = false;
			for (UClass* Class : ClassesWithConfig)
			{
				if (ValidatedClasses.Find(Class))
				{
					continue;
				}

				// Iterate over super classes and see if any has an explicit filter config
				RootClassProspects.Reset();
				RootClassProspects.Add(Class);
				for (UClass* SuperClass = Class->GetSuperClass(); SuperClass; SuperClass = SuperClass->GetSuperClass())
				{
					if (ClassesWithConfig.Find(SuperClass))
					{
						RootClassProspects.Add(SuperClass);
					}
				}

				UClass* ClassToValidate = RootClassProspects.Last();
				ValidatedClasses.FindOrAdd(ClassToValidate, &bAlreadyValidated);
				if (bAlreadyValidated)
				{
					continue;
				}

				const FClassTree* ClassTreeToValidate = FilterConfigClassTree.FindNode(ClassToValidate);
				if (ClassTreeToValidate)
				{
					constexpr bool bRecurse = false;
					ClassTreeToValidate->GetChildClasses(ClassesToValidate, bRecurse);
				}

				while (!ClassesToValidate.IsEmpty())
				{
					for (const FClassTree* ChildClassTree : ClassesToValidate)
					{
						if (!ChildClassTree)
						{
							continue;
						}

						UClass* ChildClass = ChildClassTree->GetClass();

						ValidatedClasses.FindOrAdd(ChildClass, &bAlreadyValidated);
						if (bAlreadyValidated)
						{
							continue;
						}

						// Validate classes that are replicated by default
						bool bShouldValidateChildClasses = true;
						if (IsClassReplicatedByDefault(ChildClass))
						{
							constexpr bool bRequireForceEnabled = false;
							FName FilterProfile;
							const FNetObjectFilterHandle ChildClassFilter = GetDynamicFilter(ChildClass, bRequireForceEnabled, FilterProfile);

							// If the child class doesn't have an explicit config make sure its closest parent class with a config uses the same filter
							bShouldValidateChildClasses = ClassesWithConfig.Contains(ChildClass);
							if (!bShouldValidateChildClasses)
							{
								for (UClass* SuperClass = ChildClass->GetSuperClass(); SuperClass; SuperClass = SuperClass->GetSuperClass())
								{
									const FName SuperClassPath = GetConfigClassPathName(SuperClass);
									if (const FClassFilterInfo* SuperClassFilterInfo = ClassesWithDynamicFilter.Find(SuperClassPath))
									{
										if (SuperClassFilterInfo->FilterHandle == ChildClassFilter || (bSkipActorChildIssues && SuperClass->GetFName() == NAME_Actor))
										{
											bShouldValidateChildClasses = true;
										}
										else
										{
											// Log a message and don't recurse into child classes
											UE_LOGF(LogIrisFilterConfig, Display, "Child class %ls uses a different filter than its closest configured super class %ls. Child class filter: %ls Parent class filter: %ls. Recommend adding an explicit filter config.", ToCStr(GetConfigClassPathName(ChildClass).ToString()), ToCStr(SuperClassPath.ToString()), ToCStr(RepSystem->GetFilterName(ChildClassFilter).ToString()), ToCStr(RepSystem->GetFilterName(SuperClassFilterInfo->FilterHandle).ToString()));
										}
										break;
									}
								}
							}
						}

						// Don't traverse subclasses to classes with potential issues.
						if (bShouldValidateChildClasses)
						{
							TArray<const FClassTree*> ChildClasses;
							constexpr bool bRecurse = false;
							ChildClassTree->GetChildClasses(ChildClasses, bRecurse);
							ClassesToValidateNext.Append(ChildClasses);
						}
					}

					Swap(ClassesToValidate, ClassesToValidateNext);
					ClassesToValidateNext.Reset();
				}
			}
		}
		UE_LOGF(LogIrisFilterConfig, Display, "### End Dynamic Filter Class Config Issue Dection ###");
	}
}

//-----------------------------------------------
FAutoConsoleCommand ObjectBridgePrintReplicatedObjects(
	TEXT("Net.Iris.PrintReplicatedObjects"), 
	TEXT("Prints the list of replicated objects registered for replication in Iris"), 
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	UReplicationSystem* RepSystem = FindReplicationSystemFromArg(Args);
	if (RepSystem)
	{
		if (UObjectReplicationBridge* ObjectBridge = RepSystem->GetReplicationBridge())
		{
			EPrintTraits ArgTraits = FindPrintTraitsFromArgs(Args);
			ObjectBridge->PrintReplicatedObjects((uint32)ArgTraits);
		}
	}
}));

void UObjectReplicationBridge::PrintReplicatedObjects(uint32 ArgTraits) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	UE_LOGF(LogIris, Display, "################ Start Printing ALL Replicated Objects ################");
	UE_LOGF(LogIris, Display, "");

	uint32 TotalRootObjects = 0;
	uint32 TotalSubObjects = 0;

	FNetBitArray RootObjects;
	RootObjects.Init(NetRefHandleManager->GetCurrentMaxInternalNetRefIndex());
	FNetBitArrayView RootObjectsView = MakeNetBitArrayView(RootObjects);
	RootObjectsView.Set(NetRefHandleManager->GetGlobalScopableInternalIndices(), FNetBitArrayView::AndNotOp, NetRefHandleManager->GetSubObjectInternalIndicesView());

	TArray<FRootObjectData> RootObjectArray;
	{
		FillRootObjectArrayFromBitArray(RootObjectArray, RootObjectsView, NetRefHandleManager);
		SortViaTrait(RootObjectArray, (EPrintTraits)ArgTraits, EPrintTraits::Default);
	}

	auto PrintClassOrProtocol = [&](FInternalNetRefIndex ObjectIndex) -> FString
	{
		const FNetRefHandleManager::FReplicatedObjectData& NetObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
		UObject* ObjectPtr = NetRefHandleManager->GetReplicatedObjectInstance(ObjectIndex);

		return FString::Printf(TEXT("Class %s"), ObjectPtr ? *(ObjectPtr->GetClass()->GetName()) : NetObjectData.Protocol->DebugName->Name);
	};

	FLogContext LogContext = {.NetRefHandleManager = NetRefHandleManager, .RootObjectArray = RootObjectArray, .OptionalObjectPrint = PrintClassOrProtocol};
	LogViaTrait(LogContext, (EPrintTraits)ArgTraits, EPrintTraits::Default);

	UE_LOGF(LogIris, Display, "");
	UE_LOGF(LogIris, Display, "Printed %u root objects and %u sub objects", LogContext.NumRootObjects, LogContext.NumSubObjects);
	UE_LOGF(LogIris, Display, "################ Stop Printing ALL Replicated Objects ################");
}

//-----------------------------------------------
FAutoConsoleCommand ObjectBridgePrintRelevantObjects(
	TEXT("Net.Iris.PrintRelevantObjects"), 
	TEXT("Prints the list of netobjects currently relevant to any connection"), 
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	if (UReplicationSystem* RepSystem = FindReplicationSystemFromArg(Args))
	{
		if (UObjectReplicationBridge* ObjectBridge = RepSystem->GetReplicationBridge())
		{
			EPrintTraits ArgTraits = FindPrintTraitsFromArgs(Args);
			ObjectBridge->PrintRelevantObjects((uint32)ArgTraits);
		}
	}
}));

void UObjectReplicationBridge::PrintRelevantObjects(uint32 ArgTraits) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	UE_LOGF(LogIris, Display, "################ Start Printing Relevant Objects ################");
	UE_LOGF(LogIris, Display, "");

	FNetBitArray RootObjects;
	RootObjects.Init(NetRefHandleManager->GetCurrentMaxInternalNetRefIndex());
	FNetBitArrayView RootObjectsView = MakeNetBitArrayView(RootObjects);
	RootObjectsView.Set(NetRefHandleManager->GetRelevantObjectsInternalIndices(), FNetBitArrayView::AndNotOp, NetRefHandleManager->GetSubObjectInternalIndicesView());

	TArray<FRootObjectData> RootObjectArray;
	{
		FillRootObjectArrayFromBitArray(RootObjectArray, RootObjectsView, NetRefHandleManager);
		SortViaTrait(RootObjectArray, (EPrintTraits)ArgTraits, EPrintTraits::Default);
	}

	FLogContext LogContext = {.NetRefHandleManager = NetRefHandleManager, .RootObjectArray = RootObjectArray };
	LogViaTrait(LogContext, (EPrintTraits)ArgTraits, EPrintTraits::Default);

	UE_LOGF(LogIris, Display, "");
	UE_LOGF(LogIris, Display, "Printed %u root objects and %u sub objects", LogContext.NumRootObjects, LogContext.NumSubObjects);
	UE_LOGF(LogIris, Display, "################ Stop Printing Relevant Objects ################");
}

//-----------------------------------------------
FAutoConsoleCommand ObjectBridgePrintAlwaysRelevantObjects(
	TEXT("Net.Iris.PrintAlwaysRelevantObjects"),
	TEXT("Prints the list of netobjects always relevant to every connection"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	if (UReplicationSystem* RepSystem = FindReplicationSystemFromArg(Args))
	{
		if (UObjectReplicationBridge* ObjectBridge = RepSystem->GetReplicationBridge())
		{
			EPrintTraits ArgTraits = FindPrintTraitsFromArgs(Args);
			ObjectBridge->PrintAlwaysRelevantObjects((uint32)ArgTraits);
		}
	}
}));

void UObjectReplicationBridge::PrintAlwaysRelevantObjects(uint32 ArgTraits) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();

	UE_LOGF(LogIris, Display, "################ Start Printing Always Relevant Objects ################");
	UE_LOGF(LogIris, Display, "");

	FNetBitArray AlwaysRelevantList;
	AlwaysRelevantList.Init(NetRefHandleManager->GetCurrentMaxInternalNetRefIndex());
	
	ReplicationSystemInternal->GetFiltering().BuildAlwaysRelevantList(MakeNetBitArrayView(AlwaysRelevantList), ReplicationSystemInternal->GetNetRefHandleManager().GetGlobalScopableInternalIndices());

	// Include objects configured with an AlwaysRelveantNetObjectFilter
	{
		const FName AlwaysRelevantFilterName("/Script/IrisCore.AlwaysRelevantNetObjectFilter");
		const UNetObjectFilterDefinitions* FilterDefinitions = GetDefault<UNetObjectFilterDefinitions>();
		for (const FNetObjectFilterDefinition& FilterConfig : FilterDefinitions->GetFilterDefinitions())
		{
			// In theory there can be multiple filters using the AlwaysRelevantNetObjectFilter. While unlikely we do support it.
			if (FilterConfig.ClassName == AlwaysRelevantFilterName)
			{
				FNetBitArray DynamicFilterAlwaysRelevantRelevant;
				DynamicFilterAlwaysRelevantRelevant.Init(NetRefHandleManager->GetCurrentMaxInternalNetRefIndex());
				ReplicationSystemInternal->GetFiltering().BuildObjectsInFilterList(MakeNetBitArrayView(DynamicFilterAlwaysRelevantRelevant), FilterConfig.FilterName);
				MakeNetBitArrayView(AlwaysRelevantList).Combine(MakeNetBitArrayView(DynamicFilterAlwaysRelevantRelevant), FNetBitArrayView::OrOp);
			}
		}
	}

	// Remove subobjects from the list.
	MakeNetBitArrayView(AlwaysRelevantList).Combine(NetRefHandleManager->GetSubObjectInternalIndicesView(), FNetBitArrayView::AndNotOp);

	TArray<FRootObjectData> AlwaysRelevantObjects;
	{
		FillRootObjectArrayFromBitArray(AlwaysRelevantObjects, MakeNetBitArrayView(AlwaysRelevantList), NetRefHandleManager);
		SortViaTrait(AlwaysRelevantObjects, (EPrintTraits)ArgTraits, EPrintTraits::SortByClass);
	}

	FLogContext LogContext = {.NetRefHandleManager=NetRefHandleManager, .RootObjectArray=AlwaysRelevantObjects };
	LogViaTrait(LogContext, (EPrintTraits)ArgTraits, EPrintTraits::Default);

	UE_LOGF(LogIris, Display, "");
	UE_LOGF(LogIris, Display, "Printed %u root objects and %u subobjects", LogContext.NumRootObjects, LogContext.NumSubObjects);
	UE_LOGF(LogIris, Display, "################ Stop Printing Always Relevant Objects ################");
}

//-----------------------------------------------
FAutoConsoleCommand ObjectBridgePrintRelevantObjectsToConnection(
TEXT("Net.Iris.PrintRelevantObjectsToConnection"),
TEXT("Prints the list of replicated objects relevant to a specific connection.")
TEXT(" OptionalParams: WithFilter"),
FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	UReplicationSystem* RepSystem = FindReplicationSystemFromArg(Args);
	if (RepSystem)
	{
		if (UObjectReplicationBridge* ObjectBridge = RepSystem->GetReplicationBridge())
		{
			FReplicationSystemInternal* ReplicationSystemInternal = RepSystem->GetReplicationSystemInternal();

			ObjectBridge->PrintRelevantObjectsForConnections(Args);
		}
	}
}));

void UObjectReplicationBridge::PrintRelevantObjectsForConnections(const TArray<FString>& Args) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();

	FReplicationConnections& Connections = ReplicationSystemInternal->GetConnections();
	const FNetBitArray& ValidConnections = Connections.GetValidConnections();

	const FReplicationFiltering& Filtering = ReplicationSystemInternal->GetFiltering();

	// Default to all connections
	FNetBitArray ConnectionsToPrint;
	ConnectionsToPrint.InitAndCopy(ValidConnections);

	// Filter down the list if users wanted specific connections
	TArray<uint32> RequestedConnectionList = FindConnectionsFromArgs(Args);
	if (RequestedConnectionList.Num())
	{
		ConnectionsToPrint.ClearAllBits();
		for (uint32 ConnectionId : RequestedConnectionList)
		{
			if (ValidConnections.IsBitSet(ConnectionId))
			{
				ConnectionsToPrint.SetBit(ConnectionId);
			}
			else
			{
				UE_LOGF(LogIris, Warning, "UObjectReplicationBridge::PrintRelevantObjectsForConnections ConnectionId: %u is not valid", ConnectionId);
			}
		}
	}

	UE_LOGF(LogIris, Display, "################ Start Printing Relevant Objects of %d Connections ################", ConnectionsToPrint.CountSetBits());
	UE_LOGF(LogIris, Display, "");

	const bool bWithFilterInfo = nullptr != Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("WithFilter")); });

	EPrintTraits ArgTraits = FindPrintTraitsFromArgs(Args);

	ConnectionsToPrint.ForAllSetBits([&](uint32 ConnectionId)
	{
		const FReplicationView& ConnectionViews = Connections.GetReplicationView(ConnectionId);
		FString ViewLocs;
		for (const FReplicationView::FView& UserView : ConnectionViews.Views )
		{
			ViewLocs += FString::Printf(TEXT("%s "), *UserView.Pos.ToCompactString());
		}

		UE_LOGF(LogIris, Display, "");
		UE_LOGF(LogIris, Display, "###### Begin Relevant list of Connection:%u ViewPos:%ls Named: %ls ######", ConnectionId, *ViewLocs, *PrintConnectionInfo(ConnectionId));
		UE_LOGF(LogIris, Display, "");

		FNetBitArray RootObjects;
		RootObjects.Init(NetRefHandleManager->GetCurrentMaxInternalNetRefIndex());
		MakeNetBitArrayView(RootObjects).Set(GetReplicationSystem()->GetReplicationSystemInternal()->GetFiltering().GetRelevantObjectsInScope(ConnectionId), FNetBitArrayView::AndNotOp, NetRefHandleManager->GetSubObjectInternalIndicesView());

		TArray<FRootObjectData> RelevantObjects;
		{
			FillRootObjectArrayFromBitArray(RelevantObjects, MakeNetBitArrayView(RootObjects), NetRefHandleManager);
			SortViaTrait(RelevantObjects, ArgTraits, EPrintTraits::SortByClass);
		}

		auto AddFilterInfo = [&](FInternalNetRefIndex ObjectIndex) -> FString
		{
            // TODO: When printing with subobjects. Try to tell if they are relevant or not to the connection.
			return FString::Printf(TEXT("\t%s"), *Filtering.PrintFilterObjectInfo(ObjectIndex, ConnectionId));
		};

		FLogContext LogContext = {.NetRefHandleManager=NetRefHandleManager, .RootObjectArray=RelevantObjects, .OptionalObjectPrint=AddFilterInfo};
		LogViaTrait(LogContext, ArgTraits, EPrintTraits::Default);

		UE_LOGF(LogIris, Display, "");
		UE_LOGF(LogIris, Display, "###### Stop Relevant list of Connection:%u | Total: %u root objects relevant ######", ConnectionId, LogContext.NumRootObjects);
		UE_LOGF(LogIris, Display, "");
	});

	UE_LOGF(LogIris, Display, "");
	UE_LOGF(LogIris, Display, "################ Stop Printing Relevant Objects of %d Connections ################", ConnectionsToPrint.CountSetBits());
}

//-----------------------------------------------
FAutoConsoleCommand ObjectBridgePrintNetCullDistances(
TEXT("Net.Iris.PrintNetCullDistances"),
TEXT("Prints the list of replicated objects and their current netculldistance. Add -NumClasses=X to limit the printing to the X classes with the largest net cull distances."),
FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	UReplicationSystem* RepSystem = FindReplicationSystemFromArg(Args);
	if (RepSystem)
	{
		if (UObjectReplicationBridge* ObjectBridge = RepSystem->GetReplicationBridge())
		{
			ObjectBridge->PrintNetCullDistances(Args);
		}
	}
}));

void UObjectReplicationBridge::PrintNetCullDistances(const TArray<FString>& Args) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	// Number of classes to print. If 0, print all.
	int32 NumClassesToPrint = 0;
	if (const FString* ClassCountArg = Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("NumClasses=")); }))
	{
		FParse::Value(**ClassCountArg, TEXT("NumClasses="), NumClassesToPrint);
	}

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();
	const FWorldLocations& WorldLocations = ReplicationSystemInternal->GetWorldLocations();

	struct FCullDistanceInfo
	{
		UClass* Class = nullptr;

		// Total replicated root objects of this class
		uint32 NumTotal = 0;

		// Total objects with a cull distance override
		uint32 NumOverrides = 0;

		// Track unique culldistance values for replicated root objects
		TMap<float /*CullDistance*/, uint32 /*ActorCount with culldistance value*/> UniqueCullDistances; 

		const float FindMostUsedCullDistance() const
		{
			float MostUsedCullDistance = 0.0;
			uint32 MostUsedCount = 0;
			for (auto It : UniqueCullDistances)
			{
				if (It.Value >= MostUsedCount)
				{
					MostUsedCount = It.Value;
					MostUsedCullDistance = FMath::Max(It.Key, MostUsedCullDistance);
				}
			}

			return MostUsedCullDistance;
		}
	};

	TMap<UClass*, FCullDistanceInfo> ClassCullDistanceMap;

	FNetBitArray RootObjects;
	RootObjects.InitAndCopy(NetRefHandleManager->GetGlobalScopableInternalIndices());
	
	// Remove objects that didn't register world location info
	MakeNetBitArrayView(RootObjects).Combine(WorldLocations.GetObjectsWithWorldInfo(), FNetBitArrayView::AndOp);

	// Filter down to objects in the GridFilter. Other filters do not use net culling
	{
		FNetBitArray GridFilterList;
		GridFilterList.Init(NetRefHandleManager->GetCurrentMaxInternalNetRefIndex());
		ReplicationSystemInternal->GetFiltering().BuildObjectsInFilterList(MakeNetBitArrayView(GridFilterList), TEXT("Spatial"));
		RootObjects.Combine(GridFilterList, FNetBitArray::AndOp);
	}

	RootObjects.ForAllSetBits([&](uint32 RootObjectIndex)
	{
		if( UObject* RepObj = NetRefHandleManager->GetReplicatedObjectInstance(RootObjectIndex) )
		{
			UClass* RepObjClass = RepObj->GetClass();

			FCullDistanceInfo& Info = ClassCullDistanceMap.FindOrAdd(RepObjClass);
			Info.Class = RepObjClass;
			Info.NumTotal++;

			if (WorldLocations.HasCullDistanceOverride(RootObjectIndex))
			{
				Info.NumOverrides++;
			}

			// Find this object's current net cull distance
			float RootObjectCullDistance = WorldLocations.GetCullDistance(RootObjectIndex);

			uint32& NumUsingCullDistance = Info.UniqueCullDistances.FindOrAdd(RootObjectCullDistance, 0);
			++NumUsingCullDistance;
		}
	});

	EPrintTraits PrintArgs = FindPrintTraitsFromArgs(Args);


	if (PrintArgs == EPrintTraits::Default)
	{
		// Sort from highest to lowest
		ClassCullDistanceMap.ValueSort([](const FCullDistanceInfo& lhs, const FCullDistanceInfo& rhs) 
		{ 
			const float LHSSortingCullDistance = lhs.FindMostUsedCullDistance();
			const float RHSSortingCullDistance = rhs.FindMostUsedCullDistance();
			return LHSSortingCullDistance >= RHSSortingCullDistance;
		});
	}
	else if (PrintArgs == EPrintTraits::SortByClass)
	{
		ClassCullDistanceMap.KeySort([](const UClass& lhs, const UClass& rhs)
		{
				return lhs.GetName() < rhs.GetName();
		});
	}

	UE_LOGF(LogIris, Display, "################ Start Printing NetCullDistance Values ################");
	UE_LOGF(LogIris, Display, "");

	int32 NumClassesPrinted = 0;
	for (auto ClassIt = ClassCullDistanceMap.CreateIterator(); ClassIt; ++ClassIt)
	{
		FCullDistanceInfo& Info = ClassIt.Value();
		UClass* Class = Info.Class;

		UE_LOGF(LogIris, Display, "MostCommon NetCullDistance: %f | Class: %ls | Instances: %u | Overrides: %u", Info.FindMostUsedCullDistance(), *Info.Class->GetName(), Info.NumTotal, Info.NumOverrides);

		Info.UniqueCullDistances.KeySort([](const float& lhs, const float& rhs){ return lhs >= rhs; });
		for (auto DivergentIt = Info.UniqueCullDistances.CreateConstIterator(); DivergentIt; ++DivergentIt)
		{
			UE_LOGF(LogIris, Display, "\tNetCullDistance: %f | UseCount: %d/%d (%.2f%%)", DivergentIt.Key(), DivergentIt.Value(), Info.NumTotal,((float)DivergentIt.Value()/(float)Info.NumTotal)*100.f);
		}

		if (++NumClassesPrinted == NumClassesToPrint)
		{
			break;
		}
	}
	
	UE_LOGF(LogIris, Display, "");
	UE_LOGF(LogIris, Display, "################ Stop Printing NetCullDistance Values ################");
}

//-----------------------------------------------
FAutoConsoleCommand ObjectBridgePrintPushBasedStatuses(
	TEXT("Net.Iris.PrintPushBasedStatuses"), 
	TEXT("Prints the push-based statuses of all classes."), 
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	using namespace UE::Net::Private;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	UReplicationSystem* RepSystem = FindReplicationSystemFromArg(Args);
	if (!RepSystem)
	{
		UE_LOGF(LogIris, Error, "Could not find ReplicationSystem.");
		return;
	}

	UObjectReplicationBridge* ObjectBridge = RepSystem->GetReplicationBridge();
	if (!ObjectBridge)
	{
		UE_LOGF(LogIris, Error, "Could not find ObjectReplicationBridge.");
		return;
	}

	ObjectBridge->PrintPushBasedStatuses();
}));

void UObjectReplicationBridge::PrintPushBasedStatuses() const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FReplicationProtocolManager* ProtocolManager = GetReplicationProtocolManager();
	if (!ProtocolManager)
	{
		UE_LOGF(LogIris, Error, "Could not find ReplicationProtocolManager.");
		return;
	}

	struct FPushBasedInfo
	{
		const UClass* Class = nullptr;
		int32 RefCount = 0;
		bool bIsFullyPushBased = false;
	};

	TArray<FPushBasedInfo> PushBasedInfos;
	ProtocolManager->ForEachProtocol([&](const FReplicationProtocol* Protocol, const FObjectKey TemplateKey)
	{
		UObject* Template = TemplateKey.ResolveObjectPtr();
		if (!Template)
		{
			return;
		}

		for (const FReplicationStateDescriptor* StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
		{
			if (!EnumHasAnyFlags(StateDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness))
			{
				PushBasedInfos.Add({ Template->GetClass(), Protocol->GetRefCount(), false});
				return;
			}
		}

		PushBasedInfos.Add({ Template->GetClass(), Protocol->GetRefCount(), true});
	});

	// Print by push-based status (not push-based first), then by ref count, then by name.
	Algo::Sort(PushBasedInfos, [](const FPushBasedInfo& A, const FPushBasedInfo& B) 
	{
		if (A.bIsFullyPushBased != B.bIsFullyPushBased)
		{
			return B.bIsFullyPushBased;
		}
		else if (A.RefCount != B.RefCount)
		{
			return A.RefCount > B.RefCount;
		}
		return A.Class->GetName() < B.Class->GetName();
	});

	UE_LOGF(LogIris, Display, "################ Start Printing Push-Based Statuses ################");
	UE_LOGF(LogIris, Display, "");

	for (const FPushBasedInfo& Info : PushBasedInfos)
	{
		UE_LOGF(LogIris, Display, "%ls (RefCount: %d) (PushBased: %d)", ToCStr(Info.Class->GetName()), Info.RefCount, (int32)Info.bIsFullyPushBased);
		if (!Info.bIsFullyPushBased)
		{
			UE_LOGF(LogIris, Display, "\tPrinting properties that aren't push-based:");

			TArray<FLifetimeProperty> LifetimeProps;
			LifetimeProps.Reserve(Info.Class->ClassReps.Num());
			Info.Class->GetDefaultObject()->GetLifetimeReplicatedProps(LifetimeProps);
			for (const FLifetimeProperty& LifetimeProp : LifetimeProps)
			{
				if (!LifetimeProp.bIsPushBased && LifetimeProp.Condition != COND_Never)
				{
					const FRepRecord& RepRecord = Info.Class->ClassReps[LifetimeProp.RepIndex];
					const FProperty* Prop = RepRecord.Property;
					UE_LOGF(LogIris, Display, "\t\t%ls", ToCStr(Prop->GetPathName()));
				}
			}
		}
	}

	UE_LOGF(LogIris, Display, "");
	UE_LOGF(LogIris, Display, "################ Stop Printing Push-Based Statuses ################");
}

//-----------------------------------------------
FAutoConsoleCommand ObjectBridgePrintNetInfoOfObject(
	TEXT("Net.Iris.PrintNetInfoOfObject"),
	TEXT("Prints all relevant information about replicated objects and their relevancy status for clients in the session."),
/**
 * Search params
 * FindByName=<object name> : Wildcard match for replicated UObject name.
 * FindByClass=<class name> : Wildcard match the class names (with inheritance).
 * FindById=<id list> : Find the objects via their NetRefHandle Id. ex: '(Id=2010):(RepSystemId=0)' would be 'FindById=2010'
 * 
 * Connection params
 * ConnectionId=<id list> : Filter relevancy status of specific connections
 */
FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	UReplicationSystem* RepSystem = FindReplicationSystemFromArg(Args);
	if (!RepSystem)
	{
		UE_LOGF(LogIris, Error, "Could not find ReplicationSystem.");
		return;
	}

	UObjectReplicationBridge* ObjectBridge = RepSystem->GetReplicationBridge();
	if (!ObjectBridge)
	{
		UE_LOGF(LogIris, Error, "Could not find ObjectReplicationBridge.");
		return;
	}

	const TArray<FNetRefHandle> NetHandles = FindNetRefHandlesFromArg(RepSystem, Args);

	UObjectReplicationBridge::EPrintDebugInfoTraits PrintTraits = UObjectReplicationBridge::EPrintDebugInfoTraits::Default;
	ObjectBridge->PrintDebugInfoForNetRefHandlesAndConnections(NetHandles, Args, PrintTraits);
	
}));

void UObjectReplicationBridge::PrintDebugInfoForNetRefHandlesAndConnections(const TArray<FNetRefHandle>& NetHandles, const TArray<FString>& Args, EPrintDebugInfoTraits PrintTraits)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	FReplicationConnections& Connections = GetReplicationSystem()->GetReplicationSystemInternal()->GetConnections();
	const FNetBitArray& ValidConnections = Connections.GetValidConnections();

	// Default to all connections
	FNetBitArray ConnectionsToPrint(ValidConnections);

	// Filter down the list of specific connections if requested
	{
		const TArray<uint32> RequestedConnectionList = FindConnectionsFromArgs(Args);
		if (!RequestedConnectionList.IsEmpty())
		{
			ConnectionsToPrint.ClearAllBits();

			for (uint32 ConnectionId : RequestedConnectionList)
			{
				ConnectionsToPrint.SetBitValue(ConnectionId, ValidConnections.GetBit(ConnectionId));
			}
		}
	}

	EPrintDebugInfoTraits DebugPrintTraits = (EPrintDebugInfoTraits)PrintTraits;

	ConnectionsToPrint.ForAllSetBits([&](uint32 ConnectionId)
	{
		for (const FNetRefHandle& NetHandle : NetHandles)
		{
			PrintDebugInfoForNetRefHandle(NetHandle, ConnectionId, DebugPrintTraits);
		}

		// Do not print the object state for the other connections since there is only one global state for all
		EnumAddFlags(DebugPrintTraits, EPrintDebugInfoTraits::NoProtocolState);
	});
}


void UObjectReplicationBridge::PrintDebugInfoForNetRefHandle(const FNetRefHandle NetRefHandle, uint32 ConnectionId, EPrintDebugInfoTraits PrintTraits) const
{
	using namespace UE::Net::Private;
	using namespace UE::Net;
	using namespace UE::Net::Private::ObjectBridgeDebugging;

	FReplicationSystemInternal* ReplicationSystemInternal = GetReplicationSystem()->GetReplicationSystemInternal();

	const FWorldLocations& WorldLocations = ReplicationSystemInternal->GetWorldLocations();
	const FReplicationConnections& Connections = ReplicationSystemInternal->GetConnections();
	const FReplicationPrioritization& Prioritization = ReplicationSystemInternal->GetPrioritization();
	const FDeltaCompressionBaselineManager& DeltaManager = ReplicationSystemInternal->GetDeltaCompressionBaselineManager();
	const FReplicationFiltering& Filtering = ReplicationSystemInternal->GetFiltering();

	const FInternalNetRefIndex ObjectIndex = NetRefHandleManager->GetInternalIndex(NetRefHandle);

	UE_LOGF(LogIris, Display, "################ Start Printing debug info for: %ls ################", *NetRefHandleManager->PrintObjectFromNetRefHandle(NetRefHandle));
	UE_LOGF(LogIris, Display, "");

#if !UE_BUILD_SHIPPING
	if (!EnumHasAnyFlags(PrintTraits, EPrintDebugInfoTraits::NoProtocolState))
	{
		TStringBuilder<4096> StringBuilder;
		IrisDebugHelper::NetObjectStateToString(StringBuilder, NetRefHandle);

		// Protocol state
		UE_LOGF(LogIris, Display, "\tNetObjectState: %ls", *StringBuilder);
	}
#endif

	UE_LOGF(LogIris, Display, "");
	UE_LOGF(LogIris, Display, "For Connection: %ls", *PrintConnectionInfo(ConnectionId))
	UE_LOGF(LogIris, Display, "");

	if (GetReplicationSystem()->IsServer())
	{
		// Connection info
		if (const FReplicationConnection* Connection = Connections.GetConnection(ConnectionId))
		{
			if (Filtering.GetRelevantObjectsInScope(ConnectionId).IsBitSet(ObjectIndex))
			{
				UE_LOGF(LogIris, Display, "\tRelevant to connection: %d", ConnectionId);
			}
			else
			{
				UE_LOGF(LogIris, Display, "\tNot relevant to connection: %d", ConnectionId);
			}

			if (const FReplicationWriter* ReplicationWriter = Connection->ReplicationWriter)
			{
				// Replication Writer Info
				UE_LOGF(LogIris, Display, "\tReplicationWriter: %ls", *ReplicationWriter->PrintObjectInfo(ObjectIndex));
			}
			else
			{
				UE_LOGF(LogIris, Display, "\tReplicationWriter: not found ?");
			}

			// Prioritizer
			UE_LOGF(LogIris, Display, "\tPrioritizer: Priority: %f", Prioritization.GetObjectPriorityForConnection(ConnectionId, ObjectIndex));

			// DeltaCompression info
			if (DeltaManager.GetDeltaCompressionStatus(ObjectIndex) == ENetObjectDeltaCompressionStatus::Allow)
			{
				UE_LOGF(LogIris, Display, "\tDeltaCompression: %ls", *DeltaManager.PrintDeltaCompressionStatus(ConnectionId, ObjectIndex));
			}
			else
			{
				UE_LOGF(LogIris, Display, "\tDeltaCompression: not enabled");
			}
		}
		else
		{
			UE_LOGF(LogIris, Display, "\tNo valid connection with id: %u", ConnectionId);
		}

		// World location
		if (const TOptional<FWorldLocations::FWorldInfo> WorldInfo = WorldLocations.GetWorldInfo(ObjectIndex))
		{
			UE_LOGF(LogIris, Display, "\tWorldLocation: %ls | CullDistance: %f", *WorldInfo->WorldLocation.ToCompactString(), WorldInfo->CullDistance);
		}
		else
		{
			UE_LOGF(LogIris, Display, "\tWorldLocation: None");
		}

		// Filter Info
		UE_LOGF(LogIris, Display, "\tFilterInfo: %ls", *Filtering.PrintFilterObjectInfo(ObjectIndex, ConnectionId));

		// Child Dependents
		{
			const TArrayView<const FDependentObjectInfo> DependentInfos = NetRefHandleManager->GetDependentObjectInfos(ObjectIndex);
			
			UE_CLOGF(!DependentInfos.IsEmpty(), LogIris, Display, "\tChildDependents: %d childs", DependentInfos.Num());
			for (int32 i=0; i < DependentInfos.Num(); ++i)
			{
				UE_LOGF(LogIris, Display, "\t\t[%i] %ls (%ls)", i, *NetRefHandleManager->PrintObjectFromIndex(DependentInfos[i].NetRefIndex), LexToString(DependentInfos[i].SchedulingHint));
			}

		}

		// Parent Dependents
		{
			const TArrayView<const FInternalNetRefIndex> ParentDependents = NetRefHandleManager->GetDependentObjectParents(ObjectIndex);
			UE_CLOGF(!ParentDependents.IsEmpty(), LogIris, Display, "\tParentDependents: %d parents", ParentDependents.Num());
			for (int32 i = 0; i < ParentDependents.Num(); ++i)
			{
				UE_LOGF(LogIris, Display, "\t\t[%i] %ls", i, *NetRefHandleManager->PrintObjectFromIndex(ParentDependents[i]));
			}
		}

		// Creation dependencies
		{
			TConstArrayView<const FInternalNetRefIndex> CreationDependencies = NetRefHandleManager->GetCreationDependencies(ObjectIndex);
			UE_CLOGF(!CreationDependencies.IsEmpty(), LogIris, Display, "\tCreationDependencies: %d parents", CreationDependencies.Num());
			for (int32 i=0; i < CreationDependencies.Num(); i++)
			{
				UE_LOGF(LogIris, Display, "\t\t[%i] %ls", i, *NetRefHandleManager->PrintObjectFromIndex(CreationDependencies[i]));
			}
		}
	}
	else
	{
		if (const FReplicationConnection* Connection = Connections.GetConnection(ConnectionId))
		{
			if (const FReplicationReader* ReplicationReader = Connection->ReplicationReader)
			{
				UE_LOGF(LogIris, Display, "\tReplicationReader: %ls", *ReplicationReader->PrintObjectInfo(ObjectIndex, NetRefHandle));
			}
			else
			{
				UE_LOGF(LogIris, Display, "\tReplicationReader not found ?");
			}
		}
		else
		{
			UE_LOGF(LogIris, Display, "\tNo valid connection with id: %u", ConnectionId);
		}
	}
	
	UE_LOGF(LogIris, Display, "");
	UE_LOGF(LogIris, Display, "################ Stopped Printing debug info for: %ls ################", *NetRefHandleManager->PrintObjectFromNetRefHandle(NetRefHandle));

	// Log dependents if requested
	if (GetReplicationSystem()->IsServer() && EnumHasAnyFlags(PrintTraits, EPrintDebugInfoTraits::WithDependents))
	{
		const TArrayView<const FDependentObjectInfo> ChildDependents = NetRefHandleManager->GetDependentObjectInfos(ObjectIndex);
		for (const FDependentObjectInfo& DependentInfo : ChildDependents)
		{
			UE_LOGF(LogIris, Display, "");
			UE_LOGF(LogIris, Display, "################ Start printing dependent %ls of parent %ls ################", *NetRefHandleManager->PrintObjectFromIndex(DependentInfo.NetRefIndex), *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex));
			UE_LOGF(LogIris, Display, "");
			PrintDebugInfoForNetRefHandle(NetRefHandleManager->GetNetRefHandleFromInternalIndex(DependentInfo.NetRefIndex), ConnectionId, PrintTraits);
		}
	}
}