// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/ObjectReferenceTypes.h"

#include "Templates/UniquePtr.h"

namespace UE::Net::Private
{

enum EReplicationDataStreamDebugFeatures : uint32;

}

namespace UE::Net::Private
{
// Queued data chunk
struct FQueuedDataChunk
{
	FQueuedDataChunk()
	: StorageOffset(0U)
	, NumBits(0U)
	, bHasBatchOwnerData(0U)
	, bIsEndReplicationChunk(0U)
	{
	}

	uint32 StorageOffset;
	uint32 NumBits : 30;
	uint32 bHasBatchOwnerData : 1;
	uint32 bIsEndReplicationChunk : 1;
	EReplicationDataStreamDebugFeatures StreamDebugFeatures = static_cast<EReplicationDataStreamDebugFeatures>(0U);
};

// Struct to contain storage and required data for queued batches pending must be mapped references
struct FPendingBatchData
{
	// We use a single array to store the actual data, it will grow if required.
	TArray<uint32, TInlineAllocator<32>> DataChunkStorage;		
	TArray<FQueuedDataChunk, TInlineAllocator<4>> QueuedDataChunks;

	// The MustBeMapped references that are still not resolved
	TArray<FNetRefHandle, TInlineAllocator<4>> PendingMustBeMappedReferences;

	// Resolved references for which we have are holding on to references to avoid GC
	TArray<FNetRefHandle, TInlineAllocator<4>> ResolvedReferences;

	// Owner of the queued data chunks
	FNetRefHandle Owner;

	// The list of parents that must exist before the owner can be created
	TArray<FNetRefHandle, TInlineAllocator<4>> CreationDependentParents;

	// Time when we started to accumulate data for this object
	uint64 PendingBatchStartCycles = 0;

	// At what time should we warn about being blocked too long
	double NextWarningTimeout = 0.0;
	
	// Incremented every time we try to process queued batches, reset each time we output warning.
	int32 PendingBatchTryProcessCount = 0;

	// The async loading priority of the Owner
	EIrisAsyncLoadingPriority IrisAsyncLoadingPriority = EIrisAsyncLoadingPriority::Invalid;
};

typedef TUniquePtr<FPendingBatchData> FPendingBatchDataPtr;

struct FPendingBatchHolder
{
public:

	bool Contains(FNetRefHandle NetRefHandle) const
	{
		return PendingBatches.Contains(NetRefHandle);
	}

	FPendingBatchData* Find(FNetRefHandle NetRefHandle)
	{
		FPendingBatchDataPtr* Ptr = PendingBatches.Find(NetRefHandle);
		return Ptr ? Ptr->Get() : nullptr;
	}

	const FPendingBatchData* Find(FNetRefHandle NetRefHandle) const
	{
		const FPendingBatchDataPtr* Ptr = PendingBatches.Find(NetRefHandle);
		return Ptr ? Ptr->Get() : nullptr;
	}

	FPendingBatchData* FindOrCreate(FNetRefHandle NetRefHandle)
	{
		FPendingBatchDataPtr* Ptr = PendingBatches.Find(NetRefHandle);
		return Ptr ? Ptr->Get() : CreatePendingBatch(NetRefHandle);
	}

	void Remove(FNetRefHandle NetRefHandle)
	{
		PendingBatches.Remove(NetRefHandle);
	}
	
	bool IsEmpty() const
	{
		return PendingBatches.IsEmpty();
	}

	int32 Num() const
	{
		return PendingBatches.Num();
	}

	auto CreateConstIterator() const
	{
		return PendingBatches.CreateConstIterator();
	}

	void Empty()
	{
		PendingBatches.Empty();
	}

private:

	FPendingBatchData* CreatePendingBatch(FNetRefHandle Owner);

	TMap<FNetRefHandle, FPendingBatchDataPtr> PendingBatches;
};

} // namespace UE::Net::Private

