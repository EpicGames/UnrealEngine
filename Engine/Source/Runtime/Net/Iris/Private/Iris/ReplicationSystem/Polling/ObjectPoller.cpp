// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectPoller.h"

#include "Async/AsyncWork.h"
#include "Async/TaskGraphInterfaces.h"

#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationState/ReplicationStateStorage.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/LegacyPushModel.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"

#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisCsv.h"
#include "Iris/Stats/NetStatsContext.h"

#include "Net/Core/DirtyNetObjectTracker/GlobalDirtyNetObjectTracker.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "HAL/IConsoleManager.h"

#include "Tasks/Task.h"

#include "VerseVM/VVMContext.h"

namespace UE::Net::Private
{
namespace CVars
{
	static bool bFilterOutNonDirtyPushBasedObjects = true;
	static FAutoConsoleVariableRef CVarFilterOutNonDirtyPushBasedObjects(
		TEXT("net.Iris.Poll.FilterOutNonDirtyPushBasedObjects"),
		bFilterOutNonDirtyPushBasedObjects,
		TEXT("When true fully push based objects that are not considered to require polling are masked off before the poll loop.")
	);

	static bool bEnableVerbosePollProfiling = false;
	static FAutoConsoleVariableRef CVarEnableVerbosePollProfiling(
		TEXT("net.Iris.Poll.EnableVerboseProfiling"),
		bEnableVerbosePollProfiling,
		TEXT("Enable conditional profiler scopes for polling. Useful to see exactly what properties gets polled in cpu profiler..")
	);

	static int32 GNumPollingTasks = 32;
	static FAutoConsoleVariableRef CvarNumPollingTasks(
		TEXT("net.Iris.NumPollingTasks"),
		GNumPollingTasks,
		TEXT("Number of Tasks that will be created to poll dirty objects in a single frame. Setting to 0 will run Polling Synchronously instead.")
	);

	static int32 GNumWordsPerChunk = 16;
	static FAutoConsoleVariableRef CvarNumWordsToInterleave(
		TEXT("net.Iris.NumWordsToInterleave"),
		GNumWordsPerChunk,
		TEXT("Number of words in a row to interleave before queuing to the next task")
	);
}

	struct FReplicationPollTaskData
	{
		const FNetBitArrayView ObjectsConsideredForPolling; // Represents the entire set of objects to poll across all tasks, and this Task will Poll a subset of them
		FNetTypeStats* NetTypeStats = nullptr; // Task will use NetTypeStats to get an available ChildNetStatsContext to use for the duration of this Task
		FObjectPoller::FPreUpdateAndPollStats PollStats; // Standalone PollStats object which will be combined with the main instance when tasks are joined
		uint32 CurrentTaskIndex = 0; // Unique index per task, must be contiguous as it is used to define which set of chunks will be processed by this Task
		uint32 BitsPerChunk = 0; // Number of bits are processed in a single chunk, should be at least 1 cache line to avoid false sharing
		uint32 ChunkStride = 0; // When a Chunk has been processed, we move on by ChunkStride which skips over the bits processed by the other tasks
		bool bUsePushModel = false; // Selects between PushModelPollObject and ForcePollObject

		FReplicationPollTaskData(const FNetBitArrayView& ObjectsConsideredForPollingIn)
		: ObjectsConsideredForPolling(ObjectsConsideredForPollingIn)
		{}
	};

	class FReplicationPollTask
	{
	protected:
		FObjectPoller* Poller = nullptr;
		FReplicationPollTaskData& PollTaskData;
		FReplicationSystemInternal* ReplicationSystemInternal = nullptr;

	public:
		

		FReplicationPollTask(
			FObjectPoller* InPoller,
			FReplicationPollTaskData& InPollTaskData,
			FReplicationSystemInternal* InReplicationSystemInternal
		)
			: Poller(InPoller)
			, PollTaskData(InPollTaskData)
			, ReplicationSystemInternal(InReplicationSystemInternal)
		{
		}

		void DoTask() const
		{
			check(ReplicationSystemInternal->IsInParallelPhase());

			IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationPollAndCopyTask);

			IRIS_PROFILER_SCOPE_CONDITIONAL(PollAndCopyPushBased, PollTaskData.bUsePushModel);
			IRIS_PROFILER_SCOPE_CONDITIONAL(ForcePollAndCopy, !PollTaskData.bUsePushModel);

			FNetStatsContext* ChildNetStatsContext = PollTaskData.NetTypeStats->AcquireChildNetStatsContext();

			//Set up the lambda to use for polling, selecting a different one for PushModel or ForcePoll
			auto PushModelPoll = [this, ChildNetStatsContext](uint32 ObjectIndex)
				{
					Poller->PushModelPollObject(ObjectIndex, ChildNetStatsContext, PollTaskData.PollStats);
				};
			auto ForcePoll = [this, ChildNetStatsContext](uint32 ObjectIndex)
				{
					Poller->ForcePollObject(ObjectIndex, ChildNetStatsContext, PollTaskData.PollStats);
				};

			// Named TFunctionRefs needed to give the ternary a common type
			TFunctionRef<void(uint32 ObjectIndex)> PushModelPollFunc = PushModelPoll;
			TFunctionRef<void(uint32 ObjectIndex)> ForcePollFunc = ForcePoll;
			TFunctionRef<void(uint32 ObjectIndex)> PollingLambdaFunc = PollTaskData.bUsePushModel ? PushModelPollFunc : ForcePollFunc;

			//   For each Task, we want to take the objects list and process it based on
			//Chunks of a specific size, and interleave these Chunks between tasks
			//   This is more balanced than dividing into contiguous chunks as the object bit array
			//ends up with long sets of 0 values that would result in some very short tasks
			uint32 CurrentBit = PollTaskData.CurrentTaskIndex * PollTaskData.BitsPerChunk;

			while (CurrentBit < PollTaskData.ObjectsConsideredForPolling.GetNumBits())
			{
				const uint32 ObjectIndexRangeStart = CurrentBit;
				uint32 ObjectIndexRangeEnd = CurrentBit + PollTaskData.BitsPerChunk - 1;

				if (ObjectIndexRangeEnd >= PollTaskData.ObjectsConsideredForPolling.GetNumBits())
				{
					ObjectIndexRangeEnd = PollTaskData.ObjectsConsideredForPolling.GetNumBits();
				}

				
				PollTaskData.ObjectsConsideredForPolling.ForAllSetBitsInRange(ObjectIndexRangeStart, ObjectIndexRangeEnd, PollingLambdaFunc);
				
				//Move the CurrentBit on by ChunkStride so we access the next part of the array which is designated for this Task to process
				CurrentBit += PollTaskData.ChunkStride;
			}

			PollTaskData.NetTypeStats->ReleaseChildNetStatsContext(ChildNetStatsContext);
		}
	};

FObjectPoller::FObjectPoller(const FInitParams& InitParams)
	: ObjectReplicationBridge(InitParams.ObjectReplicationBridge)
	, ReplicationSystemInternal(InitParams.ReplicationSystemInternal)
	, LocalNetRefHandleManager(ReplicationSystemInternal->GetNetRefHandleManager())
	, NetStatsContext(nullptr)
	, ReplicatedInstances(LocalNetRefHandleManager.GetReplicatedInstances())
	, AccumulatedDirtyObjects(ReplicationSystemInternal->GetDirtyNetObjectTracker().GetAccumulatedDirtyNetObjects())
	, DirtyObjectsToQuantize(LocalNetRefHandleManager.GetDirtyObjectsToQuantize())
{
	GarbageCollectionAffectedObjects = MakeNetBitArrayView(ObjectReplicationBridge->GarbageCollectionAffectedObjects);

	// DirtyObjectsThisFrame is acquired only during polling 
	bUsePerPropertyDirtyTracking = FGlobalDirtyNetObjectTracker::IsUsingPerPropertyDirtyTracking();
}

void FObjectPoller::PollAndCopyObjects(const FNetBitArrayView& InObjectsConsideredForPolling)
{
	FDirtyObjectsAccessor DirtyObjectsAccessor(ReplicationSystemInternal->GetDirtyNetObjectTracker());
	DirtyObjectsThisFrame = DirtyObjectsAccessor.GetDirtyNetObjects();

	NetStatsContext = ReplicationSystemInternal->GetNetTypeStats().GetNetStatsContext();

	const bool bIsIrisPushModelEnabled = IsIrisPushModelEnabled();
	
	FNetBitArray TempObjectsToPoll;
	
	// Prepare bitarray with objects to poll, filtering out all fully push based that we can skip.
	const bool bUseFilteredObjectsConsideredForPolling = CVars::bFilterOutNonDirtyPushBasedObjects && bIsIrisPushModelEnabled;
	if (bUseFilteredObjectsConsideredForPolling)
	{
		IRIS_PROFILER_SCOPE(FilterObjectsConsideredForPolling);

		using StorageWordType = UE::Net::FNetBitArrayBase::StorageWordType;

		TempObjectsToPoll.Init(InObjectsConsideredForPolling.GetNumBits());

		const FNetBitArrayView ObjectsWithFullPushBasedDirtiness = LocalNetRefHandleManager.GetObjectsWithFullPushBasedDirtiness();

		const StorageWordType* ObjectsConsideredForPollingData = InObjectsConsideredForPolling.GetData();
		const StorageWordType* ObjectsWithFullPushBasedDirtinessData = ObjectsWithFullPushBasedDirtiness.GetData();
		const StorageWordType* DirtyObjectsThisFrameData = DirtyObjectsThisFrame.GetData();
		const StorageWordType* AccumulatedDirtyObjectsData = AccumulatedDirtyObjects.GetData();
		const StorageWordType* GarbageCollectionAffectedObjectsData = GarbageCollectionAffectedObjects.GetData();
		StorageWordType* TempObjectsToPollData = TempObjectsToPoll.GetData();

		// Build a bitarray for all objects that we need to poll, masking off all objects we can skip (fully push based, not dirty, and not affected by GC).
		// Assumes that "new"-objects are properly marked as dirty.
		for (uint32 WordIt = 0U; WordIt < InObjectsConsideredForPolling.GetNumWords(); ++WordIt)
		{
			const FNetBitArrayBase::StorageWordType SkippableObjects = ObjectsWithFullPushBasedDirtinessData[WordIt] & ~(DirtyObjectsThisFrameData[WordIt] | AccumulatedDirtyObjectsData[WordIt] | GarbageCollectionAffectedObjectsData[WordIt]);
			TempObjectsToPollData[WordIt] = ObjectsConsideredForPollingData[WordIt] & ~SkippableObjects;
		}
	}

	// Pick filtered or non-filtered polling list.
	const FNetBitArrayView ObjectsConsideredForPolling = bUseFilteredObjectsConsideredForPolling ? UE::Net::MakeNetBitArrayView(TempObjectsToPoll, FNetBitArrayBase::NoResetNoValidate) : InObjectsConsideredForPolling;

	// We need to split the work somewhat evenly across multiple tasks, so we define a Chunk as
	// a number of contiguous words (each 32-bit word represents 32 objects), so 16 words per chunk 
	// is 64-bytes and should fill out a whole cache line on standard platforms.
	// The Chunk size must be small enough to result in at least one chunk per task, otherwise fall back to synchronous
	const uint32 NumPollingTasks = CVars::GNumPollingTasks;
	const uint32 BitsPerChunk = CVars::GNumWordsPerChunk * FNetBitArray::WordBitCount;
	const bool bEachTaskHasAChunkOfWork = (BitsPerChunk * NumPollingTasks) < ObjectsConsideredForPolling.GetNumBits();
	
	if (ReplicationSystemInternal->AreParallelTasksAllowed() && bEachTaskHasAChunkOfWork && NumPollingTasks > 0)
	{
		ReplicationSystemInternal->StartParallelPhase();

		TArray<UE::Tasks::TTask<void>> TasksToComplete;
		TArray<FReplicationPollTaskData> TaskData;
		TaskData.Init(FReplicationPollTaskData(ObjectsConsideredForPolling), NumPollingTasks);

		for (uint32 CurrentTaskIndex = 0; CurrentTaskIndex < NumPollingTasks; CurrentTaskIndex++)
		{
			IRIS_PROFILER_SCOPE(PrepareAndDispatchTask);
			
			FReplicationPollTaskData& Data = TaskData[CurrentTaskIndex];
			Data.bUsePushModel = bIsIrisPushModelEnabled;//If pushmodel is disabled, call ForcePollObject instead
			Data.NetTypeStats = &ReplicationSystemInternal->GetNetTypeStats();
			Data.CurrentTaskIndex = CurrentTaskIndex;
			Data.BitsPerChunk = BitsPerChunk;
			Data.ChunkStride = NumPollingTasks * BitsPerChunk;//How many bits should the task skip to get to the next Chunk it owns?

			{
				IRIS_PROFILER_SCOPE(UObjectPoller_DispatchTask); 
				
				TasksToComplete.Add(UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &Data]()
					{
						FReplicationPollTask NewTask = FReplicationPollTask(this, Data, ReplicationSystemInternal);
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
						if (IsInGameThread())
#endif
						{
							NewTask.DoTask();
						}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
						else
						{
							// Acquire Verse heap access in case the property data to be copied includes VCell references.
							Verse::FRunningContext::Create([&NewTask](Verse::FRunningContext Context) { NewTask.DoTask(); });
						}
#endif
					}
				));
			}
		}
	
		//This can also run some of the task list on the current thread (GameThread), but only ones from TaskToComplete
		UE::Tasks::Wait(TasksToComplete);

		ReplicationSystemInternal->StopParallelPhase();
	
		//Combine the PollStats from individual tasks into the shared one
		for (const FReplicationPollTaskData& ThisTaskData : TaskData)
		{
			PollStats.Accumulate(ThisTaskData.PollStats);
		} 
	}
	else
	{
		//Synchronous, non-task based polling of all objects considered for polling
		if (IsIrisPushModelEnabled())
		{
			IRIS_PROFILER_SCOPE(PollAndCopyPushBased);

			ObjectsConsideredForPolling.ForAllSetBits([this](FInternalNetRefIndex Objectindex)
			{
				PushModelPollObject(Objectindex, NetStatsContext, PollStats);
			});
		}
		else
		{
			IRIS_PROFILER_SCOPE(ForcePollAndCopy);

			ObjectsConsideredForPolling.ForAllSetBits([this](FInternalNetRefIndex Objectindex)
			{
				ForcePollObject(Objectindex, NetStatsContext, PollStats);
			});
		}
	}

	NetStatsContext = nullptr;
}

void FObjectPoller::PollAndCopySingleObject(FInternalNetRefIndex ObjectIndex)
{
	FDirtyObjectsAccessor DirtyObjectsAccessor(ReplicationSystemInternal->GetDirtyNetObjectTracker());
	DirtyObjectsThisFrame = DirtyObjectsAccessor.GetDirtyNetObjects();

	//We don't run this PollAndCopySingleObject during the parallel Poll phase, so we can use the parent context NetStatsContext and PollStats
	check(!ReplicationSystemInternal->IsInParallelPhase());

	IRIS_PROFILER_SCOPE_VERBOSE(ForcePollAndCopy);
	ForcePollObject(ObjectIndex, NetStatsContext, PollStats);

	// Clear ref to locked dirty bit array
	DirtyObjectsThisFrame = FNetBitArrayView();
	NetStatsContext = nullptr;
}

void FObjectPoller::ForcePollObject(FInternalNetRefIndex ObjectIndex, FNetStatsContext* InNetStatsContext, FPreUpdateAndPollStats& InPollStats)
{
	FNetRefHandleManager::FReplicatedObjectData& ObjectData = LocalNetRefHandleManager.GetReplicatedObjectDataNoCheck(ObjectIndex);
	if (UNLIKELY(ObjectData.InstanceProtocol == nullptr))
	{
		return;
	}

	// We always poll all states here.
	ObjectData.bWantsFullPoll = 0U;

	// Poll properties if the instance protocol requires it
	if (EnumHasAnyFlags(ObjectData.InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::NeedsPoll))
	{
		IRIS_PROFILER_PROTOCOL_NAME_CONDITIONAL(ObjectData.Protocol->DebugName->Name, CVars::bEnableVerbosePollProfiling);
		UE_NET_IRIS_STATS_TIMER(Timer, InNetStatsContext);
		UE_NET_TRACE_POLL_OBJECT_SCOPE(ObjectData.RefHandle, Timer);

		const bool bIsGCAffectedObject = GarbageCollectionAffectedObjects.GetBit(ObjectIndex);
		GarbageCollectionAffectedObjects.ClearBit(ObjectIndex);

		// If this object has been around for a garbage collect and it has object references we must make sure that we update all cached object references
		EReplicationFragmentPollFlags PollOptions = EReplicationFragmentPollFlags::PollAllState;
		PollOptions |= CVars::bEnableVerbosePollProfiling ? EReplicationFragmentPollFlags::EnableVerboseProfiling : EReplicationFragmentPollFlags::None;
		PollOptions |= bIsGCAffectedObject ? EReplicationFragmentPollFlags::ForceRefreshCachedObjectReferencesAfterGC : EReplicationFragmentPollFlags::None;

		const bool bWasAlreadyDirty = DirtyObjectsThisFrame.IsBitSet(ObjectIndex);
		const bool bPollFoundDirty = FReplicationInstanceOperations::PollAndCopyPropertyData(ObjectData.InstanceProtocol, PollOptions);
		if (bWasAlreadyDirty || bPollFoundDirty)
		{
			UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT(Timer, Poll, ObjectIndex);

			DirtyObjectsToQuantize.SetBit(ObjectIndex);
			DirtyObjectsThisFrame.SetBit(ObjectIndex);
		}
		else
		{
			UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT_AS_WASTE(Timer, Poll, ObjectIndex);
			UE_NET_TRACE_POLL_OBJECT_IS_WASTE();
		}
		++InPollStats.PolledObjectCount;
	}
	else
	{
		DirtyObjectsToQuantize.SetBit(ObjectIndex);
		DirtyObjectsThisFrame.SetBit(ObjectIndex);
	}
}

void FObjectPoller::PushModelPollObject(FInternalNetRefIndex ObjectIndex, FNetStatsContext* InNetStatsContext, FPreUpdateAndPollStats& InPollStats)
{
	FNetRefHandleManager::FReplicatedObjectData& ObjectData = LocalNetRefHandleManager.GetReplicatedObjectDataNoCheck(ObjectIndex);
	const FReplicationInstanceProtocol* InstanceProtocol = ObjectData.InstanceProtocol;
	if (UNLIKELY(InstanceProtocol == nullptr))
	{
		return;
	}

	const EReplicationInstanceProtocolTraits InstanceTraits = InstanceProtocol->InstanceTraits;
	const bool bNeedsPoll = EnumHasAnyFlags(InstanceTraits, EReplicationInstanceProtocolTraits::NeedsPoll);

	bool bIsDirtyObject = AccumulatedDirtyObjects.GetBit(ObjectIndex) || DirtyObjectsThisFrame.GetBit(ObjectIndex);

	if (bIsDirtyObject)
	{
		DirtyObjectsToQuantize.SetBit(ObjectIndex);
		DirtyObjectsThisFrame.SetBit(ObjectIndex);
	}

	const bool bIsGCAffectedObject = GarbageCollectionAffectedObjects.GetBit(ObjectIndex);
	GarbageCollectionAffectedObjects.ClearBit(ObjectIndex);

	// Early out if the instance does not require polling
	if (!bNeedsPoll)
	{
		return;
	}

	// Does the object need to poll all states once.
	const bool bWantsFullPoll = ObjectData.bWantsFullPoll;
	ObjectData.bWantsFullPoll = 0U;
	
	// If the object is fully push model we only need to poll it if it's dirty, unless it's a new object or was garbage collected.
	bool bPollFoundDirty = false;
	if (EnumHasAnyFlags(InstanceTraits, EReplicationInstanceProtocolTraits::HasFullPushBasedDirtiness))
	{
		if (bIsDirtyObject || bWantsFullPoll)
		{
			IRIS_PROFILER_PROTOCOL_NAME_CONDITIONAL(ObjectData.Protocol->DebugName->Name, CVars::bEnableVerbosePollProfiling);
			UE_NET_IRIS_STATS_TIMER(Timer, InNetStatsContext);
			UE_NET_TRACE_POLL_OBJECT_SCOPE(ObjectData.RefHandle, Timer);

			// We need to do a poll if object is marked as dirty
			EReplicationFragmentPollFlags PollOptions = bUsePerPropertyDirtyTracking && !bWantsFullPoll ? EReplicationFragmentPollFlags::PollDirtyState : EReplicationFragmentPollFlags::PollAllState;
			PollOptions |= CVars::bEnableVerbosePollProfiling ? EReplicationFragmentPollFlags::EnableVerboseProfiling : EReplicationFragmentPollFlags::None;
			PollOptions |= bIsGCAffectedObject ? EReplicationFragmentPollFlags::ForceRefreshCachedObjectReferencesAfterGC : EReplicationFragmentPollFlags::None;
			bPollFoundDirty = FReplicationInstanceOperations::PollAndCopyPropertyData(InstanceProtocol, EReplicationFragmentTraits::None, PollOptions);
			++InPollStats.PolledObjectCount;

			UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT(Timer, Poll, ObjectIndex);
		}
		else if (bIsGCAffectedObject)
		{
			IRIS_PROFILER_PROTOCOL_NAME_CONDITIONAL(ObjectData.Protocol->DebugName->Name, CVars::bEnableVerbosePollProfiling);
			UE_NET_IRIS_STATS_TIMER(Timer, InNetStatsContext);
			UE_NET_TRACE_POLL_OBJECT_SCOPE(ObjectData.RefHandle, Timer);

			// If this object might have been affected by GC, only refresh cached references
			bPollFoundDirty = FReplicationInstanceOperations::PollAndCopyObjectReferences(InstanceProtocol, EReplicationFragmentTraits::None);
			++InPollStats.PolledReferencesObjectCount;

			UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT(Timer, Poll, ObjectIndex);
		}
		else
		{
			++PollStats.SkippedObjectCount;
		}
	}
	else
	{
		IRIS_PROFILER_PROTOCOL_NAME_CONDITIONAL(ObjectData.Protocol->DebugName->Name, CVars::bEnableVerbosePollProfiling);
		UE_NET_IRIS_STATS_TIMER(Timer, InNetStatsContext);
		UE_NET_TRACE_POLL_OBJECT_SCOPE(ObjectData.RefHandle, Timer);

		// If the object has fragments with pushed based properties, and is not marked dirty and object is affected by GC we need to make sure that we refresh cached references for all fragments with push based properties
		const bool bHasPushBasedFragments = EnumHasAnyFlags(InstanceTraits, EReplicationInstanceProtocolTraits::HasPushBasedDirtiness);
		const bool bHasObjectReferences = EnumHasAnyFlags(InstanceTraits, EReplicationInstanceProtocolTraits::HasObjectReference);
		const bool bNeedsRefreshOfCachedObjectReferences = ((!(bWantsFullPoll | bIsDirtyObject)) & bIsGCAffectedObject & bHasPushBasedFragments & bHasObjectReferences);
		if (bNeedsRefreshOfCachedObjectReferences)
		{
			// Only states which has full push based dirtiness need to be updated as the other states will be at least partially polled anyway.
			const EReplicationFragmentTraits RequiredTraits = EReplicationFragmentTraits::HasFullPushBasedDirtiness;
			bPollFoundDirty = FReplicationInstanceOperations::PollAndCopyObjectReferences(InstanceProtocol, RequiredTraits);
			++InPollStats.PolledReferencesObjectCount;
		}

		// We currently cannot trust changemask for multi-owner instances so we need to poll.
		const bool bIsMultiOwnerInstance = EnumHasAnyFlags(InstanceTraits, EReplicationInstanceProtocolTraits::IsMultiObjectInstance);

		// If this object has been around for a garbage collect and it has object references we must make sure that we update all cached object references 
		EReplicationFragmentPollFlags PollOptions = bUsePerPropertyDirtyTracking && !bWantsFullPoll && !bIsMultiOwnerInstance ? EReplicationFragmentPollFlags::PollDirtyState : EReplicationFragmentPollFlags::PollAllState;
		PollOptions |= CVars::bEnableVerbosePollProfiling ? EReplicationFragmentPollFlags::EnableVerboseProfiling : EReplicationFragmentPollFlags::None;
		PollOptions |= bIsGCAffectedObject ? EReplicationFragmentPollFlags::ForceRefreshCachedObjectReferencesAfterGC : EReplicationFragmentPollFlags::None;

		// If the object is not new or dirty at this point we only need to poll non-fully push based fragments as we know that fully pushed based states have not been modified or have already had their 
		const EReplicationFragmentTraits ExcludeTraits = (bIsDirtyObject || bWantsFullPoll || bIsMultiOwnerInstance) ? EReplicationFragmentTraits::None : EReplicationFragmentTraits::HasFullPushBasedDirtiness;
		bPollFoundDirty |= FReplicationInstanceOperations::PollAndCopyPropertyData(InstanceProtocol, ExcludeTraits, PollOptions);
		++InPollStats.PolledObjectCount;

		if (bPollFoundDirty)
		{
			UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT(Timer, Poll, ObjectIndex);

			DirtyObjectsToQuantize.SetBit(ObjectIndex);
			DirtyObjectsThisFrame.SetBit(ObjectIndex);
		}
		else
		{
			UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT_AS_WASTE(Timer, Poll, ObjectIndex);
			UE_NET_TRACE_POLL_OBJECT_IS_WASTE();
		}
	}
}

} // end namespace UE::Net::Private