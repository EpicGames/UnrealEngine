// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/NetBitArray.h"
#include "Iris/Core/NetChunkedArray.h"

// Forward declarations
class UObjectReplicationBridge;

namespace UE::Net
{
	class FNetRefHandle;

	typedef uint32 FInternalNetRefIndex;

	namespace Private
	{
		class FReplicationSystemInternal;
		class FNetRefHandleManager;
		class FNetStatsContext;
	}
}

namespace UE::Net::Private
{

/** Class that holds the required information needed to execute the poll phase on one or multiple replicated objects. */
class FObjectPoller
{
	friend class FReplicationPollTask;

public:

	/** Holds statistics on how the polling went */
	struct FPreUpdateAndPollStats
	{
		uint32 PreUpdatedObjectCount = 0;
		uint32 PolledObjectCount = 0;
		uint32 SkippedObjectCount = 0;
		uint32 PolledReferencesObjectCount = 0;

		void Accumulate(const FPreUpdateAndPollStats& StatsToAdd)
		{
			PreUpdatedObjectCount += StatsToAdd.PreUpdatedObjectCount;
			PolledObjectCount += StatsToAdd.PolledObjectCount;
			PolledReferencesObjectCount += StatsToAdd.PolledReferencesObjectCount;
		}
	};

	struct FInitParams
	{
		FReplicationSystemInternal* ReplicationSystemInternal = nullptr;
		UObjectReplicationBridge* ObjectReplicationBridge = nullptr;
	};

public:

	FObjectPoller(const FInitParams& InitParams);

	const FPreUpdateAndPollStats& GetPollStats() const { return PollStats; }

	/** Poll all the objects whose bit index is set in the array and copy any dirty data into ReplicationState buffers*/
	void PollAndCopyObjects(const FNetBitArrayView& ObjectsConsideredForPolling);

	/** Poll a single replicated object */
	void PollAndCopySingleObject(FInternalNetRefIndex ObjectIndex);

private:

	/** Polls an object in every circumstance */
	void ForcePollObject(FInternalNetRefIndex ObjectIndex, FNetStatsContext* InNetStatsContext, FPreUpdateAndPollStats& InPollStats);

	/** Polls an object only if it is required or considered dirty */
	void PushModelPollObject(FInternalNetRefIndex ObjectIndex, FNetStatsContext* InNetStatsContext, FPreUpdateAndPollStats& InPollStats);

private:

	UObjectReplicationBridge* ObjectReplicationBridge;
	FReplicationSystemInternal* ReplicationSystemInternal;

	FNetRefHandleManager& LocalNetRefHandleManager;
	FNetStatsContext* NetStatsContext = nullptr;
	const TNetChunkedArray<TObjectPtr<UObject>>& ReplicatedInstances;

	const FNetBitArrayView AccumulatedDirtyObjects;

	FNetBitArrayView DirtyObjectsToQuantize;
	FNetBitArrayView DirtyObjectsThisFrame;
	FNetBitArrayView GarbageCollectionAffectedObjects;

	FPreUpdateAndPollStats PollStats;

	bool bUsePerPropertyDirtyTracking = false;
};

} // end namespace UE::Net::Private

