// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Prioritization/FieldOfViewNetObjectPrioritizer.h"
#include "Iris/Core/IrisCsv.h"
#include "Iris/Core/IrisProfiler.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/MemStack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FieldOfViewNetObjectPrioritizer)

void UFieldOfViewNetObjectPrioritizer::Init(FNetObjectPrioritizerInitParams& Params)
{
	checkf(Params.Config != nullptr, TEXT("Need config to operate."));
	Config = TStrongObjectPtr<UFieldOfViewNetObjectPrioritizerConfig>(CastChecked<UFieldOfViewNetObjectPrioritizerConfig>(Params.Config));

	Super::Init(Params);
}

void UFieldOfViewNetObjectPrioritizer::Deinit()
{
	Super::Deinit();

	Config.Reset();
}

void UFieldOfViewNetObjectPrioritizer::Prioritize(FNetObjectPrioritizationParams& PrioritizationParams)
{
	IRIS_CSV_PROFILER_SCOPE(Iris, UFieldOfViewNetObjectPrioritizer_Prioritize);
	
	FMemStack& Mem = FMemStack::Get();
	FMemMark MemMark(Mem);

	// Trade-off memory/performance
	constexpr uint32 MaxBatchObjectCount = 1024U;

	uint32 BatchObjectCount = FMath::Min((PrioritizationParams.ObjectCount + 3U) & ~3U, MaxBatchObjectCount);
	FBatchParams BatchParams;
	SetupBatchParams(BatchParams, PrioritizationParams, BatchObjectCount, Mem);

	for (uint32 ObjectIt = 0U, ObjectEndIt = PrioritizationParams.ObjectCount; ObjectIt < ObjectEndIt; )
	{
		const uint32 CurrentBatchObjectCount = FMath::Min(ObjectEndIt - ObjectIt, MaxBatchObjectCount);

		BatchParams.ObjectCount = CurrentBatchObjectCount;
		PrepareBatch(BatchParams, PrioritizationParams, ObjectIt);
		PrioritizeBatch(BatchParams);
		FinishBatch(BatchParams, PrioritizationParams, ObjectIt);

		ObjectIt += CurrentBatchObjectCount;
	}
}

void UFieldOfViewNetObjectPrioritizer::PrepareBatch(FBatchParams& BatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset)
{
	IRIS_PROFILER_SCOPE(UFieldOfViewNetObjectPrioritizer_PrepareBatch);
	const float* ExternalPriorities = PrioritizationParams.Priorities;
	const uint32* ObjectIndices = PrioritizationParams.ObjectIndices;
	const FNetObjectPrioritizationInfo* PrioritizationInfos = PrioritizationParams.PrioritizationInfos;

	float* LocalPriorities = BatchParams.Priorities;
	VectorRegister* Positions = BatchParams.Positions;

	// Copy priorities.
	{
		uint32 LocalObjIt = 0;
		for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = ObjIt + BatchParams.ObjectCount; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
		{
			const uint32 ObjectIndex = ObjectIndices[ObjIt];
			LocalPriorities[LocalObjIt] = ExternalPriorities[ObjectIndex];
		}
	}

	// Copy positions. 
	uint32 LocalObjIt = 0;
	{
		for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = ObjIt + BatchParams.ObjectCount; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
		{
			const uint32 ObjectIndex = ObjectIndices[ObjIt];
			const FObjectLocationInfo& Info = static_cast<const FObjectLocationInfo&>(PrioritizationInfos[ObjectIndex]);
			Positions[LocalObjIt] = GetLocation(Info);
		}
	}

	// Make sure we have a multiple of four valid entries.
	for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = (ObjIt + 3U) & ~3U; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
	{
		LocalPriorities[LocalObjIt] = 0.0f;
		Positions[LocalObjIt] = VectorZero();
	}
}

void UFieldOfViewNetObjectPrioritizer::PrioritizeBatch(FBatchParams& BatchParams)
{
	IRIS_PROFILER_SCOPE(UFieldOfViewNetObjectPrioritizer_PrioritizeBatch);

	TArray<VectorRegister, TInlineAllocator<16>> ViewPositions;
	TArray<VectorRegister, TInlineAllocator<16>> ViewDirs;
	for (const UE::Net::FReplicationView::FView& View : BatchParams.View.Views)
	{
		const FVector& ViewPos = View.Pos;
		const FVector& ViewDir = View.Dir;
		ViewPositions.Add(VectorLoadFloat3_W0(&ViewPos));
		ViewDirs.Add(VectorLoadFloat3_W0(&ViewDir));
	}

	const VectorRegister* Positions = BatchParams.Positions;
	float* Priorities = BatchParams.Priorities;
	const int ViewCount = BatchParams.View.Views.Num();
	for (uint32 ObjIt = 0, ObjEndIt = BatchParams.ObjectCount; ObjIt < ObjEndIt; ObjIt += 4)
	{
		VectorRegister Priorities0123 = VectorMax(VectorLoadAligned(Priorities + ObjIt), BatchParams.PriorityCalculationConstants.OutsidePriority);
		// As the cone and line of sight capsule are view direction dependent there's not a lot we can do to optimize multi-view calculations.
		for (int ViewIt = 0, ViewEndIt = ViewCount; ViewIt < ViewEndIt; ++ViewIt)
		{
			const VectorRegister ViewPos = ViewPositions[ViewIt];
			const VectorRegister ViewDir = ViewDirs[ViewIt];
			const VectorRegister ReverseViewDir = VectorNegate(ViewDir);

			// Object directions
			const VectorRegister ObjectDir0 = VectorSubtract(Positions[ObjIt + 0], ViewPos);
			const VectorRegister ObjectDir1 = VectorSubtract(Positions[ObjIt + 1], ViewPos);
			const VectorRegister ObjectDir2 = VectorSubtract(Positions[ObjIt + 2], ViewPos);
			const VectorRegister ObjectDir3 = VectorSubtract(Positions[ObjIt + 3], ViewPos);

			// Calculate squared distances to ViewPos
			const VectorRegister DistSqrToViewPos0 = VectorDot4(ObjectDir0, ObjectDir0);
			const VectorRegister DistSqrToViewPos1 = VectorDot4(ObjectDir1, ObjectDir1);
			const VectorRegister DistSqrToViewPos2 = VectorDot4(ObjectDir2, ObjectDir2);
			const VectorRegister DistSqrToViewPos3 = VectorDot4(ObjectDir3, ObjectDir3);

			// Assemble all distances into a single vector
			const VectorRegister DistSqrToViewPos0101 = VectorSwizzle(VectorCombineHigh(DistSqrToViewPos0, DistSqrToViewPos1), 0, 2, 1, 3);
			const VectorRegister DistSqrToViewPos2323 = VectorSwizzle(VectorCombineHigh(DistSqrToViewPos2, DistSqrToViewPos3), 0, 2, 1, 3);
			const VectorRegister DistSqrToViewPos0123 = VectorCombineHigh(DistSqrToViewPos0101, DistSqrToViewPos2323);
			const VectorRegister DistToViewPos0123 = VectorSqrt(DistSqrToViewPos0123);

			// Project object direction onto cone center axis to retrieve the distance on the cone
			const VectorRegister ConeDist0 = VectorDot4(ObjectDir0, ViewDir);
			const VectorRegister ConeDist1 = VectorDot4(ObjectDir1, ViewDir);
			const VectorRegister ConeDist2 = VectorDot4(ObjectDir2, ViewDir);
			const VectorRegister ConeDist3 = VectorDot4(ObjectDir3, ViewDir);

			// $IRIS TODO Worth doing VectorMaskBits on ConeDistInRangeMask before doing the more expensive cone checks?
			// Calculate the distance to the center axis at the cone distances
			// Simplified VectorSubtract(Positions[ObjIt + N], VectorMultiplyAdd(ConeDistN, ViewDir, ViewPos)) to VectorMultiplyAdd(ConeDistN, -ViewDir, ObjectDirN)
			VectorRegister DistSqrToConeCenterAxis0 = VectorMultiplyAdd(ConeDist0, ReverseViewDir, ObjectDir0);
			VectorRegister DistSqrToConeCenterAxis1 = VectorMultiplyAdd(ConeDist1, ReverseViewDir, ObjectDir1);
			VectorRegister DistSqrToConeCenterAxis2 = VectorMultiplyAdd(ConeDist2, ReverseViewDir, ObjectDir2);
			VectorRegister DistSqrToConeCenterAxis3 = VectorMultiplyAdd(ConeDist3, ReverseViewDir, ObjectDir3);

			DistSqrToConeCenterAxis0 = VectorDot4(DistSqrToConeCenterAxis0, DistSqrToConeCenterAxis0);
			DistSqrToConeCenterAxis1 = VectorDot4(DistSqrToConeCenterAxis1, DistSqrToConeCenterAxis1);
			DistSqrToConeCenterAxis2 = VectorDot4(DistSqrToConeCenterAxis2, DistSqrToConeCenterAxis2);
			DistSqrToConeCenterAxis3 = VectorDot4(DistSqrToConeCenterAxis3, DistSqrToConeCenterAxis3);

			// Assemble all cone distances into a single vector
			const VectorRegister ConeDist0101 = VectorSwizzle(VectorCombineHigh(ConeDist0, ConeDist1), 0, 2, 1, 3);
			const VectorRegister ConeDist2323 = VectorSwizzle(VectorCombineHigh(ConeDist2, ConeDist3), 0, 2, 1, 3);
			const VectorRegister ConeDist0123 = VectorCombineHigh(ConeDist0101, ConeDist2323);

			// Assemble all distances to cone center axis into a single vector
			const VectorRegister DistSqrToConeCenterAxis0101 = VectorSwizzle(VectorCombineHigh(DistSqrToConeCenterAxis0, DistSqrToConeCenterAxis1), 0, 2, 1, 3);
			const VectorRegister DistSqrToConeCenterAxis2323 = VectorSwizzle(VectorCombineHigh(DistSqrToConeCenterAxis2, DistSqrToConeCenterAxis3), 0, 2, 1, 3);
			const VectorRegister DistSqrToConeCenterAxis0123 = VectorCombineHigh(DistSqrToConeCenterAxis0101, DistSqrToConeCenterAxis2323);

			// Validate that the cone distances fall into the valid range [0, OuterConeLength]
			const VectorRegister ConeDistGEZeroMask = VectorCompareGE(ConeDist0123, VectorZeroVectorRegister());
			const VectorRegister ConeDistLEDistMask = VectorCompareLE(ConeDist0123, BatchParams.PriorityCalculationConstants.ConeLength);
			const VectorRegister ConeDistInRangeMask = VectorBitwiseAnd(ConeDistGEZeroMask, ConeDistLEDistMask);

			// Validate that projected points fall within the cone radius at their respective distances
			VectorRegister ConeRadiusAtDistSqr = VectorMultiply(ConeDist0123, BatchParams.PriorityCalculationConstants.ConeRadiusFactor);
			ConeRadiusAtDistSqr = VectorMultiply(ConeRadiusAtDistSqr, ConeRadiusAtDistSqr);

			// Calculate priorities. Go through the different shapes and always take the max of the current priorities and the shape priorities, assuming the object point is inside it.
			// No assumptions are being made with regards to the different shapes' priorities- always take the max of all viable priorities.

			// Cone priorities.
			// Use the distance to the view pos for the cone priorities. It's more correct versus using the distance along the center axis but costs a VectorSqrt.
			// InsideConeMask are for positions inside the cone. However the cone priorities starts getting lower at InnerConeLength. For distances shorter than InnerConeLength the priority is InnerConePriority.
			const VectorRegister InsideConeMask = VectorBitwiseAnd(VectorCompareLE(DistSqrToConeCenterAxis0123, ConeRadiusAtDistSqr), ConeDistInRangeMask);
			const VectorRegister InsideInnerConeMask = VectorCompareLE(ConeDist0123, BatchParams.PriorityCalculationConstants.InnerConeLength);
			const VectorRegister ConeLengthFactor = VectorMultiply(VectorSubtract(DistToViewPos0123, BatchParams.PriorityCalculationConstants.InnerConeLength), BatchParams.PriorityCalculationConstants.InvConeLengthDiff);
			VectorRegister ConePriorities0123 = VectorMultiplyAdd(ConeLengthFactor, BatchParams.PriorityCalculationConstants.ConePriorityDiff, BatchParams.PriorityCalculationConstants.InnerConePriority);
			ConePriorities0123 = VectorSelect(InsideInnerConeMask, BatchParams.PriorityCalculationConstants.InnerConePriority, ConePriorities0123);
			ConePriorities0123 = VectorBitwiseAnd(ConePriorities0123, InsideConeMask);

			// Line of sight priorities.
			const VectorRegister InsideLineOfSightMask = VectorBitwiseAnd(VectorCompareLE(DistSqrToConeCenterAxis0123, BatchParams.PriorityCalculationConstants.LineOfSightRadiusSqr), ConeDistInRangeMask);
			const VectorRegister LoSPriorities0123 = VectorBitwiseAnd(InsideLineOfSightMask, BatchParams.PriorityCalculationConstants.LineOfSightPriority);

			// Outer sphere priorities
			const VectorRegister InsideOuterSphereMask = VectorCompareLE(DistSqrToViewPos0123, BatchParams.PriorityCalculationConstants.OuterSphereRadiusSqr);
			const VectorRegister OuterSpherePriorities0123 = VectorBitwiseAnd(InsideOuterSphereMask, BatchParams.PriorityCalculationConstants.OuterSpherePriority);

			// Inner sphere priorities
			const VectorRegister InsideInnerSphereMask = VectorCompareLE(DistSqrToViewPos0123, BatchParams.PriorityCalculationConstants.InnerSphereRadiusSqr);
			const VectorRegister InsideSpherePriorities0123 = VectorBitwiseAnd(InsideInnerSphereMask, BatchParams.PriorityCalculationConstants.InnerSpherePriority);

			// Compute max value of all priorities
			Priorities0123 = VectorMax(Priorities0123, VectorMax(ConePriorities0123, LoSPriorities0123));
			Priorities0123 = VectorMax(Priorities0123, VectorMax(OuterSpherePriorities0123, InsideSpherePriorities0123));
		}

		// Store our calculated priority which takes into account the priorities prior to the above calculations.
		VectorStoreAligned(Priorities0123, Priorities + ObjIt);
	}
}

void UFieldOfViewNetObjectPrioritizer::FinishBatch(const FBatchParams& BatchParams, FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset)
{
	IRIS_PROFILER_SCOPE(UFieldOfViewNetObjectPrioritizer_FinishBatch);
	float* ExternalPriorities = PrioritizationParams.Priorities;
	const uint32* ObjectIndices = PrioritizationParams.ObjectIndices;

	const float* LocalPriorities = BatchParams.Priorities;

	// Update the object priority array
	uint32 LocalObjIt = 0;
	for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = ObjIt + BatchParams.ObjectCount; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
	{
		const uint32 ObjectIndex = ObjectIndices[ObjIt];
		ExternalPriorities[ObjectIndex] = LocalPriorities[LocalObjIt];
	}
}

void UFieldOfViewNetObjectPrioritizer::SetupBatchParams(FBatchParams& OutBatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 MaxBatchObjectCount, FMemStackBase& Mem)
{
	OutBatchParams.View = PrioritizationParams.View;
	OutBatchParams.ConnectionId = PrioritizationParams.ConnectionId;
	OutBatchParams.Positions = static_cast<VectorRegister*>(Mem.Alloc(MaxBatchObjectCount*sizeof(VectorRegister), alignof(VectorRegister)));
	OutBatchParams.Priorities = static_cast<float*>(Mem.Alloc(MaxBatchObjectCount*sizeof(float), 16));

	SetupCalculationConstants(OutBatchParams.PriorityCalculationConstants);

	FMemory::Memzero(OutBatchParams.Positions, MaxBatchObjectCount*sizeof(VectorRegister));
}

void UFieldOfViewNetObjectPrioritizer::SetupCalculationConstants(FPriorityCalculationConstants& OutConstants)
{
	// Cone constants
	VectorRegister InnerConeLength = VectorSetFloat1(Config->InnerConeLength);
	VectorRegister ConeLength = VectorSetFloat1(Config->ConeLength);
	VectorRegister ConeLengthDiff = VectorSubtract(ConeLength, InnerConeLength);
	VectorRegister InvConeLengthDiff = VectorReciprocalAccurate(ConeLengthDiff);
	VectorRegister ConeRadius = VectorSetFloat1(Config->ConeLength*FMath::Tan(0.5f*FMath::DegreesToRadians(Config->ConeFieldOfViewDegrees)));
	VectorRegister ConeRadiusFactor = VectorDivide(ConeRadius, ConeLength);
	VectorRegister InnerConePriority = VectorSetFloat1(Config->MaxConePriority); 
	VectorRegister OuterConePriority = VectorSetFloat1(Config->MinConePriority);
	VectorRegister ConePriorityDiff = VectorSubtract(OuterConePriority, InnerConePriority);

	// Inner and outer sphere constants
	VectorRegister InnerSphereRadiusSqr = VectorSetFloat1(FMath::Square(Config->InnerSphereRadius));
	VectorRegister OuterSphereRadiusSqr = VectorSetFloat1(FMath::Square(Config->OuterSphereRadius));
	VectorRegister InnerSpherePriority = VectorSetFloat1(Config->InnerSpherePriority);
	VectorRegister OuterSpherePriority = VectorSetFloat1(Config->OuterSpherePriority);

	// Line of sight constants
	VectorRegister LineOfSightRadiusSqr = VectorSetFloat1(FMath::Square(0.5f*Config->LineOfSightWidth));
	VectorRegister LineOfSightPriority = VectorSetFloat1(Config->LineOfSightPriority);

	// Outside priority
	VectorRegister OutsidePriority = VectorSetFloat1(Config->OutsidePriority);

	// Store all constants
	OutConstants.InnerConeLength = InnerConeLength;
	OutConstants.ConeLength = ConeLength;
	OutConstants.ConeLengthDiff = ConeLengthDiff;
	OutConstants.InvConeLengthDiff = InvConeLengthDiff;
	OutConstants.ConeRadiusFactor = ConeRadiusFactor;
	OutConstants.InnerConePriority = InnerConePriority;
	OutConstants.OuterConePriority = OuterConePriority;
	OutConstants.ConePriorityDiff = ConePriorityDiff;

	OutConstants.InnerSphereRadiusSqr = InnerSphereRadiusSqr;
	OutConstants.OuterSphereRadiusSqr = OuterSphereRadiusSqr;
	OutConstants.InnerSpherePriority = InnerSpherePriority;
	OutConstants.OuterSpherePriority = OuterSpherePriority;

	OutConstants.LineOfSightRadiusSqr = LineOfSightRadiusSqr;
	OutConstants.LineOfSightPriority = LineOfSightPriority;

	OutConstants.OutsidePriority = OutsidePriority;
}
