// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Prioritization/LocationBasedNetObjectPrioritizer.h"
#include "Iris/ReplicationSystem/RepTag.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/WorldLocations.h"
#include "Iris/Serialization/VectorNetSerializers.h"
#include "Iris/Core/IrisProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocationBasedNetObjectPrioritizer)

ULocationBasedNetObjectPrioritizer::ULocationBasedNetObjectPrioritizer()
{
	static_assert(sizeof(FObjectLocationInfo) == sizeof(FNetObjectPrioritizationInfo), "Can't add members to FNetObjectPrioritizationInfo.");
}

void ULocationBasedNetObjectPrioritizer::Init(FNetObjectPrioritizerInitParams& Params)
{
	AssignedLocationIndices.Init(Params.CurrentMaxInternalIndex);
	WorldLocations = &Params.ReplicationSystem->GetWorldLocations();
}

void ULocationBasedNetObjectPrioritizer::Deinit()
{
	WorldLocations = nullptr;
}

void ULocationBasedNetObjectPrioritizer::OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex)
{
	AssignedLocationIndices.SetNumBits(NewMaxInternalIndex);
}

bool ULocationBasedNetObjectPrioritizer::AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerAddObjectParams& Params)
{
	// We support either a world location in the state, tagged with RepTag_WorldLocation, or via the WorldLocations instance.
	UE::Net::FRepTagFindInfo TagInfo;
	bool bHasWorldLocation = false;
	if (WorldLocations->HasInfoForObject(ObjectIndex))
	{
		bHasWorldLocation = true;
		// Craft tag info that will let us know we need to retrieve the location from WorldLocations
		TagInfo.StateIndex = InvalidStateIndex;
		TagInfo.ExternalStateOffset = InvalidStateOffset;
	}
	else if (!UE::Net::FindRepTag(Params.Protocol, UE::Net::RepTag_WorldLocation, TagInfo))
	{
		return false;
	}

	// If state index or external offset to tag is too high we can't store the relevant information easily.
	if (!bHasWorldLocation && ((TagInfo.ExternalStateOffset >= MAX_uint16) || (TagInfo.StateIndex >= MAX_uint16)))
	{
		return false;
	}

	FObjectLocationInfo& ObjectInfo = static_cast<FObjectLocationInfo&>(Params.OutInfo);
	ObjectInfo.SetLocationStateOffset(static_cast<uint16>(TagInfo.ExternalStateOffset));
	ObjectInfo.SetLocationStateIndex(static_cast<uint16>(TagInfo.StateIndex));
	const uint32 LocationIndex = AllocLocation();
	ObjectInfo.SetLocationIndex(LocationIndex);

	UpdateLocation(ObjectIndex, ObjectInfo, Params.InstanceProtocol);

	return true;
}

void ULocationBasedNetObjectPrioritizer::RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectPrioritizationInfo& Info)
{
	const FObjectLocationInfo& ObjectInfo = static_cast<const FObjectLocationInfo&>(Info);
	FreeLocation(ObjectInfo.GetLocationIndex());
}

void ULocationBasedNetObjectPrioritizer::UpdateObjects(FNetObjectPrioritizerUpdateParams& Params)
{
	for (SIZE_T ObjectIt = 0, ObjectEndIt = Params.ObjectCount; ObjectIt != ObjectEndIt; ++ObjectIt)
	{
		const uint32 ObjectIndex = Params.ObjectIndices[ObjectIt];

		const FObjectLocationInfo& ObjectInfo = static_cast<const FObjectLocationInfo&>(Params.PrioritizationInfos[ObjectIndex]);
		const UE::Net::FReplicationInstanceProtocol* InstanceProtocol = Params.InstanceProtocols[ObjectIt];
		UpdateLocation(ObjectIndex, ObjectInfo, InstanceProtocol);
	}
}

uint32 ULocationBasedNetObjectPrioritizer::AllocLocation()
{
	uint32 Index = AssignedLocationIndices.FindFirstZero();
	if (Index >= uint32(Locations.Num()))
	{
		constexpr int32 NumElementsPerChunk = LocationsChunkSize / sizeof(VectorRegister);
		Locations.Add(NumElementsPerChunk);
	}

	AssignedLocationIndices.SetBit(Index);
	return Index;
}

void ULocationBasedNetObjectPrioritizer::FreeLocation(uint32 Index)
{
	AssignedLocationIndices.ClearBit(Index);
}

VectorRegister ULocationBasedNetObjectPrioritizer::GetLocation(const FObjectLocationInfo& Info) const
{
	return Locations[Info.GetLocationIndex()];
}

void ULocationBasedNetObjectPrioritizer::SetLocation(const FObjectLocationInfo& Info, VectorRegister Location)
{
	Locations[Info.GetLocationIndex()] = Location;
}

void ULocationBasedNetObjectPrioritizer::UpdateLocation(const UE::Net::FInternalNetRefIndex ObjectIndex, const FObjectLocationInfo& Info, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol)
{
	if (Info.IsUsingWorldLocations())
	{
		const FVector WorldLocation = WorldLocations->GetWorldLocation(ObjectIndex);
		SetLocation(Info, VectorLoadFloat3_W0(&WorldLocation));
	}
	else
	{
		TArrayView<const UE::Net::FReplicationInstanceProtocol::FFragmentData> FragmentDatas = MakeArrayView(InstanceProtocol->FragmentData, InstanceProtocol->FragmentCount);
		const UE::Net::FReplicationInstanceProtocol::FFragmentData& FragmentData = FragmentDatas[Info.GetLocationStateIndex()];
		const uint8* LocationOffset = FragmentData.ExternalSrcBuffer + Info.GetLocationStateOffset();
		SetLocation(Info, VectorLoadFloat3_W0(reinterpret_cast<const FVector*>(LocationOffset)));
	}
}
