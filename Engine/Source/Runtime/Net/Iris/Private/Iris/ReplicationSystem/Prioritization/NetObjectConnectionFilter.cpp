// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Filtering/NetObjectConnectionFilter.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetObjectConnectionFilter)

void UNetObjectConnectionFilter::SetReplicateToConnection(UE::Net::FNetRefHandle RefHandle, uint32 ConnectionId, UE::Net::ENetFilterStatus FilterStatus)
{
	if (ConnectionId >= static_cast<uint32>(PerConnectionInfos.Num()))
	{
		ensureMsgf(false, TEXT("UNetObjectConnectionFilter::SetReplicateToConnection received invalid ConnectionId:%u | Max:%u"), ConnectionId, PerConnectionInfos.Num());
		return;
	}

	const UE::Net::FInternalNetRefIndex ObjectIndex = GetObjectIndex(RefHandle);

	if (ObjectIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return;
	}

	if (const FFilteringInfo* FilteringInfo = static_cast<const FFilteringInfo*>(GetFilteringInfo(ObjectIndex)))
	{
		const uint16 LocalIndex = FilteringInfo->GetLocalObjectIndex();

		FPerConnectionInfo& PerConnectionInfo = PerConnectionInfos[ConnectionId];
		ensureMsgf(LocalIndex < PerConnectionInfo.ReplicationEnabledObjects.GetNumBits(), TEXT("UNetObjectConnectionFilter::SetReplicateToConnection Object %u mapped to invalid local index %u"), ObjectIndex, LocalIndex);
		PerConnectionInfo.ReplicationEnabledObjects.SetBitValue(LocalIndex, FilterStatus == UE::Net::ENetFilterStatus::Allow);
	}
}

void UNetObjectConnectionFilter::OnInit(const FNetObjectFilterInitParams& Params)
{
	MaxInternalIndex = Params.CurrentMaxInternalIndex; 

	Config = TStrongObjectPtr<UNetObjectConnectionFilterConfig>(CastChecked<UNetObjectConnectionFilterConfig>(Params.Config));

	const uint16 MaxLocalObjectCount = static_cast<uint16>(FPlatformMath::Min<uint32>(Params.AbsoluteMaxNetObjectCount, Config->MaxObjectCount));
	UsedLocalInfoIndices.Init(MaxLocalObjectCount);
	LocalToNetRefIndex.SetNumZeroed(MaxLocalObjectCount);

	PerConnectionInfos.SetNum(Params.MaxConnectionCount + 1);
}

void UNetObjectConnectionFilter::OnDeinit()
{
	Config = nullptr;

	UsedLocalInfoIndices.Empty();
	LocalToNetRefIndex.Empty();
}

void UNetObjectConnectionFilter::AddConnection(uint32 ConnectionId)
{
	FPerConnectionInfo& ConnInfo = PerConnectionInfos[ConnectionId];
	ConnInfo.ReplicationEnabledObjects.Init(UsedLocalInfoIndices.GetNumBits());
}

void UNetObjectConnectionFilter::RemoveConnection(uint32 ConnectionId)
{
	// Free everything that was allocated for the connection.
	FPerConnectionInfo& ConnInfo = PerConnectionInfos[ConnectionId];
	ConnInfo = FPerConnectionInfo();
}

bool UNetObjectConnectionFilter::AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectFilterAddObjectParams& Params)
{
	const uint32 LocalIndex = UsedLocalInfoIndices.FindFirstZero();
	if (LocalIndex == UE::Net::FNetBitArrayBase::InvalidIndex)
	{
		ensureMsgf(false, TEXT("%hs. MaxObjectCount: %u. Config type %s."), "Too many objects added to NetObjectConnectionFilter. Object will not be handled by filter!", (Config.IsValid() ? Config->MaxObjectCount : uint16(0)), ToCStr(GetNameSafe(Config.Get())));
		return false;
	}

	UE_LOGF(LogIris, Verbose, "UNetObjectConnectionFilter::AddObject added %u | Mapped to LocalIndex %u", ObjectIndex, LocalIndex);

	UsedLocalInfoIndices.SetBit(LocalIndex);
	LocalToNetRefIndex[LocalIndex] = ObjectIndex;

	FFilteringInfo& Info = static_cast<FFilteringInfo&>(Params.OutInfo);
	Info.SetLocalObjectIndex((uint16)(LocalIndex)); //LPS: Assert if overflow

	return true;
}

void UNetObjectConnectionFilter::RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectFilteringInfo& InInfo)
{
	bObjectRemoved = true;

	const FFilteringInfo& Info = static_cast<const FFilteringInfo&>(InInfo);

	const uint16 LocalIndex = Info.GetLocalObjectIndex();
	UsedLocalInfoIndices.ClearBit(LocalIndex);

	UE_LOGF(LogIris, Verbose, "UNetObjectConnectionFilter::RemoveObject removed %u | Mapped to LocalIndex %u", ObjectIndex, LocalIndex);

	// For good measure. It's not strictly needed.
	LocalToNetRefIndex[LocalIndex] = UE::Net::InvalidInternalNetRefIndex;
}

void UNetObjectConnectionFilter::PreFilter(FNetObjectPreFilteringParams& Params)
{
	if (!bObjectRemoved)
	{
		return;
	}

	bObjectRemoved = false;

	// Mask out no longer filtered objects to minimize looping during filtering.
	Params.ValidConnections.ForAllSetBits(
		[this](uint32 ConnectionId)
		{
			FPerConnectionInfo& Info = this->PerConnectionInfos[ConnectionId];
			Info.ReplicationEnabledObjects.Combine(this->UsedLocalInfoIndices, UE::Net::FNetBitArrayBase::AndOp);
		}
	);
}

void UNetObjectConnectionFilter::Filter(FNetObjectFilteringParams& Params)
{
	IRIS_PROFILER_SCOPE(UNetObjectConnectionFilter_Filter);

	UE::Net::FNetBitArrayView& AllowedObjects = Params.OutAllowedObjects;
	AllowedObjects.ClearAllBits();

	FPerConnectionInfo& ConnectionInfo = PerConnectionInfos[Params.ConnectionId];
	ConnectionInfo.ReplicationEnabledObjects.ForAllSetBits(
		[&AllowedObjects, &ToNetRefIndex = LocalToNetRefIndex](uint32 LocalObjectIndex)
		{
			const uint32 ObjectIndex = ToNetRefIndex[LocalObjectIndex];
			AllowedObjects.SetBit(ObjectIndex);
		}
	);
}
