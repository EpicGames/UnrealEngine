// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Prioritization/NetObjectCountLimiter.h"
#include "Iris/IrisConstants.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFiltering.h"
#include "Iris/Core/IrisProfiler.h"
#include "Algo/Sort.h"
#include "Containers/ArrayView.h"
#include "HAL/PlatformMemory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetObjectCountLimiter)

UNetObjectCountLimiter::UNetObjectCountLimiter()
: PrioFrame(0)
, ReplicationSystemId(UE::Net::InvalidReplicationSystemId)
{
	static_assert(sizeof(FObjectInfo) == sizeof(FNetObjectPrioritizationInfo), "Can't add members to FNetObjectPrioritizationInfo.");
}

void UNetObjectCountLimiter::Init(FNetObjectPrioritizerInitParams& Params)
{
	checkf(Params.Config != nullptr, TEXT("Need config to operate."));
	checkf(Params.MaxConnectionCount < 65536U, TEXT("Assumption being able to use uint16 for ConnectionIds is incorrect."));
	Config = TStrongObjectPtr<UNetObjectCountLimiterConfig>(CastChecked<UNetObjectCountLimiterConfig>(Params.Config));
	ensureMsgf(Config->MaxObjectCount >= 1U, TEXT("Prioritizer will not consider any object for replication. They will be replicated once when constructed, but never again."));

	InternalObjectIndices.Init(ObjectGrowCount);
	PerConnectionInfos.SetNum(Params.MaxConnectionCount + 1);

	ReplicationSystem = Params.ReplicationSystem;
}

void UNetObjectCountLimiter::Deinit()
{
	ReplicationSystem = nullptr;
	Config = nullptr;

	InternalObjectIndices.Empty();
}

void UNetObjectCountLimiter::AddConnection(uint32 ConnectionId)
{
	FPerConnectionInfo& Info = PerConnectionInfos[ConnectionId];
	Info.LastConsiderFrames.Init(PrioFrame, InternalObjectIndices.GetNumBits());
}

void UNetObjectCountLimiter::RemoveConnection(uint32 ConnectionId)
{
	FPerConnectionInfo& Info = PerConnectionInfos[ConnectionId];
	FPerConnectionInfo Empty;
	Info = MoveTemp(Empty);
}

bool UNetObjectCountLimiter::AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerAddObjectParams& Params)
{
	FObjectInfo& Info = static_cast<FObjectInfo&>(Params.OutInfo);
	const uint16 Index = AllocInternalIndex();
	Info.SetPrioritizerInternalIndex(Index);
	return true;
}

void UNetObjectCountLimiter::RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectPrioritizationInfo& InInfo)
{
	const FObjectInfo& Info = static_cast<const FObjectInfo&>(InInfo);
	FreeInternalIndex(Info.GetPrioritizerInternalIndex());
}

void UNetObjectCountLimiter::UpdateObjects(FNetObjectPrioritizerUpdateParams& Params)
{
	const UE::Net::Private::FReplicationFiltering& ReplicationFiltering = ReplicationSystem->GetReplicationSystemInternal()->GetFiltering();

	for (const uint32 ObjectIndex : MakeArrayView(Params.ObjectIndices, Params.ObjectCount))
	{
		FObjectInfo& ObjectInfo = static_cast<FObjectInfo&>(Params.PrioritizationInfos[ObjectIndex]);
		const uint32 OwningConnection = ReplicationFiltering.GetOwningConnection(ObjectIndex);
		ObjectInfo.SetOwningConnection(OwningConnection);
	}
}

void UNetObjectCountLimiter::PrePrioritize(FNetObjectPrePrioritizationParams& Params)
{
	++PrioFrame;
	switch (Config->Mode)
	{
		case ENetObjectCountLimiterMode::RoundRobin:
		{
			PrePrioritizeForRoundRobin();
			break;
		};

		case ENetObjectCountLimiterMode::Fill:
		{
			// There's no meaningful work to do as everything is connection specific.
			break;
		};

		default:
		{
			break;
		}
	};
}

void UNetObjectCountLimiter::Prioritize(FNetObjectPrioritizationParams& Params)
{
	switch (Config->Mode)
	{
		case ENetObjectCountLimiterMode::RoundRobin:
		{
			PrioritizeForRoundRobin(Params);
			break;
		};

		case ENetObjectCountLimiterMode::Fill:
		{
			PrioritizeForFill(Params);
			break;
		};
	}
}

uint16 UNetObjectCountLimiter::AllocInternalIndex()
{
	uint32 Index = InternalObjectIndices.FindFirstZero();
	if (Index == UE::Net::FNetBitArray::InvalidIndex)
	{
		Index = InternalObjectIndices.GetNumBits();
		check(Index <= 65535U);
		InternalObjectIndices.AddBits(ObjectGrowCount);
	}

	InternalObjectIndices.SetBit(Index);
	return static_cast<uint16>(Index);
}

void UNetObjectCountLimiter::FreeInternalIndex(uint16 Index)
{
	InternalObjectIndices.ClearBit(Index);
}

void UNetObjectCountLimiter::PrePrioritizeForRoundRobin()
{
	IRIS_PROFILER_SCOPE(UNetObjectCountLimiter_PrePrioritizeForRoundRobin);

	// Find the next N viable objects. These will be considered for replication for all connections if they're dirty.
	const uint32 IndexCount = InternalObjectIndices.GetNumBits();
	RoundRobinState.InternalObjectIndices.Init(IndexCount);
	if (RoundRobinState.NextIndexToConsider >= IndexCount)
	{
		RoundRobinState.NextIndexToConsider = 0;
	}

	const uint32 MaxObjectCount = Config->MaxObjectCount;
	// Obey the user's wishes. No objects will be prioritized. There's an ensure in Init().
	if (MaxObjectCount == 0)
	{
		return;
	}

	uint32* Indices = static_cast<uint32*>(FMemory_Alloca(MaxObjectCount*sizeof(uint32)));
	uint32 ObjectCount = InternalObjectIndices.GetSetBitIndices(RoundRobinState.NextIndexToConsider, ~0U, Indices, MaxObjectCount);
	if (RoundRobinState.NextIndexToConsider > 0 && ObjectCount < MaxObjectCount)
	{
		ObjectCount += InternalObjectIndices.GetSetBitIndices(0U, RoundRobinState.NextIndexToConsider - 1U, Indices + ObjectCount, MaxObjectCount - ObjectCount);
	}

	if (ObjectCount)
	{
		RoundRobinState.NextIndexToConsider = static_cast<uint16>(Indices[ObjectCount - 1U] + 1U);
		for (uint32 Index : MakeArrayView(Indices, ObjectCount))
		{
			RoundRobinState.InternalObjectIndices.SetBit(Index);
		}
	}
}

void UNetObjectCountLimiter::PrioritizeForRoundRobin(FNetObjectPrioritizationParams& Params) const
{
	IRIS_PROFILER_SCOPE(UNetObjectCountLimiter_PrioritizeForRoundRobin);

	const bool bEnableOwnedObjectsFastLane = Config->bEnableOwnedObjectsFastLane;
	const float StandardPriority = Config->Priority;
	const float OwningConnectionPriority = Config->OwningConnectionPriority;

	const uint32 MaxConsiderCount = Config->MaxObjectCount;
	uint32 ConsiderCount = 0U;
	if (bEnableOwnedObjectsFastLane)
	{
		for (const uint32 ObjectIndex : MakeArrayView(Params.ObjectIndices, Params.ObjectCount))
		{
			const FObjectInfo& Info = static_cast<const FObjectInfo&>(Params.PrioritizationInfos[ObjectIndex]);
			const bool bIsRoundRobin = RoundRobinState.InternalObjectIndices.GetBit(Info.GetPrioritizerInternalIndex());
			const bool bIsOwnedByConnection = Info.GetOwningConnection() == Params.ConnectionId;
			if (!(bIsRoundRobin | bIsOwnedByConnection))
			{
				continue;
			}

			if (bIsOwnedByConnection)
			{
				Params.Priorities[ObjectIndex] = OwningConnectionPriority;
			}
			else if (ConsiderCount < MaxConsiderCount)
			{
				++ConsiderCount;
				Params.Priorities[ObjectIndex] = StandardPriority;
			}
		}
	}
	else
	{
		for (const uint32 ObjectIndex : MakeArrayView(Params.ObjectIndices, Params.ObjectCount))
		{
			const FObjectInfo& Info = static_cast<const FObjectInfo&>(Params.PrioritizationInfos[ObjectIndex]);
			if (!RoundRobinState.InternalObjectIndices.GetBit(Info.GetPrioritizerInternalIndex()))
			{
				continue;
			}

			const float Priority = (Info.GetOwningConnection() == Params.ConnectionId ? OwningConnectionPriority : StandardPriority);
			Params.Priorities[ObjectIndex] = Priority;
			if (++ConsiderCount == MaxConsiderCount)
			{
				break;
			}
		}
	}
}

void UNetObjectCountLimiter::PrioritizeForFill(FNetObjectPrioritizationParams& Params)
{
	IRIS_PROFILER_SCOPE(UNetObjectCountLimiter_PrioritizeForFill);

	/*
	 * Warn about entering this function twice in the same frame for the same connection
	 * as that means there were so many objects that they were split into multiple batches.
	 * With Fill mode it's vital that we get all the dirty objects in the same batch so
	 * we can see which of those were replicated least recently. There's no proper fix
	 * as we cannot/shouldn't rely on priority pointers being valid outside
	 * the scope of the Prioritize() call. The priority system batch size must be increased
	 * or the mode for this instance changed to RoundRobin. What will happen when this ensure
	 * happens is that each batch could potentially prioritize N objects.
	 */

	ensureMsgf(FillState.LastPrioFrame != PrioFrame || FillState.LastConnectionId != Params.ConnectionId, TEXT("%s"), TEXT("UNetObjectCountLimiter::PrioritizeForFill. Too many objects are being prioritized"));

	FillState.LastPrioFrame = PrioFrame;
	FillState.LastConnectionId = Params.ConnectionId;

	const uint32 ConnectionId = Params.ConnectionId;
	FPerConnectionInfo& ConnectionInfo = PerConnectionInfos[Params.ConnectionId];

	// Make sure we can get/set info for all potential objects
	if (static_cast<uint32>(ConnectionInfo.LastConsiderFrames.Num()) < InternalObjectIndices.GetNumBits())
	{
		/*
		 * When new objects have been added it's ok if the LastConsiderFrame is 0.
		 * That's why we don't fill new objects with some special value.
		 * New objects will be replicated for creation anyway and if they are considered 
		 * by this prioritizer we won't waste bandwidth by adding even more objects. 
		 */
		ConnectionInfo.LastConsiderFrames.SetNum(InternalObjectIndices.GetNumBits());
	}

	uint32* LastConsideredFrames = ConnectionInfo.LastConsiderFrames.GetData();

	struct FSortInfo
	{
		uint32 ObjectIndex;
		uint32 InternalIndex;
		uint32 FrameCountSinceConsidered;
		bool bIsOwnedByConnection;
	};

	// Can use some sort of scratch pad if this is problematic. We don't expect this to require more than 16KiB.
	FSortInfo* SortInfosAlloc = static_cast<FSortInfo*>(FMemory_Alloca(sizeof(FSortInfo)*Params.ObjectCount));
	TArrayView<FSortInfo> SortInfos = MakeArrayView(SortInfosAlloc, Params.ObjectCount);

	// Prep sort
	{
		FSortInfo* SortInfoIter = SortInfosAlloc;
		for (const uint32 ObjectIndex : MakeArrayView(Params.ObjectIndices, Params.ObjectCount))
		{
			const FObjectInfo& ObjectInfo = static_cast<const FObjectInfo&>(Params.PrioritizationInfos[ObjectIndex]);
			const uint32 PrioritizerInternalIndex = ObjectInfo.GetPrioritizerInternalIndex();

			FSortInfo& SortInfo = *SortInfoIter++;
			SortInfo.ObjectIndex = ObjectIndex;
			SortInfo.InternalIndex = PrioritizerInternalIndex;
			// This should work reasonably well even with FrameIndex wraparound.
			SortInfo.FrameCountSinceConsidered = PrioFrame - LastConsideredFrames[PrioritizerInternalIndex];
			SortInfo.bIsOwnedByConnection = (ObjectInfo.GetOwningConnection() == ConnectionId);
		}
	}

	// Sort and prioritize
	{
		const float StandardPriority = Config->Priority;
		const float OwningConnectionPriority = Config->OwningConnectionPriority;
		const uint32 MaxConsiderCount = Config->MaxObjectCount;
		uint32 ConsiderCount = 0U;

		if (Config->bEnableOwnedObjectsFastLane)
		{
			auto ByOwnerAndLeastConsidered = [](const FSortInfo& A, const FSortInfo& B)
			{
				if (A.bIsOwnedByConnection != B.bIsOwnedByConnection)
				{
					return A.bIsOwnedByConnection;
				}

				if (A.FrameCountSinceConsidered != B.FrameCountSinceConsidered)
				{
					return A.FrameCountSinceConsidered > B.FrameCountSinceConsidered;
				}

				// Tie-breaker
				return (A.InternalIndex < B.InternalIndex);
			};

			Algo::Sort(SortInfos, ByOwnerAndLeastConsidered);

			for (const FSortInfo& Info : SortInfos)
			{
				LastConsideredFrames[Info.InternalIndex] = PrioFrame;

				const float Priority = (Info.bIsOwnedByConnection ? OwningConnectionPriority : StandardPriority);
				Params.Priorities[Info.ObjectIndex] = Priority;
				
				// We're sorting so that owned objects end up first so if we've reached MaxConsiderCount we're done.
				ConsiderCount += !Info.bIsOwnedByConnection;
				if (ConsiderCount == MaxConsiderCount)
				{
					break;
				}
			}
		}
		else
		{
			auto ByLeastConsidered = [](const FSortInfo& A, const FSortInfo& B)
			{
				if (A.FrameCountSinceConsidered != B.FrameCountSinceConsidered)
				{
					return A.FrameCountSinceConsidered > B.FrameCountSinceConsidered;
				}

				// Tie-breaker
				return (A.InternalIndex < B.InternalIndex);
			};
			Algo::Sort(SortInfos, ByLeastConsidered);

			for (const FSortInfo& Info : SortInfos)
			{
				LastConsideredFrames[Info.InternalIndex] = PrioFrame;

				const float Priority = (Info.bIsOwnedByConnection ? OwningConnectionPriority : StandardPriority);
				Params.Priorities[Info.ObjectIndex] = Priority;
				
				if (++ConsiderCount == MaxConsiderCount)
				{
					break;
				}
			}
		}
	}
}
