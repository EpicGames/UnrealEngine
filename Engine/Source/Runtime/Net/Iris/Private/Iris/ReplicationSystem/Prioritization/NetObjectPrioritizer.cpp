// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetObjectPrioritizer)

/**
Have spatial prioritizers and non-spatial prioritizers. Kind of like ReplicationGraphNodes if you will.
How many spatial prioritizers can be set per object?
How many non-spatial prioritizers can be set per object?

Could fairly easily support up to 32 (64) prioritizers per object if using a mask. Could be somewhat expensive to traverse all objects per prioritizer,
or complex logic to splat out prioritizer IDs. Another idea is to have a BitArray per prioritizer, but that bit array must be large enough to hold MaxObjectCount.
Ok for thousands of objects (a few KB), but expensive for millions of objects.

If supporting multiple prioritizers each prioritizer needs to be careful to honor the existing priority? Or enforce via logic MAXing between runs, requires temp storage.
*/

UNetObjectPrioritizer::UNetObjectPrioritizer()
{
}

void UNetObjectPrioritizer::AddConnection(uint32 ConnectionId)
{
}

void UNetObjectPrioritizer::RemoveConnection(uint32 ConnectionId)
{
}

void UNetObjectPrioritizer::PrePrioritize(FNetObjectPrePrioritizationParams&)
{
}

void UNetObjectPrioritizer::PostPrioritize(FNetObjectPostPrioritizationParams&)
{
}
