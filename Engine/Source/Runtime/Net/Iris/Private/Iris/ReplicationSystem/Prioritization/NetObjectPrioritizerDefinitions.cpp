// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizerDefinitions.h"
#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizer.h"
#include "Iris/Core/IrisLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetObjectPrioritizerDefinitions)

void UNetObjectPrioritizerDefinitions::GetValidDefinitions(TArray<FNetObjectPrioritizerDefinition>& OutDefinitions) const
{
	OutDefinitions.Reserve(NetObjectPrioritizerDefinitions.Num());
	for (const FNetObjectPrioritizerDefinition& Definition : NetObjectPrioritizerDefinitions)
	{
		if (Definition.Class != nullptr && (Definition.ConfigClassName.IsNone() || Definition.ConfigClass != nullptr))
		{
			OutDefinitions.Push(Definition);
		}
	}
}

void UNetObjectPrioritizerDefinitions::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		LoadDefinitions();
	}
}

void UNetObjectPrioritizerDefinitions::PostReloadConfig(FProperty* PropertyToLoad)
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		LoadDefinitions();
	}
}

void UNetObjectPrioritizerDefinitions::LoadDefinitions()
{
	for (FNetObjectPrioritizerDefinition& Definition : NetObjectPrioritizerDefinitions)
	{
		Definition.Class = StaticLoadClass(UNetObjectPrioritizer::StaticClass(), nullptr, *Definition.ClassName.ToString(), nullptr, LOAD_Quiet);
		UE_CLOGF(Definition.Class == nullptr, LogIris, Error, "NetObjectPrioritizer class could not be loaded: %ls", *Definition.ClassName.GetPlainNameString());

		if (!Definition.ConfigClassName.IsNone())
		{
			Definition.ConfigClass = StaticLoadClass(UNetObjectPrioritizerConfig::StaticClass(), nullptr, *Definition.ConfigClassName.ToString(), nullptr, LOAD_Quiet);
			UE_CLOGF(Definition.ConfigClass == nullptr, LogIris, Error, "NetObjectPrioritizerConfig class could not be loaded: %ls", *Definition.ConfigClassName.GetPlainNameString());
		}
	}
}
