// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "NetObjectPrioritizerDefinitions.generated.h"

/** Prioritizer definition. Configurable via UNetObjectPrioritizerDefinitions. */
USTRUCT()
struct FNetObjectPrioritizerDefinition
{
	GENERATED_BODY()

	/** Prioritizer identifier. Used to uniquely identify a prioritizer in various APIs. */
	UPROPERTY()
	FName PrioritizerName;

	/**
	 * UClass name, specified by its fully qualified path, used to create the UNetObjectPrioritizer. You can have multiple instances of the same prioritizer as long as 
	 * their PrioritizerNames are unique.
	 */
	UPROPERTY()
	FName ClassName;		

	/** UClass used to create the UNetObjectPrioritizer. Filled in automatically when reading the config. */
	UPROPERTY()
	TObjectPtr<UClass> Class = nullptr;

	/**
	 * Optional UClass, specified by its fully qualified path, used to create the UNetObjectPrioritizerConfig. The class default instance will be passed at prioritizer initialization.
	 * If you want multiple instances of the same prioritizer then use subclassing to create unique prioritizer configs.
	 */
	UPROPERTY()
	FName ConfigClassName;		

	/** UClass used to create the UNetObjectPrioritizerConfig. Filled in automatically when reading the config. */
	UPROPERTY()
	TObjectPtr<UClass> ConfigClass = nullptr;
};

/** Configurable prioritizer definitions. Valid prioritizer definitions are auto-created by the prioritization system. */
UCLASS(transient, config=Engine)
class UNetObjectPrioritizerDefinitions final : public UObject
{
	GENERATED_BODY()

public:
	/** Retrieve the valid prioritizer definitions- those that should be able to create valid prioritizers. */
	void GetValidDefinitions(TArray<FNetObjectPrioritizerDefinition>& OutDefinitions) const;

private:
	// UObject
	virtual void PostInitProperties() override;
	virtual void PostReloadConfig(FProperty* PropertyToLoad) override;

	//
	void LoadDefinitions();

private:
	/**
	 * Prioritizer definitions.
	 * The first valid definition will assume the role as default spatial prioritizer. All objects with a RepTag_WorldLocation tag will 
	 * be added to the default prioritizer. To override the behavior a prioritizer must be set via calls to the ReplicationSystem.
	 */
	UPROPERTY(Config)
	TArray<FNetObjectPrioritizerDefinition> NetObjectPrioritizerDefinitions;
};
