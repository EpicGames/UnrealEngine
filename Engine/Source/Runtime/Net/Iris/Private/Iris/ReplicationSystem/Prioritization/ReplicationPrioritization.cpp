// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationPrioritization.h"
#include "Iris/IrisConstants.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"
#include "Net/Core/Misc/NetCVars.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationConnections.h"
#include "Iris/ReplicationSystem/ReplicationWriter.h"
#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizerDefinitions.h"
#include "Containers/ChunkedArray.h"
#include "Templates/Sorting.h"
#include "UObject/WeakObjectPtr.h"

namespace UE::Net::Private
{

static_assert(InvalidNetObjectPrioritizerHandle == ~FNetObjectPrioritizerHandle(0), "ObjectIndexToPrioritizer code needs attention. Contact the UE Networking team.");
static constexpr uint8 FReplicationPrioritization_InvalidNetObjectPrioritizerIndex = 0xFF;

static const FName NAME_DefaultPrioritizer("DefaultPrioritizer");

/**
 * Most logic in here revolves around batches. As such we need access to the Chunks.
 */
template<class InElementType, uint32 BytesPerChunk>
class TChunkedArrayWithChunkManagement : public ::TChunkedArray<InElementType, BytesPerChunk>
{
private:
	using Super = ::TChunkedArray<InElementType, BytesPerChunk>;

public:
	/** Removes the first chunk and all elements in it, if it exists. */
	void PopChunkSafe()
	{
		if (Super::NumElements > 0)
		{
			Super::NumElements -= FPlatformMath::Min(Super::NumElements, static_cast<int32>(Super::NumElementsPerChunk));

			constexpr int32 Index = 0;
			constexpr int32 Count = 1;
			Super::Chunks.RemoveAt(Index, Count, EAllowShrinking::No);
		}
	}

	/** Constructs a new element at the end of the array. Returns a reference to it. */
	InElementType& Emplace_GetRef()
	{
		if ((static_cast<uint32>(Super::NumElements) % static_cast<uint32>(Super::NumElementsPerChunk)) == 0U)
		{
			++Super::NumElements;
			typename Super::FChunk* Chunk = new typename Super::FChunk;
			Super::Chunks.Add(Chunk);
			return Chunk->Elements[0];
		}
		else
		{
			return this->operator[](Super::NumElements++);
		}
	}

	/** Returns the number of elements in first chunk. */
	int32 GetFirstChunkNum() const
	{
		return FPlatformMath::Min(Super::NumElements, static_cast<int32>(Super::NumElementsPerChunk));
	}

	/** Returns a pointer to the first element in the first chunk. */
	const InElementType* GetFirstChunkData() const
	{
		if (typename Super::FChunk const** ChunkPtr = Super::Chunks.GetData())
		{
			return &(*ChunkPtr)->Elements[0];
		}

		return nullptr;
	}

	/** Sets num elements to zero but keeps allocations. */
	void Reset()
	{
		Super::Chunks.Reset(0);
		Super::NumElements = 0;
	}
};

class FReplicationPrioritization::FPrioritizerBatchHelper
{
public:
	enum EConstants : unsigned
	{
		MaxObjectCountPerBatch = 1024U,
	};

	enum EBatchProcessStatus : unsigned
	{
		ProcessFullBatchesAndContinue,
		ProcessAllBatchesAndStop,
		NothingToProcess,
	};

	struct FPerPrioritizerInfo
	{
		TChunkedArrayWithChunkManagement<FInternalNetRefIndex, MaxObjectCountPerBatch*sizeof(uint32)> ObjectIndices;
	};

	explicit FPrioritizerBatchHelper(uint32 PrioritizerCount)
	{
		PerPrioritizerInfos.SetNum(PrioritizerCount);
	}

	void InitForConnection()
	{
		BatchInfo = FBatchInfo();

		for (FPerPrioritizerInfo& PerPrioritizerInfo : PerPrioritizerInfos)
		{
			PerPrioritizerInfo.ObjectIndices.Reset();
		}
	}

	EBatchProcessStatus PrepareBatch(FPerConnectionInfo& ConnInfo, const FNetBitArrayView Objects, const uint8* PrioritizerIndices, const float* InDefaultPriorities)
	{
		IRIS_PROFILER_SCOPE(FReplicationPrioritization_PrioritizeForConnection_PrepareBatch);

		if (!ensure(BatchInfo.CurrentObjectIndex < Objects.GetNumBits()))
		{
			return EBatchProcessStatus::NothingToProcess;
		}

		float* ConnPriorities = ConnInfo.Priorities.GetData();
		FPerPrioritizerInfo* PerPrioritizerInfosData = PerPrioritizerInfos.GetData();

		uint32 ObjectIndices[MaxObjectCountPerBatch];
		/**
		 * Algorithm will get MaxBatchObjectCount dirty objects and sort them into the correct prioritizer.
		 * Return on the following conditions:
		 * - If a prioritizer has reached at least MaxBatchObjectCount
		 * - If all dirty objects have been sorted.
		 */
		{
			constexpr uint32 BitCount = ~0U;
			for (uint32 ObjectCount = 0; (ObjectCount = Objects.GetSetBitIndices(BatchInfo.CurrentObjectIndex, BitCount, ObjectIndices, MaxObjectCountPerBatch)) > 0; )
			{
				if (ObjectCount < MaxObjectCountPerBatch)
				{
					// This is so we can trigger an ensure if PrepareBatch() is called again.
					BatchInfo.CurrentObjectIndex = Objects.GetNumBits();
				}
				else
				{
					BatchInfo.CurrentObjectIndex =  ObjectIndices[ObjectCount - 1] + 1U;
				}

				for (const uint32 ObjectIndex : MakeArrayView(ObjectIndices, ObjectCount))
				{
					const uint8 PrioritizerIndex = PrioritizerIndices[ObjectIndex];
					if (PrioritizerIndex == FReplicationPrioritization_InvalidNetObjectPrioritizerIndex)
					{
						continue;
					}

					FPerPrioritizerInfo& PerPrioritizerInfo = PerPrioritizerInfosData[PrioritizerIndex];
					PerPrioritizerInfo.ObjectIndices.Emplace_GetRef() = ObjectIndex;

					// Reset priority to default.
					ConnPriorities[ObjectIndex] = InDefaultPriorities[ObjectIndex];
				}

				/**
				 * If we've processed all objects return ProcessAll.
				 * If any prioritizer has a full batch then return ProcessFull. This limits the memory footprint to a maximum of two chunks per prioritizer.
				 * Else continue.
				 */
				if ((ObjectCount < MaxObjectCountPerBatch) || (BatchInfo.CurrentObjectIndex == Objects.GetNumBits()))
				{
					return EBatchProcessStatus::ProcessAllBatchesAndStop;
				}
				/**
				 * Is there a full batch for any prioritizer?
				 * We expect checking it after the fact to be a lot faster in typical scenarios versus checking after each object addition to a prioritizer.
				 */
				else
				{
					for (const FPerPrioritizerInfo& PerPrioritizerInfo : PerPrioritizerInfos)
					{
						if (PerPrioritizerInfo.ObjectIndices.Num() >= MaxObjectCountPerBatch)
						{
							return EBatchProcessStatus::ProcessFullBatchesAndContinue;
						}
					}

					// Continue sorting objects by prioritizer.
				}
			}
		}

		// We've sorted all objects.
		return EBatchProcessStatus::ProcessAllBatchesAndStop;
	}

	TArray<FPerPrioritizerInfo, TInlineAllocator<16>> PerPrioritizerInfos;

private:
	struct FBatchInfo
	{
		uint32 CurrentObjectIndex = 0;
	};

	FBatchInfo BatchInfo;
};

class FReplicationPrioritization::FUpdateDirtyObjectsBatchHelper
{
public:
	enum Constants : uint32
	{
		MaxObjectCountPerBatch = 512U,
	};

	struct FPerPrioritizerInfo
	{
		FInternalNetRefIndex* ObjectIndices;
		uint32 ObjectCount;

		FReplicationInstanceProtocol const** InstanceProtocols;
	};

	FUpdateDirtyObjectsBatchHelper(const FNetRefHandleManager* InNetRefHandleManager, uint32 PrioritizerCount)
	: NetRefHandleManager(InNetRefHandleManager)
	{
		PerPrioritizerInfos.SetNumUninitialized(PrioritizerCount);
		ObjectIndicesStorage.SetNumUninitialized(PrioritizerCount*MaxObjectCountPerBatch);
		InstanceProtocolsStorage.SetNumUninitialized(PrioritizerCount*MaxObjectCountPerBatch);

		uint32 PrioritizerIndex = 0;
		for (FPerPrioritizerInfo& PerPrioritizerInfo : PerPrioritizerInfos)
		{
			PerPrioritizerInfo.ObjectIndices = ObjectIndicesStorage.GetData() + PrioritizerIndex*MaxObjectCountPerBatch;
			PerPrioritizerInfo.InstanceProtocols = InstanceProtocolsStorage.GetData() + PrioritizerIndex*MaxObjectCountPerBatch;
			++PrioritizerIndex;
		}
	}

	void PrepareBatch(const uint32* ObjectIndices, uint32 ObjectCount, const uint8* PrioritizerIndices)
	{
		ResetBatch();

		FPerPrioritizerInfo* PerPrioritizerInfosData = PerPrioritizerInfos.GetData();
		for (const uint32 ObjectIndex : MakeArrayView(ObjectIndices, ObjectCount))
		{
			const uint8 PrioritizerIndex = PrioritizerIndices[ObjectIndex];
			if (PrioritizerIndex == FReplicationPrioritization_InvalidNetObjectPrioritizerIndex)
			{
				continue;
			}

			if (const FReplicationInstanceProtocol* InstanceProtocol = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex).InstanceProtocol)
			{
				FPerPrioritizerInfo& PerPrioritizerInfo = PerPrioritizerInfosData[PrioritizerIndex];
				PerPrioritizerInfo.ObjectIndices[PerPrioritizerInfo.ObjectCount] = ObjectIndex;
				PerPrioritizerInfo.InstanceProtocols[PerPrioritizerInfo.ObjectCount] = InstanceProtocol;
				++PerPrioritizerInfo.ObjectCount;
			}
		}
	}

	TArray<FPerPrioritizerInfo, TInlineAllocator<16>> PerPrioritizerInfos;

private:
	void ResetBatch()
	{
		for (FPerPrioritizerInfo& PerPrioritizerInfo : PerPrioritizerInfos)
		{
			PerPrioritizerInfo.ObjectCount = 0U;
		}
	}

	TArray<UE::Net::FInternalNetRefIndex> ObjectIndicesStorage;
	TArray<const FReplicationInstanceProtocol*> InstanceProtocolsStorage;
	const FNetRefHandleManager* NetRefHandleManager;
};

FReplicationPrioritization::FReplicationPrioritization()
: HasNewObjectsWithStaticPriority(0)
{
}

void FReplicationPrioritization::Init(FReplicationPrioritizationInitParams& Params)
{
	check(Params.Connections != nullptr);
	check(Params.NetRefHandleManager != nullptr);

	ReplicationSystem = Params.ReplicationSystem;

	Connections = Params.Connections;
	NetRefHandleManager = Params.NetRefHandleManager;

	MaxInternalNetRefIndex = Params.MaxInternalNetRefIndex;

	SetNetObjectListsSize(MaxInternalNetRefIndex);

	ConnectionInfos.Reserve(Params.Connections->GetMaxConnectionCount());

	InitPrioritizers();
}

void FReplicationPrioritization::Deinit()
{
	for (FPrioritizerInfo& Info : PrioritizerInfos)
	{
		Info.Prioritizer->Deinit();
		Info.Prioritizer = nullptr;
	}

	PrioritizerDefinitions = nullptr;

	ReplicationSystem = nullptr;
	Connections = nullptr;
	NetRefHandleManager = nullptr;

	NetObjectPrioritizationInfos.Empty();
	ObjectIndexToPrioritizer.Empty();
	DefaultPriorities.Empty();
	ObjectsWithNewStaticPriority.Empty();
}

void FReplicationPrioritization::SetNetObjectListsSize(FInternalNetRefIndex MaxInternalIndex)
{
	constexpr EAllowShrinking NoShrinking = EAllowShrinking::No;

	// $IRIS TODO: This can be quite wasteful in terms of memory assuming many objects will use a static priority. Need object pool!
	NetObjectPrioritizationInfos.SetNumUninitialized(MaxInternalIndex, NoShrinking);

	// Properly Initialize ObjectIndexToPrioritizer
	{
		const int32 PreviousSize = ObjectIndexToPrioritizer.Num();
		ObjectIndexToPrioritizer.SetNumUninitialized(MaxInternalIndex, NoShrinking);

		uint8* BufferData = ObjectIndexToPrioritizer.GetData();
		BufferData += PreviousSize;

		check((int32)MaxInternalIndex >= PreviousSize);
		const int32 UninitNum = (int32)MaxInternalIndex - PreviousSize;

		FMemory::Memset(BufferData, FReplicationPrioritization_InvalidNetObjectPrioritizerIndex, UninitNum * sizeof(decltype(ObjectIndexToPrioritizer)::ElementType));
	}

	// Property initialize DefaultPriorities
	ResizePrioritiesList(DefaultPriorities, MaxInternalIndex);

	ObjectsWithNewStaticPriority.SetNumBits(MaxInternalIndex);
}

void FReplicationPrioritization::ResizePrioritiesList(TArray<float>& OutPriorities, FInternalNetRefIndex MaxInternalIndex)
{
	const int32 PreviousSize = OutPriorities.Num();
	OutPriorities.SetNumUninitialized(MaxInternalIndex, EAllowShrinking::No);

	for (int32 PrioIndex = PreviousSize; PrioIndex < (int32)MaxInternalIndex; ++PrioIndex)
	{
		OutPriorities[PrioIndex] = DefaultPriority;
	}
}

void FReplicationPrioritization::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	MaxInternalNetRefIndex = NewMaxInternalIndex;

	SetNetObjectListsSize(NewMaxInternalIndex);

	for (FPrioritizerInfo& PrioritizerInfo : PrioritizerInfos)
	{
		PrioritizerInfo.Prioritizer->OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
	}

	for (FPerConnectionInfo& ConnectionInfo : ConnectionInfos)
	{
		if (ConnectionInfo.IsValid)
		{
			ResizePrioritiesList(ConnectionInfo.Priorities, NewMaxInternalIndex);
		}
	}
}

void FReplicationPrioritization::SetStaticPriority(FInternalNetRefIndex ObjectIndex, float NewPrio)
{
	if (!ensureMsgf(NewPrio >= 0.0f, TEXT("Trying to set invalid priority %f"), NewPrio))
	{
		return;
	}

	uint8& Prioritizer = ObjectIndexToPrioritizer[ObjectIndex];
	float& Prio = DefaultPriorities[ObjectIndex];

	bool bPrioritizerDiffers = Prioritizer != FReplicationPrioritization_InvalidNetObjectPrioritizerIndex;
	bool bPrioDiffers = Prio != NewPrio;
	if (bPrioritizerDiffers || bPrioDiffers)
	{
		Prio = NewPrio;
		if (bPrioritizerDiffers)
		{
			FPrioritizerInfo& PrioritizerInfo = PrioritizerInfos[Prioritizer];
			--PrioritizerInfo.ObjectCount;
			PrioritizerInfo.Prioritizer->RemoveObject(ObjectIndex, NetObjectPrioritizationInfos[ObjectIndex]);
			Prioritizer = FReplicationPrioritization_InvalidNetObjectPrioritizerIndex;
		}

		ObjectsWithNewStaticPriority.SetBit(ObjectIndex);
	}
}

bool FReplicationPrioritization::SetPrioritizer(FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerHandle NewPrioritizer)
{
	if (!ensureMsgf(NewPrioritizer != InvalidNetObjectPrioritizerHandle, TEXT("%s"), TEXT("Call SetStaticPriority if you want to use a static priority for the object.")))
	{
		return false;
	}

	if (PrioritizerInfos.Num() == 0 || !ensureMsgf(NewPrioritizer < FNetObjectPrioritizerHandle(uint32(PrioritizerInfos.Num())), TEXT("Trying to set invalid prioritizer 0x%08x"), NewPrioritizer))
	{
		return false;
	}

	/**
	  * Not marking this object as in need of copying a new priority to each connection. 
	  * We keep the old priority value regardless of which prioritizer was previously used.
	  * That should work ok when we're throttling priority calculations. Let's just set the
	  * new default priority.
	  */
	DefaultPriorities[ObjectIndex] = 0.0f;

	// Unregister object from old prioritizer
	uint8& Prioritizer = ObjectIndexToPrioritizer[ObjectIndex];
	FNetObjectPrioritizationInfo& NetObjectPrioritizationInfo = NetObjectPrioritizationInfos[ObjectIndex];
	const bool bWasUsingStaticPriority = (Prioritizer == FReplicationPrioritization_InvalidNetObjectPrioritizerIndex);
	if (!bWasUsingStaticPriority)
	{
		FPrioritizerInfo& OldPrioritizerInfo = PrioritizerInfos[Prioritizer];
		--OldPrioritizerInfo.ObjectCount;
		OldPrioritizerInfo.Prioritizer->RemoveObject(ObjectIndex, NetObjectPrioritizationInfo);
	}

	// Register object with new prioritizer
	{
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);

		NetObjectPrioritizationInfo = FNetObjectPrioritizationInfo{};
		FNetObjectPrioritizerAddObjectParams AddParams = {NetObjectPrioritizationInfo, ObjectData.InstanceProtocol, ObjectData.Protocol, NetRefHandleManager->GetReplicatedObjectStateBufferNoCheck(ObjectIndex)};
		FPrioritizerInfo& PrioritizerInfo = PrioritizerInfos[NewPrioritizer];
		if (PrioritizerInfo.Prioritizer->AddObject(ObjectIndex, AddParams))
		{
			Prioritizer = static_cast<uint8>(NewPrioritizer);
			++PrioritizerInfo.ObjectCount;
			return true;
		}

		// If we fail setting the new prioritizer we default to use default static priority.
		UE_LOGF(LogIris, Verbose, "Prioritizer '%ls' does not support prioritizing object %u", ToCStr(PrioritizerInfo.Prioritizer->GetFName().GetPlainNameString()), ObjectIndex);

		// If we were previously using static priority we don't have to do anything, otherwise force set default priority.
		if (!bWasUsingStaticPriority)
		{
			DefaultPriorities[ObjectIndex] = DefaultPriority;
			Prioritizer = FReplicationPrioritization_InvalidNetObjectPrioritizerIndex;
			ObjectsWithNewStaticPriority.SetBit(ObjectIndex);
		}
	}

	return false;
}

FNetObjectPrioritizerHandle FReplicationPrioritization::GetPrioritizerHandle(const FName PrioritizerName) const
{
	FNetObjectPrioritizerHandle Handle = 0;
	// Expect few prioritizers to be registered. Just do a linear search.
	for (const FPrioritizerInfo& Info : PrioritizerInfos)
	{
		if (Info.Name == PrioritizerName)
		{
			return Handle;
		}

		++Handle;
	}

	if (PrioritizerName == UE::Net::Private::NAME_DefaultPrioritizer)
	{
		return DefaultSpatialNetObjectPrioritizerHandle;
	}

	return InvalidNetObjectPrioritizerHandle;
}

UNetObjectPrioritizer* FReplicationPrioritization::GetPrioritizer(const FName PrioritizerName) const
{
	// Expect few prioritizers to be registered. Just do a linear search.
	for (const FPrioritizerInfo& Info : PrioritizerInfos)
	{
		if (Info.Name == PrioritizerName)
		{
			return Info.Prioritizer.Get();
		}
	}

	return nullptr;
}

/**
  * There are many different ways to determine which objects should be prioritized. They have different performance
  * characteristics. Here are some ways:
  *
  * - Prioritize all objects that have been marked as dirty this frame along with each connection's wishes due to packet loss.
  *   packet loss should be rare and the list of objects with lost state should be small. This way works well if the number of
  *   dirty objects is relatively small. This should save a lot of time on the setup of the prioritization data. This can be coupled
  *   with having lost objects get a priority bump every frame until it's been resent. The latter would mean that there's no extra per
  *   connection setup cost at all other than filling in the right pointer to the priority data. A similar approach can also be taken
  *   with throttling of how often connections are replicated to. For example if a third of all connections are considered each net
  *   tick then one would maintain three different dirty object structures.
  *
  * - Get the list of objects to prioritize from each connection. This can make sense if there are many connections and they have 
  *   very different sets of scoped objects, for example if spatial filtering is applied and players are far away from eachother. 
  *   Prioritizing all connections' relevant objects would then cause ConnectionCount*PerConnectionObjectCount to be prioritized
  *   per connection which could be very expensive.
  *
  * If performance issues arise in certain scenarios like initial world spawn or late joing here are some additional thoughts.
  *
  *  During world spawn or late join it's likely a connection wants to replicate many objects. This can stress this system
  *  if there are a lot of objects using dynamic prioritization. Perhaps having the ReplicationDataStream in a special state 
  *  where many (or special) objects are replicated first before starting prioritizing dirty objects as well is something to consider.
  *  One could also just consider a smaller amount of objects (say 200) each frame and continue from the last object index
  *  prioritized the next time we get here. This works not only at initial replication but also if there is often a very large
  *  amount of dynamically prioritized objects. The connection will get the last priority value for the objects that aren't
  *  prioritized this frame. A very simple yet effective optimization! There can of course be artifacts such as an object
  *  receiving a low priority and then the player rotates quickly so that object is in the line of sight and it doesn't get replicated
  *  for a few fames until it receives the high priority it should have had. Perhaps one could mark either certain objects as not
  *  being subject to priority calculation throttling or mark a prioritizer as not being subject to throttling, in which case
  *  all dirty objects with such a prioritizer would always get an updated priority.
  */

/**
  * Prioritize objects as per connection's wishes.
  */
void FReplicationPrioritization::Prioritize(const FNetBitArrayView& ReplicatingConnections, const FNetBitArrayView& DirtyObjectsThisFrame)
{
	UpdatePrioritiesForNewAndDeletedObjects();
	NotifyPrioritizersOfDirtyObjects(DirtyObjectsThisFrame);

	if (!ReplicatingConnections.IsAnyBitSet())
	{
		return;
	}

	// Give prioritizers a chance to prepare for prioritization. It's only called if there's a chance Prioritize() will be called.
	{
		FNetObjectPrePrioritizationParams PrePrioParams;
		for (FPrioritizerInfo& Info : PrioritizerInfos)
		{
			if (Info.ObjectCount == 0U)
			{
				continue;
			}
			
			Info.Prioritizer->PrePrioritize(PrePrioParams);
		}
	}
	

	uint32* ConnectionIds = static_cast<uint32*>(FMemory_Alloca(ReplicatingConnections.GetNumBits()*4));
	uint32 ReplicatingConnectionCount = 0;
	ReplicatingConnections.ForAllSetBits([ConnectionIds, &ReplicatingConnectionCount](uint32 Bit) { ConnectionIds[ReplicatingConnectionCount++] = Bit; });

	FPrioritizerBatchHelper BatchHelper(PrioritizerInfos.Num());

	for (const uint32 ConnId : MakeArrayView(ConnectionIds, ReplicatingConnectionCount))
	{
		// If there's no view we do not prioritize
		if (Connections->GetReplicationView(ConnId).Views.Num() <= 0)
		{
			continue;
		}

		FReplicationConnection* Connection = Connections->GetConnection(ConnId);
		FReplicationWriter* ReplicationWriter = Connection->ReplicationWriter;
		const FNetBitArray& Objects = ReplicationWriter->GetObjectsRequiringPriorityUpdate();
		if (Objects.GetNumBits() == 0)
		{
			continue;
		}

		PrioritizeForConnection(ConnId, BatchHelper, MakeNetBitArrayView(Objects));

		// Pass updated priorities to the ReplicationWriter. Currently it is assumed priorities are stored persistently per object per connection.
		{
			FPerConnectionInfo& ConnInfo = ConnectionInfos[ConnId];
			const float* Priorities = ConnInfo.Priorities.GetData();
			ReplicationWriter->UpdatePriorities(Priorities);
		}
	}

	// Give prioritizers a chance to cleanup after prioritization. It's called if PrePrioritize() was called.
	{
		FNetObjectPostPrioritizationParams PostPrioParams;
		for (FPrioritizerInfo& Info : PrioritizerInfos)
		{
			if (Info.ObjectCount == 0U)
			{
				continue;
			}

			Info.Prioritizer->PostPrioritize(PostPrioParams);
		}
	}
}

void FReplicationPrioritization::AddConnection(uint32 ConnectionId, ENetConnectionTraits NetConnectionTraits)
{
	if (ConnectionId >= (uint32)ConnectionInfos.Num())
	{
		ConnectionInfos.SetNum(ConnectionId + 1U, EAllowShrinking::No);
	}

	++ConnectionCount;
	FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
	ConnectionInfo.Priorities = DefaultPriorities;
	ConnectionInfo.NextObjectIndexToProcess = 0;
	ConnectionInfo.IsValid = 1;

	for (FPrioritizerInfo& Info : PrioritizerInfos)
	{
		Info.Prioritizer->AddConnection(ConnectionId);
	}
}

void FReplicationPrioritization::RemoveConnection(uint32 ConnectionId)
{
	checkSlow(ConnectionId < (uint32)ConnectionInfos.Num());

	--ConnectionCount;
	FPerConnectionInfo& ConnectionInfo = ConnectionInfos[ConnectionId];
	ConnectionInfo.IsValid = 0;
	ConnectionInfo.Priorities.Empty();

	for (FPrioritizerInfo& Info : PrioritizerInfos)
	{
		Info.Prioritizer->RemoveConnection(ConnectionId);
	}
}

/**
  * For new objects we copy the most recently set priority to each connection.
  * For deleted objects we reset the priority to default. Propagation of this priority to each connection is unnecessary
  * as we propagate the priority, which may have changed from the default, when the object is detected as new.
  */
void FReplicationPrioritization::UpdatePrioritiesForNewAndDeletedObjects()
{
	IRIS_PROFILER_SCOPE(FReplicationPrioritization_UpdatePrioritiesForNewAndDeletedObjects);

	const FNetBitArrayView PrevScopedIndices = NetRefHandleManager->GetPrevFrameScopableInternalIndices();
	const FNetBitArrayView ScopedIndices = NetRefHandleManager->GetCurrentFrameScopableInternalIndices();

	auto ForEachRemovedObject = [this](uint32 ObjectIndex)
	{
		uint8& Prioritizer = ObjectIndexToPrioritizer[ObjectIndex];
		if (Prioritizer != FReplicationPrioritization_InvalidNetObjectPrioritizerIndex)
		{
			FPrioritizerInfo& OldPrioritizerInfo = PrioritizerInfos[Prioritizer];
			--OldPrioritizerInfo.ObjectCount;
			OldPrioritizerInfo.Prioritizer->RemoveObject(ObjectIndex, NetObjectPrioritizationInfos[ObjectIndex]);
		}

		Prioritizer = FReplicationPrioritization_InvalidNetObjectPrioritizerIndex;
		DefaultPriorities[ObjectIndex] = DefaultPriority;
	};

	TArray<FInternalNetRefIndex> NewIndices;
	TFunction<void(FInternalNetRefIndex)> DoNothing = [](FInternalNetRefIndex ObjectIndex){};
	TFunction<void(FInternalNetRefIndex)> AddIndexAndClearFromNewPriority = [&NewIndices, this](FInternalNetRefIndex ObjectIndex)
	{
		NewIndices.Add(ObjectIndex);
		// Prevent the same index from being added twice
		this->ObjectsWithNewStaticPriority.ClearBit(ObjectIndex);
	};

	TFunction<void(FInternalNetRefIndex)> ForEachNewObject = (ConnectionCount > 0 ? AddIndexAndClearFromNewPriority : DoNothing);
	if (ConnectionCount > 0)
	{
		NewIndices.Reserve(FMath::Min(1024U, MaxInternalNetRefIndex));
	}

	FNetBitArrayView::ForAllExclusiveBits(ScopedIndices, PrevScopedIndices, ForEachNewObject, ForEachRemovedObject);

	if (HasNewObjectsWithStaticPriority)
	{
		HasNewObjectsWithStaticPriority = 0;

		if (ConnectionCount > 0)
		{
			ObjectsWithNewStaticPriority.ForAllSetBits([this, &NewIndices](FInternalNetRefIndex ObjectIndex)
			{ 
				NewIndices.Add(ObjectIndex); 
			});
		}

		// $IRIS TODO: Want ForAllSetBits with clear.
		ObjectsWithNewStaticPriority.ClearAllBits();
	}

	// Copy the priorities for the new objects to each connection
	if (NewIndices.Num() > 0 && ConnectionCount > 0)
	{
		const TArrayView<FInternalNetRefIndex> ObjectIndices = MakeArrayView(NewIndices);
		for (FPerConnectionInfo& ConnectionInfo : ConnectionInfos)
		{
			if (!ConnectionInfo.IsValid)
			{
				continue;
			}

			for (const FInternalNetRefIndex ObjectIndex : ObjectIndices)
			{
				ConnectionInfo.Priorities[ObjectIndex] = DefaultPriorities[ObjectIndex];
			}
		}
	}
}

/**
 * 1. Sort indices by prioritizer first, index second. Try to keep static priority objects last.
 * 2. Loop through index list until new prioritizer is found and pass the info to the prioritizer for processing.
 */
void FReplicationPrioritization::PrioritizeForConnection(uint32 ConnId, FPrioritizerBatchHelper& BatchHelper, const FNetBitArrayView Objects)
{
	IRIS_PROFILER_SCOPE(FReplicationPrioritization_PrioritizeForConnection);

	FPerConnectionInfo& ConnInfo = ConnectionInfos[ConnId];

	FNetObjectPrioritizationParams PrioParameters;
	// Setup static part of the prio parameters.
	{
		PrioParameters.Priorities = ConnInfo.Priorities.GetData();
		PrioParameters.PrioritizationInfos = NetObjectPrioritizationInfos.GetData();
		PrioParameters.ConnectionId = ConnId;
		PrioParameters.View = Connections->GetReplicationView(ConnId);
	}

	/**
	  * Split objects per prioritizer and copy priorities for the objects to the connection specific priority array.
	  * The latter allows throttling of priority calculations by keeping the latest priority until a new one is calculated.
	  */
	BatchHelper.InitForConnection();
	while (true)
	{
		const FPrioritizerBatchHelper::EBatchProcessStatus ProcessStatus = BatchHelper.PrepareBatch(ConnInfo, Objects, ObjectIndexToPrioritizer.GetData(), DefaultPriorities.GetData());
		if (ProcessStatus == FPrioritizerBatchHelper::EBatchProcessStatus::ProcessAllBatchesAndStop)
		{
			for (FPrioritizerBatchHelper::FPerPrioritizerInfo& PerPrioritizerInfo : BatchHelper.PerPrioritizerInfos)
			{
				for (int32 ObjectCount = PerPrioritizerInfo.ObjectIndices.Num(); ObjectCount > 0; ObjectCount = PerPrioritizerInfo.ObjectIndices.Num())
				{
					PrioParameters.ObjectIndices = PerPrioritizerInfo.ObjectIndices.GetFirstChunkData();
					PrioParameters.ObjectCount = PerPrioritizerInfo.ObjectIndices.GetFirstChunkNum();

					const int32 PrioritizerIndex = static_cast<int32>(&PerPrioritizerInfo - BatchHelper.PerPrioritizerInfos.GetData());
					UNetObjectPrioritizer* Prioritizer = PrioritizerInfos[PrioritizerIndex].Prioritizer.Get();
					Prioritizer->Prioritize(PrioParameters);

					PerPrioritizerInfo.ObjectIndices.PopChunkSafe();
				}
			}
			break;
		}
		else if (ProcessStatus == FPrioritizerBatchHelper::EBatchProcessStatus::ProcessFullBatchesAndContinue)
		{
			for (FPrioritizerBatchHelper::FPerPrioritizerInfo& PerPrioritizerInfo : BatchHelper.PerPrioritizerInfos)
			{
				if (PerPrioritizerInfo.ObjectIndices.Num() < FPrioritizerBatchHelper::MaxObjectCountPerBatch)
				{
					continue;
				}

				PrioParameters.ObjectIndices = PerPrioritizerInfo.ObjectIndices.GetFirstChunkData();
				PrioParameters.ObjectCount = PerPrioritizerInfo.ObjectIndices.GetFirstChunkNum();

				const int32 PrioritizerIndex = static_cast<int32>(&PerPrioritizerInfo - BatchHelper.PerPrioritizerInfos.GetData());
				UNetObjectPrioritizer* Prioritizer = PrioritizerInfos[PrioritizerIndex].Prioritizer.Get();
				Prioritizer->Prioritize(PrioParameters);

				PerPrioritizerInfo.ObjectIndices.PopChunkSafe();
			}

			continue;
		}

		// We're done
		break;
	}

	// Optionally force very high priority on view targets
	if (CVar_ForceConnectionViewerPriority > 0)
	{
		SetHighPriorityOnViewTargets(MakeArrayView(ConnInfo.Priorities), PrioParameters.View);
	}
}

void FReplicationPrioritization::SetHighPriorityOnViewTargets(const TArrayView<float>& Priorities, const FReplicationView& ReplicationView)
{
	using namespace UE::Net::Private;

	// We allow a view target to appear multiple times. It will get the same priority regardless.
	TArray<FNetHandle, TInlineAllocator<16>> ViewTargets;
	for (const FReplicationView::FView& View : ReplicationView.Views)
	{
		if (View.Controller.IsValid())
		{
			ViewTargets.Add(View.Controller);
		}
		if (View.ViewTarget != View.Controller && View.ViewTarget.IsValid())
		{
			ViewTargets.Add(View.ViewTarget);
		}
	}

	for (FNetHandle NetHandle : ViewTargets)
	{
		const FInternalNetRefIndex ViewTargetInternalIndex = NetRefHandleManager->GetInternalIndexFromNetHandle(NetHandle);
		if (ViewTargetInternalIndex != InvalidInternalNetRefIndex)
		{
			Priorities[ViewTargetInternalIndex] = ViewTargetHighPriority;
		}
	}
}

/**
 * Notify prioritizers of which objects have been updated since last frame.	 
 */
void FReplicationPrioritization::NotifyPrioritizersOfDirtyObjects(const FNetBitArrayView& DirtyObjectsThisFrame)
{
	IRIS_PROFILER_SCOPE(FReplicationPrioritization_NotifyPrioritizersOfDirtyObjects);

	FUpdateDirtyObjectsBatchHelper BatchHelper(NetRefHandleManager, PrioritizerInfos.Num());

	constexpr SIZE_T MaxBatchObjectCount = FUpdateDirtyObjectsBatchHelper::Constants::MaxObjectCountPerBatch;
	FInternalNetRefIndex ObjectIndices[MaxBatchObjectCount];

	const uint32 BitCount = ~0U;
	for (uint32 ObjectCount, StartIndex = 0; (ObjectCount = DirtyObjectsThisFrame.GetSetBitIndices(StartIndex, BitCount, ObjectIndices, MaxBatchObjectCount)) > 0; )
	{
		BatchNotifyPrioritizersOfDirtyObjects(BatchHelper, ObjectIndices, ObjectCount);

		StartIndex = ObjectIndices[ObjectCount - 1] + 1U;
		if ((StartIndex == DirtyObjectsThisFrame.GetNumBits()) | (ObjectCount < MaxBatchObjectCount))
		{
			break;
		}
	}
}

void FReplicationPrioritization::BatchNotifyPrioritizersOfDirtyObjects(FUpdateDirtyObjectsBatchHelper& BatchHelper, FInternalNetRefIndex* ObjectIndices, uint32 ObjectCount)
{
	BatchHelper.PrepareBatch(ObjectIndices, ObjectCount, ObjectIndexToPrioritizer.GetData());

	FNetObjectPrioritizerUpdateParams UpdateParameters;
	UpdateParameters.StateBuffers = &NetRefHandleManager->GetReplicatedObjectStateBuffers();
	UpdateParameters.PrioritizationInfos = NetObjectPrioritizationInfos.GetData();

	for (const FUpdateDirtyObjectsBatchHelper::FPerPrioritizerInfo& PerPrioritizerInfo : BatchHelper.PerPrioritizerInfos)
	{
		if (PerPrioritizerInfo.ObjectCount == 0)
		{
			continue;
		}

		UpdateParameters.ObjectIndices = PerPrioritizerInfo.ObjectIndices;
		UpdateParameters.ObjectCount = PerPrioritizerInfo.ObjectCount;
		UpdateParameters.InstanceProtocols = PerPrioritizerInfo.InstanceProtocols;

		const int32 PrioritizerIndex = static_cast<int32>(&PerPrioritizerInfo - BatchHelper.PerPrioritizerInfos.GetData());
		UNetObjectPrioritizer* Prioritizer = PrioritizerInfos[PrioritizerIndex].Prioritizer.Get();
		Prioritizer->UpdateObjects(UpdateParameters);
	}
}

void FReplicationPrioritization::InitPrioritizers()
{
	/**
	  * $IRIS TODO: Figure out what kind of hotfixing support we need. There are different trade-offs
	  * depending how we set prioritizer on an object and how the user decides to cache or not cache
	  * prioritizer handles. Not having handles and always set by name doesn't really solve much
	  * as the SetPrioritizer call would just return false anyway if the prioritizer does not exist.
	  * However we currently do not invalidate handles, which is perhaps a good thing as it allows
	  * switching prioritizers behind the scenes.
	  * As for hotfixing prioritizer configs that's up to the implementor of the prioritizer.
	  */
	PrioritizerDefinitions = TStrongObjectPtr<UNetObjectPrioritizerDefinitions>(NewObject<UNetObjectPrioritizerDefinitions>());
	TArray<FNetObjectPrioritizerDefinition> Definitions;
	PrioritizerDefinitions->GetValidDefinitions(Definitions);

	// We store a uint8 per object to prioritizer.
	check(Definitions.Num() <= 256);

	PrioritizerInfos.Reserve(Definitions.Num());
	for (FNetObjectPrioritizerDefinition& Definition : Definitions)
	{
		TStrongObjectPtr<UNetObjectPrioritizer> Prioritizer(NewObject<UNetObjectPrioritizer>((UObject*)GetTransientPackage(), Definition.Class, MakeUniqueObjectName(nullptr, Definition.Class, Definition.PrioritizerName)));

		FNetObjectPrioritizerInitParams InitParams;
		InitParams.ReplicationSystem = ReplicationSystem;
		InitParams.Config = (Definition.ConfigClass != nullptr ? NewObject<UNetObjectPrioritizerConfig>((UObject*)GetTransientPackage(), Definition.ConfigClass) : nullptr);
		InitParams.AbsoluteMaxNetObjectCount = NetRefHandleManager->GetMaxActiveObjectCount();
		InitParams.CurrentMaxInternalIndex = MaxInternalNetRefIndex;
		InitParams.MaxConnectionCount = Connections->GetMaxConnectionCount();

		Prioritizer->Init(InitParams);

		FPrioritizerInfo& Info = PrioritizerInfos.Emplace_GetRef();
		Info.Prioritizer = Prioritizer;
		Info.Name = Definition.PrioritizerName;
		Info.ObjectCount = 0;
	}

#if UE_GAME || UE_SERVER
	UE_CLOGF(PrioritizerInfos.Num() == 0, LogIris, Warning, "%ls", TEXT("No prioritizers have been registered. This may result in a bad gameplay experience because nearby actors will not have higher priority than actors far away."));
#endif
}

}
