// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizer.h"
#include "Net/Core/NetBitArray.h"
#include "UObject/StrongObjectPtr.h"

class UNetObjectPrioritizerDefinitions;
class UReplicationSystem;
namespace UE::Net
{
	class FNetBitArrayView;
	
	typedef uint32 FInternalNetRefIndex;

	enum class ENetConnectionTraits : uint32;
	namespace Private
	{
		class FNetRefHandleManager;
		class FReplicationConnections;
	}

	// For testing
	class FTestNetObjectPrioritizerFixture;
}

namespace UE::Net::Private
{

struct FReplicationPrioritizationInitParams
{
	TObjectPtr<const UReplicationSystem> ReplicationSystem;
	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	FReplicationConnections* Connections = nullptr;
	FInternalNetRefIndex MaxInternalNetRefIndex = 0;
};

class FReplicationPrioritization
{
public:
	FReplicationPrioritization();

	void Init(FReplicationPrioritizationInitParams& Params);
	void Deinit();

	/** Called when the maximum InternalNetRefIndex increased and we need to realloc our lists */
	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

	void Prioritize(const FNetBitArrayView& ConnectionsToSend, const FNetBitArrayView& DirtyObjectsThisFrame);

	void SetStaticPriority(FInternalNetRefIndex ObjectIndex, float Prio);
	bool SetPrioritizer(FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerHandle Prioritizer);
	FNetObjectPrioritizerHandle GetPrioritizerHandle(const FName PrioritizerName) const;
	UNetObjectPrioritizer* GetPrioritizer(const FName PrioritizerName) const;

	void AddConnection(uint32 ConnectionId, ENetConnectionTraits NetConnectionTraits);
	void RemoveConnection(uint32 ConnectionId);

	float GetObjectPriorityForConnection(uint32 ConnectionId, FInternalNetRefIndex InternalIndex) const
	{
		return GetPrioritiesForConnection(ConnectionId)[InternalIndex];
	}

private:
	class FPrioritizerBatchHelper;
	class FUpdateDirtyObjectsBatchHelper;

	void SetNetObjectListsSize(FInternalNetRefIndex MaxInternalIndex);
	void ResizePrioritiesList(TArray<float>& OutPriorities, FInternalNetRefIndex MaxInternalIndex);

	void UpdatePrioritiesForNewAndDeletedObjects();
	void PrioritizeForConnection(uint32 ConnId, FPrioritizerBatchHelper& BatchHelper, FNetBitArrayView Objects);
	void SetHighPriorityOnViewTargets(const TArrayView<float>& Priorities, const FReplicationView& View);
	void NotifyPrioritizersOfDirtyObjects(const FNetBitArrayView& DirtyObjectsThisFrame);
	void BatchNotifyPrioritizersOfDirtyObjects(FUpdateDirtyObjectsBatchHelper& BatchHelper, FInternalNetRefIndex* ObjectIndices, uint32 ObjectCount);
	void InitPrioritizers();

private:
	friend UE::Net::FTestNetObjectPrioritizerFixture;

	// For testing
	TConstArrayView<float> GetPrioritiesForConnection(uint32 ConnectionId) const;

	struct FPrioritizerInfo
	{
		TStrongObjectPtr<UNetObjectPrioritizer> Prioritizer;
		FName Name;
		uint32 ObjectCount;
	};

	struct FPerConnectionInfo
	{
		FPerConnectionInfo() : NextObjectIndexToProcess(0), IsValid(0) {}

		TArray<float> Priorities;
		uint32 NextObjectIndexToProcess;
		uint32 IsValid : 1;
	};

	static constexpr float DefaultPriority = 1.0f;
	static constexpr float ViewTargetHighPriority = 1.0E7f;

	TObjectPtr<const UReplicationSystem> ReplicationSystem = nullptr;
	FReplicationConnections* Connections = nullptr;
	const FNetRefHandleManager* NetRefHandleManager = nullptr;

	TStrongObjectPtr<UNetObjectPrioritizerDefinitions> PrioritizerDefinitions;

	TArray<FNetObjectPrioritizationInfo> NetObjectPrioritizationInfos;
	TArray<uint8> ObjectIndexToPrioritizer;
	TArray<FPrioritizerInfo> PrioritizerInfos;
	TArray<FPerConnectionInfo> ConnectionInfos;
	TArray<float> DefaultPriorities;
	FNetBitArray ObjectsWithNewStaticPriority;

	FInternalNetRefIndex MaxInternalNetRefIndex = 0;

	uint32 ConnectionCount = 0;
	uint32 HasNewObjectsWithStaticPriority : 1;
};

inline TConstArrayView<float> FReplicationPrioritization::GetPrioritiesForConnection(uint32 ConnectionId) const
{
	return MakeArrayView(ConnectionInfos[ConnectionId].Priorities);
}

}
