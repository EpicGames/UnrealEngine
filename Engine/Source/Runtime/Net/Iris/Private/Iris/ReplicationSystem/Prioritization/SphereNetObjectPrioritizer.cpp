// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Prioritization/SphereNetObjectPrioritizer.h"
#include "Iris/ReplicationSystem/RepTag.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/Serialization/VectorNetSerializers.h"
#include "Iris/Core/IrisProfiler.h"
#include "Misc/MemStack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SphereNetObjectPrioritizer)

void USphereNetObjectPrioritizer::Init(FNetObjectPrioritizerInitParams& Params)
{
	checkf(Params.Config != nullptr, TEXT("Need config to operate."));
	Config = TStrongObjectPtr<USphereNetObjectPrioritizerConfig>(CastChecked<USphereNetObjectPrioritizerConfig>(Params.Config));

	Super::Init(Params);
}

void USphereNetObjectPrioritizer::Deinit()
{
	Super::Deinit();

	Config = nullptr;
}

void USphereNetObjectPrioritizer::Prioritize(FNetObjectPrioritizationParams& PrioritizationParams)
{
	IRIS_PROFILER_SCOPE(USphereNetObjectPrioritizer_Prioritize);
	
	FMemStack& Mem = FMemStack::Get();
	FMemMark MemMark(Mem);

	// Trade-off memory/performance
	constexpr uint32 MaxBatchObjectCount = 1024U;

	uint32 BatchObjectCount = FMath::Min((PrioritizationParams.ObjectCount + 3U) & ~3U, MaxBatchObjectCount);
	FBatchParams BatchParams;
	SetupBatchParams(BatchParams, PrioritizationParams, BatchObjectCount, Mem);

	for (uint32 ObjectIt = 0, ObjectEndIt = PrioritizationParams.ObjectCount; ObjectIt < ObjectEndIt; )
	{
		const uint32 CurrentBatchObjectCount = FMath::Min(ObjectEndIt - ObjectIt, MaxBatchObjectCount);

		BatchParams.ObjectCount = CurrentBatchObjectCount;
		PrepareBatch(BatchParams, PrioritizationParams, ObjectIt);
		PrioritizeBatch(BatchParams);
		FinishBatch(BatchParams, PrioritizationParams, ObjectIt);

		ObjectIt += CurrentBatchObjectCount;
	}
}

void USphereNetObjectPrioritizer::PrepareBatch(FBatchParams& BatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset)
{
	IRIS_PROFILER_SCOPE(USphereNetObjectPrioritizer_PrepareBatch);
	const float* ExternalPriorities = PrioritizationParams.Priorities;
	const uint32* ObjectIndices = PrioritizationParams.ObjectIndices;
	const FNetObjectPrioritizationInfo* PrioritizationInfos = PrioritizationParams.PrioritizationInfos;

	float* LocalPriorities = BatchParams.Priorities;
	VectorRegister* Positions = BatchParams.Positions;

	// Copy priorities.
	{
		uint32 LocalObjIt = 0;
		for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = ObjIt + BatchParams.ObjectCount; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
		{
			const uint32 ObjectIndex = ObjectIndices[ObjIt];
			LocalPriorities[LocalObjIt] = ExternalPriorities[ObjectIndex];
		}
	}

	// Copy positions. 
	uint32 LocalObjIt = 0;
	{
		for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = ObjIt + BatchParams.ObjectCount; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
		{
			const uint32 ObjectIndex = ObjectIndices[ObjIt];
			const FObjectLocationInfo& Info = static_cast<const FObjectLocationInfo&>(PrioritizationInfos[ObjectIndex]);
			Positions[LocalObjIt] = GetLocation(Info);
		}
	}

	// Make sure we have a multiple of four valid entries.
	for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = (ObjIt + 3U) & ~3U; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
	{
		LocalPriorities[LocalObjIt] = 0.0f;
		Positions[LocalObjIt] = VectorZero();
	}
}

void USphereNetObjectPrioritizer::PrioritizeBatch(FBatchParams& BatchParams)
{
	const int32 ViewCount = BatchParams.View.Views.Num();
	if (ViewCount == 1)
	{
		PrioritizeBatchForSingleView(BatchParams);
	}
	else if (ViewCount == 2)
	{
		PrioritizeBatchForDualView(BatchParams);
	}
	else
	{
		PrioritizeBatchForMultiView(BatchParams);
	}
}

/**
  * Priority falls off linearly with distance from the object position to the view position.
  *
  * The equation is:
  * OuterPriority + (OuterPriority - InnerPriority)*(Clamp(Distance(ObjPos, ViewPos), InnerRadius, OuterRadius)/(OuterRadius - InnerRadius))
  */

void USphereNetObjectPrioritizer::PrioritizeBatchForSingleView(FBatchParams& BatchParams)
{
	IRIS_PROFILER_SCOPE(USphereNetObjectPrioritizer_PrioritizeBatchForSingleView);
	const FVector& ViewPosVector = BatchParams.View.Views[0].Pos;
	const VectorRegister ViewPos = VectorLoadFloat3_W0(&ViewPosVector);

	const VectorRegister* Positions = BatchParams.Positions;
	float* Priorities = BatchParams.Priorities;
	for (uint32 ObjIt = 0, ObjEndIt = BatchParams.ObjectCount; ObjIt < ObjEndIt; ObjIt += 4)
	{
		const VectorRegister Pos0 = Positions[ObjIt + 0];
		const VectorRegister Pos1 = Positions[ObjIt + 1];
		const VectorRegister Pos2 = Positions[ObjIt + 2];
		const VectorRegister Pos3 = Positions[ObjIt + 3];

		const VectorRegister OriginalPriorities0123 = VectorLoadAligned(Priorities + ObjIt);

		// Distance from point to view center
		const VectorRegister Dist0 = VectorSubtract(Pos0, ViewPos);
		const VectorRegister Dist1 = VectorSubtract(Pos1, ViewPos);
		const VectorRegister Dist2 = VectorSubtract(Pos2, ViewPos);
		const VectorRegister Dist3 = VectorSubtract(Pos3, ViewPos);

		const VectorRegister ScalarDistSqr0 = VectorDot4(Dist0, Dist0);
		const VectorRegister ScalarDistSqr1 = VectorDot4(Dist1, Dist1);
		const VectorRegister ScalarDistSqr2 = VectorDot4(Dist2, Dist2);
		const VectorRegister ScalarDistSqr3 = VectorDot4(Dist3, Dist3);

		// Assemble all distances into a single vector
		// $IRIS TODO: This can be optimized with SSE 4.1 using _mm_blend_ps. VectorDot4 or similar would only have to store the result in the X component too.
		const VectorRegister ScalarDistSqr0101 = VectorSwizzle(VectorCombineHigh(ScalarDistSqr0, ScalarDistSqr1), 0, 2, 1, 3);
		const VectorRegister ScalarDistSqr2323 = VectorSwizzle(VectorCombineHigh(ScalarDistSqr2, ScalarDistSqr3), 0, 2, 1, 3);
		const VectorRegister ScalarDistSqr0123 = VectorCombineHigh(ScalarDistSqr0101, ScalarDistSqr2323);

		const VectorRegister ScalarDist0123 = VectorSqrt(ScalarDistSqr0123);
		const VectorRegister ClampedScalarDist0123 = VectorMax(VectorSubtract(ScalarDist0123, BatchParams.PriorityCalculationConstants.InnerRadius), VectorZeroVectorRegister());

		// Calculate priority assuming the object is inside the sphere
		const VectorRegister RadiusFactor = VectorMultiply(ClampedScalarDist0123, BatchParams.PriorityCalculationConstants.InvRadiusDiff);
		VectorRegister Priorities0123 = VectorMultiplyAdd(RadiusFactor, BatchParams.PriorityCalculationConstants.PriorityDiff, BatchParams.PriorityCalculationConstants.InnerPriority);

		// If object is outside the sphere we use the OutsidePriority
		const VectorRegister OutsideSphereMask = VectorCompareGT(ClampedScalarDist0123, BatchParams.PriorityCalculationConstants.RadiusDiff);
		Priorities0123 = VectorSelect(OutsideSphereMask, BatchParams.PriorityCalculationConstants.OutsidePriority, Priorities0123);

		// Store the max of our calculated priority and the provided priorities
		Priorities0123 = VectorMax(Priorities0123, OriginalPriorities0123);
		VectorStoreAligned(Priorities0123, Priorities + ObjIt);
	}
}

void USphereNetObjectPrioritizer::PrioritizeBatchForDualView(FBatchParams& BatchParams)
{
	IRIS_PROFILER_SCOPE(USphereNetObjectPrioritizer_PrioritizeBatchForDualView);
	const FVector& ViewPos0Vector = BatchParams.View.Views[0].Pos;
	const VectorRegister ViewPos0 = VectorLoadFloat3_W0(&ViewPos0Vector);

	const FVector& ViewPos1Vector = BatchParams.View.Views[1].Pos;
	const VectorRegister ViewPos1 = VectorLoadFloat3_W0(&ViewPos1Vector);

	const VectorRegister* Positions = BatchParams.Positions;
	float* Priorities = BatchParams.Priorities;
	for (uint32 ObjIt = 0, ObjEndIt = BatchParams.ObjectCount; ObjIt < ObjEndIt; ObjIt += 4)
	{
		const VectorRegister Pos0 = Positions[ObjIt + 0];
		const VectorRegister Pos1 = Positions[ObjIt + 1];
		const VectorRegister Pos2 = Positions[ObjIt + 2];
		const VectorRegister Pos3 = Positions[ObjIt + 3];

		const VectorRegister OriginalPriorities0123 = VectorLoadAligned(Priorities + ObjIt);

		// Distance from point to view centers
		const VectorRegister Dist0_0 = VectorSubtract(Pos0, ViewPos0);
		const VectorRegister Dist1_0 = VectorSubtract(Pos1, ViewPos0);
		const VectorRegister Dist2_0 = VectorSubtract(Pos2, ViewPos0);
		const VectorRegister Dist3_0 = VectorSubtract(Pos3, ViewPos0);

		VectorRegister ScalarDistSqr0 = VectorDot4(Dist0_0, Dist0_0);
		VectorRegister ScalarDistSqr1 = VectorDot4(Dist1_0, Dist1_0);
		VectorRegister ScalarDistSqr2 = VectorDot4(Dist2_0, Dist2_0);
		VectorRegister ScalarDistSqr3 = VectorDot4(Dist3_0, Dist3_0);

		const VectorRegister Dist0_1 = VectorSubtract(Pos0, ViewPos1);
		const VectorRegister Dist1_1 = VectorSubtract(Pos1, ViewPos1);
		const VectorRegister Dist2_1 = VectorSubtract(Pos2, ViewPos1);
		const VectorRegister Dist3_1 = VectorSubtract(Pos3, ViewPos1);

		const VectorRegister ScalarDistSqr0_1 = VectorDot4(Dist0_1, Dist0_1);
		const VectorRegister ScalarDistSqr1_1 = VectorDot4(Dist1_1, Dist1_1);
		const VectorRegister ScalarDistSqr2_1 = VectorDot4(Dist2_1, Dist2_1);
		const VectorRegister ScalarDistSqr3_1 = VectorDot4(Dist3_1, Dist3_1);

		// Pick closest points
		ScalarDistSqr0 = VectorMin(ScalarDistSqr0, ScalarDistSqr0_1);
		ScalarDistSqr1 = VectorMin(ScalarDistSqr1, ScalarDistSqr1_1);
		ScalarDistSqr2 = VectorMin(ScalarDistSqr2, ScalarDistSqr2_1);
		ScalarDistSqr3 = VectorMin(ScalarDistSqr3, ScalarDistSqr3_1);

		// Assemble all distances into a single vector
		const VectorRegister ScalarDistSqr0101 = VectorSwizzle(VectorCombineHigh(ScalarDistSqr0, ScalarDistSqr1), 0, 2, 1, 3);
		const VectorRegister ScalarDistSqr2323 = VectorSwizzle(VectorCombineHigh(ScalarDistSqr2, ScalarDistSqr3), 0, 2, 1, 3);
		const VectorRegister ScalarDistSqr0123 = VectorCombineHigh(ScalarDistSqr0101, ScalarDistSqr2323);

		const VectorRegister ScalarDist0123 = VectorSqrt(ScalarDistSqr0123);
		const VectorRegister ClampedScalarDist0123 = VectorMax(VectorSubtract(ScalarDist0123, BatchParams.PriorityCalculationConstants.InnerRadius), VectorZeroVectorRegister());

		// Calculate priority assuming the object is inside the sphere
		const VectorRegister RadiusFactor = VectorMultiply(ClampedScalarDist0123, BatchParams.PriorityCalculationConstants.InvRadiusDiff);
		VectorRegister Priorities0123 = VectorMultiplyAdd(RadiusFactor, BatchParams.PriorityCalculationConstants.PriorityDiff, BatchParams.PriorityCalculationConstants.InnerPriority);

		// If object is outside the sphere we use the OutsidePriority
		const VectorRegister OutsideSphereMask = VectorCompareGT(ClampedScalarDist0123, BatchParams.PriorityCalculationConstants.RadiusDiff);
		Priorities0123 = VectorSelect(OutsideSphereMask, BatchParams.PriorityCalculationConstants.OutsidePriority, Priorities0123);

		// Store the max of our calculated priority and the provided priorities
		Priorities0123 = VectorMax(Priorities0123, OriginalPriorities0123);
		VectorStoreAligned(Priorities0123, Priorities + ObjIt);
	}
}

void USphereNetObjectPrioritizer::PrioritizeBatchForMultiView(FBatchParams& BatchParams)
{
	IRIS_PROFILER_SCOPE(USphereNetObjectPrioritizer_PrioritizeBatchForMultiView);
	TArray<VectorRegister, TInlineAllocator<8>> ViewPositions;
	for (const UE::Net::FReplicationView::FView& View : BatchParams.View.Views)
	{
		const FVector& ViewPosVector = View.Pos;
		ViewPositions.Add(VectorLoadFloat3_W0(&ViewPosVector));
	}

	const VectorRegister MaxFloatVector = VectorSetFloat1(MAX_flt);

	const VectorRegister* Positions = BatchParams.Positions;
	float* Priorities = BatchParams.Priorities;
	for (uint32 ObjIt = 0, ObjEndIt = BatchParams.ObjectCount; ObjIt < ObjEndIt; ObjIt += 4)
	{
		const VectorRegister Pos0 = Positions[ObjIt + 0];
		const VectorRegister Pos1 = Positions[ObjIt + 1];
		const VectorRegister Pos2 = Positions[ObjIt + 2];
		const VectorRegister Pos3 = Positions[ObjIt + 3];

		const VectorRegister OriginalPriorities0123 = VectorLoadAligned(Priorities + ObjIt);

		// Distance from point to view centers
		VectorRegister ScalarDistSqr0 = MaxFloatVector;
		VectorRegister ScalarDistSqr1 = MaxFloatVector;
		VectorRegister ScalarDistSqr2 = MaxFloatVector;
		VectorRegister ScalarDistSqr3 = MaxFloatVector;

		for (VectorRegister ViewPos : ViewPositions)
		{
			const VectorRegister Dist0 = VectorSubtract(Pos0, ViewPos);
			const VectorRegister Dist1 = VectorSubtract(Pos1, ViewPos);
			const VectorRegister Dist2 = VectorSubtract(Pos2, ViewPos);
			const VectorRegister Dist3 = VectorSubtract(Pos3, ViewPos);

 			ScalarDistSqr0 = VectorMin(ScalarDistSqr0, VectorDot4(Dist0, Dist0));
			ScalarDistSqr1 = VectorMin(ScalarDistSqr1, VectorDot4(Dist1, Dist1));
			ScalarDistSqr2 = VectorMin(ScalarDistSqr2, VectorDot4(Dist2, Dist2));
			ScalarDistSqr3 = VectorMin(ScalarDistSqr3, VectorDot4(Dist3, Dist3));
		}

		// Assemble all distances into a single vector
		const VectorRegister ScalarDistSqr0101 = VectorSwizzle(VectorCombineHigh(ScalarDistSqr0, ScalarDistSqr1), 0, 2, 1, 3);
		const VectorRegister ScalarDistSqr2323 = VectorSwizzle(VectorCombineHigh(ScalarDistSqr2, ScalarDistSqr3), 0, 2, 1, 3);
		const VectorRegister ScalarDistSqr0123 = VectorCombineHigh(ScalarDistSqr0101, ScalarDistSqr2323);

		const VectorRegister ScalarDist0123 = VectorSqrt(ScalarDistSqr0123);
		const VectorRegister ClampedScalarDist0123 = VectorMax(VectorSubtract(ScalarDist0123, BatchParams.PriorityCalculationConstants.InnerRadius), VectorZeroVectorRegister());

		// Calculate priority assuming the object is inside the sphere
		const VectorRegister RadiusFactor = VectorMultiply(ClampedScalarDist0123, BatchParams.PriorityCalculationConstants.InvRadiusDiff);
		VectorRegister Priorities0123 = VectorMultiplyAdd(RadiusFactor, BatchParams.PriorityCalculationConstants.PriorityDiff, BatchParams.PriorityCalculationConstants.InnerPriority);

		// If object is outside the sphere we use the OutsidePriority
		const VectorRegister OutsideSphereMask = VectorCompareGT(ClampedScalarDist0123, BatchParams.PriorityCalculationConstants.RadiusDiff);
		Priorities0123 = VectorSelect(OutsideSphereMask, BatchParams.PriorityCalculationConstants.OutsidePriority, Priorities0123);

		// Store the max of our calculated priority and the provided priorities
		Priorities0123 = VectorMax(Priorities0123, OriginalPriorities0123);
		VectorStoreAligned(Priorities0123, Priorities + ObjIt);
	}
}

void USphereNetObjectPrioritizer::FinishBatch(const FBatchParams& BatchParams, FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset)
{
	IRIS_PROFILER_SCOPE(USphereNetObjectPrioritizer_FinishBatch);
	float* ExternalPriorities = PrioritizationParams.Priorities;
	const uint32* ObjectIndices = PrioritizationParams.ObjectIndices;

	const float* LocalPriorities = BatchParams.Priorities;

	// Update the object priority array
	uint32 LocalObjIt = 0;
	for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = ObjIt + BatchParams.ObjectCount; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
	{
		const uint32 ObjectIndex = ObjectIndices[ObjIt];
		ExternalPriorities[ObjectIndex] = LocalPriorities[LocalObjIt];
	}
}

void USphereNetObjectPrioritizer::SetupBatchParams(FBatchParams& OutBatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 MaxBatchObjectCount, FMemStackBase& Mem)
{
	OutBatchParams.View = PrioritizationParams.View;
	OutBatchParams.ConnectionId = PrioritizationParams.ConnectionId;
	OutBatchParams.Positions = static_cast<VectorRegister*>(Mem.Alloc(MaxBatchObjectCount*sizeof(VectorRegister), alignof(VectorRegister)));
	OutBatchParams.Priorities = static_cast<float*>(Mem.Alloc(MaxBatchObjectCount*sizeof(float), 16));

	SetupCalculationConstants(OutBatchParams.PriorityCalculationConstants);

	FMemory::Memzero(OutBatchParams.Positions, MaxBatchObjectCount*sizeof(VectorRegister));
}

void USphereNetObjectPrioritizer::SetupCalculationConstants(FPriorityCalculationConstants& OutConstants)
{
	const VectorRegister InnerRadius = VectorSetFloat1(Config->InnerRadius);
	const VectorRegister OuterRadius = VectorSetFloat1(Config->OuterRadius);
	const VectorRegister InnerPriority = VectorSetFloat1(Config->InnerPriority);
	const VectorRegister OuterPriority = VectorSetFloat1(Config->OuterPriority);
	const VectorRegister OutsidePriority = VectorSetFloat1(Config->OutsidePriority);

	const VectorRegister RadiusDiff = VectorSubtract(OuterRadius, InnerRadius);
	const VectorRegister PriorityDiff = VectorSubtract(OuterPriority, InnerPriority);

	OutConstants.InnerRadius = InnerRadius;
	OutConstants.OuterRadius = OuterRadius;
	OutConstants.RadiusDiff = RadiusDiff;
	OutConstants.InvRadiusDiff = VectorReciprocalAccurate(RadiusDiff);
	OutConstants.InnerPriority = InnerPriority;
	OutConstants.OuterPriority = OuterPriority;
	OutConstants.OutsidePriority = OutsidePriority;
	OutConstants.PriorityDiff = PriorityDiff;
}
