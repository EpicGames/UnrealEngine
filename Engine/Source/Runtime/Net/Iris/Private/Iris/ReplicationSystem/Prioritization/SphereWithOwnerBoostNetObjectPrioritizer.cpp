// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/Prioritization/SphereWithOwnerBoostNetObjectPrioritizer.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFiltering.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Misc/MemStack.h"
#include <limits>

#include UE_INLINE_GENERATED_CPP_BY_NAME(SphereWithOwnerBoostNetObjectPrioritizer)

void USphereWithOwnerBoostNetObjectPrioritizer::Init(FNetObjectPrioritizerInitParams& Params)
{
	// Make sure our ConnectionId type is sufficient.
	checkf(Params.MaxConnectionCount < std::numeric_limits<ConnectionId>::max(), TEXT("Need to increase size of ConnectionId type."));

	// We have no need for a special init, but need to make sure the config is of the expected type.
	check(CastChecked<USphereWithOwnerBoostNetObjectPrioritizerConfig>(Params.Config));
	Super::Init(Params);

	ReplicationSystem = Params.ReplicationSystem;

	AssignedOwningConnectionIndices.Init(Params.AbsoluteMaxNetObjectCount);
}

void USphereWithOwnerBoostNetObjectPrioritizer::Deinit()
{
	Super::Deinit();

	ReplicationSystem = nullptr;

	AssignedOwningConnectionIndices.Empty();
	OwningConnections.Empty();
}

bool USphereWithOwnerBoostNetObjectPrioritizer::AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerAddObjectParams& Params)
{
	if (!Super::AddObject(ObjectIndex, Params))
	{
		return false;
	}

	const uint32 OwningConnectionIndex = AllocOwningConnection();
	// We depend on the index being the same as the one stored in the info. If this check triggers something must have changed in a base class and AllocOwningConnection() must adapt.
	checkSlow(OwningConnectionIndex == static_cast<FObjectLocationInfo&>(Params.OutInfo).GetLocationIndex());

	// Defer updating the owning connection until the UpdateObjects call.

	return true;
}

void USphereWithOwnerBoostNetObjectPrioritizer::RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectPrioritizationInfo& Info)
{
	const FObjectLocationInfo& ObjectInfo = static_cast<const FObjectLocationInfo&>(Info);
	const uint32 OwningConnectionIndex = ObjectInfo.GetLocationIndex();
	OwningConnections[OwningConnectionIndex] = InvalidConnectionID;
	FreeOwningConnection(OwningConnectionIndex);

	return Super::RemoveObject(ObjectIndex, Info);
}

void USphereWithOwnerBoostNetObjectPrioritizer::UpdateObjects(FNetObjectPrioritizerUpdateParams& Params)
{
	Super::UpdateObjects(Params);

	// Update owning connections.
	const UE::Net::Private::FReplicationFiltering& ReplicationFiltering = ReplicationSystem->GetReplicationSystemInternal()->GetFiltering();

	for (const uint32 ObjectIndex : MakeArrayView(Params.ObjectIndices, Params.ObjectCount))
	{
		FObjectLocationInfo& ObjectInfo = static_cast<FObjectLocationInfo&>(Params.PrioritizationInfos[ObjectIndex]);
		const uint32 OwningConnection = ReplicationFiltering.GetOwningConnection(ObjectIndex);
		OwningConnections[ObjectInfo.GetLocationIndex()] = static_cast<uint16>(OwningConnection);
	}
}

void USphereWithOwnerBoostNetObjectPrioritizer::Prioritize(FNetObjectPrioritizationParams& PrioritizationParams)
{
	IRIS_PROFILER_SCOPE(USphereWithOwnerBoostNetObjectPrioritizer_Prioritize);

	FMemStack& Mem = FMemStack::Get();
	FMemMark MemMark(Mem);

	// Trade-off memory/performance
	constexpr uint32 MaxBatchObjectCount = 1024U;

	FOwnerBoostBatchParams BatchParams;
	const uint32 BatchObjectCount = FMath::Min((PrioritizationParams.ObjectCount + 3U) & ~3U, MaxBatchObjectCount);
	SetupBatchParams(BatchParams, PrioritizationParams, BatchObjectCount, Mem);

	for (uint32 ObjectIt = 0, ObjectEndIt = PrioritizationParams.ObjectCount; ObjectIt < ObjectEndIt; )
	{
		const uint32 CurrentBatchObjectCount = FMath::Min(ObjectEndIt - ObjectIt, MaxBatchObjectCount);

		BatchParams.ObjectCount = CurrentBatchObjectCount;
		PrepareBatch(BatchParams, PrioritizationParams, ObjectIt);
		PrioritizeBatch(BatchParams);
		FinishBatch(BatchParams, PrioritizationParams, ObjectIt);

		ObjectIt += CurrentBatchObjectCount;
	}
}

void USphereWithOwnerBoostNetObjectPrioritizer::PrepareBatch(FOwnerBoostBatchParams& BatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset)
{
	IRIS_PROFILER_SCOPE(USphereWithOwnerBoostNetObjectPrioritizer_PrepareBatch);
	const float* ExternalPriorities = PrioritizationParams.Priorities;
	const uint32* ObjectIndices = PrioritizationParams.ObjectIndices;
	const FNetObjectPrioritizationInfo* PrioritizationInfos = PrioritizationParams.PrioritizationInfos;

	float* LocalPriorities = BatchParams.Priorities;
	VectorRegister* Positions = BatchParams.Positions;
	uint32* OwnedObjectsLocalIndices = BatchParams.OwnedObjectsLocalIndices;

	// Copy priorities
	{
		uint32 LocalObjIt = 0;
		for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = ObjIt + BatchParams.ObjectCount; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
		{
			const uint32 ObjectIndex = ObjectIndices[ObjIt];
			LocalPriorities[LocalObjIt] = ExternalPriorities[ObjectIndex];
		}
	}

	// Copy positions and owning connections.
	uint32 LocalObjIt = 0;
	{
		const uint32 OwningConnectionId = PrioritizationParams.ConnectionId;
		uint32 OwnedObjectCount = 0;
		for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = ObjIt + BatchParams.ObjectCount; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
		{
			const uint32 ObjectIndex = ObjectIndices[ObjIt];
			const FObjectLocationInfo& Info = static_cast<const FObjectLocationInfo&>(PrioritizationInfos[ObjectIndex]);
			Positions[LocalObjIt] = GetLocation(Info);
			if (GetOwningConnection(Info) == OwningConnectionId)
			{
				OwnedObjectsLocalIndices[OwnedObjectCount++] = LocalObjIt;
			}
		}
		BatchParams.OwnedObjectCount = OwnedObjectCount;
	}

	// Make sure we have a multiple of four valid entries
	for (uint32 ObjIt = ObjectIndexStartOffset, ObjEndIt = (ObjIt + 3U) & ~3U; ObjIt != ObjEndIt; ++ObjIt, ++LocalObjIt)
	{
		LocalPriorities[LocalObjIt] = 0.0f;
		Positions[LocalObjIt] = VectorZero();
	}
}

void USphereWithOwnerBoostNetObjectPrioritizer::PrioritizeBatch(FOwnerBoostBatchParams& BatchParams)
{
	IRIS_PROFILER_SCOPE(USphereWithOwnerBoostNetObjectPrioritizer_PrioritizeBatch);

	// Let the super class deal with the prioritization and do fixup afterwards.
	Super::PrioritizeBatch(BatchParams);

	BoostOwningConnectionPriorities(BatchParams);
}

void USphereWithOwnerBoostNetObjectPrioritizer::FinishBatch(const FOwnerBoostBatchParams& BatchParams, FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset)
{
	Super::FinishBatch(BatchParams, PrioritizationParams, ObjectIndexStartOffset);
}

void USphereWithOwnerBoostNetObjectPrioritizer::BoostOwningConnectionPriorities(FOwnerBoostBatchParams& BatchParams) const
{
	const float OwnerPriorityBoost = BatchParams.OwnerPriorityBoost;
	float* LocalPriorities = BatchParams.Priorities;
	const uint32* OwnedObjects = BatchParams.OwnedObjectsLocalIndices;
	for (uint32 ObjIt = 0, ObjEndIt = BatchParams.OwnedObjectCount; ObjIt < ObjEndIt; ++ObjIt)
	{
		const uint32 LocalIndex = OwnedObjects[ObjIt];
		LocalPriorities[LocalIndex] += OwnerPriorityBoost;
	}
}

void USphereWithOwnerBoostNetObjectPrioritizer::SetupBatchParams(FOwnerBoostBatchParams& OutBatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 MaxBatchObjectCount, FMemStackBase& Mem)
{
	Super::SetupBatchParams(OutBatchParams, PrioritizationParams, MaxBatchObjectCount, Mem);
	OutBatchParams.OwnedObjectsLocalIndices = static_cast<uint32*>(Mem.Alloc(MaxBatchObjectCount*sizeof(uint32), alignof(uint32)));

	USphereWithOwnerBoostNetObjectPrioritizerConfig* OwnerBoostConfig = Cast<USphereWithOwnerBoostNetObjectPrioritizerConfig>(Config.Get());
	OutBatchParams.OwnerPriorityBoost = OwnerBoostConfig->OwnerPriorityBoost;
}

uint32 USphereWithOwnerBoostNetObjectPrioritizer::AllocOwningConnection()
{
	uint32 Index = AssignedOwningConnectionIndices.FindFirstZero();
	if (Index >= uint32(OwningConnections.Num()))
	{
		constexpr int32 NumElementsPerChunk = OwningConnectionsChunkSize/sizeof(ConnectionId);
		OwningConnections.Add(NumElementsPerChunk);
	}

	AssignedOwningConnectionIndices.SetBit(Index);
	return Index;
}

void USphereWithOwnerBoostNetObjectPrioritizer::FreeOwningConnection(uint32 Index)
{
	AssignedOwningConnectionIndices.ClearBit(Index);
}
