// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/PropertyReplicationFragment.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisLog.h"
#include "HAL/IConsoleManager.h"
#include "Iris/IrisConfigInternal.h"

namespace UE::Net
{

static bool bUsePrevReceivedStateForOnReps = false;
static FAutoConsoleVariableRef CVarUsePrevReceivedStateForOnReps(
		TEXT("net.Iris.UsePrevReceivedStateForOnReps"),
		bUsePrevReceivedStateForOnReps,
		TEXT("If true OnReps will use the previous received state when doing onreps and not do any compares, if set to false we will copy the local state and do a compare before issuing onreps"
		));

FPropertyReplicationFragment::FPropertyReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor)
: FReplicationFragment(InTraits)
, ReplicationStateDescriptor(InDescriptor)
, Owner(InOwner)
{
	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReplicate))
	{
		SrcReplicationState = MakeUnique<FPropertyReplicationState>(InDescriptor);
	}
	
	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReceive))
	{
		if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasRepNotifies))
		{
			// We need to store the previous state if the onreps require this information or we are not using the previous received state received for onreps as we need to store the local state before overwriting the current state
			if (!bUsePrevReceivedStateForOnReps || EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::KeepPreviousState))
			{
				PrevReplicationState = MakeUnique<FPropertyReplicationState>(InDescriptor);

				// Full store of initial value for repnotifies
				PrevReplicationState->StoreCurrentPropertyReplicationStateForRepNotifies(InOwner, nullptr);

				Traits |= EReplicationFragmentTraits::KeepPreviousState;
			}

			Traits |= EReplicationFragmentTraits::HasRepNotifies;
		}

		// For now we always expect pre/post operations for legacy states, we might make this
		Traits |= EReplicationFragmentTraits::NeedsLegacyCallbacks;
	}
	
	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReplicate))
	{
		// For PropertyReplicationStates, except for pure function states, we need to poll properties from our owner in order to detect state changes.
		if (InDescriptor->FunctionCount == 0)
		{
			// In theory we wouldn't need CanReplicate/CanReceive but there's a lot of logic that would then have to check whether the fragment has the appropriate buffers.
			Traits |= EReplicationFragmentTraits::NeedsPoll;
		}
	}

	// Propagate push based dirtiness.
	if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasPushBasedDirtiness))
	{
		Traits |= EReplicationFragmentTraits::HasPushBasedDirtiness;
		if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasFullPushBasedDirtiness))
		{
			Traits |= EReplicationFragmentTraits::HasFullPushBasedDirtiness;
		}
	}

	// Propagate object reference.
	if (EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::HasObjectReference))
	{
		Traits |= EReplicationFragmentTraits::HasObjectReference;
	}

	Traits |= EReplicationFragmentTraits::HasPropertyReplicationState;
	// We can handle partial state in all apply operations
	Traits |= EReplicationFragmentTraits::SupportsPartialDequantizedState;
}

FPropertyReplicationFragment::~FPropertyReplicationFragment() = default;

void FPropertyReplicationFragment::Register(FFragmentRegistrationContext& Context)
{
	Context.RegisterReplicationFragment(this, ReplicationStateDescriptor.GetReference(), SrcReplicationState ? SrcReplicationState->GetStateBuffer() : nullptr);
}

void FPropertyReplicationFragment::CollectOwner(FReplicationStateOwnerCollector* Owners) const
{
	Owners->AddOwner(Owner);
}

void FPropertyReplicationFragment::CallRepNotifies(FReplicationStateApplyContext& Context)
{
	IRIS_PROFILER_SCOPE(PropertyReplicationFragment_InvokeRepNotifies);

	const FPropertyReplicationState ReceivedState(Context.Descriptor, Context.StateBufferData.ExternalStateBuffer);

	if (PrevReplicationState || !Context.bIsInit)
	{
		FPropertyReplicationState::FCallRepNotifiesParameters Params;
		Params.PreviousState = PrevReplicationState.Get();
		Params.bIsInit = Context.bIsInit;
		Params.bOnlyCallIfDiffersFromLocal = !bUsePrevReceivedStateForOnReps;

		ReceivedState.CallRepNotifies(Owner, Params);

		// We keep a copy of the previous state for RepNotifies that need the value
		// If we rely on received data for the onreps, we just copy the received state, otherwise we must store the local state before applying received data.
		if (bUsePrevReceivedStateForOnReps && PrevReplicationState)
		{
			// Init is always a full state so we can just copy it
			if (Context.bIsInit)
			{
				*PrevReplicationState = ReceivedState;
			}
			else
			{
				// As apply now might provide us with partial states we should only copy dirty members.
				PrevReplicationState->CopyDirtyProperties(ReceivedState);
			}
		}
	}
	else
	{
		// As our default initial states is always treated as all dirty we need to compare against the default before applying initial repnotifies
		const FPropertyReplicationState DefaultState(Context.Descriptor);

		ReceivedState.CallRepNotifies(Owner, &DefaultState, Context.bIsInit);
	}
}

void FPropertyReplicationFragment::ApplyReplicatedState(FReplicationStateApplyContext& Context) const
{
	IRIS_PROFILER_SCOPE(PropertyReplicationFragment_ApplyReplicatedState);

	// Create a wrapping property replication state, cheap as we are simply injecting the already constructed state
	const FPropertyReplicationState ReceivedState(Context.Descriptor, Context.StateBufferData.ExternalStateBuffer);

	// If we do not rely on received data to issue rep notifies we need to store a copy of the local state before we apply the new received state.
	if (!bUsePrevReceivedStateForOnReps && PrevReplicationState)
	{
		PrevReplicationState->StoreCurrentPropertyReplicationStateForRepNotifies(Owner, &ReceivedState);
	}

	// Just push the state data to owner
	ReceivedState.PushPropertyReplicationState(Owner, static_cast<void*>(Owner));
}

bool FPropertyReplicationFragment::PollReplicatedState(EReplicationFragmentPollFlags PollOption)
{
	bool bIsDirty = false;
	const bool bEnableVerboseProfiling = EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::EnableVerboseProfiling);

	if (EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollAllState))
	{
		IRIS_PROFILER_SCOPE_CONDITIONAL(PropertyReplicationFragment_PollAllState, bEnableVerboseProfiling);

		// Since PollPropertyReplicationState always copies the new value we do not need to do anything special
		// with object references if the EReplicationFragmentPollOption::All flag is set
		const void* ExternalSourceState = static_cast<void*>(Owner);
		return SrcReplicationState->PollPropertyReplicationState(ExternalSourceState);
	}
	else if (EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollDirtyState))
	{
		const bool bForceRefreshObjectReferences = EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::ForceRefreshCachedObjectReferencesAfterGC) && ReplicationStateDescriptor->HasObjectReference();

		// We can skip, fully push based states if they are not dirty and we are not refreshing object references
		if (!EnumHasAnyFlags(ReplicationStateDescriptor->Traits, EReplicationStateTraits::HasFullPushBasedDirtiness) || bForceRefreshObjectReferences || SrcReplicationState->IsDirtyForPolling())
		{
			IRIS_PROFILER_SCOPE_CONDITIONAL(PropertyReplicationFragment_PollDirtyState, bEnableVerboseProfiling);

			const void* ExternalSourceState = static_cast<void*>(Owner);
			const FPropertyReplicationState::FPollParameters Params = {.bEnableVerboseProfiling = bEnableVerboseProfiling, .bForceRefreshObjectReferences = bEnableVerboseProfiling };
			bIsDirty = SrcReplicationState->PollPushBasedPropertyReplicationState(ExternalSourceState, Params);
		}
	}
	else if (EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::ForceRefreshCachedObjectReferencesAfterGC))
	{
		IRIS_PROFILER_SCOPE_CONDITIONAL(PropertyReplicationFragment_PollObjectReferences, bEnableVerboseProfiling);

		const void* ExternalSourceState = static_cast<void*>(Owner);
		return SrcReplicationState->PollObjectReferences(ExternalSourceState);
	}

	return bIsDirty;
}

void FPropertyReplicationFragment::ReplicatedStateToString(FStringBuilderBase& StringBuilder, FReplicationStateApplyContext& Context, EReplicationStateToStringFlags Flags) const
{
	const FPropertyReplicationState ReceivedState(Context.Descriptor, Context.StateBufferData.ExternalStateBuffer);

	const bool bIncludeAll = EnumHasAnyFlags(Flags, EReplicationStateToStringFlags::OnlyIncludeDirtyMembers) == false;
	ReceivedState.ToString(StringBuilder, bIncludeAll);
};

FPropertyReplicationFragment* FPropertyReplicationFragment::CreateAndRegisterFragment(UObject* InOwner, const FReplicationStateDescriptor* InDescriptor, FFragmentRegistrationContext& Context)
{
	// Fast arrays needs to be bound explicitly
	if (InDescriptor == nullptr || EnumHasAnyFlags(InDescriptor->Traits, EReplicationStateTraits::IsFastArrayReplicationState))
	{
		return nullptr;
	}
	
	FPropertyReplicationFragment* Fragment = new FPropertyReplicationFragment(Context.GetFragmentTraits(), InOwner, InDescriptor);

	Fragment->Traits |= EReplicationFragmentTraits::DeleteWithInstanceProtocol;

	Fragment->Register(Context);

	return Fragment;
}

}
