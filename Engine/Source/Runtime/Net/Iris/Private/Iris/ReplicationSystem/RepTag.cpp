// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/RepTag.h"
#include "Iris/ReplicationState/InternalReplicationStateDescriptorUtils.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/Serialization/NetSerializers.h" // FStructNetSerializerConfig
#include "Containers/ArrayView.h"
#include "HAL/PlatformString.h"
#include "Hash/CityHash.h"

namespace UE::Net
{

FRepTag MakeRepTag(const char* TagName)
{
	constexpr uint64 Seed = 0x5265705461675F00ULL; // Think of this as "RepTag_"
	return CityHash64WithSeed(TagName, FPlatformString::Strlen(TagName), Seed);
}

bool HasRepTag(const FReplicationProtocol* Protocol, FRepTag RepTag)
{
	check(Protocol);
	for (const FReplicationStateDescriptor* StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
	{
		for (const FReplicationStateMemberTagDescriptor& TagDescriptor : MakeArrayView(StateDescriptor->MemberTagDescriptors, StateDescriptor->TagCount))
		{
			if (TagDescriptor.Tag == RepTag)
			{
				return true;
			}
		}
	}

	return false;
}

bool FindRepTag(const FReplicationStateDescriptor* Descriptor, FRepTag RepTag, FRepTagFindInfo& OutRepTagFindInfo)
{
	check(Descriptor);
	for (const FReplicationStateMemberTagDescriptor& TagDescriptor : MakeArrayView(Descriptor->MemberTagDescriptors, Descriptor->TagCount))
	{
		if (TagDescriptor.Tag != RepTag)
		{
			continue;
		}

		// Lookup offsets and serializers from the descriptor. We may have to dig deep to find the relevant information.
		OutRepTagFindInfo.StateIndex = 0;
		OutRepTagFindInfo.ExternalStateOffset = 0;
		OutRepTagFindInfo.InternalStateAbsoluteOffset = 0;
		const FReplicationStateDescriptor* CurrentDescriptor = Descriptor;
		const FReplicationStateMemberTagDescriptor* CurrentTagDescriptor = &TagDescriptor;
		for (;;)
		{
			const uint16 MemberIndex = CurrentTagDescriptor->MemberIndex;
			const FReplicationStateMemberDescriptor& MemberDescriptor = CurrentDescriptor->MemberDescriptors[MemberIndex];
			OutRepTagFindInfo.ExternalStateOffset += MemberDescriptor.ExternalMemberOffset;
			OutRepTagFindInfo.InternalStateAbsoluteOffset += MemberDescriptor.InternalMemberOffset;

			const FReplicationStateMemberSerializerDescriptor* SerializerInfo = &CurrentDescriptor->MemberSerializerDescriptors[MemberIndex];
			const uint16 InnerTagIndex = CurrentTagDescriptor->InnerTagIndex;
			if (InnerTagIndex == MAX_uint16)
			{
				OutRepTagFindInfo.Serializer = SerializerInfo->Serializer;
				OutRepTagFindInfo.SerializerConfig = SerializerInfo->SerializerConfig;
				return true;
			}

			// Since the inner tag index is valid we assume the member is a struct.
			if (!ensure(IsUsingStructNetSerializer(*SerializerInfo)))
			{
				return false;
			}

			const FStructNetSerializerConfig* StructConfig = static_cast<const FStructNetSerializerConfig*>(SerializerInfo->SerializerConfig);
			CurrentDescriptor = StructConfig->StateDescriptor;
			CurrentTagDescriptor = &StructConfig->StateDescriptor->MemberTagDescriptors[InnerTagIndex];
		}
	}

	return false;
}

bool FindRepTag(const FReplicationProtocol* Protocol, FRepTag RepTag, FRepTagFindInfo& OutRepTagFindInfo)
{
	check(Protocol);
	uint32 InternalOffset = 0;
	for (const FReplicationStateDescriptor*& StateDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
	{
		InternalOffset = Align(InternalOffset, StateDescriptor->InternalAlignment);
		
		if (FindRepTag(StateDescriptor, RepTag, OutRepTagFindInfo))
		{
			OutRepTagFindInfo.StateIndex = static_cast<uint32>(&StateDescriptor - Protocol->ReplicationStateDescriptors);
			OutRepTagFindInfo.InternalStateAbsoluteOffset += InternalOffset;
			return true;
		}
		
		InternalOffset += StateDescriptor->InternalSize;
	}

	return false;
}

}
