// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ReplicationBridge.h"

#include "Containers/ArrayView.h"

#include "HAL/IConsoleManager.h"

#include "Iris/Core/IrisCsv.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/IrisConfigInternal.h"

#include "Iris/ReplicationState/ReplicationStateUtil.h"

#include "Iris/ReplicationSystem/ChangeMaskCache.h"
#include "Iris/ReplicationSystem/LegacyPushModel.h"
#include "Iris/ReplicationSystem/InternalNetRefIndexManager.h"
#include "Iris/ReplicationSystem/NetRefHandleManagerTypes.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/ReplicationFragmentInternal.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationWriter.h"
#include "Iris/ReplicationSystem/RepTag.h"
#include "Iris/ReplicationSystem/WorldLocations.h"

#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizer.h"

#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/ObjectNetSerializer.h"

#include "Net/Core/NetBitArray.h"
#include "Net/Core/NetHandle/NetHandleManager.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Net/Core/Trace/NetTrace.h"

#include "ReplicationFragmentInternal.h"
#include "ReplicationOperationsInternal.h"
#include "Iris/ReplicationSystem/ReplicationBridgeTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplicationBridge)

#define UE_LOG_REPLICATIONBRIDGE(Category, Format, ...)  UE_LOG(LogIris, Category, TEXT("RepBridge(%u)::") Format, ReplicationSystem->GetId(), ##__VA_ARGS__)

static bool bEnableFlushReliableRPCOnDestroy = true;
static FAutoConsoleVariableRef CVarEnableFlushReliableRPCOnDestroy(
	TEXT("net.Iris.EnableFlushReliableRPCOnDestroy"),
	bEnableFlushReliableRPCOnDestroy,
	TEXT("When true EEndReplicationFlags::Flush flag will be appended in EndReplication if we have pending unprocessed attachments/RPC:s when destroying a replicated object.")
);

/**
 * ReplicationBridge Implementation
 */
UReplicationBridge::UReplicationBridge()
: ReplicationSystem(nullptr)
, ReplicationProtocolManager(nullptr)
, ReplicationStateDescriptorRegistry(nullptr)
, NetRefHandleManager(nullptr)
{
}

void UReplicationBridge::PreReceiveUpdate()
{
	check(bInReceiveUpdate == false);
	bInReceiveUpdate = true;
}

void UReplicationBridge::PostReceiveUpdate()
{
	check(bInReceiveUpdate == true);
	bInReceiveUpdate = false;

	// Now process all StopReplication calls done while inside ReceiveUpdate
	for (const auto& It : HandlesToStopReplicating)
	{
		StopReplicatingNetRefHandle(It.Key, It.Value);
	}
	HandlesToStopReplicating.Reset();
	
	OnPostReceiveUpdate();
}

bool UReplicationBridge::CacheNetRefHandleCreationInfo(FNetRefHandle Handle)
{
	return false;
}

void UReplicationBridge::PruneStaleObjects()
{
}

void UReplicationBridge::GetInitialDependencies(FNetRefHandle Handle, FNetDependencyInfoArray& OutDependencies) const
{
	return;
}

void UReplicationBridge::CallPreSendUpdate(float DeltaSeconds)
{
	// Prune stale async stop replication instances and tear-off all handles pending tear-off
	PreUpdateHandlesPendingEndReplication();

	PreSendUpdate();
}

void UReplicationBridge::CallPreSendUpdateSingleHandle(FNetRefHandle Handle)
{
	PreSendUpdateSingleHandle(Handle);
}

void UReplicationBridge::CallUpdateInstancesWorldLocation()
{
	UpdateInstancesWorldLocation();
}

void UReplicationBridge::CallPruneStaleObjects()
{
	PruneStaleObjects();
}

bool UReplicationBridge::CallCacheNetRefHandleCreationInfo(FNetRefHandle Handle)
{
	return CacheNetRefHandleCreationInfo(Handle);
}

UReplicationBridge::~UReplicationBridge()
{
}

void UReplicationBridge::UnregisterInstance(UE::Net::FInternalNetRefIndex InternalNetRefIndex)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// If a dynamic replicated object stops replicating, it means that any reference to it or relative to it must be invalidated in the cache.
	// If object actually is destroyed we can also evict cached entries from cache as it will not replicate again.
	FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalNetRefIndex);
	if (ObjectData.RefHandle.IsDynamic())
	{
		const UObject* Instance = ObjectReferenceCache->GetObjectFromReferenceHandle(ObjectData.RefHandle);
		// In particular for subobjects its likely to get duplicate calls to UnregisterInstance in which case GetObjectFromReferenceHandle will fail to retrieve the instance. We really want to pass a valid object pointer to avoid a slow path iterating over every NetRefHandle.
		if (!Instance)
		{
			Instance = NetRefHandleManager->GetReplicatedObjectInstance(InternalNetRefIndex);
		}
		bool bInvalidatedSubObjectReferences = false;
		ObjectReferenceCache->RemoveReference(ObjectData.RefHandle, Instance, bInvalidatedSubObjectReferences);

		// Flag for eviction from cache.
		const bool bDestroyed = ObjectData.bTearOff || (ObjectData.InternalDetachReason == EInternalDetachReason::Normal || ObjectData.InternalDetachReason == EInternalDetachReason::TornOff);
		if (bDestroyed && bInvalidatedSubObjectReferences)
		{
			ObjectReferenceCache->MarkTrackedSubObjectHandlesForEviction(InternalNetRefIndex, ObjectData.RefHandle);
		}
	}
}

void UReplicationBridge::Initialize(UReplicationSystem* InReplicationSystem)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	Private::FReplicationSystemInternal* ReplicationSystemInternal = InReplicationSystem->GetReplicationSystemInternal();

	ReplicationSystem = InReplicationSystem;
	ReplicationProtocolManager = &ReplicationSystemInternal->GetReplicationProtocolManager();
	ReplicationStateDescriptorRegistry = &ReplicationSystemInternal->GetReplicationStateDescriptorRegistry();
	NetRefHandleManager = &ReplicationSystemInternal->GetNetRefHandleManager();
	ObjectReferenceCache = &ReplicationSystemInternal->GetObjectReferenceCache();
	Groups = &ReplicationSystemInternal->GetGroups();
}

void UReplicationBridge::Deinitialize()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// Detach all replicated instances that have not yet been destroyed as part of shutting down the rest of the game.
	NetRefHandleManager->GetAssignedInternalIndices().ForAllSetBits([this](uint32 InternalObjectIndex) 
	{
		if (InternalObjectIndex == InvalidInternalNetRefIndex)
		{
			return;
		}

		FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);
		if (ObjectData.InstanceProtocol)
		{
			// Clear out, tracking data or not?, currently we only have a single replication-system so it should not be a big issue if we do.
			// Note: Currently we opted to leave it as is.
			// -- This only applies to server. If it is a restart of ReplicationSystem, the same actors will be re-registered and global handle will be destroyed later, otherwise it does not matter what we do.
			// DestroyGlobalNetHandle(InternalObjectIndex);
			// ClearNetPushIds(InternalObjectIndex);

			// Detach and destroy instance protocol
			ObjectData.bPendingEndReplication = 1U;
			InternalDetachInstanceFromNetRefHandle(InternalObjectIndex);
		}
	});

	ReplicationSystem = nullptr;
	ReplicationProtocolManager = nullptr;
	ReplicationStateDescriptorRegistry = nullptr;
	NetRefHandleManager = nullptr;
	ObjectReferenceCache = nullptr;
	Groups = nullptr;
}

UE::Net::FNetRefHandle UReplicationBridge::InternalCreateNetObject(FNetRefHandle AllocatedHandle, FNetHandle GlobalHandle, const UE::Net::Private::FCreateNetObjectParams& Params)
{
	check(AllocatedHandle.IsValid() && AllocatedHandle.IsCompleteHandle());

	FNetRefHandle Handle = NetRefHandleManager->CreateNetObject(AllocatedHandle, GlobalHandle, Params);

	if (Handle.IsValid())
	{
		UE_NET_TRACE_NETHANDLE_CREATED(Handle, Params.ReplicationProtocol->DebugName, (uint64)Params.ReplicationProtocol->ProtocolIdentifier, 0/*Local*/);
	}

	return Handle;
}

UE::Net::FNetRefHandle UReplicationBridge::InternalCreateNetObjectFromRemote(FNetRefHandle WantedNetHandle, const UE::Net::Private::FCreateNetObjectParams& Params)
{
	FNetRefHandle Handle = NetRefHandleManager->CreateNetObjectFromRemote(WantedNetHandle, Params);

	if (Handle.IsValid())
	{
		UE_NET_TRACE_NETHANDLE_CREATED(Handle, Params.ReplicationProtocol->DebugName, (uint64)Params.ReplicationProtocol->ProtocolIdentifier, 1/*Remote*/);
	}

	return Handle;
}

void UReplicationBridge::InternalAttachInstanceToNetRefHandle(FNetRefHandle RefHandle, bool bBindInstanceProtocol, UE::Net::FReplicationInstanceProtocol* InstanceProtocol, UObject* Instance, FNetHandle NetHandle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const uint32 ReplicationSystemId = RefHandle.GetReplicationSystemId();
	const FInternalNetRefIndex InternalReplicationIndex = NetRefHandleManager->GetInternalIndex(RefHandle);

	NetRefHandleManager->AttachInstanceProtocol(InternalReplicationIndex, InstanceProtocol, Instance);
	UE_LOG_REPLICATIONBRIDGE(Verbose, TEXT("InternalAttachInstanceToNetHandle Attached: %s (0x%p) %s to (InternalIndex: %u)"), *Instance->GetName(), Instance, *RefHandle.ToString(), InternalReplicationIndex);

	// Bind instance protocol to dirty state tracking
	if (bBindInstanceProtocol)
	{
		FReplicationInstanceOperationsInternal::BindInstanceProtocol(NetHandle, InstanceProtocol, NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalReplicationIndex).Protocol);
		ForceNetUpdate(ReplicationSystemId, InternalReplicationIndex);
	}
}

void UReplicationBridge::InternalDetachInstanceFromNetRefHandle(UE::Net::FInternalNetRefIndex InternalObjectIndex, bool bUnBindInstanceProtocol)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (FReplicationInstanceProtocol* InstanceProtocol = const_cast<FReplicationInstanceProtocol*>(NetRefHandleManager->DetachInstanceProtocol(InternalObjectIndex)))
	{
		UE_LOG_REPLICATIONBRIDGE(Verbose, TEXT("InternalDetachInstanceFromNetHandle Detached: %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalObjectIndex));

		if (bUnBindInstanceProtocol && EnumHasAnyFlags(InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::IsBound))
		{
			FReplicationInstanceOperationsInternal::UnbindInstanceProtocol(InstanceProtocol, NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex).Protocol);
		}
		ReplicationProtocolManager->DestroyInstanceProtocol(InstanceProtocol);
	}
}

void UReplicationBridge::InternalDestroyNetObject(UE::Net::FInternalNetRefIndex InternalObjectIndex)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (InternalObjectIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	FReplicationSystemInternal* ReplicationSystemInternal = ReplicationSystem->GetReplicationSystemInternal();

	FWorldLocations& WorldLocations = ReplicationSystemInternal->GetWorldLocations();
	WorldLocations.RemoveObjectInfoCache(InternalObjectIndex);

	// Remove from handles to stop replicating as we might respawn the same object again in a later packet
	HandlesToStopReplicating.Remove(NetRefHandleManager->GetNetRefHandleFromInternalIndex(InternalObjectIndex));

	NetRefHandleManager->DestroyNetObjectByIndex(InternalObjectIndex);
}

void UReplicationBridge::InternalStartDestroyLocalObject(UE::Net::FInternalNetRefIndex InternalReplicationIndex, EEndReplicationFlags EndReplicationFlags)
{
	UE_LOG_REPLICATIONBRIDGE(Verbose, TEXT("InternalStartDestroyLocalObject for %s | EndReplicationFlags: %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalReplicationIndex), *LexToString(EndReplicationFlags));

	if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::DestroyNetHandle | EEndReplicationFlags::ClearNetPushId))
	{
		if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::DestroyNetHandle))
		{
			DestroyGlobalNetHandle(InternalReplicationIndex);
		}

		if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::ClearNetPushId))
		{
			ClearNetPushIds(InternalReplicationIndex);
		}
	}

	// Detach instance protocol
	InternalDetachInstanceFromNetRefHandle(InternalReplicationIndex);

	// Allow derived bridges to cleanup any instance info they have stored
	FNetRefHandle Handle = NetRefHandleManager->GetNetRefHandleFromInternalIndex(InternalReplicationIndex);

	UnregisterInstance(InternalReplicationIndex);

	if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::Flush))
	{
		// Initiate flush
		const bool bKeepInScopeAndMarkForFlush = true;
		NetRefHandleManager->DestroyNetObjectByIndex(InternalReplicationIndex, bKeepInScopeAndMarkForFlush);
	}

	// If we have any attached SubObjects, begin destroying them as well.
	InternalStartDestroyLocalSubObjects(InternalReplicationIndex, EndReplicationFlags);
}

void UReplicationBridge::InternalFinishDestroyLocalObject(UE::Net::FInternalNetRefIndex InternalReplicationIndex, EEndReplicationFlags EndReplicationFlags)
{
	// If the object are a member of any groups we need to remove it to make sure that we update filtering
	GetReplicationSystem()->GetReplicationSystemInternal()->RemoveFromAllGroups(InternalReplicationIndex);

	// Destroy NetObject and remove it from all scoping.
	InternalDestroyNetObject(InternalReplicationIndex);

	// If we have any attached SubObjects, finish destroy for them as well.
	InternalFinishDestroyLocalSubObjects(InternalReplicationIndex, EndReplicationFlags);
}

void UReplicationBridge::DestroyLocalNetHandle(FNetRefHandle Handle, EEndReplicationFlags EndReplicationFlags)
{
	using namespace UE::Net;

	const FInternalNetRefIndex InternalReplicationIndex = NetRefHandleManager->GetInternalIndex(Handle);
	if (InternalReplicationIndex != InvalidInternalNetRefIndex)
	{
		UE_LOG_REPLICATIONBRIDGE(Verbose, TEXT("DestroyLocalNetHandle for %s | EndReplicationFlags: %s"), *NetRefHandleManager->PrintObjectFromNetRefHandle(Handle), *LexToString(EndReplicationFlags));
		InternalStartDestroyLocalObject(InternalReplicationIndex, EndReplicationFlags);
		InternalFinishDestroyLocalObject(InternalReplicationIndex, EndReplicationFlags);
	}
}

void UReplicationBridge::InternalAddSubObject(FNetRefHandle ParentHandle, FNetRefHandle SubObjectHandle, FNetRefHandle InsertRelativeToSubObjectHandle, UE::Net::ESubObjectInsertionOrder InsertionOrder, UE::Net::EChildSubObjectsReplicationOrder ChildSubObjectsReplicationOrder)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	EAddSubObjectFlags AddSubObjectFlags = EAddSubObjectFlags::None;

	switch(InsertionOrder)
	{
		case ESubObjectInsertionOrder::None: 
			break;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		case ESubObjectInsertionOrder::ReplicateWith:
		{
			// This flag is being deprecated.
			ensureMsgf(false, TEXT("ESubObjectInsertionOrder::ReplicateWith is being deprecated in favor of explicitly specifying the parent using FSubObjectReplicationParams::ParentSubObjectHandle"));
			if (InsertRelativeToSubObjectHandle.IsValid())
			{
				ParentHandle = InsertRelativeToSubObjectHandle;
				InsertRelativeToSubObjectHandle = FNetRefHandle();
			}
			break;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		case ESubObjectInsertionOrder::InsertAtStart:
			AddSubObjectFlags |= EAddSubObjectFlags::InsertAtStart;
			break;
		case ESubObjectInsertionOrder::InsertAtStartOrAfterOther:
			AddSubObjectFlags |= EAddSubObjectFlags::InsertAtStartOrAfterOther;
			break;
		default:
			checkf(false, TEXT("Missing implementation of ESubObjectInsertionOrder enum"));
			break;
	}

	if (ChildSubObjectsReplicationOrder == UE::Net::EChildSubObjectsReplicationOrder::AfterParent)
	{
		AddSubObjectFlags |= EAddSubObjectFlags::ReplicateChildSubObjectsAfterParent;	
	}

	if (NetRefHandleManager->AddSubObject(ParentHandle, SubObjectHandle, InsertRelativeToSubObjectHandle, AddSubObjectFlags))
	{
		// If the subobject is new we need to update it immediately to pick it up for replication with its new parent
		ForceNetUpdate(ReplicationSystem->GetId(), NetRefHandleManager->GetInternalIndex(SubObjectHandle));

		// We set the priority of subobjects to be static as they will be prioritized with owner
		ReplicationSystem->SetStaticPriority(SubObjectHandle, 1.0f);
	}
}

void UReplicationBridge::InternalStartDestroyLocalSubObjects(UE::Net::FInternalNetRefIndex OwnerInternalIndex, EEndReplicationFlags Flags)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// Destroy SubObjects
	if (OwnerInternalIndex != InvalidInternalNetRefIndex)
	{
		for (FInternalNetRefIndex SubObjectInternalIndex : NetRefHandleManager->GetChildSubObjects(OwnerInternalIndex))
		{
			FNetRefHandleManager::FReplicatedObjectData& SubObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(SubObjectInternalIndex);
			if (!(SubObjectData.bFlush || SubObjectData.bNetObjectIsPendingDestroy))
			{
				SubObjectData.bPendingEndReplication = 1U;
				InternalStartDestroyLocalObject(SubObjectInternalIndex, Flags);				
			}
		}
	}
}

void UReplicationBridge::InternalFinishDestroyLocalSubObjects(UE::Net::FInternalNetRefIndex OwnerInternalIndex, EEndReplicationFlags Flags)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// Destroy SubObjects
	if (OwnerInternalIndex != InvalidInternalNetRefIndex)
	{
		for (FInternalNetRefIndex SubObjectInternalIndex : NetRefHandleManager->GetChildSubObjects(OwnerInternalIndex))
		{
			FNetRefHandleManager::FReplicatedObjectData& SubObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(SubObjectInternalIndex);
			SubObjectData.bPendingEndReplication = 1U;
			InternalFinishDestroyLocalObject(SubObjectInternalIndex, Flags);
		}
	}
}

void UReplicationBridge::InternalRequestAsyncStopReplication(UE::Net::FInternalNetRefIndex ObjectInternalIndex, EEndReplicationFlags EndReplicationFlags)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (ObjectInternalIndex != InvalidInternalNetRefIndex && NetRefHandleManager->IsScopableIndex(ObjectInternalIndex))
	{
		UE_LOG_REPLICATIONBRIDGE(Verbose, TEXT("%hs: %s | EndFlags: %s"), __FUNCTION__, *NetRefHandleManager->PrintObjectFromIndex(ObjectInternalIndex), ToCStr(LexToString(EndReplicationFlags)));

		FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectInternalIndex);
		AddPendingEndReplication(ObjectData.RefHandle, ObjectInternalIndex, EndReplicationFlags);

		ObjectData.bIsAsyncStopping = true;

		// If flush is required we need to wait to after scope update before we initialize async end replication, otherwise we can do it now.
		if (!EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::Flush))
		{
			if (!ReplicationSystem->IsInGroup(ReplicationSystem->GetAsyncStopReplicationNetObjectGroup(), ObjectData.RefHandle))
			{
				ReplicationSystem->AddToGroup(ReplicationSystem->GetAsyncStopReplicationNetObjectGroup(), ObjectData.RefHandle);
			}
		}

		for (FInternalNetRefIndex SubObjectInternalIndex : NetRefHandleManager->GetChildSubObjects(ObjectInternalIndex))
		{
			InternalRequestAsyncStopReplication(SubObjectInternalIndex, EndReplicationFlags);
		}
	}
}

void UReplicationBridge::StopReplicatingNetRefHandle(FNetRefHandle Handle, EEndReplicationFlags EndReplicationFlags)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (!IsReplicatedHandle(Handle))
	{
		return;
	}

	const FInternalNetRefIndex InternalReplicationIndex = NetRefHandleManager->GetInternalIndex(Handle);
	if (NetRefHandleManager->IsLocal(InternalReplicationIndex))
	{
		FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalReplicationIndex);

		if (ObjectData.bPendingEndReplication)
		{
			UE_LOG_REPLICATIONBRIDGE(Verbose, TEXT("Ignoring EndReplication called on object already PendingEndReplication %s."), *NetRefHandleManager->PrintObjectFromNetRefHandle(Handle));
			return;
		}

		// Store the stop replication reason
		ObjectData.InternalDetachReason = EInternalDetachReason::Normal;

		if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::ProxyReuse))
		{
			ObjectData.InternalDetachReason = EInternalDetachReason::ProxyReuse;
		}

		if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::TearOff))
		{
			ObjectData.InternalDetachReason = EInternalDetachReason::TornOff;

			// We do however copy the final state data and mark object to stop propagating state changes
			InternalTearOff(Handle);

			// Add handle to list of objects pending EndReplication indicate that it should be destroyed during next update
			// We need to do this to cover the case where the torn off object not yet has been added to the scope.
			AddPendingEndReplication(Handle, InternalReplicationIndex, EndReplicationFlags);
		}
		else 
		{
			// New objects, destroyed during the same frame with posted attachments(RPC) need to request a flush to ensure that they get a scope update
			const UE::Net::Private::FNetBlobManager& NetBlobManager = GetReplicationSystem()->GetReplicationSystemInternal()->GetNetBlobManager();
			const bool bAllowAutoFlushOfUnProcessedReliableRPCs = bEnableFlushReliableRPCOnDestroy && ObjectData.bNeedsFullCopyAndQuantize;
			if (bAllowAutoFlushOfUnProcessedReliableRPCs && NetBlobManager.HasUnprocessedReliableAttachments(InternalReplicationIndex))
			{
				EnumAddFlags(EndReplicationFlags, EEndReplicationFlags::Flush);
			}

			const bool bFlushRequested = EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::Flush);
			const bool bAsyncStopRequested = EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::AllowAsyncStopReplication);
			
			if (bAsyncStopRequested)
			{
				if (bFlushRequested)
				{
					InternalFlushStateData(InternalReplicationIndex);
				}

				// Note: When using async stop we only detach the instance when it is no longer referenced by any client.
				InternalRequestAsyncStopReplication(InternalReplicationIndex, EndReplicationFlags);
			}
			else if (bFlushRequested)
			{
				// Capture final state
				InternalFlushStateData(InternalReplicationIndex);

				// Object will finalize the flush-destroy Immediately after the next NetUpdate.
				AddPendingEndReplication(Handle, InternalReplicationIndex, EndReplicationFlags, EPendingEndReplicationImmediate::Yes);

				// After this call, the NetObject is only accessible through its InternalReplicationIndex.
				ObjectData.bPendingEndReplication = 1U;
				InternalStartDestroyLocalObject(InternalReplicationIndex, EndReplicationFlags);
			}
			else
			{
				ObjectData.bPendingEndReplication = 1U;
				DestroyLocalNetHandle(Handle, EndReplicationFlags);	
			}	
		}
	}
	else
	{
		// While we are inside ReceiveUpdate, queue stop replication requests instead of immediately stopping replication
		// This allows us the apply any received updates before we cut off this object
		if (IsInReceiveUpdate())
		{
			UE_LOGF(LogIris, Verbose, "Delayed request to StopReplicating %ls (flags: %ls) because it was called while inside ReceiveUpdate", *NetRefHandleManager->PrintObjectFromNetRefHandle(Handle), *LexToString(EndReplicationFlags));
		
			// Detect if we have diverging EndReplicationFlags for the same netobject
	#if DO_ENSURE
			{
				EEndReplicationFlags* PreviousFlags = HandlesToStopReplicating.Find(Handle);
				const bool bPreviousFlagsMatch = PreviousFlags ? (*PreviousFlags) == EndReplicationFlags : true;
				ensureMsgf(bPreviousFlagsMatch, TEXT("Received multiple StopReplicating calls for %s with different EndReplicationFlags: Previous: %s | Newest: %s"),
					*NetRefHandleManager->PrintObjectFromNetRefHandle(Handle), *LexToString(*PreviousFlags), *LexToString(EndReplicationFlags));
			}
	#endif

			HandlesToStopReplicating.Add(Handle, EndReplicationFlags);
			return;
		}

		if (InternalReplicationIndex != InvalidInternalNetRefIndex && EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::SkipPendingEndReplicationValidation))
		{
			NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalReplicationIndex).bPendingEndReplication = 1U;
		}
		// If we get a call to end replication on the client, we need to detach the instance as it might be garbage collected
		InternalDetachInstanceFromNetRefHandle(InternalReplicationIndex);
	}
}

void UReplicationBridge::PreUpdateHandlesPendingEndReplication()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	for (const FPendingEndReplicationInfo& Info : MakeArrayView(HandlesPendingEndReplication))
	{
		const FInternalNetRefIndex ObjectInternalIndex = Info.ObjectIndex;
		if (NetRefHandleManager->GetAssignedInternalIndices().GetBit(ObjectInternalIndex))
		{
			// Due to async stop replication not explicitly detaching instances in the call to StopReplication we detach the instance protocol if the instance has been invalidated
			if (!IsValid(NetRefHandleManager->GetReplicatedInstances()[ObjectInternalIndex]))
			{
				FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectInternalIndex);
				if (ObjectData.InstanceProtocol)
				{
					// Detach instance as we should not access the instance.
					ObjectData.bPendingEndReplication = 1U;
					const bool bUnbindInstanceProtocol = false;
					InternalDetachInstanceFromNetRefHandle(ObjectInternalIndex, bUnbindInstanceProtocol);
				}
			}
		}

		// Tear off handle pending tear off.
		if (EnumHasAnyFlags(Info.EndReplicationFlags, EEndReplicationFlags::TearOff))
		{
			InternalTearOff(Info.Handle);
		}
	}
}

void UReplicationBridge::UpdateHandlesPendingEndReplication()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FNetBitArray& AssignedInternalIndices = NetRefHandleManager->GetAssignedInternalIndices();

	TArray<FPendingEndReplicationInfo, TInlineAllocator<32>> ObjectsStillPendingEndReplication;
	for (FPendingEndReplicationInfo Info : MakeArrayView(HandlesPendingEndReplication))
	{
		const FInternalNetRefIndex ObjectInternalIndex = Info.ObjectIndex;
		if (NetRefHandleManager->GetAssignedInternalIndices().GetBit(ObjectInternalIndex))
		{
			FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectInternalIndex);
			check(ObjectData.RefHandle == Info.Handle);

			// Immediate destroy or objects that are no longer are referenced by any connections are destroyed
			if (NetRefHandleManager->GetNetObjectRefCount(ObjectInternalIndex) == 0U || Info.Immediate == EPendingEndReplicationImmediate::Yes)
			{
				ObjectData.bPendingEndReplication = 1U;
				if (ObjectData.bFlush)
				{
					// If we have initiated a flush we just need to finish the destroy
					InternalFinishDestroyLocalObject(Info.ObjectIndex, Info.EndReplicationFlags);
				}
				else
				{
					DestroyLocalNetHandle(Info.Handle, Info.EndReplicationFlags);
				}
			}
			else
			{
				if (EnumHasAnyFlags(Info.EndReplicationFlags, EEndReplicationFlags::AllowAsyncStopReplication))
				{
					// We keep the object in scope but we mark it a not replicated which will stop replicating it for all clients
					if (!ReplicationSystem->IsInGroup(ReplicationSystem->GetAsyncStopReplicationNetObjectGroup(), Info.Handle))
					{
						ReplicationSystem->AddToGroup(ReplicationSystem->GetAsyncStopReplicationNetObjectGroup(), Info.Handle);
					}
				}
				else if (NetRefHandleManager->IsScopableIndex(ObjectInternalIndex))
				{
					// If the object is still in scope remove it from scope as objects pending EndReplication should not be added to new connections after the first update.
					// Mark object and subobjects as no longer scopeable, and that we should not propagate changed states
					NetRefHandleManager->RemoveFromScope(ObjectInternalIndex);
					for (FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(ObjectInternalIndex))
					{
						NetRefHandleManager->RemoveFromScope(SubObjectIndex);
					}
				}
			
				// Keep object in the pending EndReplication-list until the object is no longer referenced by any ReplicationWriter
				ObjectsStillPendingEndReplication.Add(FPendingEndReplicationInfo(Info.Handle, ObjectInternalIndex, Info.EndReplicationFlags, EPendingEndReplicationImmediate::No));
			}
		}
	}

	HandlesPendingEndReplication.Reset();
	HandlesPendingEndReplication.Insert(ObjectsStillPendingEndReplication.GetData(), ObjectsStillPendingEndReplication.Num(), 0);

	CSV_CUSTOM_STAT(Iris, NumHandlesPendingEndRepliction, (float)HandlesPendingEndReplication.Num(), ECsvCustomStatOp::Set);
}

void UReplicationBridge::AddPendingEndReplication(FNetRefHandle Handle, UE::Net::FInternalNetRefIndex InternalObjectIndex, EEndReplicationFlags DestroyFlags, EPendingEndReplicationImmediate Immediate)
{
	if (ensure(EnumHasAnyFlags(DestroyFlags, EEndReplicationFlags::Flush | EEndReplicationFlags::TearOff | EEndReplicationFlags::AllowAsyncStopReplication)))
	{
		if (!HandlesPendingEndReplication.FindByPredicate([&InternalObjectIndex](const FPendingEndReplicationInfo& Info){ return Info.ObjectIndex == InternalObjectIndex; }))
		{
			HandlesPendingEndReplication.Add(FPendingEndReplicationInfo(Handle, InternalObjectIndex, DestroyFlags, Immediate));
		}
	}
}

bool UReplicationBridge::IsPendingStopReplication(FNetRefHandle Handle) const
{
	const int32 Index = HandlesPendingEndReplication.FindLastByPredicate([Handle](const FPendingEndReplicationInfo& Info)
	{ 
		return Info.Handle == Handle;
	});
	return Index != INDEX_NONE && EnumHasAnyFlags(HandlesPendingEndReplication[Index].EndReplicationFlags, EEndReplicationFlags::Flush | EEndReplicationFlags::AllowAsyncStopReplication);
}

bool UReplicationBridge::RestartOrStopObjectPendingAsyncStopReplication(FNetRefHandle Handle, bool bForceStopReplication)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	bool bRestarted = false;

	const int32 Index = HandlesPendingEndReplication.FindLastByPredicate([Handle](const FPendingEndReplicationInfo& Info){ return Info.Handle == Handle; });
	// We allow canceling StopReplication for objects that are marked for AsyncStopReplication
	if (Index != INDEX_NONE && EnumHasAnyFlags(HandlesPendingEndReplication[Index].EndReplicationFlags, EEndReplicationFlags::AllowAsyncStopReplication))
	{
		UE_LOG_REPLICATIONBRIDGE(Verbose, TEXT("RestartOrStopObjectPendingAsyncStopReplication: %s ForceStop: %u Flags:%s"), *PrintObjectFromNetRefHandle(Handle), bForceStopReplication ? 1U : 0U, *LexToString(HandlesPendingEndReplication[Index].EndReplicationFlags));

		const uint32 InternalObjectIndex = NetRefHandleManager->GetInternalIndex(HandlesPendingEndReplication[Index].Handle);
		if (InternalObjectIndex != InvalidInternalNetRefIndex)
		{
			FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);
			
			// In some cases we want to just create a new net object instead so we should just carry on and stop replication now.
			if (bForceStopReplication)
			{
				ObjectData.bPendingEndReplication = 1U;
				DestroyLocalNetHandle(HandlesPendingEndReplication[Index].Handle, HandlesPendingEndReplication[Index].EndReplicationFlags);
				bRestarted = false;
			}
			else
			{
				ObjectData.bIsAsyncStopping = false;
				bRestarted = true;

				if (NetRefHandleManager->IsDestroyedStartupObject(InternalObjectIndex))
				{
					// Need to cleanse destruction info if we are going to cancel async stop replication.
					const FInternalNetRefIndex DestructionInfoInternalIndex = NetRefHandleManager->GetOriginalDestroyedStartupObjectIndex(InternalObjectIndex);
					NetRefHandleManager->ResetDestroyedStartupObject(InternalObjectIndex);
					RemoveDestructionInfo(DestructionInfoInternalIndex);
				}
				
				// Remove cached creation info.
				NetRefHandleManager->ClearCachedCreationInfo(InternalObjectIndex);
			}
		}

		HandlesPendingEndReplication.RemoveAt(Index);
		const FNetObjectGroupHandle AsyncStopReplicationNetObjectGroupHandle = ReplicationSystem->GetAsyncStopReplicationNetObjectGroup();
		if (ReplicationSystem->IsInGroup(AsyncStopReplicationNetObjectGroupHandle, Handle))
		{
			ReplicationSystem->RemoveFromGroup(AsyncStopReplicationNetObjectGroupHandle, Handle);
		}

		return bRestarted;
	}

	return Index == INDEX_NONE;
}

void UReplicationBridge::InternalFlushStateData(UE::Net::FNetSerializationContext& SerializationContext, UE::Net::Private::FChangeMaskCache& ChangeMaskCache, UE::Net::FNetBitStreamWriter& ChangeMaskWriter, UE::Net::FInternalNetRefIndex InternalObjectIndex)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (InternalObjectIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);

	// Copy state data, if object already is torn off there is nothing to do
	if (ObjectData.bTearOff)
	{
		return;
	}

	UE_LOG_REPLICATIONBRIDGE(Verbose, TEXT("InternalFlushStateData Initiating flush for %s (InternalIndex: %u)"), *ObjectData.RefHandle.ToString(), InternalObjectIndex);

	if (ObjectData.InstanceProtocol)
	{
		if (EnumHasAnyFlags(ObjectData.InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::NeedsPoll | EReplicationInstanceProtocolTraits::NeedsPreSendUpdate))
		{
			CallPreSendUpdateSingleHandle(ObjectData.RefHandle);
		}

		// Cache creation info
		ObjectData.bHasCachedCreationInfo =  CallCacheNetRefHandleCreationInfo(ObjectData.RefHandle) ? 1U : 0U;

		FReplicationInstanceOperationsInternal::QuantizeObjectStateData(ChangeMaskWriter, ChangeMaskCache, *NetRefHandleManager, SerializationContext, InternalObjectIndex);

		// Clear the quantize flag since it was done directly here.
		NetRefHandleManager->GetDirtyObjectsToQuantize().ClearBit(InternalObjectIndex);
	}

	for (FInternalNetRefIndex SubObjectInternalIndex : NetRefHandleManager->GetChildSubObjects(InternalObjectIndex))
	{
		InternalFlushStateData(SerializationContext, ChangeMaskCache, ChangeMaskWriter, SubObjectInternalIndex);
	}
}

void UReplicationBridge::InternalFlushStateData(UE::Net::FInternalNetRefIndex InternalObjectIndex)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	IRIS_PROFILER_SCOPE(InternalFlushStateData);

	if (InternalObjectIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);

	FChangeMaskCache ChangeMaskCache;
	FNetBitStreamWriter ChangeMaskWriter;

	// Setup context
	FNetSerializationContext SerializationContext;
	FInternalNetSerializationContext InternalContext(ReplicationSystem);
	SerializationContext.SetInternalContext(&InternalContext);
	SerializationContext.SetNetStatsContext(ReplicationSystem->GetReplicationSystemInternal()->GetNetTypeStats().GetNetStatsContext());

	InternalFlushStateData(SerializationContext, ChangeMaskCache, ChangeMaskWriter, InternalObjectIndex);

	// Iterate over connections and propagate dirty changemasks to all connections already scoping this object
	if (ChangeMaskCache.Indices.Num() > 0)
	{
		FReplicationConnections& Connections = ReplicationSystem->GetReplicationSystemInternal()->GetConnections();

		auto&& UpdateDirtyChangeMasks = [&Connections, &ChangeMaskCache](uint32 ConnectionId)
		{
			FReplicationConnection* Conn = Connections.GetConnection(ConnectionId);			

			const bool bMarkForTearOff = false;
			Conn->ReplicationWriter->ForceUpdateDirtyChangeMasks(ChangeMaskCache, FReplicationWriter::FlushFlags_FlushState, bMarkForTearOff);
		};
		const FNetBitArray& ValidConnections = Connections.GetValidConnections();
		ValidConnections.ForAllSetBits(UpdateDirtyChangeMasks);		
	}
}

void UReplicationBridge::InternalTearOff(FNetRefHandle Handle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	IRIS_PROFILER_SCOPE(InternalTearOff);

	const uint32 InternalObjectIndex = NetRefHandleManager->GetInternalIndex(Handle);
	if (InternalObjectIndex == InvalidInternalNetRefIndex)
	{
		return;
	}

	FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);

	if (ObjectData.bTearOff)
	{
		// Already torn off
		return;
	}

	// Copy state data and tear off now
	UE_LOG_REPLICATIONBRIDGE(Verbose, TEXT("TearOff: %s"), *PrintObjectFromNetRefHandle(Handle));

	// Force copy of final state data as we will detach the object after scope update
	FChangeMaskCache ChangeMaskCache;
	FNetBitStreamWriter ChangeMaskWriter;

	// Setup context
	FNetSerializationContext SerializationContext;
	FInternalNetSerializationContext InternalContext(ReplicationSystem);
	SerializationContext.SetInternalContext(&InternalContext);
	SerializationContext.SetNetStatsContext(ReplicationSystem->GetReplicationSystemInternal()->GetNetTypeStats().GetNetStatsContext());

	if (ObjectData.InstanceProtocol)
	{
		if (EnumHasAnyFlags(ObjectData.InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::NeedsPoll | EReplicationInstanceProtocolTraits::NeedsPreSendUpdate))
		{
			CallPreSendUpdateSingleHandle(Handle);
		} 

		// Cache creation info
		ObjectData.bHasCachedCreationInfo =  CallCacheNetRefHandleCreationInfo(Handle) ? 1U : 0U;
	}

	if (ObjectData.InstanceProtocol && ObjectData.Protocol->InternalTotalSize > 0U)
	{
		FReplicationInstanceOperationsInternal::QuantizeObjectStateData(ChangeMaskWriter, ChangeMaskCache, *NetRefHandleManager, SerializationContext, InternalObjectIndex);

		// Clear the quantize flag since it was done directly here.
		NetRefHandleManager->GetDirtyObjectsToQuantize().ClearBit(InternalObjectIndex);
	}
	else
	{
		// Nothing to copy, but we must still propagate the tear-off state.
		FChangeMaskCache::FCachedInfo& Info = ChangeMaskCache.AddEmptyChangeMaskForObject(InternalObjectIndex);
		// If we are a subobject we must also mark owner as dirty.
		const uint32 SubObjectOwnerIndex = ObjectData.SubObjectRootIndex;
		if (SubObjectOwnerIndex != InvalidInternalNetRefIndex) 
		{
			ChangeMaskCache.AddSubObjectOwnerDirty(SubObjectOwnerIndex);
		}			
	}

	// Propagate changes to all connections that we currently have in scope
	FReplicationConnections& Connections = ReplicationSystem->GetReplicationSystemInternal()->GetConnections();

	// Iterate over connections and propagate dirty changemasks to all connections already scoping this object
	auto UpdateDirtyChangeMasks = [&Connections, &ChangeMaskCache](uint32 ConnectionId)
	{
		FReplicationConnection* Conn = Connections.GetConnection(ConnectionId);
		const bool bMarkForTearOff = true;
		Conn->ReplicationWriter->ForceUpdateDirtyChangeMasks(ChangeMaskCache, FReplicationWriter::FlushFlags_None, bMarkForTearOff);
	};
	const FNetBitArray& ValidConnections = Connections.GetValidConnections();
	ValidConnections.ForAllSetBits(UpdateDirtyChangeMasks);		

	// TearOff subobjects as well.
	for (FInternalNetRefIndex SubObjectInternalIndex : NetRefHandleManager->GetChildSubObjects(InternalObjectIndex))
	{
		InternalTearOff(NetRefHandleManager->GetNetRefHandleFromInternalIndex(SubObjectInternalIndex));
	}	

	// Mark object as being torn-off and that we should no longer propagate state changes
	ObjectData.bTearOff = 1U;
	ObjectData.bShouldPropagateChangedStates = 0U;

	// Detach instance as we must assume that we should not access the object after this call.
	ObjectData.bPendingEndReplication = 1U;
	InternalDetachInstanceFromNetRefHandle(InternalObjectIndex);
}

bool UReplicationBridge::IsReplicatedHandle(FNetRefHandle Handle) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FInternalNetRefIndex InternalObjectIndex = NetRefHandleManager->GetInternalIndex(Handle);
	if (InternalObjectIndex == InvalidInternalNetRefIndex)
	{
		return false;
	}

	// We do not consider objects pending stop due to flush or tearoff to be replicated to prevent unwanted API calls.
	FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);
	const bool bPendingStopDueToFlushOrTearOff = ObjectData.bTearOff || ObjectData.bFlush;

	return !bPendingStopDueToFlushOrTearOff;
}

void UReplicationBridge::SetNetPushIdOnFragments(const TArrayView<const UE::Net::FReplicationFragment*const>& Fragments, const UE::Net::Private::FNetPushObjectHandle& PushHandle)
{
#if WITH_PUSH_MODEL
	using namespace UE::Net;

	constexpr uint32 MaxFragmentOwnerCount = 1U;
	UObject* FragmentOwners[MaxFragmentOwnerCount] = {};
	FReplicationStateOwnerCollector FragmentOwnerCollector(FragmentOwners, MaxFragmentOwnerCount);
	UObject* Instance = nullptr;
	for (const FReplicationFragment* Fragment : Fragments)
	{
		if (!EnumHasAnyFlags(Fragment->GetTraits(), EReplicationFragmentTraits::HasPushBasedDirtiness))
		{
			continue;
		}

		FragmentOwnerCollector.Reset();
		Fragment->CollectOwner(&FragmentOwnerCollector);
		UObject* FragmentOwner = (FragmentOwnerCollector.GetOwnerCount() > 0 ? FragmentOwnerCollector.GetOwners()[0] : static_cast<UObject*>(nullptr));
		if (FragmentOwner != nullptr && FragmentOwner != Instance)
		{
			Instance = FragmentOwner;
			UE_NET_IRIS_SET_PUSH_ID(Instance, PushHandle);
		}
	}
#endif
}

void UReplicationBridge::ClearNetPushIdOnFragments(const TArrayView<const UE::Net::FReplicationFragment*const>& Fragments)
{
#if WITH_PUSH_MODEL
	using namespace UE::Net;

	constexpr uint32 MaxFragmentOwnerCount = 1U;
	UObject* FragmentOwners[MaxFragmentOwnerCount] = {};
	FReplicationStateOwnerCollector FragmentOwnerCollector(FragmentOwners, MaxFragmentOwnerCount);
	UObject* Instance = nullptr;
	for (const FReplicationFragment* Fragment : Fragments)
	{
		if (!EnumHasAnyFlags(Fragment->GetTraits(), EReplicationFragmentTraits::HasPushBasedDirtiness))
		{
			continue;
		}

		FragmentOwnerCollector.Reset();
		Fragment->CollectOwner(&FragmentOwnerCollector);
		UObject* FragmentOwner = (FragmentOwnerCollector.GetOwnerCount() > 0 ? FragmentOwnerCollector.GetOwners()[0] : static_cast<UObject*>(nullptr));
		if (FragmentOwner != nullptr && FragmentOwner != Instance)
		{
			Instance = FragmentOwner;
			if (IsValid(FragmentOwner))
			{
				UE_NET_IRIS_CLEAR_PUSH_ID(FragmentOwner);
			}
		}
	}
#endif
}

void UReplicationBridge::NotifyStreamingContainerUnload(const UObject* Container)
{
	// Destroy group associated with container
	UE::Net::FNetObjectGroupHandle ContainerGroupHandle;
	if (ContainerGroups.RemoveAndCopyValue(FObjectKey(Container), ContainerGroupHandle))
	{
		RemoveDestructionInfosForGroup(ContainerGroupHandle);
		ReplicationSystem->DestroyGroup(ContainerGroupHandle);
	}
}

UE::Net::FNetObjectGroupHandle UReplicationBridge::CreateContainerGroup(const UObject* Container, FName PackageName)
{
	using namespace UE::Net;

	FNetObjectGroupHandle ContainerGroupHandle = ReplicationSystem->CreateGroup(PackageName);
	if (ensure(ContainerGroupHandle.IsValid()))
	{
		ReplicationSystem->AddExclusionFilterGroup(ContainerGroupHandle);
		ContainerGroups.Emplace(FObjectKey(Container), ContainerGroupHandle);
	}

	return ContainerGroupHandle;
}

UE::Net::FNetObjectFactoryId UReplicationBridge::GetNetObjectFactoryId(FNetRefHandle RefHandle) const
{
	return NetRefHandleManager->GetReplicatedObjectData(NetRefHandleManager->GetInternalIndex(RefHandle)).NetFactoryId;
}

UE::Net::FNetObjectGroupHandle UReplicationBridge::GetContainerGroup(const UObject* Container) const
{
	const UE::Net::FNetObjectGroupHandle* ContainerGroupHandle = ContainerGroups.Find(FObjectKey(Container));
	return (ContainerGroupHandle != nullptr ? *ContainerGroupHandle : UE::Net::FNetObjectGroupHandle());
}

const TMap<FObjectKey, UE::Net::FNetObjectGroupHandle>& UReplicationBridge::GetAllContainerGroups() const
{
	return ContainerGroups;
}

void UReplicationBridge::DestroyGlobalNetHandle(UE::Net::FInternalNetRefIndex InternalReplicationIndex)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalReplicationIndex);
	if (ObjectData.NetHandle.IsValid())
	{
		FNetHandleDestroyer::DestroyNetHandle(ObjectData.NetHandle);
	}
}

void UReplicationBridge::ClearNetPushIds(UE::Net::FInternalNetRefIndex InternalReplicationIndex)
{
#if WITH_PUSH_MODEL
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalReplicationIndex);
	if (const FReplicationInstanceProtocol* InstanceProtocol = ObjectData.InstanceProtocol)
	{
		if (EnumHasAnyFlags(InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::HasPushBasedDirtiness))
		{
			TArrayView<const FReplicationFragment* const> Fragments(InstanceProtocol->Fragments, InstanceProtocol->FragmentCount);
			ClearNetPushIdOnFragments(Fragments);
		}
	}
#endif
}

FString UReplicationBridge::PrintObjectFromNetRefHandle(FNetRefHandle RefHandle) const
{
	return NetRefHandleManager->PrintObjectFromNetRefHandle(RefHandle);
}

void UReplicationBridge::PreSeamlessTravelGarbageCollect()
{
	RemoveDestructionInfosForGroup(UE::Net::FNetObjectGroupHandle());

	for (const TPair<FObjectKey, UE::Net::FNetObjectGroupHandle>& ContainerAndGroup : ContainerGroups)
	{
		ReplicationSystem->DestroyGroup(ContainerAndGroup.Value);
	}
	ContainerGroups.Empty();

	OnPreSeamlessTravelGarbageCollect();
}

void UReplicationBridge::OnPreSeamlessTravelGarbageCollect()
{
}

void UReplicationBridge::PostSeamlessTravelGarbageCollect()
{
	OnPostSeamlessTravelGarbageCollect();
}

void UReplicationBridge::OnPostSeamlessTravelGarbageCollect()
{
}
