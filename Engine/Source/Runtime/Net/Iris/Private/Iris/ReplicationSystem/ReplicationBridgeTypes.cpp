// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ReplicationBridgeTypes.h"


FString LexToString(EEndReplicationFlags EndReplicationFlags)
{
	
	if (EndReplicationFlags == EEndReplicationFlags::None)
	{
		return TEXT("None");
	}

	FString Flags;

	if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::Destroy))
	{
		Flags += TEXT("Destroy");
	}
	if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::TearOff))
	{
		if (!Flags.IsEmpty()) { Flags += TEXT(','); }
		Flags += TEXT("TearOff");
	}
	if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::Flush))
	{
		if (!Flags.IsEmpty()) { Flags += TEXT(','); }
		Flags += TEXT("Flush");
	}
	if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::DestroyNetHandle))
	{
		if (!Flags.IsEmpty()) { Flags += TEXT(','); };
		Flags += TEXT("DestroyNetHandle");
	}
	if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::ClearNetPushId))
	{
		if (!Flags.IsEmpty()) { Flags += TEXT(','); };
		Flags += TEXT("ClearNetPushId");
	}
	if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::SkipPendingEndReplicationValidation))
	{
		if (!Flags.IsEmpty()) { Flags += TEXT(','); };
		Flags += TEXT("SkipPendingEndReplicationValidation");
	}
	if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::ProxyReuse))
	{
		if (!Flags.IsEmpty()) { Flags += TEXT(','); };
		Flags += TEXT("ProxyReuse");
	}
	if (EnumHasAnyFlags(EndReplicationFlags, EEndReplicationFlags::AllowAsyncStopReplication))
	{
		if (!Flags.IsEmpty()) { Flags += TEXT(','); };
		Flags += TEXT("AllowAsyncStopReplication");
	}

	return Flags;
}