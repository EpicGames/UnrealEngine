// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationConnections.h"
#include "Iris/DataStream/DataStreamManager.h"
#include "Iris/ReplicationSystem/NetTokenDataStream.h"
#include "Iris/ReplicationSystem/ReplicationDataStream.h"
#include "Iris/ReplicationSystem/ReplicationWriter.h"
#include "Iris/ReplicationSystem/ReplicationReader.h"

namespace UE::Net::Private
{

void FReplicationConnections::Deinit()
{
	ValidConnections.ForAllSetBits([this](uint32 ConnectionId) { RemoveConnection(ConnectionId); });
}

void FReplicationConnections::InitDataStreamManager(uint32 ReplicationSystemId, uint32 ConnectionId, UDataStreamManager* DataStreamManager)
{
	FReplicationConnection* Connection = GetConnection(ConnectionId);
	if (Connection == nullptr)
	{
		return;
	}

	// Init data stream manager and create all DataStreams
	UDataStream::FInitParameters InitParams;
	InitParams.ReplicationSystemId = ReplicationSystemId;
	InitParams.ConnectionId = ConnectionId;
	InitParams.PacketWindowSize = 256;

	DataStreamManager->Init(InitParams);

	// Store it.
	Connection->DataStreamManager = DataStreamManager;
}

void FReplicationConnections::SetReplicationView(uint32 ConnectionId, const FReplicationView& View)
{
	ReplicationViews[ConnectionId] = View;
}

void FReplicationConnections::AddConnection(uint32 ConnectionId, ENetConnectionTraits NetConnectionTraits)
{
	check(ValidConnections.GetBit(ConnectionId) == false);
	ValidConnections.SetBit(ConnectionId);

	Connections[ConnectionId].NetConnectionTraits = NetConnectionTraits;
}

void FReplicationConnections::RemoveConnection(uint32 ConnectionId)
{
	check(ValidConnections.GetBit(ConnectionId));
	SetReplicationView(ConnectionId, FReplicationView());

	DeinitDataStreamManager(ConnectionId);

	Connections[ConnectionId] = FReplicationConnection();
	ValidConnections.ClearBit(ConnectionId);
}

bool FReplicationConnections::HasConnectionWithAnyTraits(ENetConnectionTraits NetConnectionTraits) const
{
	// For None we  check if there are any valid connections whatsoever.
	if (NetConnectionTraits == ENetConnectionTraits::None)
	{
		return ValidConnections.FindFirstOne() != FNetBitArray::InvalidIndex;
	}

	for (const FNetBitArray::StorageWordType ConnectionId : ValidConnections)
	{
		const FReplicationConnection& Connection = Connections[ConnectionId];
		if (EnumHasAnyFlags(Connection.NetConnectionTraits, NetConnectionTraits))
		{
			return true;
		}
	}

	return false;
}

FNetBitArray FReplicationConnections::GetOpenConnections() const
{
	FNetBitArray OpenConnections(ValidConnections.GetNumBits());

	for (int32 ConnectionId = 0; ConnectionId < Connections.Num(); ++ConnectionId)
	{
		if (ValidConnections.IsBitSet(ConnectionId) && !Connections[ConnectionId].bIsClosing)
		{
			OpenConnections.SetBit(ConnectionId);
		}
	}

	return OpenConnections;
}

void FReplicationConnections::DeinitDataStreamManager(uint32 ConnectionId)
{
	if (FReplicationConnection* Connection = GetConnection(ConnectionId))
	{
		if (UDataStreamManager* DataStreamManager = Connection->DataStreamManager.Get())
		{
			DataStreamManager->Deinit();
		}

		// This is a bit special as these are owned by ReplicationSystem rather than the ReplicationDataStream
		Connection->ReplicationReader->Deinit();
		Connection->ReplicationWriter->Deinit();

		// Delete replication reader / writer
		delete Connection->ReplicationReader;
		Connection->ReplicationReader = nullptr;
		delete Connection->ReplicationWriter;
		Connection->ReplicationWriter = nullptr;
	}
}

}
