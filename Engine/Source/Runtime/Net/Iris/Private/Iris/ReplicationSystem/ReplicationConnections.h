// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Array.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationSystem/ReplicationView.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"

class UDataStreamManager;
namespace UE::Net
{
	enum class ENetConnectionTraits : uint32;
}

namespace UE::Net::Private
{
	class FReplicationWriter;
	class FReplicationReader;
}

namespace UE::Net::Private
{

struct FReplicationConnection
{
	FReplicationWriter* ReplicationWriter = nullptr;
	FReplicationReader* ReplicationReader = nullptr;
	TWeakObjectPtr<UDataStreamManager> DataStreamManager;
	FObjectPtr UserData = nullptr;
	ENetConnectionTraits NetConnectionTraits = {};
	bool bIsClosing = false; // Should be set when a connection starts the graceful close process to finish flushing reliable data
};

class FReplicationConnections
{
public:
	explicit FReplicationConnections(uint32 MaxConnections = 128)
	: ValidConnections(MaxConnections)
	{
		Connections.SetNumZeroed(MaxConnections);
		ReplicationViews.SetNum(MaxConnections);
	}

	void Deinit();

	const FReplicationConnection* GetConnection(uint32 ConnectionId) const
	{
		if (ValidConnections.GetBit(ConnectionId))
		{
			return &Connections[ConnectionId];
		}
		
		return nullptr;
	}

	FReplicationConnection* GetConnection(uint32 ConnectionId)
	{
		if (ValidConnections.GetBit(ConnectionId))
		{
			return &Connections[ConnectionId];
		}
		
		return nullptr;
	}

	bool IsValidConnection(uint32 ConnectionId) const
	{
		return ConnectionId < GetMaxConnectionCount() && ValidConnections.GetBit(ConnectionId);
	}

	bool IsOpenConnection(uint32 ConnectionId) const
	{
		return ConnectionId < GetMaxConnectionCount() && ValidConnections.GetBit(ConnectionId) && !Connections[ConnectionId].bIsClosing;
	}

	void AddConnection(uint32 ConnectionId, ENetConnectionTraits NetConnectionTraits);
	void RemoveConnection(uint32 ConnectionId);

	bool HasConnectionWithAnyTraits(ENetConnectionTraits NetConnectionTraits) const;

	uint32 GetMaxConnectionCount() const { return ValidConnections.GetNumBits(); }

	const FNetBitArray& GetValidConnections() const { return ValidConnections; }

	// Returns connections that are not in the closing state
	FNetBitArray GetOpenConnections() const;

	void InitDataStreamManager(uint32 ReplicationSystemId, uint32 ConnectionId, UDataStreamManager* DataStreamManager);
	void DeinitDataStreamManager(uint32 ConnectionId);

	void SetReplicationView(uint32 ConnectionId, const FReplicationView& ViewInfo);
	const FReplicationView& GetReplicationView(uint32 ConnectionId) const { return ReplicationViews[ConnectionId]; }

	// Flag a connection as being in a graceful-close state meant to flush pending reliable data.
	void SetConnectionIsClosing(uint32 ConnectionId)
	{
		check(ValidConnections.GetBit(ConnectionId) == true);
		Connections[ConnectionId].bIsClosing = true;
	}
private:
	TArray<FReplicationConnection> Connections;
	TArray<FReplicationView> ReplicationViews;
	FNetBitArray ValidConnections;
};

}
