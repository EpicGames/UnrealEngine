// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationDataStream.h"
#include "Iris/ReplicationSystem/ReplicationReader.h"
#include "Iris/ReplicationSystem/ReplicationWriter.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationConnections.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplicationDataStream)

UReplicationDataStream::UReplicationDataStream()
: ReplicationReader(nullptr)
, ReplicationWriter(nullptr)
{
}

UReplicationDataStream::~UReplicationDataStream()
{
}

void UReplicationDataStream::Init(const UDataStream::FInitParameters& Params)
{
	Super::Init(Params);

	using namespace UE::Net::Private;

	UReplicationSystem* ReplicationSystem = UE::Net::GetReplicationSystem(Params.ReplicationSystemId);
	if (ensure(ReplicationSystem))
	{
		// Init ReplicationReader and writer.
		if (FReplicationSystemInternal* ReplicationSystemInternal = ReplicationSystem->GetReplicationSystemInternal())
		{
			FReplicationConnections& Connections = ReplicationSystemInternal->GetConnections();
			FReplicationConnection* Connection = Connections.GetConnection(Params.ConnectionId);
			if (ensure(Connection))
			{
				ReplicationWriter = Connection->ReplicationWriter;
				ReplicationReader = Connection->ReplicationReader;

				if (ensure(Params.NetExports))
				{
					ReplicationWriter->SetNetExports(*Params.NetExports);
				}
			}
		}
	}
}

void UReplicationDataStream::Deinit()
{
	Super::Deinit();

	ReplicationWriter = nullptr;
	ReplicationReader = nullptr;
}

void UReplicationDataStream::Update(const FUpdateParameters& Params)
{
	if (ReplicationWriter)
	{
		ReplicationWriter->Update(Params);
	}
}

UDataStream::EWriteResult UReplicationDataStream::BeginWrite(const FBeginWriteParameters& Params)
{
	if (ReplicationWriter != nullptr)
	{
		return ReplicationWriter->BeginWrite(Params);
	}

	return EWriteResult::NoData;
}

UDataStream::EWriteResult UReplicationDataStream::WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord)
{
	if (ReplicationWriter != nullptr)
	{
		return ReplicationWriter->Write(Context);
	}

	return EWriteResult::NoData;
}

void UReplicationDataStream::EndWrite()
{
	if (ReplicationWriter != nullptr)
	{
		return ReplicationWriter->EndWrite();
	}
}

void UReplicationDataStream::ReadData(UE::Net::FNetSerializationContext& Context)
{
	if (ReplicationReader != nullptr)
	{
		ReplicationReader->Read(Context);
	}
}

void UReplicationDataStream::ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record)
{
	if (ReplicationWriter != nullptr)
	{
		ReplicationWriter->ProcessDeliveryNotification(Status);
	}
}

bool UReplicationDataStream::HasAcknowledgedAllReliableData() const
{
	if (ReplicationWriter != nullptr)
	{
		return ReplicationWriter->AreAllReliableAttachmentsSentAndAcked();
	}

	return true;
}

bool UReplicationDataStream::HasDebugDataToSend() const
{
	if (ReplicationWriter != nullptr)
	{
		return ReplicationWriter->HasDebugDataToSend();
	}

	return false;
}
