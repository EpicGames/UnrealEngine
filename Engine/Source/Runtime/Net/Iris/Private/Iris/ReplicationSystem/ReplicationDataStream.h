// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/DataStream/DataStream.h"
#include "ReplicationDataStream.generated.h"

namespace UE::Net::Private
{
	class FReplicationReader;
	class FReplicationWriter;
}

UCLASS()
class UReplicationDataStream final : public UDataStream
{
	GENERATED_BODY()

private:
	UReplicationDataStream();
	virtual ~UReplicationDataStream();

	// UDataStream interface
	virtual void Init(const UDataStream::FInitParameters& Params) override;
	virtual void Deinit() override;
	virtual void Update(const FUpdateParameters& Params) override;
	virtual EWriteResult BeginWrite(const FBeginWriteParameters& Params) override;
	virtual EWriteResult WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord) override;
	virtual void EndWrite() override;
	virtual void ReadData(UE::Net::FNetSerializationContext& Context) override;
	virtual void ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record) override;
	virtual bool HasAcknowledgedAllReliableData() const override;
	virtual bool HasDebugDataToSend() const override;

private:
	UE::Net::Private::FReplicationReader* ReplicationReader;
	UE::Net::Private::FReplicationWriter* ReplicationWriter;
};
