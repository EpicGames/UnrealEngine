// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"

// Whether said features should be compiled in or not. Individual features should be togglable by cvars.
#ifndef UE_NET_REPLICATIONDATASTREAM_DEBUG
	#define UE_NET_REPLICATIONDATASTREAM_DEBUG !(UE_BUILD_SHIPPING)
#endif

namespace UE::Net::Private
{

// CVar net.Iris.
extern bool bReplicationDataStreamDebugBatchSizePerObjectEnabled;
// CVar net.Iris.
extern bool bReplicationDataStreamDebugSentinelsEnabled;

enum EReplicationDataStreamDebugFeatures : uint32
{
	None = 0U,
	BatchSizePerObject = 1U << 0U,
	Sentinels = 1U << 1U,
};
ENUM_CLASS_FLAGS(EReplicationDataStreamDebugFeatures);

static constexpr uint32 ReplicationDataStreamDebugFeaturesBitCount = 2U;

inline void WriteReplicationDataStreamDebugFeatures(FNetBitStreamWriter* Writer, EReplicationDataStreamDebugFeatures Features)
{
	Writer->WriteBits(static_cast<uint32>(Features), ReplicationDataStreamDebugFeaturesBitCount);
}

inline EReplicationDataStreamDebugFeatures ReadReplicationDataStreamDebugFeatures(FNetBitStreamReader* Reader)
{
	EReplicationDataStreamDebugFeatures Features = static_cast<EReplicationDataStreamDebugFeatures>(Reader->ReadBits(ReplicationDataStreamDebugFeaturesBitCount));
	return Features;
}

}
