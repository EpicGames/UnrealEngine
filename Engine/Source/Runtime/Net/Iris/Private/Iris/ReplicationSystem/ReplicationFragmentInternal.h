// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/ReplicationFragment.h"

namespace UE::Net::Private
{
	struct FFragmentRegistrationContextPrivateAccessor
	{
		static const FReplicationFragments& GetReplicationFragments(const FFragmentRegistrationContext& Context)
		{ 
			return Context.Fragments;
		}
		
		static const Private::FReplicationStateDescriptorRegistry* GetReplicationStateRegistry(const FFragmentRegistrationContext& Context)
		{ 
			return Context.ReplicationStateRegistry;
		}
		
		static Private::FReplicationStateDescriptorRegistry* GetReplicationStateRegistry(FFragmentRegistrationContext& Context)
		{ 
			return Context.ReplicationStateRegistry;
		}
		
		static UReplicationSystem* GetReplicationSystem(const FFragmentRegistrationContext& Context)
		{ 
			return Context.ReplicationSystem;
		}

		static bool IsMainObject(const FFragmentRegistrationContext& Context, UObject* ObjectInstance)
		{
			return Context.MainObjectInstance == ObjectInstance;
		}

		static void SetDefaultStateSource(FFragmentRegistrationContext& Context, const UObject* DefaultStateSource)
		{
			Context.MainObjectDefaultStateSource = DefaultStateSource;
		}

		static const UObject* GetDefaultStateSource(const FFragmentRegistrationContext& Context)
		{
			return Context.MainObjectDefaultStateSource;
		}

		static void SetTemplate(FFragmentRegistrationContext& Context, const UObject* Template)
		{
			Context.Template = Template;
		}

		static const UObject* GetTemplate(const FFragmentRegistrationContext& Context)
		{
			return Context.Template;
		}
	};
}
