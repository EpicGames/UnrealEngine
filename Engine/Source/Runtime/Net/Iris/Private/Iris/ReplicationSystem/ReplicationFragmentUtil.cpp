// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/PropertyReplicationFragment.h"
#include "Iris/ReplicationSystem/ReplicationFragmentInternal.h"
#include "Iris/ReplicationSystem/FastArrayReplicationFragment.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisLog.h"

namespace UE::Net
{

uint32 FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(UObject* Object, FFragmentRegistrationContext& Context, EFragmentRegistrationFlags RegistrationFlags, TArray<FReplicationFragment*>* OutCreatedFragments)
{
	const FCreateFragmentParams CreateParams
	{
		.ObjectInstance = Object,
		.RegistrationFlags = RegistrationFlags,
	};

	return CreateAndRegisterFragmentsForObject(CreateParams, Context, OutCreatedFragments);
}

uint32 FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(const FCreateFragmentParams& Params, FFragmentRegistrationContext& Context, TArray<FReplicationFragment*>* OutCreatedFragments)
{
	using namespace UE::Net::Private;

	IRIS_PROFILER_SCOPE(FReplicationFragmentUtil_CreateAndRegisterFragmentsForObject);

	FReplicationStateDescriptorBuilder::FParameters BuilderParameters;
	BuilderParameters.DescriptorRegistry = FFragmentRegistrationContextPrivateAccessor::GetReplicationStateRegistry(Context);
	BuilderParameters.ReplicationSystem = FFragmentRegistrationContextPrivateAccessor::GetReplicationSystem(Context);

	UClass* ObjectClass = Params.ObjectInstance->GetClass();

	const bool bIsMainObject = FFragmentRegistrationContextPrivateAccessor::IsMainObject(Context, Params.ObjectInstance);

	// Find the state source needed by some fragments
	{
		const UObject* DefaultStateSource = nullptr;

		// Some objects want to always build the init state from the CDO
		if (EnumHasAnyFlags(Params.RegistrationFlags, EFragmentRegistrationFlags::InitializeDefaultStateFromClassDefaults))
		{
			DefaultStateSource = ObjectClass->GetDefaultObject();
		}
		// For the main object look if a template was provided
		else if (bIsMainObject)
		{
			DefaultStateSource = FFragmentRegistrationContextPrivateAccessor::GetTemplate(Context);
		}
	
		// Default to use the archetype otherwise
		if (DefaultStateSource == nullptr)
		{
			DefaultStateSource = Params.ObjectInstance->GetArchetype();
		}

		if (!IsValid(DefaultStateSource))
		{
			UE_LOGF(LogIris, Error, "FPropertyReplicationFragment::CreateAndRegisterFragmentsForObject: Invalid object archetype for object %ls, default state will use the CDO", *GetFullNameSafe(Params.ObjectInstance));
			DefaultStateSource = ObjectClass->GetDefaultObject();
		}

		if (bIsMainObject)
		{
			// Store the state source so the bridge can retrieve it
			FFragmentRegistrationContextPrivateAccessor::SetDefaultStateSource(Context, DefaultStateSource);
		}

		BuilderParameters.DefaultStateSource = DefaultStateSource;
	}

	// Pass-on that we allow FastAarrays with extra replicated properties for this object.
	// NOTE: this is not perfect as it allows this for all FastArray properties of the object but as there 
	// is further validation in the actual ReplicationStateFragment implementation for FastArrays it is good enough.
	if (EnumHasAnyFlags(Params.RegistrationFlags, EFragmentRegistrationFlags::AllowFastArraysWithAdditionalProperties))
	{
		BuilderParameters.AllowFastArrayWithExtraReplicatedProperties = 1U;
	}

	FReplicationStateDescriptorBuilder::FResult Result;
	FReplicationStateDescriptorBuilder::CreateDescriptorsForClass(Result, ObjectClass, BuilderParameters);

	const bool bRegisterFunctionsOnly = EnumHasAnyFlags(Params.RegistrationFlags, EFragmentRegistrationFlags::RegisterRPCsOnly);

	uint32 NumCreatedReplicationFragments = 0U;
	// create fragments and let the instance protocol own them.
	for (TRefCountPtr<const FReplicationStateDescriptor>& Desc : Result)
	{
		if (bRegisterFunctionsOnly && (Desc->FunctionCount == 0))
		{
			continue;
		}

		FReplicationFragment* Fragment = nullptr;
		// If descriptor provides CreateAndRegisterReplicationFragment function we use that to instantiate fragment
		if (Desc->CreateAndRegisterReplicationFragmentFunction)
		{
			Fragment = Desc->CreateAndRegisterReplicationFragmentFunction(Params.ObjectInstance, Desc.GetReference(), Context);
		}
		else
		{
			Fragment = FPropertyReplicationFragment::CreateAndRegisterFragment(Params.ObjectInstance, Desc.GetReference(), Context);
		}
		if (Fragment && OutCreatedFragments)
		{
			OutCreatedFragments->Add(Fragment);
			++NumCreatedReplicationFragments;
		}
	}

	// If we did not find any fragments to create, tell the context it's known.
	if (Context.NumFragments() <= 0)
	{
		Context.SetIsFragmentlessNetObject(true);
	}

	return NumCreatedReplicationFragments;
}

}
