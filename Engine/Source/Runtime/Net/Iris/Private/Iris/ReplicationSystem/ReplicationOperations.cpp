// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisLog.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationState/InternalPropertyReplicationState.h"
#include "Iris/ReplicationState/InternalReplicationStateDescriptorUtils.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Misc/CoreMiscDefines.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "DequantizeAndApplyHelper.h"
#include "HAL/IConsoleManager.h"
#include "HAL/UnrealMemory.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/CoreNetTypes.h"

#include "CoreTypes.h"


namespace UE::Net::Private
{

static FNetDebugName s_LifetimeConditionDebugNames[] = 
{
	{TEXT("None"), 0},
	{TEXT("InitialOnly"), 0},
	{TEXT("OwnerOnly"), 0},
	{TEXT("SkipOwner"), 0},
	{TEXT("SimulatedOnly"), 0},
	{TEXT("AutonomousOnly"), 0},
	{TEXT("SimulatedOrPhysics"), 0},
	{TEXT("InitialOrOwner"), 0},
	{TEXT("Custom"), 0},
	{TEXT("ReplayOrOwner"), 0},
	{TEXT("ReplayOnly"), 0},
	{TEXT("SimulatedOnlyNoReplay"), 0},
	{TEXT("SimulatedOrPhysicsNoReplay"), 0},
	{TEXT("SkipReplay"), 0},
	{TEXT("Dynamic"), 0},
	{TEXT("Never"), 0},
	{TEXT("NetGroup"), 0},
};
static_assert(COND_Max == 17, "s_LifetimeConditionDebugNames may need updating.");

// Append changemask bits to ChangeMaskWriter and conditionally to the ConditionalChangeMaskWriter. If the state is dirty, the function will return true.
static bool AppendMemberChangeMasks(FNetBitStreamWriter* ChangeMaskWriter, FNetBitStreamWriter* ConditionalChangeMaskWriter, uint8* ExternalStateBuffer, const FReplicationStateDescriptor* Descriptor);
static void ResetMemberChangeMasks(uint8* ExternalStateBuffer, const FReplicationStateDescriptor* Descriptor);
static void CheckChangeMask(const FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask) { check(Context.GetChangeMask() && (Context.GetChangeMask()->GetData() == ChangeMask.GetData()));}

}

namespace UE::Net
{

static bool bDeltaCompressInitialState = true;
static FAutoConsoleVariableRef CVarbDeltaCompressInitialState(
	TEXT("net.Iris.DeltaCompressInitialState"),
	bDeltaCompressInitialState,
	TEXT("if true we compare with default state when serializing initial state."
	));

static bool bOnlyQuantizeDirtyMembers = true;
static FAutoConsoleVariableRef CVarbOnlyQuantizeDirtyMembers(
	TEXT("net.Iris.OnlyQuantizeDirtyMembers"),
	bOnlyQuantizeDirtyMembers,
	TEXT("if true we only quantize members marked as dirty unless this is a new object."
	));

void FReplicationStateOperations::Quantize(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT SrcExternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	check(IsAligned(DstInternalBuffer, Descriptor->InternalAlignment) && IsAligned(SrcExternalBuffer, Descriptor->ExternalAlignment));

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];

		FNetQuantizeArgs Args;
		Args.Version = 0;
		Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcExternalBuffer + MemberDescriptor.ExternalMemberOffset);
		Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstInternalBuffer + MemberDescriptor.InternalMemberOffset);

		MemberSerializerDescriptor.Serializer->Quantize(Context, Args);
	}
}

void FReplicationStateOperations::QuantizeWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT SrcExternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	check(!Descriptor->IsInitState());
	Private::CheckChangeMask(Context, ChangeMask);
#endif

	check(IsAligned(DstInternalBuffer, Descriptor->InternalAlignment) && IsAligned(SrcExternalBuffer, Descriptor->ExternalAlignment));

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = Descriptor->MemberChangeMaskDescriptors[MemberIt];
		const uint32 MemberChangeMaskOffset = ChangeMaskOffset + MemberChangeMaskDescriptor.BitOffset;

		if (ChangeMask.IsAnyBitSet(MemberChangeMaskOffset, MemberChangeMaskDescriptor.BitCount))
		{
			FNetQuantizeArgs Args;
			Args.Version = 0;
			Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcExternalBuffer + MemberDescriptor.ExternalMemberOffset);
			Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstInternalBuffer + MemberDescriptor.InternalMemberOffset);

			Args.ChangeMaskInfo.BitCount = MemberChangeMaskDescriptor.BitCount;
			Args.ChangeMaskInfo.BitOffset = MemberChangeMaskOffset;

			MemberSerializerDescriptor.Serializer->Quantize(Context, Args);
		}
	}
}

void FReplicationStateOperations::DequantizeWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	check(!Descriptor->IsInitState());
	Private::CheckChangeMask(Context, ChangeMask);
#endif

	check(IsAligned(SrcInternalBuffer, Descriptor->InternalAlignment) && IsAligned(DstExternalBuffer, Descriptor->ExternalAlignment));

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	// If adding code to only dequantize dirty states then special consideration is needed for init states.
	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = Descriptor->MemberChangeMaskDescriptors[MemberIt];
		const uint32 MemberChangeMaskOffset = ChangeMaskOffset + MemberChangeMaskDescriptor.BitOffset;

		if (ChangeMask.IsAnyBitSet(MemberChangeMaskOffset, MemberChangeMaskDescriptor.BitCount))
		{
			FNetDequantizeArgs Args;
			Args.Version = 0;
			Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstExternalBuffer + MemberDescriptor.ExternalMemberOffset);
			Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);

			// NOTE: Currently we only forward changemask info for fastarrays this is to avoid propagating it by accident where it does not belong
			if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::IsFastArrayReplicationState))
			{
				Args.ChangeMaskInfo.BitCount = MemberChangeMaskDescriptor.BitCount;
				Args.ChangeMaskInfo.BitOffset = MemberChangeMaskOffset;
			}

			MemberSerializerDescriptor.Serializer->Dequantize(Context, Args);
		}
	}
}

void FReplicationStateOperations::Dequantize(FNetSerializationContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	check(IsAligned(SrcInternalBuffer, Descriptor->InternalAlignment) && IsAligned(DstExternalBuffer, Descriptor->ExternalAlignment));

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	// If adding code to only dequantize dirty states then special consideration is needed for init states.
	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];

		FNetDequantizeArgs Args;
		Args.Version = 0;
		Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstExternalBuffer + MemberDescriptor.ExternalMemberOffset);
		Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);

		MemberSerializerDescriptor.Serializer->Dequantize(Context, Args);
	}
}

void FReplicationStateOperations::FreeDynamicState(FNetSerializationContext& Context, uint8* RESTRICT StateInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	Private::FReplicationStateOperationsInternal::FreeDynamicState(Context, StateInternalBuffer, Descriptor);
}

bool FReplicationStateOperations::IsEqualQuantizedState(FNetSerializationContext& Context, const uint8* RESTRICT Source0, const uint8* RESTRICT Source1, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	// $IRIS TODO Memcmp if quantized check and possible (no dynamic arrays)?
	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetIsEqualArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		const uint32 MemberOffset = MemberDescriptor.InternalMemberOffset;
		MemberArgs.Source0 = NetSerializerValuePointer(Source0 + MemberOffset);
		MemberArgs.Source1 = NetSerializerValuePointer(Source1 + MemberOffset);
		MemberArgs.bStateIsQuantized = true;

		if (!Serializer->IsEqual(Context, MemberArgs))
		{
			return false;
		}
	}

	return true;
}

bool FReplicationStateOperations::Validate(FNetSerializationContext& Context, const uint8* RESTRICT SrcExternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	if (Descriptor == nullptr)
	{
		return false;
	}

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetValidateArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = NetSerializerValuePointer(SrcExternalBuffer + MemberDescriptor.ExternalMemberOffset);

		if (!Serializer->Validate(Context, MemberArgs))
		{
			return false;
		}
	}

	return true;
}

static void OutputDescriptorMemberHashToString(FNetSerializationContext Context, uint32* HashBuffer, uint32 Indentation, FStringBuilderBase& StringBuilder, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	auto AppendMemberAndIndentation = [](FStringBuilderBase& StringBuilder, uint32 MemberIndex, uint32 Indentation)
	{
		StringBuilder.Appendf(TEXT("%*sMember: %u "), 4*Indentation, TEXT(""), MemberIndex);
	};

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];

		AppendMemberAndIndentation(StringBuilder, MemberIt, Indentation);

		// Traverse struct members as well.
		if (IsUsingStructNetSerializer(MemberSerializerDescriptor))
		{
			const FStructNetSerializerConfig* StructConfig = static_cast<const FStructNetSerializerConfig*>(MemberSerializerDescriptor.SerializerConfig);
			const FReplicationStateDescriptor* StructDescriptor = StructConfig->StateDescriptor;

			StringBuilder.Appendf(TEXT("%s\n"), ToCStr(MemberDebugDescriptors[MemberIt].DebugName));
			OutputDescriptorMemberHashToString(Context, HashBuffer, Indentation + 1, StringBuilder, SrcInternalBuffer + MemberDescriptor.InternalMemberOffset, StructDescriptor);
		}
		else
		{
			// Reset writer as we want to capture this per member
			Writer.Seek(0U);

			FNetSerializeArgs Args;
			Args.Version = 0;
			Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);

			MemberSerializerDescriptor.Serializer->Serialize(Context, Args);

			// Make sure last byte has well defined data as we include it in hash
			if (const uint32 BitsToFill = (8U - (Writer.GetPosBits() & 7U)) & 7U)
			{
				Writer.WriteBits(0U, BitsToFill);
			}

			Writer.CommitWrites();

			// Calculate and print hash 
			// Hash the serialized buffer
			const uint64 MemberDefaultHash = CityHash64((const char*)HashBuffer, Writer.GetPosBytes());
		
			StringBuilder.Appendf(TEXT("%s Serializer: %s Hash: 0x%" UINT64_x_FMT "\n"), ToCStr(MemberDebugDescriptors[MemberIt].DebugName), ToCStr(MemberSerializerDescriptor.Serializer->Name), MemberDefaultHash);
		}
	}
};

void FReplicationStateOperations::OutputDefaultStateMembersHashToString(UReplicationSystem* ReplicationSystem, FStringBuilderBase& StringBuilder, const FReplicationStateDescriptor* Descriptor)
{
	if (!ReplicationSystem)
	{
		return;
	}

	if (!Descriptor || !Descriptor->DefaultStateBuffer)
	{
		return;
	}

	// Setup a temporary buffer for this.
	constexpr uint32 BufferSize = 4096;
	TArray<uint32> TempBuffer;
	TempBuffer.SetNum(BufferSize/sizeof(uint32));

	FNetBitStreamWriter Writer;
	Writer.InitBytes(TempBuffer.GetData(), BufferSize);

	// Setup context to serialize default state
	FNetSerializationContext Context(&Writer);
	Private::FInternalNetSerializationContext InternalContext(ReplicationSystem);
	Context.SetInternalContext(&InternalContext);

	// Tell serializers we are serializing default state. It allows serializers to opt out of being part of the checksum by simply not serializing any data.
	Context.SetIsInitializingDefaultState(true);

	OutputDescriptorMemberHashToString(Context, TempBuffer.GetData(), 0, StringBuilder, Descriptor->DefaultStateBuffer, Descriptor);
}


void FReplicationStateOperations::Serialize(FNetSerializationContext& Context, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		FNetSerializeArgs Args;
		Args.Version = 0;
		Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);

		MemberSerializerDescriptor.Serializer->Serialize(Context, Args);
	}
}

void FReplicationStateOperations::Deserialize(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		UE_ADD_READ_JOURNAL_ENTRY(Context, MemberDebugDescriptors[MemberIt].DebugName);

		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		FNetDeserializeArgs Args;
		Args.Version = 0;
		Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstInternalBuffer + MemberDescriptor.InternalMemberOffset);

		MemberSerializerDescriptor.Serializer->Deserialize(Context, Args);
	}
}

void FReplicationStateOperations::SerializeDelta(FNetSerializationContext& Context, const uint8* RESTRICT SrcInternalBuffer, const uint8* RESTRICT PrevInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];

		FNetSerializeDeltaArgs Args;
		Args.Version = 0;
		Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);
		Args.Prev = reinterpret_cast<NetSerializerValuePointer>(PrevInternalBuffer + MemberDescriptor.InternalMemberOffset);

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
		MemberSerializerDescriptor.Serializer->SerializeDelta(Context, Args);
	}
}

void FReplicationStateOperations::DeserializeDelta(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT PrevInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		UE_ADD_READ_JOURNAL_ENTRY(Context, MemberDebugDescriptors[MemberIt].DebugName);

		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];

		FNetDeserializeDeltaArgs Args;
		Args.Version = 0;
		Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstInternalBuffer + MemberDescriptor.InternalMemberOffset);
		Args.Prev = reinterpret_cast<NetSerializerValuePointer>(PrevInternalBuffer + MemberDescriptor.InternalMemberOffset);

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
		MemberSerializerDescriptor.Serializer->DeserializeDelta(Context, Args);
	}
}


void FReplicationStateOperations::SerializeWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	check(!Descriptor->IsInitState());
	Private::CheckChangeMask(Context, ChangeMask);
#endif

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const FReplicationStateMemberLifetimeConditionDescriptor* MemberLifetimeConditionDescriptors = Descriptor->MemberLifetimeConditionDescriptors;

	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = Descriptor->MemberChangeMaskDescriptors[MemberIt];
		const uint32 MemberChangeMaskOffset = ChangeMaskOffset + MemberChangeMaskDescriptor.BitOffset;

		if (ChangeMask.IsAnyBitSet(MemberChangeMaskOffset, MemberChangeMaskDescriptor.BitCount))
		{
#if UE_NET_TRACE_ENABLED
			const int8 Condition = (MemberLifetimeConditionDescriptors != nullptr) ? MemberLifetimeConditionDescriptors[MemberIt].Condition : int8(ELifetimeCondition::COND_None);
			UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(ConditionTraceScope, &Private::s_LifetimeConditionDebugNames[Condition], *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
			if (MemberLifetimeConditionDescriptors == nullptr)
			{
				UE_NET_TRACE_EXIT_NAMED_SCOPE(ConditionTraceScope);
			}

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
#endif

			FNetSerializeArgs Args;
			Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);
			Args.ChangeMaskInfo.BitCount = MemberChangeMaskDescriptor.BitCount;
			Args.ChangeMaskInfo.BitOffset = MemberChangeMaskOffset;
			Args.Version = 0;

			MemberSerializerDescriptor.Serializer->Serialize(Context, Args);
		}
	}
}

void FReplicationStateOperations::DeserializeWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, uint8* RESTRICT DstInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	check(!Descriptor->IsInitState());
	Private::CheckChangeMask(Context, ChangeMask);
#endif

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const FReplicationStateMemberLifetimeConditionDescriptor* MemberLifetimeConditionDescriptors = Descriptor->MemberLifetimeConditionDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = Descriptor->MemberChangeMaskDescriptors[MemberIt];
		const uint32 MemberChangeMaskOffset = ChangeMaskOffset + MemberChangeMaskDescriptor.BitOffset;

		if (ChangeMask.IsAnyBitSet(MemberChangeMaskOffset, MemberChangeMaskDescriptor.BitCount))
		{
#if UE_NET_TRACE_ENABLED
			const int8 Condition = (MemberLifetimeConditionDescriptors != nullptr) ? MemberLifetimeConditionDescriptors[MemberIt].Condition : int8(ELifetimeCondition::COND_None);
			UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(ConditionTraceScope, &Private::s_LifetimeConditionDebugNames[Condition], *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
			if (MemberLifetimeConditionDescriptors == nullptr)
			{
				UE_NET_TRACE_EXIT_NAMED_SCOPE(ConditionTraceScope);
			}

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
#endif

			UE_ADD_READ_JOURNAL_ENTRY(Context, MemberDebugDescriptors[MemberIt].DebugName);

			FNetDeserializeArgs Args;
			Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstInternalBuffer + MemberDescriptor.InternalMemberOffset);
			Args.ChangeMaskInfo.BitCount = MemberChangeMaskDescriptor.BitCount;
			Args.ChangeMaskInfo.BitOffset = MemberChangeMaskOffset;
			Args.Version = 0;

			MemberSerializerDescriptor.Serializer->Deserialize(Context, Args);
		}
	}
}

void FReplicationStateOperations::SerializeDeltaWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, const uint8* RESTRICT SrcInternalBuffer, const uint8* RESTRICT PrevInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	check(!Descriptor->IsInitState());
	Private::CheckChangeMask(Context, ChangeMask);
#endif

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const FReplicationStateMemberLifetimeConditionDescriptor* MemberLifetimeConditionDescriptors = Descriptor->MemberLifetimeConditionDescriptors;

	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = Descriptor->MemberChangeMaskDescriptors[MemberIt];
		const uint32 MemberChangeMaskOffset = ChangeMaskOffset + MemberChangeMaskDescriptor.BitOffset;

		if (ChangeMask.IsAnyBitSet(MemberChangeMaskOffset, MemberChangeMaskDescriptor.BitCount))
		{
#if UE_NET_TRACE_ENABLED
			const int8 Condition = (MemberLifetimeConditionDescriptors != nullptr) ? MemberLifetimeConditionDescriptors[MemberIt].Condition : int8(ELifetimeCondition::COND_None);
			UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(ConditionTraceScope, &Private::s_LifetimeConditionDebugNames[Condition], *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
			if (MemberLifetimeConditionDescriptors == nullptr)
			{
				UE_NET_TRACE_EXIT_NAMED_SCOPE(ConditionTraceScope);
			}

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
#endif

			FNetSerializeDeltaArgs Args;
			Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);
			Args.Prev = reinterpret_cast<NetSerializerValuePointer>(PrevInternalBuffer + MemberDescriptor.InternalMemberOffset);
			Args.ChangeMaskInfo.BitCount = MemberChangeMaskDescriptor.BitCount;
			Args.ChangeMaskInfo.BitOffset = MemberChangeMaskOffset;
			Args.Version = 0;

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
			MemberSerializerDescriptor.Serializer->SerializeDelta(Context, Args);
		}
	}
}

void FReplicationStateOperations::DeserializeDeltaWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT PrevInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	check(!Descriptor->IsInitState());
	Private::CheckChangeMask(Context, ChangeMask);
#endif

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const FReplicationStateMemberLifetimeConditionDescriptor* MemberLifetimeConditionDescriptors = Descriptor->MemberLifetimeConditionDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = Descriptor->MemberChangeMaskDescriptors[MemberIt];
		const uint32 MemberChangeMaskOffset = ChangeMaskOffset + MemberChangeMaskDescriptor.BitOffset;

		if (ChangeMask.IsAnyBitSet(MemberChangeMaskOffset, MemberChangeMaskDescriptor.BitCount))
		{
#if UE_NET_TRACE_ENABLED
			const int8 Condition = (MemberLifetimeConditionDescriptors != nullptr) ? MemberLifetimeConditionDescriptors[MemberIt].Condition : int8(ELifetimeCondition::COND_None);
			UE_NET_TRACE_NAMED_DYNAMIC_NAME_SCOPE(ConditionTraceScope, &Private::s_LifetimeConditionDebugNames[Condition], *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
			if (MemberLifetimeConditionDescriptors == nullptr)
			{
				UE_NET_TRACE_EXIT_NAMED_SCOPE(ConditionTraceScope);
			}

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
#endif

			UE_ADD_READ_JOURNAL_ENTRY(Context, MemberDebugDescriptors[MemberIt].DebugName);

			FNetDeserializeDeltaArgs Args;
			Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			Args.Target = reinterpret_cast<NetSerializerValuePointer>(DstInternalBuffer + MemberDescriptor.InternalMemberOffset);
			Args.Prev = reinterpret_cast<NetSerializerValuePointer>(PrevInternalBuffer + MemberDescriptor.InternalMemberOffset);
			Args.ChangeMaskInfo.BitCount = MemberChangeMaskDescriptor.BitCount;
			Args.ChangeMaskInfo.BitOffset = MemberChangeMaskOffset;
			Args.Version = 0;

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberSerializerDescriptor.Serializer->Name, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
			MemberSerializerDescriptor.Serializer->DeserializeDelta(Context, Args);
		}
	}
}

void FReplicationStateOperations::ApplyStruct(FNetSerializationContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcExternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	Private::InternalApplyStructProperty(Descriptor, DstExternalBuffer, SrcExternalBuffer);
}

bool FReplicationInstanceOperations::PollAndCopyPropertyData(const FReplicationInstanceProtocol* InstanceProtocol, EReplicationFragmentTraits ExcludeTraits, EReplicationFragmentPollFlags PollOptions)
{
	bool bIsStateDirty = false;

	// @TODO Cache traits to avoid touching fragments
	const uint32 FragmentCount = InstanceProtocol->FragmentCount;
	for (uint32 StateIt = 0; StateIt < FragmentCount; ++StateIt)
	{
		// Only poll fragments with NeedsPoll and none of the ExcludedTraits set.
		const EReplicationFragmentTraits MaskedFragmentTraits = InstanceProtocol->FragmentData[StateIt].Traits & (EReplicationFragmentTraits::NeedsPoll | ExcludeTraits);
		if (MaskedFragmentTraits == EReplicationFragmentTraits::NeedsPoll)
		{
			bIsStateDirty |= InstanceProtocol->Fragments[StateIt]->PollReplicatedState(PollOptions);
		}
	}

	return bIsStateDirty;
}

bool FReplicationInstanceOperations::PollAndCopyPropertyData(const FReplicationInstanceProtocol* InstanceProtocol, EReplicationFragmentPollFlags PollOptions)
{
	return PollAndCopyPropertyData(InstanceProtocol, EReplicationFragmentTraits::None, PollOptions);
}

bool FReplicationInstanceOperations::PollAndCopyObjectReferences(const FReplicationInstanceProtocol* InstanceProtocol, EReplicationFragmentTraits RequiredTraits, EReplicationFragmentPollFlags PollOptions)
{
	bool bIsStateDirty = false;
	const EReplicationFragmentTraits ExpectedTraits = EReplicationFragmentTraits::HasObjectReference | RequiredTraits;

	const uint32 FragmentCount = InstanceProtocol->FragmentCount;
	for (uint32 StateIt = 0; StateIt < FragmentCount; ++StateIt)
	{
		const EReplicationFragmentTraits MaskedFragmentTraits = InstanceProtocol->FragmentData[StateIt].Traits & ExpectedTraits;
		if (MaskedFragmentTraits == ExpectedTraits)
		{			
			bIsStateDirty |= InstanceProtocol->Fragments[StateIt]->PollReplicatedState(PollOptions | EReplicationFragmentPollFlags::ForceRefreshCachedObjectReferencesAfterGC);
		}
	}

	return bIsStateDirty;
}

void FReplicationInstanceOperations::Quantize(FNetSerializationContext& Context, uint8* DstObjectStateBuffer, FNetBitStreamWriter* OutChangeMaskWriter, const FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol)
{
	IRIS_PROFILER_SCOPE_VERBOSE(Quantize);

	check(InstanceProtocol && InstanceProtocol->FragmentCount == Protocol->ReplicationStateCount);

	FReplicationInstanceProtocol::FFragmentData* FragmentData = InstanceProtocol->FragmentData;
	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;

	// Set up conditional change mask writer.
	FNetBitStreamWriter ConditionalChangeMaskWriter;
	FNetBitStreamWriter* OutConditionalChangeMaskWriter = nullptr;
	if (EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasConditionalChangeMask))
	{
		ConditionalChangeMaskWriter.InitBytes(DstObjectStateBuffer + Protocol->GetConditionalChangeMaskOffset(), FNetBitArrayView::CalculateRequiredWordCount(Protocol->ChangeMaskBitCount)*sizeof(FNetBitArrayView::StorageWordType));
		OutConditionalChangeMaskWriter = &ConditionalChangeMaskWriter;
	}

	uint8* CurrentInternalStateBuffer = DstObjectStateBuffer;	
	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);
		
		FReplicationStateOperations::Quantize(Context, CurrentInternalStateBuffer, FragmentData[StateIt].ExternalSrcBuffer, CurrentDescriptor);

		// Append changemask data to bitstreams
		Private::AppendMemberChangeMasks(OutChangeMaskWriter, OutConditionalChangeMaskWriter, FragmentData[StateIt].ExternalSrcBuffer, CurrentDescriptor);
		
		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
	}

	OutChangeMaskWriter->CommitWrites();
	if (OutConditionalChangeMaskWriter != nullptr)
	{
		OutConditionalChangeMaskWriter->CommitWrites();
	}
}

void FReplicationInstanceOperations::QuantizeIfDirty(FNetSerializationContext& Context, uint8* DstObjectStateBuffer, FNetBitStreamWriter* OutChangeMaskWriter, const FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol)
{
	IRIS_PROFILER_SCOPE_VERBOSE(QuantizeIfDirty);

	check(InstanceProtocol && InstanceProtocol->FragmentCount == Protocol->ReplicationStateCount);

	FReplicationInstanceProtocol::FFragmentData* FragmentData = InstanceProtocol->FragmentData;
	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;

	// Set up conditional change mask writer.
	FNetBitStreamWriter ConditionalChangeMaskWriter;
	FNetBitStreamWriter* OutConditionalChangeMaskWriter = nullptr;
	if (EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasConditionalChangeMask))
	{
		ConditionalChangeMaskWriter.InitBytes(DstObjectStateBuffer + Protocol->GetConditionalChangeMaskOffset(), FNetBitArrayView::CalculateRequiredWordCount(Protocol->ChangeMaskBitCount)*sizeof(FNetBitArrayView::StorageWordType));
		OutConditionalChangeMaskWriter = &ConditionalChangeMaskWriter;
	}

	uint8* CurrentInternalStateBuffer = DstObjectStateBuffer;	
	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);

		// Append changemask data to bitstreams and quantize state if mask is dirty
		if (Private::AppendMemberChangeMasks(OutChangeMaskWriter, OutConditionalChangeMaskWriter, FragmentData[StateIt].ExternalSrcBuffer, CurrentDescriptor))
		{
			IRIS_PROFILER_PROTOCOL_NAME(CurrentDescriptor->DebugName->Name);

			if (CurrentDescriptor->IsInitState() || !bOnlyQuantizeDirtyMembers)
			{
				FReplicationStateOperations::Quantize(Context, CurrentInternalStateBuffer, FragmentData[StateIt].ExternalSrcBuffer, CurrentDescriptor);
			}
			else
			{
				FNetBitArrayView ChangeMask = UE::Net::Private::GetMemberChangeMask(FragmentData[StateIt].ExternalSrcBuffer, CurrentDescriptor);
				Context.SetChangeMask(&ChangeMask);
				FReplicationStateOperations::QuantizeWithMask(Context, ChangeMask, 0, CurrentInternalStateBuffer, FragmentData[StateIt].ExternalSrcBuffer, CurrentDescriptor);
			}
		}
		
		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
	}

	OutChangeMaskWriter->CommitWrites();
	if (OutConditionalChangeMaskWriter != nullptr)
	{
		OutConditionalChangeMaskWriter->CommitWrites();
	}
}

void FReplicationInstanceOperations::ResetDirtiness(const FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol)
{
	check(InstanceProtocol && InstanceProtocol->FragmentCount == Protocol->ReplicationStateCount);

	FReplicationInstanceProtocol::FFragmentData* FragmentData = InstanceProtocol->FragmentData;
	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;

	for (uint32 StateIt = 0, StateEndIt = Protocol->ReplicationStateCount; StateIt < StateEndIt; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];
		Private::ResetMemberChangeMasks(FragmentData[StateIt].ExternalSrcBuffer, CurrentDescriptor);
		Private::ResetMemberPollMask(FragmentData[StateIt].ExternalSrcBuffer, CurrentDescriptor);
	}
}

void FReplicationInstanceOperations::DequantizeAndApply(FNetSerializationContext& NetSerializationContext, const FDequantizeAndApplyParameters& Parameters)
{
	using namespace UE::Net::Private;

	FDequantizeAndApplyHelper::FContext* Context = FDequantizeAndApplyHelper::Initialize(NetSerializationContext, Parameters);

	FDequantizeAndApplyHelper::ApplyAndCallLegacyFunctions(Context, NetSerializationContext);

	// We need to destruct temporary states (as properties might have allocated memory outside of our temporary allocator)
	FDequantizeAndApplyHelper::Deinitialize(Context);
}

void FReplicationInstanceOperations::DequantizeAndApply(FNetSerializationContext& Context, FMemStackBase& InAllocator, const uint32* ChangeMaskData, const FReplicationInstanceProtocol* InstanceProtocol, const uint8* SrcObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	FDequantizeAndApplyParameters Parameters;
	Parameters.Allocator = &InAllocator;
	Parameters.ChangeMaskData = ChangeMaskData;
	Parameters.InstanceProtocol = InstanceProtocol;
	Parameters.SrcObjectStateBuffer = SrcObjectStateBuffer;
	Parameters.Protocol = Protocol;

	DequantizeAndApply(Context, Parameters);
}

void FReplicationProtocolOperations::Serialize(FNetSerializationContext& Context, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	const uint8* CurrentInternalStateBuffer = SrcObjectStateBuffer;

	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);
		
		FReplicationStateOperations::Serialize(Context, CurrentInternalStateBuffer, CurrentDescriptor);
		
		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
	}
}

void FReplicationInstanceOperations::OutputInternalStateToString(FNetSerializationContext& NetSerializationContext, FStringBuilderBase& StringBuilder, const uint32* ChangeMaskData, const uint8* InternalStateBuffer, const FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol)
{
	// Iterate over fragments
	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	FReplicationFragment* const * ReplicationFragments = InstanceProtocol->Fragments;	

	// We need to accumulate the offsets and alignment of each internal state
	const uint8* CurrentInternalStateBuffer = InternalStateBuffer;

	for (uint32 StateIt = 0, StateEndIt = Protocol->ReplicationStateCount; StateIt != StateEndIt; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];
		const FReplicationFragment* CurrentFragment = ReplicationFragments[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);

		FReplicationStateApplyContext ReplicationStateToStringContext;
		ReplicationStateToStringContext.NetSerializationContext = &NetSerializationContext;
		ReplicationStateToStringContext.Descriptor = CurrentDescriptor;		
		ReplicationStateToStringContext.bIsInit = NetSerializationContext.IsInitState();

		// Dequantize state data
		if (!EnumHasAnyFlags(CurrentFragment->GetTraits(), EReplicationFragmentTraits::HasPersistentTargetStateBuffer))
		{
			// Allocate buffer for temporary state and construct the external state
			uint8* StateBuffer = (uint8*)FMemory::Malloc(CurrentDescriptor->ExternalSize, CurrentDescriptor->ExternalAlignment);
			CurrentDescriptor->ConstructReplicationState(StateBuffer, CurrentDescriptor);
	
			// Dequantize state data
			FReplicationStateOperations::Dequantize(NetSerializationContext, StateBuffer, (uint8*)CurrentInternalStateBuffer, CurrentDescriptor);

			// Pass in the dequantized state buffer
			ReplicationStateToStringContext.StateBufferData.ExternalStateBuffer = StateBuffer;

			CurrentFragment->ReplicatedStateToString(StringBuilder, ReplicationStateToStringContext, EReplicationStateToStringFlags::None);

			// Free buffer
			CurrentDescriptor->DestructReplicationState(StateBuffer, CurrentDescriptor);
			FMemory::Free(StateBuffer);
		}
		else
		{
			// Pass in raw internal statebuffer
			ReplicationStateToStringContext.StateBufferData.ChangeMaskData = nullptr;
			ReplicationStateToStringContext.StateBufferData.RawStateBuffer = CurrentInternalStateBuffer;

			CurrentFragment->ReplicatedStateToString(StringBuilder, ReplicationStateToStringContext, EReplicationStateToStringFlags::None);
		}

		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
	}
}

void FReplicationInstanceOperations::OutputInternalDefaultStateMemberHashesToString(UReplicationSystem* ReplicationSystem, FStringBuilderBase& StringBuilder, const FReplicationFragments& Fragments)
{
	// Iterate over fragments
	for (int32 Index=0; Index < Fragments.Num(); ++Index)
	{
		const FReplicationFragmentInfo& FragmentInfo = Fragments[Index];

		const FReplicationStateDescriptor* CurrentDescriptor = FragmentInfo.Descriptor;
		const FReplicationFragment* CurrentFragment = FragmentInfo.Fragment;

		StringBuilder.Appendf(TEXT("Printing default state hash for members of [%d/%d] Fragment: %s DescriptorId: 0x%" UINT64_x_FMT " DefaultStateHash: 0x%" UINT64_x_FMT "\n"), 
			Index+1, Fragments.Num(), ToCStr(CurrentDescriptor->DebugName), CurrentDescriptor->DescriptorIdentifier.Value, CurrentDescriptor->DescriptorIdentifier.DefaultStateHash);
		FReplicationStateOperations::OutputDefaultStateMembersHashToString(ReplicationSystem, StringBuilder, CurrentDescriptor);
	}	
}

void FReplicationInstanceOperations::OutputInternalDefaultStateToString(FNetSerializationContext& NetSerializationContext, FStringBuilderBase& StringBuilder, const FReplicationFragments& Fragments)
{
	// Iterate over fragments
	for (int32 Index=0; Index < Fragments.Num(); ++Index)
	{
		const FReplicationFragmentInfo& FragmentInfo = Fragments[Index];

		const FReplicationStateDescriptor* CurrentDescriptor = FragmentInfo.Descriptor;
		const FReplicationFragment* CurrentFragment = FragmentInfo.Fragment;

		FReplicationStateApplyContext ReplicationStateToStringContext;
		ReplicationStateToStringContext.NetSerializationContext = &NetSerializationContext;
		ReplicationStateToStringContext.Descriptor = CurrentDescriptor;
		ReplicationStateToStringContext.bIsInit = NetSerializationContext.IsInitState();

		StringBuilder.Appendf(TEXT("[%d/%d] Fragment: %s DescriptorId: 0x%" UINT64_x_FMT " DefaultStateHash: 0x%" UINT64_x_FMT "\n"), 
			Index+1, Fragments.Num(), ToCStr(CurrentDescriptor->DebugName), CurrentDescriptor->DescriptorIdentifier.Value, CurrentDescriptor->DescriptorIdentifier.DefaultStateHash);

		// Dequantize state data
		if (!EnumHasAnyFlags(CurrentFragment->GetTraits(), EReplicationFragmentTraits::HasPersistentTargetStateBuffer))
		{
			// Allocate buffer for temporary state and construct the external state
			uint8* StateBuffer = (uint8*)FMemory::Malloc(CurrentDescriptor->ExternalSize, CurrentDescriptor->ExternalAlignment);
			CurrentDescriptor->ConstructReplicationState(StateBuffer, CurrentDescriptor);

			// Dequantize state data
			FReplicationStateOperations::Dequantize(NetSerializationContext, StateBuffer, CurrentDescriptor->DefaultStateBuffer, CurrentDescriptor);

			// Pass in the dequantized state buffer
			ReplicationStateToStringContext.StateBufferData.ExternalStateBuffer = StateBuffer;

			CurrentFragment->ReplicatedStateToString(StringBuilder, ReplicationStateToStringContext, EReplicationStateToStringFlags::None);

			// Free buffer
			CurrentDescriptor->DestructReplicationState(StateBuffer, CurrentDescriptor);
			FMemory::Free(StateBuffer);
		}
		else
		{
			// Pass in raw internal statebuffer
			ReplicationStateToStringContext.StateBufferData.ChangeMaskData = nullptr;
			ReplicationStateToStringContext.StateBufferData.RawStateBuffer = CurrentDescriptor->DefaultStateBuffer;

			CurrentFragment->ReplicatedStateToString(StringBuilder, ReplicationStateToStringContext, EReplicationStateToStringFlags::None);
		}
	}
}

/** Build structured member hashes from descriptors, mirroring the logic in OutputDescriptorMemberHashToString. */
static void BuildDescriptorMemberState(FNetSerializationContext Context, uint32* HashBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor, TArray<Private::FDebugCDOMemberState>& OutMembers)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];

		Private::FDebugCDOMemberState& Member = OutMembers.AddDefaulted_GetRef();
		Member.Name = ToCStr(MemberDebugDescriptors[MemberIt].DebugName);

		if (IsUsingStructNetSerializer(MemberSerializerDescriptor))
		{
			const FStructNetSerializerConfig* StructConfig = static_cast<const FStructNetSerializerConfig*>(MemberSerializerDescriptor.SerializerConfig);
			const FReplicationStateDescriptor* StructDescriptor = StructConfig->StateDescriptor;
			BuildDescriptorMemberState(Context, HashBuffer, SrcInternalBuffer + MemberDescriptor.InternalMemberOffset, StructDescriptor, Member.SubMembers);
		}
		else
		{
			Member.SerializerName = MemberSerializerDescriptor.Serializer->Name;

			// Hash the serialized default state for this member (same as OutputDescriptorMemberHashToString)
			Writer.Seek(0U);

			FNetSerializeArgs Args;
			Args.Version = 0;
			Args.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			Args.Source = reinterpret_cast<NetSerializerValuePointer>(SrcInternalBuffer + MemberDescriptor.InternalMemberOffset);

			MemberSerializerDescriptor.Serializer->Serialize(Context, Args);

			if (Writer.IsOverflown())
			{
				UE_LOGF(LogIris, Warning, "BuildDescriptorMemberState: Writer overflow for member %ls, hash will be zero", *Member.Name);
				Member.Hash = 0;
			}
			else
			{
				if (const uint32 BitsToFill = (8U - (Writer.GetPosBits() & 7U)) & 7U)
				{
					Writer.WriteBits(0U, BitsToFill);
				}
				Writer.CommitWrites();

				Member.Hash = CityHash64((const char*)HashBuffer, Writer.GetPosBytes());
			}
		}
	}
}

void FReplicationInstanceOperations::BuildInternalDefaultState(UReplicationSystem* ReplicationSystem, const FReplicationFragments& Fragments, Private::FDebugCDOState& OutState)
{
	if (!ReplicationSystem)
	{
		return;
	}

	constexpr uint32 BufferSize = 4096;
	TArray<uint32> TempBuffer;
	TempBuffer.SetNum(BufferSize / sizeof(uint32));

	FNetBitStreamWriter Writer;
	Writer.InitBytes(TempBuffer.GetData(), BufferSize);

	FNetSerializationContext Context(&Writer);
	Private::FInternalNetSerializationContext InternalContext(ReplicationSystem);
	Context.SetInternalContext(&InternalContext);
	Context.SetIsInitializingDefaultState(true);

	for (int32 Index = 0; Index < Fragments.Num(); ++Index)
	{
		const FReplicationFragmentInfo& FragmentInfo = Fragments[Index];
		const FReplicationStateDescriptor* CurrentDescriptor = FragmentInfo.Descriptor;

		if (!CurrentDescriptor)
		{
			continue;
		}

		Private::FDebugCDOFragmentState& Fragment = OutState.Fragments.AddDefaulted_GetRef();
		Fragment.Name = ToCStr(CurrentDescriptor->DebugName);
		Fragment.DescriptorId = CurrentDescriptor->DescriptorIdentifier.Value;
		Fragment.DefaultStateHash = CurrentDescriptor->DescriptorIdentifier.DefaultStateHash;

		if (CurrentDescriptor->DefaultStateBuffer)
		{
			BuildDescriptorMemberState(Context, TempBuffer.GetData(), CurrentDescriptor->DefaultStateBuffer, CurrentDescriptor, Fragment.Members);
		}

		for (uint32 FunctionIt = 0; FunctionIt < CurrentDescriptor->FunctionCount; ++FunctionIt)
		{
			const FReplicationStateMemberFunctionDescriptor& FunctionDescriptor = CurrentDescriptor->MemberFunctionDescriptors[FunctionIt];
			Private::FDebugCDOFunctionState& Function = Fragment.Functions.AddDefaulted_GetRef();
			Function.Name = GetNameSafe(FunctionDescriptor.Function);
			Function.DescriptorId = FunctionDescriptor.Descriptor ? FunctionDescriptor.Descriptor->DescriptorIdentifier.Value : 0;
		}
	}
}

TArray<uint8> FReplicationInstanceOperations::SerializeInternalDefaultState(Private::FDebugCDOState& State)
{
	TArray<uint8> Bytes;
	FMemoryWriter Ar(Bytes);
	Ar << State;
	return Bytes;
}

bool FReplicationInstanceOperations::DeserializeInternalDefaultState(const TArray<uint8>& Bytes, Private::FDebugCDOState& OutState)
{
	FMemoryReader Ar(Bytes);
	Ar << OutState;
	if (Ar.IsError())
	{
		return false;
	}
	return true;
}

/** Diff two member lists recursively and append results to DiffBuilder. */
static void DiffDescriptorMembers(const TArray<Private::FDebugCDOMemberState>& ClientMembers, const TArray<Private::FDebugCDOMemberState>& ServerMembers, FStringBuilderBase& DiffBuilder, int32 Indent)
{
	TMap<FString, const Private::FDebugCDOMemberState*> ServerMembersByName;
	for (const Private::FDebugCDOMemberState& Member : ServerMembers)
	{
		ServerMembersByName.Add(Member.Name, &Member);
	}

	TSet<FString> MatchedServerMembers;
	for (const Private::FDebugCDOMemberState& ClientMember : ClientMembers)
	{
		const Private::FDebugCDOMemberState** ServerMemberPtr = ServerMembersByName.Find(ClientMember.Name);
		if (!ServerMemberPtr)
		{
			DiffBuilder.Appendf(TEXT("%*sClient only: %s"), Indent, TEXT(""), *ClientMember.Name);
			if (!ClientMember.SerializerName.IsEmpty())
			{
				DiffBuilder.Appendf(TEXT(" Serializer: %s Hash: 0x%" UINT64_x_FMT), *ClientMember.SerializerName, ClientMember.Hash);
			}
			DiffBuilder.AppendChar(TEXT('\n'));
		}
		else
		{
			MatchedServerMembers.Add(ClientMember.Name);
			const Private::FDebugCDOMemberState& ServerMember = **ServerMemberPtr;

			const bool bClientIsLeaf = !ClientMember.SerializerName.IsEmpty();
			const bool bServerIsLeaf = !ServerMember.SerializerName.IsEmpty();

			if (bClientIsLeaf && bServerIsLeaf)
			{
				// Leaf member — compare serializer and hash
				if (ClientMember.SerializerName != ServerMember.SerializerName || ClientMember.Hash != ServerMember.Hash)
				{
					DiffBuilder.Appendf(TEXT("%*sChanged: %s\n"), Indent, TEXT(""), *ClientMember.Name);
					DiffBuilder.Appendf(TEXT("%*s  Client: Serializer: %s Hash: 0x%" UINT64_x_FMT "\n"), Indent, TEXT(""), *ClientMember.SerializerName, ClientMember.Hash);
					DiffBuilder.Appendf(TEXT("%*s  Server: Serializer: %s Hash: 0x%" UINT64_x_FMT "\n"), Indent, TEXT(""), *ServerMember.SerializerName, ServerMember.Hash);
				}
			}
			else if (!bClientIsLeaf && !bServerIsLeaf)
			{
				// Struct member — recurse into sub-members
				DiffDescriptorMembers(ClientMember.SubMembers, ServerMember.SubMembers, DiffBuilder, Indent + 2);
			}
			else
			{
				// Shape mismatch: one side is a struct, the other isn't
				DiffBuilder.Appendf(TEXT("%*sChanged shape: %s\n"), Indent, TEXT(""), *ClientMember.Name);
				if (bClientIsLeaf)
				{
					DiffBuilder.Appendf(TEXT("%*s  Client: Serializer: %s Hash: 0x%" UINT64_x_FMT "\n"), Indent, TEXT(""), *ClientMember.SerializerName, ClientMember.Hash);
					DiffBuilder.Appendf(TEXT("%*s  Server: struct with %d sub-member(s)\n"), Indent, TEXT(""), ServerMember.SubMembers.Num());
				}
				else
				{
					DiffBuilder.Appendf(TEXT("%*s  Client: struct with %d sub-member(s)\n"), Indent, TEXT(""), ClientMember.SubMembers.Num());
					DiffBuilder.Appendf(TEXT("%*s  Server: Serializer: %s Hash: 0x%" UINT64_x_FMT "\n"), Indent, TEXT(""), *ServerMember.SerializerName, ServerMember.Hash);
				}
			}
		}
	}

	for (const Private::FDebugCDOMemberState& ServerMember : ServerMembers)
	{
		if (!MatchedServerMembers.Contains(ServerMember.Name))
		{
			DiffBuilder.Appendf(TEXT("%*sServer only: %s"), Indent, TEXT(""), *ServerMember.Name);
			if (!ServerMember.SerializerName.IsEmpty())
			{
				DiffBuilder.Appendf(TEXT(" Serializer: %s Hash: 0x%" UINT64_x_FMT), *ServerMember.SerializerName, ServerMember.Hash);
			}
			DiffBuilder.AppendChar(TEXT('\n'));
		}
	}
}

/** Diff two UFUNCTION lists by name */
static void DiffDescriptorFunctions(const TArray<Private::FDebugCDOFunctionState>& ClientFunctions, const TArray<Private::FDebugCDOFunctionState>& ServerFunctions, FStringBuilderBase& DiffBuilder, int32 Indent)
{
	TMap<FString, const Private::FDebugCDOFunctionState*> ServerFunctionsByName;
	for (const Private::FDebugCDOFunctionState& Function : ServerFunctions)
	{
		ServerFunctionsByName.Add(Function.Name, &Function);
	}

	TSet<FString> MatchedServerFunctions;
	for (const Private::FDebugCDOFunctionState& ClientFunction : ClientFunctions)
	{
		const Private::FDebugCDOFunctionState** ServerFunctionPtr = ServerFunctionsByName.Find(ClientFunction.Name);
		if (!ServerFunctionPtr)
		{
			DiffBuilder.Appendf(TEXT("%*sClient only function: %s DescriptorId: 0x%" UINT64_x_FMT "\n"), Indent, TEXT(""), *ClientFunction.Name, ClientFunction.DescriptorId);
		}
		else
		{
			MatchedServerFunctions.Add(ClientFunction.Name);
			const Private::FDebugCDOFunctionState& ServerFunction = **ServerFunctionPtr;
			if (ClientFunction.DescriptorId != ServerFunction.DescriptorId)
			{
				DiffBuilder.Appendf(TEXT("%*sChanged function signature: %s (client DescriptorId: 0x%" UINT64_x_FMT ", server DescriptorId: 0x%" UINT64_x_FMT ")\n"), Indent, TEXT(""), *ClientFunction.Name, ClientFunction.DescriptorId, ServerFunction.DescriptorId);
			}
		}
	}

	for (const Private::FDebugCDOFunctionState& ServerFunction : ServerFunctions)
	{
		if (!MatchedServerFunctions.Contains(ServerFunction.Name))
		{
			DiffBuilder.Appendf(TEXT("%*sServer only function: %s DescriptorId: 0x%" UINT64_x_FMT "\n"), Indent, TEXT(""), *ServerFunction.Name, ServerFunction.DescriptorId);
		}
	}
}

void FReplicationInstanceOperations::DiffAndLogInternalDefaultState(const Private::FDebugCDOState& ClientState, const Private::FDebugCDOState& ServerState)
{
	// Index server fragments by name
	TMap<FString, const Private::FDebugCDOFragmentState*> ServerFragmentsByName;
	for (const Private::FDebugCDOFragmentState& Fragment : ServerState.Fragments)
	{
		ServerFragmentsByName.Add(Fragment.Name, &Fragment);
	}

	TStringBuilder<4096> DiffBuilder;

	// Compare each client fragment against the server
	TSet<FString> ProcessedFragments;
	for (const Private::FDebugCDOFragmentState& ClientFrag : ClientState.Fragments)
	{
		ProcessedFragments.Add(ClientFrag.Name);
		const Private::FDebugCDOFragmentState** ServerFragPtr = ServerFragmentsByName.Find(ClientFrag.Name);

		if (!ServerFragPtr)
		{
			DiffBuilder.Appendf(TEXT("Fragment %s: client only (not present on server)\n"), *ClientFrag.Name);
			continue;
		}

		const Private::FDebugCDOFragmentState& ServerFrag = **ServerFragPtr;

		// Skip fragments that match on both identity (DescriptorId) and contents (DefaultStateHash).
		if (ClientFrag.DescriptorId == ServerFrag.DescriptorId && ClientFrag.DefaultStateHash == ServerFrag.DefaultStateHash)
		{
			continue;
		}

		DiffBuilder.Appendf(TEXT("Fragment %s: mismatch (client DescriptorId: 0x%" UINT64_x_FMT ", server DescriptorId: 0x%" UINT64_x_FMT " | client DefaultStateHash: 0x%" UINT64_x_FMT ", server DefaultStateHash: 0x%" UINT64_x_FMT ")\n"), *ClientFrag.Name, ClientFrag.DescriptorId, ServerFrag.DescriptorId, ClientFrag.DefaultStateHash, ServerFrag.DefaultStateHash);

		constexpr int32 IndentSize = 2;
		DiffDescriptorMembers(ClientFrag.Members, ServerFrag.Members, DiffBuilder, IndentSize);
		DiffDescriptorFunctions(ClientFrag.Functions, ServerFrag.Functions, DiffBuilder, IndentSize);
	}

	// Check for server-only fragments
	for (const Private::FDebugCDOFragmentState& ServerFrag : ServerState.Fragments)
	{
		if (!ProcessedFragments.Contains(ServerFrag.Name))
		{
			DiffBuilder.Appendf(TEXT("Fragment %s: server only (not present on client)\n"), *ServerFrag.Name);
		}
	}

	if (DiffBuilder.Len() == 0)
	{
		UE_LOGF(LogIris, Error, "CDO state diff: no differences found");
		return;
	}

	UE_LOGF(LogIris, Error, "CDO state differences between client and server:\n%ls", DiffBuilder.ToString());
}

void FReplicationProtocolOperations::Deserialize(FNetSerializationContext& Context, uint8* RESTRICT DstObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	uint8* CurrentInternalStateBuffer = DstObjectStateBuffer;

	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		UE_ADD_READ_JOURNAL_ENTRY(Context, CurrentDescriptor->DebugName);

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);
		
		FReplicationStateOperations::Deserialize(Context, CurrentInternalStateBuffer, CurrentDescriptor);
		
		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
	}
}

void FReplicationProtocolOperations::SerializeWithMask(FNetSerializationContext& Context, const uint32* ChangeMaskData, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	// Nothing to serialize
	if (Protocol->InternalTotalSize == 0U)
	{
		return;
	}

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	const uint8* CurrentInternalStateBuffer = SrcObjectStateBuffer;

	// Setup changemask
	const FNetBitArrayView ChangeMask = MakeNetBitArrayView(ChangeMaskData, Protocol->ChangeMaskBitCount);
	uint32 CurrentChangeMaskBitOffset = 0;
	Context.SetChangeMask(&ChangeMask);

	// We currently write the changemask for the the complete protocol
	// This will later either be RLE compressed or written in chunks per ReplicationState
	{
		UE_NET_TRACE_SCOPE(ChangeMasks, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		WriteSparseBitArray(Writer, ChangeMaskData, Protocol->ChangeMaskBitCount);
	}

	const bool bIsInitState = Context.IsInitState();
	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);
		
		// Only send init state at init
		if (CurrentDescriptor->IsInitState())
		{
			if (bIsInitState)
			{
				UE_NET_TRACE_DYNAMIC_NAME_SCOPE(CurrentDescriptor->DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
				FReplicationStateOperations::Serialize(Context, CurrentInternalStateBuffer, CurrentDescriptor);
			}
		}
		else
		{
			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(CurrentDescriptor->DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
			FReplicationStateOperations::SerializeWithMask(Context, ChangeMask, CurrentChangeMaskBitOffset, CurrentInternalStateBuffer, CurrentDescriptor);
		}
		
		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
		CurrentChangeMaskBitOffset += CurrentDescriptor->ChangeMaskBitCount;
	}

	// Remove dangling reference to ChangeMask on the stack
	Context.SetChangeMask(nullptr);
}

void FReplicationProtocolOperations::DeserializeWithMask(FNetSerializationContext& Context, uint32* DstChangeMaskData, uint8* RESTRICT DstObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	// Nothing to serialize
	if (Protocol->InternalTotalSize == 0U)
	{
		return;
	}

	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	uint8* CurrentInternalStateBuffer = DstObjectStateBuffer;
	
	// We might have garbage in the buffer, we do not need to clear it all but we must reset the padding bits
	FNetBitArrayView ChangeMask(DstChangeMaskData, Protocol->ChangeMaskBitCount, FNetBitArrayView::NoResetNoValidate);
	ChangeMask.ClearPaddingBits();
	Context.SetChangeMask(&ChangeMask);

	// Read the ChangeMask
	{
		UE_NET_TRACE_SCOPE(ChangeMasks, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		UE_ADD_READ_JOURNAL_ENTRY(Context, TEXT("ChangeMask"));
		ReadSparseBitArray(Context.GetBitStreamReader(), DstChangeMaskData, Protocol->ChangeMaskBitCount);
	}

	const bool bIsInitState = Context.IsInitState();
	uint32 CurrentChangeMaskBitOffset = 0;
	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);

		UE_ADD_READ_JOURNAL_ENTRY(Context, CurrentDescriptor->DebugName);
		
		if (CurrentDescriptor->IsInitState())
		{
			if (bIsInitState)
			{
				UE_NET_TRACE_DYNAMIC_NAME_SCOPE(CurrentDescriptor->DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
				FReplicationStateOperations::Deserialize(Context, CurrentInternalStateBuffer, CurrentDescriptor);
			}
		}
		else
		{
			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(CurrentDescriptor->DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
			FReplicationStateOperations::DeserializeWithMask(Context, ChangeMask, CurrentChangeMaskBitOffset, CurrentInternalStateBuffer, CurrentDescriptor);
		}
		
		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
		CurrentChangeMaskBitOffset += CurrentDescriptor->ChangeMaskBitCount;
	}
}

void FReplicationProtocolOperations::FreeDynamicState(FNetSerializationContext& Context, uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	if (!EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasDynamicState))
	{
		return;
	}

	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	uint8* CurrentInternalStateBuffer = SrcObjectStateBuffer;

	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);
		
		if (EnumHasAnyFlags(CurrentDescriptor->Traits, EReplicationStateTraits::HasDynamicState))
		{
			Private::FReplicationStateOperationsInternal::FreeDynamicState(Context, CurrentInternalStateBuffer, CurrentDescriptor);
		}
		
		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
	}
}

void FReplicationProtocolOperations::SerializeInitialStateWithMask(FNetSerializationContext& Context, const uint32* ChangeMaskData, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	if (!bDeltaCompressInitialState)
	{
		SerializeWithMask(Context, ChangeMaskData, SrcObjectStateBuffer, Protocol);
		return;
	}

	// Nothing to serialize
	if (Protocol->InternalTotalSize == 0U)
	{
		return;
	}

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	const uint8* CurrentInternalStateBuffer = SrcObjectStateBuffer;

	// Setup changemask
	const FNetBitArrayView ChangeMask = MakeNetBitArrayView(ChangeMaskData, Protocol->ChangeMaskBitCount);
	uint32 CurrentChangeMaskBitOffset = 0;
	Context.SetChangeMask(&ChangeMask);

	// We currently write the changemask for the complete protocol
	{
		UE_NET_TRACE_SCOPE(ChangeMasks, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		WriteSparseBitArray(Writer, ChangeMaskData, Protocol->ChangeMaskBitCount, ESparseBitArraySerializationHint::ContainsMostlyOnes);
	}

	check(Context.IsInitState());
	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(CurrentDescriptor->DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

		if (CurrentDescriptor->IsInitState())
		{
			FReplicationStateOperations::SerializeDelta(Context, CurrentInternalStateBuffer, CurrentDescriptor->DefaultStateBuffer, CurrentDescriptor);
		}
		else
		{
			FReplicationStateOperations::SerializeDeltaWithMask(Context, ChangeMask, CurrentChangeMaskBitOffset, CurrentInternalStateBuffer, CurrentDescriptor->DefaultStateBuffer, CurrentDescriptor);
		}

		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
		CurrentChangeMaskBitOffset += CurrentDescriptor->ChangeMaskBitCount;
	}

	// Remove dangling reference to ChangeMask on the stack
	Context.SetChangeMask(nullptr);
}

void FReplicationProtocolOperations::DeserializeInitialStateWithMask(FNetSerializationContext& Context, uint32* DstChangeMaskData, uint8* RESTRICT DstObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	if (!bDeltaCompressInitialState)
	{
		DeserializeWithMask(Context, DstChangeMaskData, DstObjectStateBuffer, Protocol);
		return;
	}

	// Nothing to serialize
	if (Protocol->InternalTotalSize == 0U)
	{
		return;
	}

	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	uint8* CurrentInternalStateBuffer = DstObjectStateBuffer;

	// We might have garbage in the buffer, we do not need to clear it all but we must reset the padding bits
	FNetBitArrayView ChangeMask(DstChangeMaskData, Protocol->ChangeMaskBitCount, FNetBitArrayView::NoResetNoValidate);
	ChangeMask.ClearPaddingBits();
	Context.SetChangeMask(&ChangeMask);

	// Read the ChangeMask
	{
		UE_NET_TRACE_SCOPE(ChangeMasks, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		ReadSparseBitArray(Context.GetBitStreamReader(), DstChangeMaskData, Protocol->ChangeMaskBitCount, ESparseBitArraySerializationHint::ContainsMostlyOnes);
	}

	check(Context.IsInitState());

	uint32 CurrentChangeMaskBitOffset = 0;
	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);

		// Since we might not serialize all members we must initialize the state with the default state.
		if (CurrentDescriptor->InternalSize > 0)
		{
			Private::FReplicationStateOperationsInternal::CloneQuantizedState(Context, CurrentInternalStateBuffer, CurrentDescriptor->DefaultStateBuffer, CurrentDescriptor);
		}

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(CurrentDescriptor->DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		if (CurrentDescriptor->IsInitState())
		{
			FReplicationStateOperations::DeserializeDelta(Context, CurrentInternalStateBuffer, CurrentDescriptor->DefaultStateBuffer, CurrentDescriptor);
		}
		else
		{
			FReplicationStateOperations::DeserializeDeltaWithMask(Context, ChangeMask, CurrentChangeMaskBitOffset, CurrentInternalStateBuffer, CurrentDescriptor->DefaultStateBuffer, CurrentDescriptor);
		}

		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
		CurrentChangeMaskBitOffset += CurrentDescriptor->ChangeMaskBitCount;
	}

	// Remove dangling reference to ChangeMask on the stack
	Context.SetChangeMask(nullptr);
}

void FReplicationProtocolOperations::SerializeWithMaskDelta(FNetSerializationContext& Context, const uint32* ChangeMaskData, const uint8* RESTRICT SrcObjectStateBuffer, const uint8* RESTRICT PrevObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	// Nothing to serialize
	if (Protocol->InternalTotalSize == 0U)
	{
		return;
	}

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	const uint8* CurrentInternalStateBuffer = SrcObjectStateBuffer;
	const uint8* PrevInternalStateBuffer = PrevObjectStateBuffer;

	// Setup changemask
	const FNetBitArrayView ChangeMask = MakeNetBitArrayView(ChangeMaskData, Protocol->ChangeMaskBitCount);
	uint32 CurrentChangeMaskBitOffset = 0;
	Context.SetChangeMask(&ChangeMask);

	// We currently write the changemask for the complete protocol
	{
		UE_NET_TRACE_SCOPE(ChangeMasks, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		WriteSparseBitArray(Writer, ChangeMaskData, Protocol->ChangeMaskBitCount);
	}

	const bool bIsInitState = Context.IsInitState();
	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);
		PrevInternalStateBuffer = Align(PrevInternalStateBuffer, CurrentDescriptor->InternalAlignment);

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(CurrentDescriptor->DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

		if (CurrentDescriptor->IsInitState())
		{
			if (bIsInitState)
			{
				FReplicationStateOperations::SerializeDelta(Context, CurrentInternalStateBuffer, PrevInternalStateBuffer, CurrentDescriptor);
			}
		}
		else
		{
			FReplicationStateOperations::SerializeDeltaWithMask(Context, ChangeMask, CurrentChangeMaskBitOffset, CurrentInternalStateBuffer, PrevInternalStateBuffer, CurrentDescriptor);
		}

		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
		PrevInternalStateBuffer += CurrentDescriptor->InternalSize;
		CurrentChangeMaskBitOffset += CurrentDescriptor->ChangeMaskBitCount;
	}

	// Remove dangling reference to ChangeMask on the stack
	Context.SetChangeMask(nullptr);
}

void FReplicationProtocolOperations::DeserializeWithMaskDelta(FNetSerializationContext& Context, uint32* DstChangeMaskData, uint8* RESTRICT DstObjectStateBuffer, const uint8* RESTRICT PrevObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	// Nothing to serialize
	if (Protocol->InternalTotalSize == 0U)
	{
		return;
	}

	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	uint8* CurrentInternalStateBuffer = DstObjectStateBuffer;
	const uint8* PrevInternalStateBuffer = PrevObjectStateBuffer;

	// We might have garbage in the buffer, we do not need to clear it all but we must reset the padding bits
	FNetBitArrayView ChangeMask(DstChangeMaskData, Protocol->ChangeMaskBitCount, FNetBitArrayView::NoResetNoValidate);
	ChangeMask.ClearPaddingBits();
	Context.SetChangeMask(&ChangeMask);

	// Read the ChangeMask
	{
		UE_NET_TRACE_SCOPE(ChangeMasks, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		UE_ADD_READ_JOURNAL_ENTRY(Context, TEXT("ChangeMask"));
		ReadSparseBitArray(Context.GetBitStreamReader(), DstChangeMaskData, Protocol->ChangeMaskBitCount);
	}

	const bool bIsInitState = Context.IsInitState();
	uint32 CurrentChangeMaskBitOffset = 0;
	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);
		PrevInternalStateBuffer = Align(PrevInternalStateBuffer, CurrentDescriptor->InternalAlignment);

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(CurrentDescriptor->DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		UE_ADD_READ_JOURNAL_ENTRY(Context, CurrentDescriptor->DebugName);
		if (CurrentDescriptor->IsInitState())
		{
			if (bIsInitState)
			{
				FReplicationStateOperations::DeserializeDelta(Context, CurrentInternalStateBuffer, PrevInternalStateBuffer, CurrentDescriptor);
			}
		}
		else
		{
			FReplicationStateOperations::DeserializeDeltaWithMask(Context, ChangeMask, CurrentChangeMaskBitOffset, CurrentInternalStateBuffer, PrevInternalStateBuffer, CurrentDescriptor);
		}

		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
		PrevInternalStateBuffer += CurrentDescriptor->InternalSize;
		CurrentChangeMaskBitOffset += CurrentDescriptor->ChangeMaskBitCount;
	}
}

void FReplicationProtocolOperations::InitializeFromDefaultState(FNetSerializationContext& Context, uint8* RESTRICT StateBuffer, const FReplicationProtocol* Protocol)
{
	// Nothing to serialize
	if (Protocol->InternalTotalSize == 0U)
	{
		return;
	}

	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	uint8* CurrentInternalStateBuffer = StateBuffer;

	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);

		if (CurrentDescriptor->InternalSize > 0)
		{
			ensure(CurrentDescriptor->DefaultStateBuffer != nullptr);
			Private::FReplicationStateOperationsInternal::CloneQuantizedState(Context, CurrentInternalStateBuffer, CurrentDescriptor->DefaultStateBuffer, CurrentDescriptor);
		}

		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
	}
}


bool FReplicationProtocolOperations::IsEqualQuantizedState(FNetSerializationContext& Context, const uint8* RESTRICT Source0, const uint8* RESTRICT Source1, const FReplicationProtocol* Protocol)
{
	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	const uint8* Source0InternalStateBuffer = Source0;
	const uint8* Source1InternalStateBuffer = Source1;

	for (uint32 StateIt = 0; StateIt < Protocol->ReplicationStateCount; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		Source0InternalStateBuffer = Align(Source0InternalStateBuffer, CurrentDescriptor->InternalAlignment);
		Source1InternalStateBuffer = Align(Source1InternalStateBuffer, CurrentDescriptor->InternalAlignment);

		if (!FReplicationStateOperations::IsEqualQuantizedState(Context, Source0InternalStateBuffer, Source1InternalStateBuffer, CurrentDescriptor))
		{
			return false;
		}

		Source0InternalStateBuffer += CurrentDescriptor->InternalSize;
		Source1InternalStateBuffer += CurrentDescriptor->InternalSize;
	}

	return true;
}

}

namespace UE::Net::Private
{

static bool AppendMemberChangeMasks(FNetBitStreamWriter* ChangeMaskWriter, FNetBitStreamWriter* ConditionalChangeMaskWriter, uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const uint32 BitCount = Descriptor->ChangeMaskBitCount;
	if (BitCount)
	{
		// Append change mask bits
		FNetBitArrayView::StorageWordType* ChangeMaskStorage = reinterpret_cast<FNetBitArrayView::StorageWordType*>(StateBuffer + Descriptor->GetChangeMaskOffset());
		if (BitCount <= 32)
		{
			ChangeMaskWriter->WriteBits(*ChangeMaskStorage, BitCount);
		}
		else
		{
			constexpr uint32 SrcStreamBitOffset = 0U;
			ChangeMaskWriter->WriteBitStream(ChangeMaskStorage, SrcStreamBitOffset, BitCount);
		}

		if (ConditionalChangeMaskWriter != nullptr)
		{
			if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals))
			{
				FNetBitArrayView::StorageWordType* ConditionalChangeMaskStorage = reinterpret_cast<FNetBitArrayView::StorageWordType*>(StateBuffer + Descriptor->GetConditionalChangeMaskOffset());

				// Append conditional change mask bits.
				// The conditionals are on or off rather than tracking dirtiness so we do not reset them.
				if (BitCount <= 32u)
				{
					ConditionalChangeMaskWriter->WriteBits(*ConditionalChangeMaskStorage, BitCount);
				}
				else
				{
					constexpr uint32 BitOffset = 0;
					ConditionalChangeMaskWriter->WriteBitStream(ConditionalChangeMaskStorage, BitOffset, BitCount);
				}
			}
			else
			{
				// Skip past our non-existing conditional mask.
				ConditionalChangeMaskWriter->Seek(ConditionalChangeMaskWriter->GetPosBits() + BitCount);
			}
		}
	}

	FReplicationStateHeader& ReplicationStateHeader = Private::GetReplicationStateHeader(StateBuffer, Descriptor);
	return FReplicationStateHeaderAccessor::GetIsStateDirty(ReplicationStateHeader);
}

static void ResetMemberChangeMasks(uint8* ExternalStateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	FNetBitArrayView::StorageWordType* ChangeMaskStorage = reinterpret_cast<FNetBitArrayView::StorageWordType*>(ExternalStateBuffer + Descriptor->GetChangeMaskOffset());
	const uint32 BitCount = Descriptor->ChangeMaskBitCount;

	// Reset the changemask. We assume it's fine to clear any padding bits.
	FNetBitArrayView MemberChangeMask(ChangeMaskStorage, BitCount, FNetBitArrayView::ResetOnInit);

	// Reset state dirtiness.
	FReplicationStateHeader& ReplicationStateHeader = Private::GetReplicationStateHeader(ExternalStateBuffer, Descriptor);
	Private::FReplicationStateHeaderAccessor::ClearAllStateIsDirty(ReplicationStateHeader);
}

}
