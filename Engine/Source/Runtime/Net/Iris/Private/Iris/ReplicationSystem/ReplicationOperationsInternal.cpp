// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationOperationsInternal.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"

#include "Net/Core/Trace/NetTrace.h"
#include "Net/Core/Trace/NetDebugName.h"

#include "Iris/ReplicationSystem/ChangeMaskCache.h"
#include "Iris/ReplicationSystem/ChangeMaskUtil.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Iris/ReplicationState/InternalReplicationStateDescriptorUtils.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/QuantizedObjectReference.h"
#include "Math/NumericLimits.h"
#include "HAL/IConsoleManager.h"
#include "Iris/Core/IrisProfiler.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Iris/Stats/NetStatsContext.h"

namespace UE::Net::Private
{

static bool bCVarForceFullCopyAndQuantize = false;
static FAutoConsoleVariableRef CVarForceFullCopyAndQuantize(
	TEXT("net.iris.ForceFullCopyAndQuantize"),
	bCVarForceFullCopyAndQuantize,
	TEXT("When enabled a full copy and quantize will be used, if disabled we will only copy and quantize dirty state data."));

void FReplicationInstanceOperationsInternal::BindInstanceProtocol(FNetHandle NetHandle, FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol)
{
	FReplicationInstanceProtocol::FFragmentData* FragmentData = InstanceProtocol->FragmentData;
	const FReplicationStateDescriptor** Descriptors = Protocol->ReplicationStateDescriptors;

	const uint32 FragmentCount = InstanceProtocol->FragmentCount;
	for (uint32 It = 0; It < FragmentCount; ++It)
	{
		if (FragmentData[It].ExternalSrcBuffer)
		{
			UE::Net::FReplicationStateHeader& ReplicationStateHeader = UE::Net::Private::GetReplicationStateHeader(FragmentData[It].ExternalSrcBuffer, Descriptors[It]);

			// Can't overwrite a bound header with a valid NetHandle.
			check(!ReplicationStateHeader.IsBound() || !NetHandle.IsValid());

			FReplicationStateHeaderAccessor::SetNetHandleId(ReplicationStateHeader, NetHandle);
		}
	}

	InstanceProtocol->InstanceTraits = NetHandle.IsValid() ? InstanceProtocol->InstanceTraits | EReplicationInstanceProtocolTraits::IsBound : InstanceProtocol->InstanceTraits &= ~EReplicationInstanceProtocolTraits::IsBound;
}

void FReplicationInstanceOperationsInternal::UnbindInstanceProtocol(FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol)
{ 
	if (EnumHasAnyFlags(InstanceProtocol->InstanceTraits, EReplicationInstanceProtocolTraits::IsBound))
	{
		BindInstanceProtocol(FNetHandle(), InstanceProtocol, Protocol);
	}
}

uint32 FReplicationInstanceOperationsInternal::QuantizeObjectStateData(FNetBitStreamWriter& ChangeMaskWriter, FChangeMaskCache& Cache, FNetRefHandleManager& NetRefHandleManager, FNetSerializationContext& SerializationContext, uint32 InternalIndex)
{
	if (NetRefHandleManager.IsScopableIndex(InternalIndex))
	{
		FNetRefHandleManager::FReplicatedObjectData& Object = NetRefHandleManager.GetReplicatedObjectDataNoCheck(InternalIndex);

		// We cannot quantize state data for zero sized objects or objects that no longer has an instance protocol.
		if (Object.InstanceProtocol && Object.Protocol->InternalTotalSize > 0U)
		{
			IRIS_PROFILER_PROTOCOL_NAME(Object.Protocol->DebugName->Name);
			UE_NET_IRIS_STATS_TIMER(Timer, SerializationContext.GetNetStatsContext());
			UE_NET_TRACE_QUANTIZE_OBJECT_SCOPE(Object.RefHandle, Timer);

			// if the object was scopable prev frame we can do partial copy
			bool bShouldPropagateChangedStates = Object.bShouldPropagateChangedStates;

			// We do a full CopyAndQuantize if the cvar bCVarForceFullCopyAndQuantize is set or that the object is marked as needing a FullCopyAndQuantize
			const bool bUseFullCopyAndQuantize = bCVarForceFullCopyAndQuantize || Object.bNeedsFullCopyAndQuantize;
			Object.bNeedsFullCopyAndQuantize = 0U;

			// Quantize dirty state
			{
				// Add entry to cache
				FChangeMaskCache::FCachedInfo& Info = Cache.AddChangeMaskForObject(InternalIndex, Object.Protocol->ChangeMaskBitCount);
				const uint32 ChangeMaskByteCount = FNetBitArrayView::CalculateRequiredWordCount(Object.Protocol->ChangeMaskBitCount) * 4;
				ChangeMaskStorageType* ChangeMaskData = Cache.GetChangeMaskStorage(Info);
				ChangeMaskWriter.InitBytes(ChangeMaskData, ChangeMaskByteCount);

				//UE_LOGF(LogIris, Log, "Copying state data for ( InternalIndex: %u ) with NetRefHandle (Id=%u)", InternalIndex, Object.RefHandle.GetId());
				if (bUseFullCopyAndQuantize)
				{
					FReplicationInstanceOperations::Quantize(SerializationContext, NetRefHandleManager.GetReplicatedObjectStateBufferNoCheck(InternalIndex), &ChangeMaskWriter, Object.InstanceProtocol, Object.Protocol);					
				}
				else
				{
					FReplicationInstanceOperations::QuantizeIfDirty(SerializationContext, NetRefHandleManager.GetReplicatedObjectStateBufferNoCheck(InternalIndex), &ChangeMaskWriter, Object.InstanceProtocol, Object.Protocol);
				}

				Info.bHasDirtyChangeMask = MakeNetBitArrayView(ChangeMaskData, ChangeMaskByteCount * 8U).FindFirstOne() != FNetBitArrayView::InvalidIndex;
			}

			// Mark subobject owner as dirty if this is a subobject
			if (const uint32 SubObjectOwnerIndex = Object.SubObjectRootIndex)
			{
				const bool bIsOwnerScopable = NetRefHandleManager.IsScopableIndex(SubObjectOwnerIndex);
				// Dependent objects should not ensure if the owner isn't scopable. Subobjects pending tear off is ok too.
				ensureMsgf(bIsOwnerScopable || Object.IsDependentObject() || Object.bTearOff, TEXT("SubObject ( InternaIndex: %u ) with NetRefHandle (Id=%" UINT64_FMT ") is trying to dirty parent ( InternalIndex: %u ) not in scope."), InternalIndex, Object.RefHandle.GetId(), SubObjectOwnerIndex);
				if (bIsOwnerScopable)
				{
					// Do we want to control this separately for subobjects? Or should they respect the setting on the owner?
					// For now, we do and will not mark owner as dirty if owner should not propagate statechanges
					bShouldPropagateChangedStates = bShouldPropagateChangedStates && NetRefHandleManager.GetReplicatedObjectDataNoCheck(SubObjectOwnerIndex).bShouldPropagateChangedStates;
					if (bShouldPropagateChangedStates)
					{
						//UE_LOGF(LogIris, Log, "Marking SubObjectOwner( InternalIndex: %u ) as dirty for ( InternalIndex: %u ) with NetRefHandle (Id=%u)", SubObjectOwnerIndex, InternalIndex, Object.RefHandle.GetId());
						Cache.AddSubObjectOwnerDirty(SubObjectOwnerIndex);
					}
				}
			}

			// If changed states should not be propagated, we must pop the last entry added to the cache
			if (!bShouldPropagateChangedStates)
			{
				Cache.PopLastEntry();
			}

			UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT(Timer, Quantize, InternalIndex);

			return 1U;
		}
		else if (const uint32 SubObjectOwnerIndex = Object.SubObjectRootIndex)
		{
			if (Object.bShouldPropagateChangedStates && ensure(NetRefHandleManager.IsScopableIndex(SubObjectOwnerIndex)))
			{	
				// Do we want to control this separately for subobjects? Or should they respect the setting on the owner?
				// For now, we do and will not mark owner as dirty if owner should not propagate statechanges
				if (NetRefHandleManager.GetReplicatedObjectDataNoCheck(SubObjectOwnerIndex).bShouldPropagateChangedStates)
				{
					//UE_LOGF(LogIris, Log, "Marking SubObjectOwner( InternalIndex: %u ) as dirty for ( InternalIndex: %u ) with NetRefHandle (Id=%u)", SubObjectOwnerIndex, InternalIndex, Object.Handle.GetIndex());
					Cache.AddSubObjectOwnerDirty(SubObjectOwnerIndex);
				}
			}			
		}
	}
	else
	{
		// Deal with error, object not found
		UE_LOGF(LogIris, Warning, "CopyObjectStateData called on object ( InternalIndex: %u ) not in scope", InternalIndex);
	}

	return 0U;
}

void FReplicationInstanceOperationsInternal::ResetObjectStateDirtiness(FNetRefHandleManager& NetRefHandleManager, uint32 InternalIndex)
{
	if (NetRefHandleManager.IsScopableIndex(InternalIndex))
	{
		const FNetRefHandleManager::FReplicatedObjectData& Object = NetRefHandleManager.GetReplicatedObjectDataNoCheck(InternalIndex);
		// Only instance protocols with state date can have dirtiness.
		if (Object.InstanceProtocol && Object.Protocol->InternalTotalSize > 0U)
		{
			FReplicationInstanceOperations::ResetDirtiness(Object.InstanceProtocol, Object.Protocol);
		}
	}
}

void FReplicationStateOperationsInternal::CloneDynamicState(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberTraitsDescriptor* MemberTraitsDescriptors = Descriptor->MemberTraitsDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberTraitsDescriptor& MemberTraitsDescriptor = MemberTraitsDescriptors[MemberIt];
		if (!EnumHasAnyFlags(MemberTraitsDescriptor.Traits, EReplicationStateMemberTraits::HasDynamicState))
		{
			continue;
		}

		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];

		FNetCloneDynamicStateArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = NetSerializerValuePointer(SrcInternalBuffer) + MemberDescriptor.InternalMemberOffset;
		MemberArgs.Target = NetSerializerValuePointer(DstInternalBuffer) + MemberDescriptor.InternalMemberOffset;

		Serializer->CloneDynamicState(Context, MemberArgs);
	}
}

void FReplicationStateOperationsInternal::CloneQuantizedState(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	// Memcopy state storage
	FMemory::Memcpy(DstInternalBuffer, SrcInternalBuffer, Descriptor->InternalSize);

	// If no member has dynamic state then there's nothing for us to do
	if (!EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState))
	{
		return;
	}

	// Clone dynamic state
	CloneDynamicState(Context, DstInternalBuffer, SrcInternalBuffer, Descriptor);
}

void FReplicationStateOperationsInternal::FreeDynamicState(FNetSerializationContext& Context, uint8* ObjectStateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberTraitsDescriptor* MemberTraitsDescriptors = Descriptor->MemberTraitsDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberTraitsDescriptor& MemberTraitsDescriptor = MemberTraitsDescriptors[MemberIt];
		if (!EnumHasAnyFlags(MemberTraitsDescriptor.Traits, EReplicationStateMemberTraits::HasDynamicState))
		{
			continue;
		}

		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];

		FNetFreeDynamicStateArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = NetSerializerValuePointer(ObjectStateBuffer) + MemberDescriptor.InternalMemberOffset;

		Serializer->FreeDynamicState(Context, MemberArgs);
	}
}

void FReplicationStateOperationsInternal::FreeDynamicState(uint8* ObjectStateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	FNetSerializationContext Context;
	FInternalNetSerializationContext InternalContext;
	Context.SetInternalContext(&InternalContext);

	FReplicationStateOperationsInternal::FreeDynamicState(Context, ObjectStateBuffer, Descriptor);
}

void FReplicationStateOperationsInternal::CollectReferences(FNetSerializationContext& Context, FNetReferenceCollector& Collector, const FNetSerializerChangeMaskParam& OuterChangeMaskInfo, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberReferenceDescriptor* ReferenceDescriptors = Descriptor->MemberReferenceDescriptors;
	
	for (const FReplicationStateMemberReferenceDescriptor& MemberReferenceDescriptor : MakeArrayView(ReferenceDescriptors, Descriptor->ObjectReferenceCount))
	{
		// Direct reference located in the buffer, we just need the offset to the NetObjectReference
		if (MemberReferenceDescriptor.Info.ResolveType != FNetReferenceInfo::EResolveType::Invalid)
		{
			const FQuantizedObjectReference& QuantizedReference = *reinterpret_cast<const FQuantizedObjectReference*>(SrcInternalBuffer + MemberReferenceDescriptor.Offset);
			
			// Only collect FNetObjectReferences, if it's a remote reference it doesn't need to be collected
			Collector.Add(MemberReferenceDescriptor.Info, QuantizedReference, OuterChangeMaskInfo);
		}
		else
		{
			// For dynamic arrays and serializers with custom references we need to find the actual MemberDescriptor so that we can invoke the CollectNetRefrences function
			const FReplicationStateMemberReferenceDescriptor* CurrentReferenceDescriptor = &MemberReferenceDescriptor;
			const FReplicationStateDescriptor* CurrentDescriptor = Descriptor;
			const uint8* RESTRICT CurrentInternalBuffer = SrcInternalBuffer;

			for (;;)
			{
				const uint32 MemberIndex = CurrentReferenceDescriptor->MemberIndex;
				const FReplicationStateMemberSerializerDescriptor* SerializerInfo = &CurrentDescriptor->MemberSerializerDescriptors[MemberIndex];
				const FReplicationStateMemberDescriptor& MemberDescriptor = CurrentDescriptor->MemberDescriptors[MemberIndex];	
				const uint16 InnerReferenceIndex = CurrentReferenceDescriptor->InnerReferenceIndex;

				const bool bIsMemberWithCustomReferences = InnerReferenceIndex == MAX_uint16;
				if (bIsMemberWithCustomReferences)
				{
					// Verify that this is a dynamic array, or a serializer with custom references otherwise something is seriously broken
					checkSlow(IsUsingArrayPropertyNetSerializer(*SerializerInfo) || EnumHasAnyFlags(SerializerInfo->Serializer->Traits, ENetSerializerTraits::HasCustomNetReference));

					FNetCollectReferencesArgs Args = {};
					Args.Source = reinterpret_cast<NetSerializerValuePointer>(CurrentInternalBuffer + MemberDescriptor.InternalMemberOffset);
					Args.NetSerializerConfig = SerializerInfo->SerializerConfig;
					Args.ChangeMaskInfo = OuterChangeMaskInfo;
					Args.Collector = reinterpret_cast<NetSerializerValuePointer>(&Collector);
					Args.Version = 0U;
					
					SerializerInfo->Serializer->CollectNetReferences(Context, Args);
					break;
				}

				// The reference was to a nested inner reference
				const FStructNetSerializerConfig* StructConfig = static_cast<const FStructNetSerializerConfig*>(SerializerInfo->SerializerConfig);
				CurrentDescriptor = StructConfig->StateDescriptor;
				CurrentReferenceDescriptor = &StructConfig->StateDescriptor->MemberReferenceDescriptors[InnerReferenceIndex];
				CurrentInternalBuffer = CurrentInternalBuffer + MemberDescriptor.InternalMemberOffset;
			}
		}
	}
}

void FReplicationStateOperationsInternal::CollectReferencesWithMask(FNetSerializationContext& Context, FNetReferenceCollector& Collector, const uint32 ChangeMaskOffset, const uint8* RESTRICT SrcInternalBuffer,  const FReplicationStateDescriptor* Descriptor)
{
	const FReplicationStateMemberReferenceDescriptor* ReferenceDescriptors = Descriptor->MemberReferenceDescriptors;
	const FReplicationStateMemberChangeMaskDescriptor* ChangeMaskDescriptors = Descriptor->MemberChangeMaskDescriptors;
	const FNetBitArrayView* ChangeMask = Context.GetChangeMask();

	for (const FReplicationStateMemberReferenceDescriptor& MemberReferenceDescriptor : MakeArrayView(ReferenceDescriptors, Descriptor->ObjectReferenceCount))
	{
		checkSlow((ChangeMaskOffset + ChangeMaskDescriptors[MemberReferenceDescriptor.MemberIndex].BitOffset) < MAX_uint16);

		const FNetSerializerChangeMaskParam LocalChangeMaskInfo = FNetSerializerChangeMaskParam( { (uint16)(ChangeMaskOffset + ChangeMaskDescriptors[MemberReferenceDescriptor.MemberIndex].BitOffset), ChangeMaskDescriptors[MemberReferenceDescriptor.MemberIndex].BitCount } );
		if (ChangeMask && !ChangeMask->IsAnyBitSet(LocalChangeMaskInfo.BitOffset, LocalChangeMaskInfo.BitCount))
		{
			continue;
		}
	
		// Direct reference located in the buffer, we just need the offset to the NetObjectReference
		if (MemberReferenceDescriptor.Info.ResolveType != FNetReferenceInfo::EResolveType::Invalid)
		{
			const FQuantizedObjectReference& QuantizedReference = *reinterpret_cast<const FQuantizedObjectReference*>(SrcInternalBuffer + MemberReferenceDescriptor.Offset);

			Collector.Add(MemberReferenceDescriptor.Info, QuantizedReference, LocalChangeMaskInfo);
		}
		else
		{
			// For dynamic arrays and serializers with custom references we need to find the actual MemberDescriptor so that we can invoke the CollectNetRefrences function
			const FReplicationStateMemberReferenceDescriptor* CurrentReferenceDescriptor = &MemberReferenceDescriptor;
			const FReplicationStateDescriptor* CurrentDescriptor = Descriptor;
			const uint8* RESTRICT CurrentInternalBuffer = SrcInternalBuffer;

			for (;;)
			{
				const uint32 MemberIndex = CurrentReferenceDescriptor->MemberIndex;
				const FReplicationStateMemberSerializerDescriptor* SerializerInfo = &CurrentDescriptor->MemberSerializerDescriptors[MemberIndex];
				const FReplicationStateMemberDescriptor& MemberDescriptor = CurrentDescriptor->MemberDescriptors[MemberIndex];	
				const uint16 InnerReferenceIndex = CurrentReferenceDescriptor->InnerReferenceIndex;

				const bool bIsMemberWithCustomReferences = InnerReferenceIndex == MAX_uint16;
				if (bIsMemberWithCustomReferences)
				{
					// Verify that this is a dynamic array, or a serializer with custom references otherwise something is seriously broken
					checkSlow(IsUsingArrayPropertyNetSerializer(*SerializerInfo) || EnumHasAnyFlags(SerializerInfo->Serializer->Traits, ENetSerializerTraits::HasCustomNetReference));

					FNetCollectReferencesArgs Args = {};
					Args.Source = reinterpret_cast<NetSerializerValuePointer>(CurrentInternalBuffer + MemberDescriptor.InternalMemberOffset);
					Args.NetSerializerConfig = SerializerInfo->SerializerConfig;
					Args.ChangeMaskInfo = LocalChangeMaskInfo;
					Args.Collector = reinterpret_cast<NetSerializerValuePointer>(&Collector);
					Args.Version = 0U;
					
					SerializerInfo->Serializer->CollectNetReferences(Context, Args);
					break;
				}

				// The reference was to a nested inner reference
				const FStructNetSerializerConfig* StructConfig = static_cast<const FStructNetSerializerConfig*>(SerializerInfo->SerializerConfig);
				CurrentDescriptor = StructConfig->StateDescriptor;
				CurrentReferenceDescriptor = &StructConfig->StateDescriptor->MemberReferenceDescriptors[InnerReferenceIndex];
				CurrentInternalBuffer = CurrentInternalBuffer + MemberDescriptor.InternalMemberOffset;
			}
		}
	}
}

void FReplicationProtocolOperationsInternal::CloneDynamicState(FNetSerializationContext& Context, uint8* RESTRICT DstObjectStateBuffer, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	uint8* CurrentInternalDstStateBuffer = DstObjectStateBuffer;
	const uint8* CurrentInternalSrcStateBuffer = SrcObjectStateBuffer;

	for (uint32 StateIt = 0, StateEndIt = Protocol->ReplicationStateCount; StateIt < StateEndIt; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalDstStateBuffer = Align(CurrentInternalDstStateBuffer, CurrentDescriptor->InternalAlignment);
		CurrentInternalSrcStateBuffer = Align(CurrentInternalSrcStateBuffer, CurrentDescriptor->InternalAlignment);

		if (EnumHasAnyFlags(CurrentDescriptor->Traits, EReplicationStateTraits::HasDynamicState))
		{
			FReplicationStateOperationsInternal::CloneDynamicState(Context, CurrentInternalDstStateBuffer, CurrentInternalSrcStateBuffer, CurrentDescriptor);
		}

		CurrentInternalDstStateBuffer += CurrentDescriptor->InternalSize;
		CurrentInternalSrcStateBuffer += CurrentDescriptor->InternalSize;
	}
}

void FReplicationProtocolOperationsInternal::FreeDynamicState(FNetSerializationContext& Context, uint8* RESTRICT ObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	const FReplicationStateDescriptor** ReplicationStateDescriptors = Protocol->ReplicationStateDescriptors;
	uint8* CurrentInternalStateBuffer = ObjectStateBuffer;

	for (uint32 StateIt = 0, StateEndIt = Protocol->ReplicationStateCount; StateIt < StateEndIt; ++StateIt)
	{
		const FReplicationStateDescriptor* CurrentDescriptor = ReplicationStateDescriptors[StateIt];

		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);
		
		if (EnumHasAnyFlags(CurrentDescriptor->Traits, EReplicationStateTraits::HasDynamicState))
		{
			FReplicationStateOperationsInternal::FreeDynamicState(Context, CurrentInternalStateBuffer, CurrentDescriptor);
		}
		
		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
	}
}

void FReplicationProtocolOperationsInternal::CollectReferences(FNetSerializationContext& Context, FNetReferenceCollector& Collector, const uint8* RESTRICT SrcInternalStateBuffer, const FReplicationProtocol* Protocol)
{
	const FNetSerializerChangeMaskParam InitStateChangeMaskInfo = {};
	const bool bIncludeInitStates = Context.IsInitState();
	const uint8* CurrentInternalStateBuffer = SrcInternalStateBuffer;
	uint32 CurrentChangeMaskBitOffset = 0;

	for (const FReplicationStateDescriptor* CurrentDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
	{
		CurrentInternalStateBuffer = Align(CurrentInternalStateBuffer, CurrentDescriptor->InternalAlignment);
		
		if (CurrentDescriptor->HasObjectReference())
		{
			if (!CurrentDescriptor->IsInitState())
			{
				FReplicationStateOperationsInternal::CollectReferencesWithMask(Context, Collector, CurrentChangeMaskBitOffset, CurrentInternalStateBuffer, CurrentDescriptor);
			}
			else if (bIncludeInitStates)
			{
				FReplicationStateOperationsInternal::CollectReferences(Context, Collector, InitStateChangeMaskInfo, CurrentInternalStateBuffer, CurrentDescriptor);
			}
		}
		
		CurrentInternalStateBuffer += CurrentDescriptor->InternalSize;
		CurrentChangeMaskBitOffset += CurrentDescriptor->ChangeMaskBitCount;
	}
}

void FReplicationProtocolOperationsInternal::CloneQuantizedState(FNetSerializationContext& Context, uint8* RESTRICT DstObjectStateBuffer, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol)
{
	FMemory::Memcpy(DstObjectStateBuffer, SrcObjectStateBuffer, Protocol->InternalTotalSize);
	if (EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasDynamicState))
	{
		FReplicationProtocolOperationsInternal::CloneDynamicState(Context, DstObjectStateBuffer, SrcObjectStateBuffer, Protocol);
		check(!Context.HasError());
	}
}

bool FReplicationProtocolOperationsInternal::IsEqualQuantizedState(FNetSerializationContext& Context, const uint8* RESTRICT State0, const uint8* RESTRICT State1, const FReplicationProtocol* Protocol)
{
	const bool bIncludeInitStates = Context.IsInitState();
	const uint8* CurrentState0 = State0;
	const uint8* CurrentState1 = State1;

	for (const FReplicationStateDescriptor* CurrentDescriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
	{
		CurrentState0 = Align(CurrentState0, CurrentDescriptor->InternalAlignment);
		CurrentState1 = Align(CurrentState1, CurrentDescriptor->InternalAlignment);
		
		if (!CurrentDescriptor->IsInitState() || bIncludeInitStates)
		{
			const bool bStateIsEqual = FReplicationStateOperations::IsEqualQuantizedState(Context, CurrentState0, CurrentState1, CurrentDescriptor);
			if (!bStateIsEqual)
			{
				return false;
			}
		}

		CurrentState0 += CurrentDescriptor->InternalSize;
		CurrentState1 += CurrentDescriptor->InternalSize;
	}

	return true;
}

}
