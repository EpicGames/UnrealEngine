// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Iris/Core/NetObjectReference.h"
#include "Net/Core/NetHandle/NetHandle.h"


namespace UE::Net 
{
	class FNetBitStreamWriter;
	class FNetSerializationContext;
	struct FNetSerializerChangeMaskParam;
	struct FReplicationStateDescriptor;
	struct FReplicationInstanceProtocol;
	struct FReplicationProtocol;
	class FNetReferenceCollector;

	namespace Private
	{
		struct FChangeMaskCache;
		class FNetRefHandleManager;
	}
}

namespace UE::Net::Private
{

struct FReplicationInstanceOperationsInternal
{
	/** A call to this function will inject the index of the handle into the external statebuffer contained in the Instance Protocol */
	static IRISCORE_API void BindInstanceProtocol(FNetHandle NetHandle, FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol);

	/** A call to this function will clear the index of the handle into the external statebuffer contained in the Instance Protocol */
	static IRISCORE_API void UnbindInstanceProtocol(FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol);

	/** Quantize state data for a single instance */
	static IRISCORE_API uint32 QuantizeObjectStateData(FNetBitStreamWriter& ChangeMaskWriter, FChangeMaskCache& Cache, FNetRefHandleManager& NetRefHandleManager, FNetSerializationContext& SerializationContext, uint32 InternalIndex);

	/** Reset instance state dirtiness. */
	static IRISCORE_API void ResetObjectStateDirtiness(FNetRefHandleManager& NetRefHandleManager, uint32 InternalIndex);
};

struct FReplicationStateOperationsInternal
{
	/** Clone the dynamic state from source to destination state for a single replication state */
	static IRISCORE_API void CloneDynamicState(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Free the dynamic state from a state for a single replication state */
	static IRISCORE_API void FreeDynamicState(FNetSerializationContext& Context, uint8* ObjectStateBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Free the dynamic state from a state for a single replication state */
	static IRISCORE_API void FreeDynamicState(uint8* ObjectStateBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Clone a quantized state, Note: DstInternalBuffer is expected to be uninitialized */
	static IRISCORE_API void CloneQuantizedState(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor);
	
	/** Collect references from a state that does not have any changemask information */
	static IRISCORE_API void CollectReferences(FNetSerializationContext& Context, FNetReferenceCollector& Collector, const FNetSerializerChangeMaskParam& OuterChangeMaskInfo, const uint8* RESTRICT SrcInternalBuffer,  const FReplicationStateDescriptor* Descriptor);

	/** Collect references from a state based on the provided changemask information */
	static IRISCORE_API void CollectReferencesWithMask(FNetSerializationContext& Context, FNetReferenceCollector& Collector, const uint32 ChangeMaskOffset, const uint8* RESTRICT SrcInternalBuffer,  const FReplicationStateDescriptor* Descriptor);
};

struct FReplicationProtocolOperationsInternal
{
	/** Clone the dynamic state from source to destination state for a full NetObject */
	static IRISCORE_API void CloneDynamicState(FNetSerializationContext& Context, uint8* RESTRICT DstObjectStateBuffer, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Clone a quantized state, Note: DstObjectStateBuffer is expected to be uninitialized */
	static IRISCORE_API void CloneQuantizedState(FNetSerializationContext& Context, uint8* RESTRICT DstObjectStateBuffer, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Free the dynamic state from a state for a full NetObject */
	static IRISCORE_API void FreeDynamicState(FNetSerializationContext& Context, uint8* RESTRICT ObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Collect references from protocol, if changemask is availabe in the Context it will be used */
	static IRISCORE_API void CollectReferences(FNetSerializationContext& Context, FNetReferenceCollector& Collector, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Compare two quantized states and return false if they are different. */
	static IRISCORE_API bool IsEqualQuantizedState(FNetSerializationContext& Context, const uint8* RESTRICT State0, const uint8* RESTRICT State1, const FReplicationProtocol* Protocol);
};

}
