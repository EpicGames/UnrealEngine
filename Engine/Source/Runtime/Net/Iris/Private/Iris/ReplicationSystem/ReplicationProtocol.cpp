// Copyright Epic Games, Inc. All Rights Reserved.
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "HAL/PlatformAtomics.h"

namespace UE::Net
{

void FReplicationProtocol::AddRef() const
{
	++RefCount;
}

void FReplicationProtocol::Release() const
{
	--RefCount;
}

}
