// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"
#include "Misc/EnumClassFlags.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"

namespace UE::Net
{
	struct FNetDebugName;
	typedef uint32 FReplicationProtocolIdentifier;
	class FReplicationFragment;
	struct FReplicationStateDescriptor;
}

namespace UE::Net
{

enum class EReplicationInstanceProtocolTraits : uint16
{
	None = 0,
	NeedsPoll					= 1,
	NeedsLegacyCallbacks		= NeedsPoll << 1,
	IsBound						= NeedsLegacyCallbacks << 1,
	NeedsPreSendUpdate			= IsBound << 1,
	HasPushBasedDirtiness		= NeedsPreSendUpdate << 1,
	HasFullPushBasedDirtiness	= HasPushBasedDirtiness << 1,
	// Whether this instance contains fragments from multiple owners
	IsMultiObjectInstance	 = HasFullPushBasedDirtiness << 1,
	// Whether there's a state that has object references
	HasObjectReference = IsMultiObjectInstance << 1,
};
ENUM_CLASS_FLAGS(EReplicationInstanceProtocolTraits);

/** 
 * The ReplicationInstanceProtocol stores everything we need to know in order to interact with data outside the replication system
 * This information is only used when copying state data to replication system or when pushing state data from the replication system
 */ 
struct FReplicationInstanceProtocol
{
	// Cached information, A fragment can register itself multiple times with different ExternalSrcBuffers to support multi state fragments
	struct FFragmentData
	{
		uint8* ExternalSrcBuffer;
		EReplicationFragmentTraits Traits;
	};
	
	FFragmentData* FragmentData;
	FReplicationFragment* const * Fragments;
	uint16 FragmentCount;
	EReplicationInstanceProtocolTraits InstanceTraits;
};

enum class EReplicationProtocolTraits : uint16
{
	None = 0,
	// Whether any of the replication states has dynamic state
	HasDynamicState = 1U << 0U,
	// Whether there's a state with legacy lifetime conditionals
	HasLifetimeConditionals = HasDynamicState << 1U,
	// Whether there's a changemask for conditionals stored in the internal state, such as when there are custom conditionals
	HasConditionalChangeMask = HasLifetimeConditionals << 1U,
	// Whether there's a state that has connection specific serialization
	HasConnectionSpecificSerialization = HasConditionalChangeMask << 1U,
	// Whether there's a state that has object references
	HasObjectReference = HasConnectionSpecificSerialization << 1U,
	// Whether delta compression is supported or not, essentially whether it makes sense to create baselines or not.
	SupportsDeltaCompression = HasObjectReference << 1U,
	// If there are some states in the protocol that partially uses pushbased dirtiness
	HasPushBasedDirtiness = SupportsDeltaCompression << 1U,
	// If all states uses full pushbased dirtiness
	HasFullPushBasedDirtiness	 = HasPushBasedDirtiness << 1U,
};
ENUM_CLASS_FLAGS(EReplicationProtocolTraits);

/**
 * The Replication protocols contains everything required to express the state of a replicated object.
 * This is shared for every instance of the same type. This is used for all internal operations on state data.
 */
struct FReplicationProtocol
{
	struct FRepIndexToFragmentIndex
	{
		enum : uint16
		{
			InvalidEntry = 65535U,
		};
		
		uint16 FragmentIndex = InvalidEntry;
	};
	struct FRepIndexToFragmentIndexTable
	{
		const FRepIndexToFragmentIndex* RepIndexToFragmentIndex;
		uint32 NumEntries;
	};

	uint32 GetConditionalChangeMaskOffset() const { return InternalChangeMasksOffset; }

	// RefCounting used to track usage of the replication protocol
	IRISCORE_API void AddRef() const;
	IRISCORE_API void Release() const;
	int32 GetRefCount() const { return RefCount; }

	const FReplicationStateDescriptor** ReplicationStateDescriptors;
	uint32 ReplicationStateCount;		// Number of states
	uint32 InternalTotalSize;			// Total memory required to store the complete state
	uint32 InternalTotalAlignment;		// Alignment of the internal state
	uint32 MaxExternalStateSize;		// Max external state size, required when we push temporary states to game
	uint32 MaxExternalStateAlignment;	// Max external state alignment, required when allocating temporary state buffer

	// These two members are only valid if traits include HasConditionalChangeMask. The target state has the HasLifetimeConditionals trait.
	uint16 FirstLifetimeConditionalsStateIndex;
	uint16 LifetimeConditionalsStateCount;
	uint32 FirstLifetimeConditionalsChangeMaskOffset;

	uint32 ChangeMaskBitCount;			// How many bits do we need to store the full changemask
	uint32 InternalChangeMasksOffset;

	FReplicationProtocolIdentifier ProtocolIdentifier;
	const FNetDebugName* DebugName;

	// TypeStatsIndex assigned to this protocol
	int32 TypeStatsIndex;

	// TODO: Cache parts of the descriptors in the protocol to avoid having to pull in the descriptor itself for operations iterating over the protocol

	// Selected traits from the ReplicationStateDescriptors that might be handy to cache in the protocol.
	EReplicationProtocolTraits ProtocolTraits;

	// Keep a table of repindex -> fragment for each owner with pushbased data.
	const FRepIndexToFragmentIndexTable* PushModelOwnerRepIndexToFragmentIndexTable;
	// Owner count
	uint16 PushModelOwnerCount;

	mutable int32 RefCount;
};

}
