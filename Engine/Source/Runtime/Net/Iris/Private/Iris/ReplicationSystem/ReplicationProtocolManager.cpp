// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ReplicationProtocolManager.h"
#include "CoreTypes.h"
#include "Iris/IrisConfigInternal.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/MemoryLayoutUtil.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Hash/CityHash.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformString.h"
#include "Containers/StringFwd.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Iris/Stats/NetStats.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"

#ifndef UE_NET_ENABLE_PROTOCOLMANAGER_LOG
// Default is to enable protocol logs in non-shipping
#	define UE_NET_ENABLE_PROTOCOLMANAGER_LOG !(UE_BUILD_SHIPPING)
#endif

#if UE_NET_ENABLE_PROTOCOLMANAGER_LOG
#	define UE_LOG_PROTOCOLMANAGER(Log, Format, ...)  UE_LOG(LogIris, Log, Format, ##__VA_ARGS__)
#else
#	define UE_LOG_PROTOCOLMANAGER(...)
#endif

#define UE_LOG_PROTOCOLMANAGER_WARNING(Format, ...)  UE_LOG(LogIris, Warning, Format, ##__VA_ARGS__)

namespace UE::Net::Private
{

static bool bIrisLogReplicationProtocols = false;
static FAutoConsoleVariableRef CVarIrisLogReplicationProtocols(
	TEXT("net.Iris.LogReplicationProtocols"),
	bIrisLogReplicationProtocols,
	TEXT("If true, log all created replication protocols."
	));

static void GetInstanceTraits(const FReplicationFragments& Fragments, EReplicationFragmentTraits& OutAccumulatedTraits, EReplicationFragmentTraits& OutSharedTraits, const EReplicationFragmentTraits InObjectTraits)
{
	EReplicationFragmentTraits AccumulatedTraits = InObjectTraits;
	EReplicationFragmentTraits SharedTraits = Fragments.Num() > 0 ? ~EReplicationFragmentTraits::None : EReplicationFragmentTraits::None;
	for (const FReplicationFragmentInfo& Info : Fragments)
	{
		const EReplicationFragmentTraits FragmentTraits = Info.Fragment->GetTraits();
		AccumulatedTraits |= FragmentTraits;
		SharedTraits &= FragmentTraits;
	}

	OutAccumulatedTraits = AccumulatedTraits;
	OutSharedTraits = SharedTraits;
}

FReplicationInstanceProtocol* FReplicationProtocolManager::CreateInstanceProtocol(const FReplicationFragments& Fragments, UE::Net::EReplicationFragmentTraits ObjectTraits)
{
	const uint32 FragmentCount = Fragments.Num();
	if (!ensure(FragmentCount < 65536))
	{
		return nullptr;
	}

	// We want to keep this as a single allocation so we first build a layout and allocate enough space for the InstanceProtocol and its data
	struct FReplicationInstanceProtocolLayoutData
	{
		FMemoryLayoutUtil::FOffsetAndSize ReplicationInstanceProtocolSizeAndOffset;
		FMemoryLayoutUtil::FOffsetAndSize FragmentDataSizeAndOffset;
		FMemoryLayoutUtil::FOffsetAndSize FragmentsSizeAndOffset;
	};
	FReplicationInstanceProtocolLayoutData LayoutData;

	FMemoryLayoutUtil::FLayout Layout;
	FMemoryLayoutUtil::AddToLayout<FReplicationInstanceProtocol>(Layout, LayoutData.ReplicationInstanceProtocolSizeAndOffset, 1);
	FMemoryLayoutUtil::AddToLayout<FReplicationInstanceProtocol::FFragmentData>(Layout, LayoutData.FragmentDataSizeAndOffset, FragmentCount);
	FMemoryLayoutUtil::AddToLayout<FReplicationFragment*>(Layout, LayoutData.FragmentsSizeAndOffset, FragmentCount);

	// Allocate memory for the instance protocol
	uint8* Buffer = static_cast<uint8*>(FMemory::MallocZeroed(Layout.CurrentOffset, static_cast<uint32>(Layout.MaxAlignment)));

	// Init FReplicationInstanceProtocol
	FReplicationInstanceProtocol* InstanceProtocol = new (Buffer) FReplicationInstanceProtocol;
	InstanceProtocol->FragmentData = reinterpret_cast<FReplicationInstanceProtocol::FFragmentData*>(Buffer + LayoutData.FragmentDataSizeAndOffset.Offset);
	InstanceProtocol->Fragments = reinterpret_cast<FReplicationFragment* const *>(Buffer + LayoutData.FragmentsSizeAndOffset.Offset);
	InstanceProtocol->FragmentCount = static_cast<uint16>(FragmentCount);

	// Setup owner collector
	constexpr uint32 MaxFragmentOwnerCount = 2U;
	UObject* FragmentOwnersForPushBasedFragments[MaxFragmentOwnerCount] = {};
	FReplicationStateOwnerCollector UniquePushBasedOwners(FragmentOwnersForPushBasedFragments, MaxFragmentOwnerCount);

	// Fill in fragment data and fragment pointer
	uint32 FragmentIt = 0U;

	for (const FReplicationFragmentInfo& Info : Fragments)
	{
		InstanceProtocol->FragmentData[FragmentIt].ExternalSrcBuffer = reinterpret_cast<uint8*>(Info.SrcReplicationStateBuffer);
		InstanceProtocol->FragmentData[FragmentIt].Traits = Info.Fragment->GetTraits();
		const_cast<FReplicationFragment**>(InstanceProtocol->Fragments)[FragmentIt] = Info.Fragment;

		// We collect unique owners with properties that are pushbased
		if (Info.Descriptor->MemberCount > 0 && EnumHasAnyFlags(Info.Fragment->GetTraits(), EReplicationFragmentTraits::HasPushBasedDirtiness))
		{
			Info.Fragment->CollectOwner(&UniquePushBasedOwners);		
		}
		++FragmentIt;
	}

	// Get accumulated and shared traits
	EReplicationFragmentTraits AccumulatedTraits;
	EReplicationFragmentTraits SharedTraits;
	GetInstanceTraits(Fragments, AccumulatedTraits, SharedTraits, ObjectTraits);

	// Init instance traits
	EReplicationInstanceProtocolTraits InstanceTraits = EReplicationInstanceProtocolTraits::None;
	if (EnumHasAnyFlags(AccumulatedTraits, EReplicationFragmentTraits::NeedsPoll))
	{
		InstanceTraits |= EReplicationInstanceProtocolTraits::NeedsPoll;
	}
	if (EnumHasAnyFlags(AccumulatedTraits, EReplicationFragmentTraits::NeedsLegacyCallbacks))
	{
		InstanceTraits |= EReplicationInstanceProtocolTraits::NeedsLegacyCallbacks;
	}
	if (EnumHasAnyFlags(AccumulatedTraits, EReplicationFragmentTraits::NeedsPreSendUpdate))
	{
		InstanceTraits |= EReplicationInstanceProtocolTraits::NeedsPreSendUpdate;
	}
	if (EnumHasAnyFlags(AccumulatedTraits, EReplicationFragmentTraits::HasObjectReference))
	{
		InstanceTraits |= EReplicationInstanceProtocolTraits::HasObjectReference;
	}
	// Currently we do not support push based dirtiness for protocols with multiple pushbased fragments with different owners.
	if (UniquePushBasedOwners.GetOwnerCount() > 1U)
	{
		InstanceTraits |= EReplicationInstanceProtocolTraits::IsMultiObjectInstance;
		UE_LOGF(LogIris, Warning, "Currently we do not support pushbased dirtiness for multiowner instances");
	}
	else if (EnumHasAnyFlags(AccumulatedTraits, EReplicationFragmentTraits::HasPushBasedDirtiness))
	{
		InstanceTraits |= EReplicationInstanceProtocolTraits::HasPushBasedDirtiness;
		if (EnumHasAnyFlags(SharedTraits, EReplicationFragmentTraits::HasFullPushBasedDirtiness))
		{
			InstanceTraits |= EReplicationInstanceProtocolTraits::HasFullPushBasedDirtiness;
		}
	}

	InstanceProtocol->InstanceTraits = InstanceTraits;

	return InstanceProtocol;
}

void FReplicationProtocolManager::DestroyInstanceProtocol(FReplicationInstanceProtocol* InstanceProtocol)
{
	FReplicationFragment* const * Fragments = InstanceProtocol->Fragments;

	// Destroy owned fragments
	// $IRIS, Cache the flag to avoid touching all Fragments
	const uint32 FragmentCount = InstanceProtocol->FragmentCount;
	for (uint32 StateIt = 0; StateIt < FragmentCount; ++StateIt)
	{
		if (EnumHasAnyFlags(Fragments[StateIt]->GetTraits(), EReplicationFragmentTraits::DeleteWithInstanceProtocol))
		{
			delete Fragments[StateIt];
		}
	}
	FMemory::Free(InstanceProtocol);
}

FReplicationProtocolIdentifier FReplicationProtocolManager::CalculateProtocolIdentifier(const FReplicationFragments& Fragments)
{
	TArray<uint64, TInlineAllocator<32>> IdBuffer;
	for (const FReplicationFragmentInfo& Info : Fragments)
	{
		IdBuffer.Add(Info.Descriptor->DescriptorIdentifier.Value);
		// We currently include the default state hash when building the protocol id in order to verify the integrity of default states
		IdBuffer.Add(Info.Descriptor->DescriptorIdentifier.DefaultStateHash);
	}

	FReplicationProtocolIdentifier ProtocolIdentifier;
	ProtocolIdentifier = CityHash32(reinterpret_cast<const char*>(IdBuffer.GetData()), sizeof(uint64) * IdBuffer.Num());

	return ProtocolIdentifier;
}

bool FReplicationProtocolManager::ValidateReplicationProtocol(const FReplicationProtocol* Protocol, const FReplicationFragments& Fragments, bool bLogFragmentErrors)
{
	bool bResult = true;

	const uint32 FragmentCount = Fragments.Num();
	if (ensureMsgf(Protocol->ReplicationStateCount == FragmentCount, TEXT("Protocol %s:ReplicationStateCount %u != FragmentCount:%u"), ToCStr(Protocol->DebugName), Protocol->ReplicationStateCount, FragmentCount))
	{
		// Validate individual fragments
		const FReplicationFragmentInfo* FragmentsData = Fragments.GetData();
		for (uint32 It = 0; It < FragmentCount; ++It)
		{
			const FReplicationFragmentInfo& Info = FragmentsData[It];
			const bool bIsSameDescriptor = Info.Descriptor == Protocol->ReplicationStateDescriptors[It];
	
			if (!bIsSameDescriptor)
			{
				UE_LOG_PROTOCOLMANAGER_WARNING(
					TEXT("FReplicationProtocolManager::ValidateReplicationProtocol for %s Descriptor Pointer mismatch %p != %p for index %u/%u named %s != %s identifier 0x%" UINT64_x_FMT " != 0x%" UINT64_x_FMT),
					ToCStr(Protocol->DebugName), Info.Descriptor, Protocol->ReplicationStateDescriptors[It], It, FragmentCount, ToCStr(Info.Descriptor->DebugName), ToCStr(Protocol->ReplicationStateDescriptors[It]->DebugName), 
					Info.Descriptor->DescriptorIdentifier.Value, Protocol->ReplicationStateDescriptors[It]->DescriptorIdentifier.Value);

				ensure(bIsSameDescriptor);
				
				bResult = false;
			}
			else if (Info.Descriptor->DescriptorIdentifier != Protocol->ReplicationStateDescriptors[It]->DescriptorIdentifier)
			{
				
				UE_LOG_PROTOCOLMANAGER_WARNING(
					TEXT("FReplicationProtocolManager::ValidateReplicationProtocol for %s DescriptorIdentfier mismatch for index %u/%u named %s != %s identifier 0x%" UINT64_x_FMT " != 0x%" UINT64_x_FMT),
					ToCStr(Protocol->DebugName), It, FragmentCount, ToCStr(Info.Descriptor->DebugName), ToCStr(Protocol->ReplicationStateDescriptors[It]->DebugName), Info.Descriptor->DescriptorIdentifier.Value, Protocol->ReplicationStateDescriptors[It]->DescriptorIdentifier.Value);

				ensure(Info.Descriptor->DescriptorIdentifier == Protocol->ReplicationStateDescriptors[It]->DescriptorIdentifier);
				
				bResult = false;
			}
		}
	}

	return bResult;
}

const FReplicationProtocol* FReplicationProtocolManager::GetReplicationProtocol(FReplicationProtocolIdentifier ProtocolId, FObjectKey TemplateKey)
{
	for (auto It = RegisteredProtocols.CreateConstKeyIterator(ProtocolId); It; ++It)
	{
		const FRegisteredProtocolInfo& Info = It.Value();
		if (Info.TemplateKey == TemplateKey)
		{
			return Info.Protocol;
		}
	}
	return nullptr;
}

void FReplicationProtocolManager::FragmentListToString(FStringBuilderBase& StringBuilder, const FReplicationFragments& Fragments)
{
	StringBuilder << TEXT("Fragments:\n");
	const FReplicationFragmentInfo* FragmentsData = Fragments.GetData();
	const uint32 FragmentCount = Fragments.Num();
	for (uint32 It = 0; It < FragmentCount; ++It)
	{
		const FReplicationFragmentInfo& Info = FragmentsData[It];
		StringBuilder.Appendf(TEXT("index %u/%u named %s identifier (0x%" UINT64_x_FMT ", 0x%" UINT64_x_FMT ") Pointer: %p\n"),
			It, FragmentCount, ToCStr(Info.Descriptor->DebugName), Info.Descriptor->DescriptorIdentifier.Value, Info.Descriptor->DescriptorIdentifier.DefaultStateHash, Info.Descriptor);
	}
}

const FReplicationProtocol* FReplicationProtocolManager::CreateReplicationProtocol(const FReplicationProtocolIdentifier ProtocolId, const FReplicationFragments& Fragments, const TCHAR* DebugName, const FCreateReplicationProtocolParameters& Params)
{
	if (Params.bValidateProtocolId)
	{
		const FReplicationProtocolIdentifier NewProtocolId = CalculateProtocolIdentifier(Fragments);
		if (NewProtocolId != ProtocolId)
		{
			UE_LOGF(LogIris, Warning, "FReplicationProtocolManager::CreateReplicationProtocol Id mismatch when creating protocol named %ls with in ProtocolId:0x%x Calculated ProtocolId:0x%x", DebugName, ProtocolId, NewProtocolId);
 #if UE_NET_ENABLE_PROTOCOLMANAGER_LOG
 			if (UE_LOG_ACTIVE(LogIris, Warning))
 			{
 				TStringBuilder<4096> StringBuilder;
 				FragmentListToString(StringBuilder, Fragments);

 				UE_LOGF(LogIris, Warning, "%ls", StringBuilder.ToString());
 			}
 #endif
			ensureMsgf(NewProtocolId == ProtocolId, TEXT("FReplicationProtocolManager::CreateReplicationProtocol Id mismatch when creating protocol named %s with in ProtocolId:0x%x Calculated ProtocolId:0x%x"), DebugName, ProtocolId, NewProtocolId);
			return nullptr;
		}
	}

	// Template key should be valid unless the protocol doesn't use one.
	if (Params.bHasTemplateKey && !IsValid(Params.TemplateKey))
	{
		UE_LOGF(LogIris, Error, "Cannot create replication protocol %ls due to invalid template", DebugName);
		check(Params.bHasTemplateKey == false || IsValid(Params.TemplateKey));
		return nullptr;
	}

	// Protocol should not exist already
	check(GetReplicationProtocol(ProtocolId, Params.TemplateKey) == nullptr);

	// Create the protocol
	const uint32 FragmentCount = Fragments.Num();
	if (!ensure(FragmentCount <= 65536))
	{
		return nullptr;
	}

	// Build table to map from RepIndex to fragment
	TArray<TArray<FReplicationProtocol::FRepIndexToFragmentIndex, TInlineAllocator<128>>> OwnerRepIndicesToFragment;
	{
		// $TODO: Remove when we change FReplicationFragment::GetOwner()
		auto GetFragmentOwner = [](const FReplicationFragment* Fragment)
		{
			constexpr uint32 MaxFragmentOwnerCount = 2U;
			UObject* FragmentOwnersForPushBasedFragments[MaxFragmentOwnerCount];
			FReplicationStateOwnerCollector OwnerCollector(FragmentOwnersForPushBasedFragments, MaxFragmentOwnerCount);
			Fragment->CollectOwner(&OwnerCollector);

			ensure(OwnerCollector.GetOwnerCount() > 0U);
			return OwnerCollector.GetOwnerCount() > 0U ? OwnerCollector.GetOwners()[0U] : nullptr;
		};
		
		// Fill in fragment data and fragment pointer
		TArray<const UObject*, TInlineAllocator<16>> Owners;

		for (const FReplicationFragmentInfo& Info : Fragments)
		{
			const uint16 FragmentIndex = static_cast<uint16>(&Info - Fragments.GetData());
			if (Info.Descriptor->MemberCount > 0U && EnumHasAnyFlags(Info.Fragment->GetTraits(), EReplicationFragmentTraits::HasPushBasedDirtiness))
			{
				const UObject* Owner = GetFragmentOwner(Info.Fragment);
				const int32 OwnerId = Owners.AddUnique(Owner);

				// Resize if necessary
				OwnerRepIndicesToFragment.SetNum(Owners.Num());
				OwnerRepIndicesToFragment[OwnerId].SetNum(FPlatformMath::Max(Info.Descriptor->RepIndexCount, (uint16)OwnerRepIndicesToFragment[OwnerId].Num()));

				for (const FReplicationStateMemberRepIndexToMemberIndexDescriptor& MemberIndex : MakeArrayView(Info.Descriptor->MemberRepIndexToMemberIndexDescriptors, Info.Descriptor->RepIndexCount))
				{
					const int32 RepIndex = static_cast<int32>(&MemberIndex - Info.Descriptor->MemberRepIndexToMemberIndexDescriptors);
					if (MemberIndex.MemberIndex != FReplicationStateMemberRepIndexToMemberIndexDescriptor::InvalidEntry)
					{
						//UE_LOGF(LogIris, VeryVerbose, "Mapping Property: %ls:%ls repindex %d memberindex %d to fragment %d", *GetNameSafe(Owner), Info.Descriptor->MemberDebugDescriptors[MemberIndex.MemberIndex].DebugName->Name, RepIndex, MemberIndex.MemberIndex, FragmentIndex)					
						OwnerRepIndicesToFragment[OwnerId][RepIndex].FragmentIndex = (uint16)FragmentIndex;
					}
				}
			}
		}
	}

	// We want to keep this as a single allocation so we first build a layout and allocate enough space for the InstanceProtocol and its data
	struct FReplicationProtocolLayoutData
	{
		FMemoryLayoutUtil::FOffsetAndSize ReplicationProtocolSizeAndOffset;
		FMemoryLayoutUtil::FOffsetAndSize ReplicationStateDescriptorsSizeAndOffset;
		FMemoryLayoutUtil::FOffsetAndSize PushModelOwnerTableSizeAndOffset;
		FMemoryLayoutUtil::FOffsetAndSize PushModelOwnerRepIndexToFragmentIndexDataSizeAndOffset;
	};
	FReplicationProtocolLayoutData LayoutData;

	FMemoryLayoutUtil::FLayout Layout;
	FMemoryLayoutUtil::AddToLayout<FReplicationProtocol>(Layout, LayoutData.ReplicationProtocolSizeAndOffset, 1);
	FMemoryLayoutUtil::AddToLayout<const FReplicationStateDescriptor*>(Layout, LayoutData.ReplicationStateDescriptorsSizeAndOffset, FragmentCount);

	// Fill in RepIndex -> Fragmeent lookup tables
	const uint16 OwnerCount = (uint16)OwnerRepIndicesToFragment.Num();

	uint32 TotalRepIndexToFragmentCount = 0U;
	for (const TArray<FReplicationProtocol::FRepIndexToFragmentIndex, TInlineAllocator<128>>& TableEntry : OwnerRepIndicesToFragment)
	{
		TotalRepIndexToFragmentCount += TableEntry.Num();
	}

	// Add lookup table to layout
	FMemoryLayoutUtil::AddToLayout<const FReplicationProtocol::FRepIndexToFragmentIndexTable>(Layout, LayoutData.PushModelOwnerTableSizeAndOffset, OwnerCount);
	// Actual data for all table entries.
	FMemoryLayoutUtil::AddToLayout<const FReplicationProtocol::FRepIndexToFragmentIndex>(Layout, LayoutData.PushModelOwnerRepIndexToFragmentIndexDataSizeAndOffset, TotalRepIndexToFragmentCount);

	// Allocate memory for the protocol, the replication protocol must be refcounted by all NetObjects
	// We could also choose to explicitly control the lifetime 
	uint8* Buffer = static_cast<uint8*>(FMemory::MallocZeroed(Layout.CurrentOffset, static_cast<uint32>(Layout.MaxAlignment)));

	// Init FReplicationInstanceProtocol
	FReplicationProtocol* Protocol = new (Buffer) FReplicationProtocol;
	Protocol->ReplicationStateDescriptors = FragmentCount ? reinterpret_cast<const FReplicationStateDescriptor**>(Buffer + LayoutData.ReplicationStateDescriptorsSizeAndOffset.Offset) : nullptr;
	Protocol->ReplicationStateCount = FragmentCount;

	// Fill in the data for the the RepIndex -> Fragment lookup tables.
	{
		// Setup table pointer and count
		FReplicationProtocol::FRepIndexToFragmentIndexTable* RepIndexToFragmentIndexTables = OwnerCount ? reinterpret_cast<FReplicationProtocol::FRepIndexToFragmentIndexTable*>(Buffer + LayoutData.PushModelOwnerTableSizeAndOffset.Offset) : nullptr;
		Protocol->PushModelOwnerRepIndexToFragmentIndexTable = RepIndexToFragmentIndexTables;
		Protocol->PushModelOwnerCount = OwnerCount;

		// Setup table data.
		FReplicationProtocol::FRepIndexToFragmentIndex* PushModelRepIndexToFragmentIndexData = OwnerCount ? reinterpret_cast<FReplicationProtocol::FRepIndexToFragmentIndex*>(Buffer + LayoutData.PushModelOwnerRepIndexToFragmentIndexDataSizeAndOffset.Offset) : nullptr;
		for (uint32 OwnerIt = 0U; OwnerIt < OwnerCount; ++OwnerIt)
		{
			const uint32 RepIndexToFragmentCount = OwnerRepIndicesToFragment[OwnerIt].Num();

			// Copy data
			FPlatformMemory::Memcpy(PushModelRepIndexToFragmentIndexData, OwnerRepIndicesToFragment[OwnerIt].GetData(), RepIndexToFragmentCount * sizeof(FReplicationProtocol::FRepIndexToFragmentIndex)); 

			// Setup table entry
			RepIndexToFragmentIndexTables[OwnerIt].RepIndexToFragmentIndex = PushModelRepIndexToFragmentIndexData;
			RepIndexToFragmentIndexTables[OwnerIt].NumEntries = OwnerRepIndicesToFragment[OwnerIt].Num();

			PushModelRepIndexToFragmentIndexData += RepIndexToFragmentCount;
		}
	}

	// Cached data
	uint32 MaxExternalSize = 0;
	uint32 MaxExternalAlign = 0;
	uint32 InternalAlign = 0;
	uint32 InternalSize = 0;
	uint32 ChangeMaskBitCount = 0;
	EReplicationProtocolTraits ProtocolTraits = EReplicationProtocolTraits::None;
	EReplicationStateTraits CombinedStateTraits = EReplicationStateTraits::None;

	// Get accumulated and shared traits for fragments
	EReplicationFragmentTraits AccumulatedTraits;
	EReplicationFragmentTraits SharedTraits;
	GetInstanceTraits(Fragments, AccumulatedTraits, SharedTraits, EReplicationFragmentTraits::None);

	// Fill in data for the protocol
	uint32 FirstLifetimeConditionalsStateIndex = ~0U;
	uint32 LifetimeConditionalsStateCount = 0;
	uint32 FirstLifetimeConditionalsChangeMaskOffset = ~0U;
	uint32 FragmentIt = 0;
	for (const FReplicationFragmentInfo& Info : Fragments)
	{
		CA_ASSUME(Protocol->ReplicationStateDescriptors != nullptr);

		const FReplicationStateDescriptor* Descriptor = Info.Descriptor;
		Protocol->ReplicationStateDescriptors[FragmentIt] = Descriptor;
		Descriptor->AddRef();

		// We track all protocols using a descriptor in order to be able to detect when we have to invalidate a protocol due to the descriptor being unloaded
		DescriptorToProtocolMap.AddUnique(Descriptor, Protocol);

		MaxExternalSize = FMath::Max<uint32>(MaxExternalSize, Descriptor->ExternalSize);
		MaxExternalAlign = FMath::Max<uint32>(MaxExternalAlign, Descriptor->ExternalAlignment);
		InternalAlign = FMath::Max<uint32>(InternalAlign, Descriptor->InternalAlignment);
		InternalSize = Align(InternalSize, Descriptor->InternalAlignment);
		InternalSize += Descriptor->InternalSize;
		ChangeMaskBitCount += Descriptor->ChangeMaskBitCount;

		// Traits
		CombinedStateTraits |= Descriptor->Traits;

		if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals))
		{
			ProtocolTraits |= EReplicationProtocolTraits::HasLifetimeConditionals | EReplicationProtocolTraits::HasConditionalChangeMask;

			++LifetimeConditionalsStateCount;
			if (LifetimeConditionalsStateCount == 1U)
			{
				FirstLifetimeConditionalsStateIndex = FragmentIt;
				FirstLifetimeConditionalsChangeMaskOffset = ChangeMaskBitCount - Descriptor->ChangeMaskBitCount;
				checkSlow(FirstLifetimeConditionalsChangeMaskOffset <= 65535);
			}
		}

		++FragmentIt;
	}

	if (EnumHasAnyFlags(CombinedStateTraits, EReplicationStateTraits::HasDynamicState))
	{
		ProtocolTraits |= EReplicationProtocolTraits::HasDynamicState;
	}

	if (EnumHasAnyFlags(CombinedStateTraits, EReplicationStateTraits::HasConnectionSpecificSerialization))
	{
		ProtocolTraits |= EReplicationProtocolTraits::HasConnectionSpecificSerialization;
	}

	if (EnumHasAnyFlags(CombinedStateTraits, EReplicationStateTraits::HasObjectReference))
	{
		ProtocolTraits |= EReplicationProtocolTraits::HasObjectReference;
	}

	if (EnumHasAnyFlags(CombinedStateTraits, EReplicationStateTraits::SupportsDeltaCompression))
	{
		if (InternalSize > 0)
		{
			ProtocolTraits |= EReplicationProtocolTraits::SupportsDeltaCompression;
		}
	}

	// Allocate conditional change mask if required.
	if (EnumHasAnyFlags(ProtocolTraits, EReplicationProtocolTraits::HasConditionalChangeMask))
	{
		InternalAlign = FPlatformMath::Max(InternalAlign, 4U);
		InternalSize = Align(InternalSize, 4U);

		Protocol->InternalChangeMasksOffset = InternalSize;

		Protocol->FirstLifetimeConditionalsStateIndex = static_cast<uint16>(FirstLifetimeConditionalsStateIndex);
		Protocol->LifetimeConditionalsStateCount = static_cast<uint16>(LifetimeConditionalsStateCount);
		Protocol->FirstLifetimeConditionalsChangeMaskOffset = FirstLifetimeConditionalsChangeMaskOffset;

		InternalSize += FNetBitArrayView::CalculateRequiredWordCount(ChangeMaskBitCount)*sizeof(FNetBitArrayView::StorageWordType);
	}

	// Setup Pushbased traits
	if (EnumHasAnyFlags(AccumulatedTraits, EReplicationFragmentTraits::HasPushBasedDirtiness))
	{
		ProtocolTraits |= EReplicationProtocolTraits::HasPushBasedDirtiness;
		if (EnumHasAnyFlags(SharedTraits, EReplicationFragmentTraits::HasPushBasedDirtiness))
		{
			ProtocolTraits |= EReplicationProtocolTraits::HasFullPushBasedDirtiness;	
		}
	}

	Protocol->InternalTotalAlignment = FPlatformMath::Max(InternalAlign, 1U);
	Protocol->InternalTotalSize = InternalSize;
	Protocol->MaxExternalStateSize = MaxExternalSize;
	Protocol->MaxExternalStateAlignment = FPlatformMath::Max(MaxExternalAlign, 1U);
	Protocol->ChangeMaskBitCount = ChangeMaskBitCount;

	Protocol->ProtocolIdentifier = ProtocolId;
	Protocol->ProtocolTraits = ProtocolTraits;
	Protocol->DebugName = CreatePersistentNetDebugName(DebugName);
	Protocol->TypeStatsIndex = Params.TypeStatsIndex >= 0 ? Params.TypeStatsIndex : FNetTypeStats::DefaultTypeStatsIndex;
	Protocol->RefCount = 0;

	// Register protocol
	FRegisteredProtocolInfo Info;
	Info.TemplateKey = Params.TemplateKey;
	Info.Protocol = Protocol;

	RegisteredProtocols.AddUnique(ProtocolId, Info);
	ProtocolToInfoMap.Add(Protocol, Info);

#if UE_NET_ENABLE_PROTOCOLMANAGER_LOG
	if (bIrisLogReplicationProtocols)
	{
		TStringBuilder<4096> StringBuilder;
		FragmentListToString(StringBuilder, Fragments);
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogIris, ELogVerbosity::Log);
		UE_LOG_PROTOCOLMANAGER(Log, TEXT("FReplicationProtocolManager::CreateReplicationProtocol Created new protocol %s with ProtocolId:0x%x"), ToCStr(Protocol->DebugName), ProtocolId);
		UE_LOG_PROTOCOLMANAGER(Log, TEXT("%s"), StringBuilder.ToString());
	}
	else
	{
		UE_LOG_PROTOCOLMANAGER(Verbose, TEXT("FReplicationProtocolManager::CreateReplicationProtocol Created new protocol %s with ProtocolId:0x%x"), ToCStr(Protocol->DebugName), ProtocolId);	
	}
#endif

	return Protocol;
}

void FReplicationProtocolManager::DestroyReplicationProtocol(const FReplicationProtocol* ReplicationProtocol)
{
	FRegisteredProtocolInfo Info;
	if (ProtocolToInfoMap.RemoveAndCopyValue(ReplicationProtocol, Info))
	{
		InternalDeferDestroyReplicationProtocol(Info.Protocol);
		RegisteredProtocols.RemoveSingle(ReplicationProtocol->ProtocolIdentifier, Info);		
	}
	
	PruneProtocolsPendingDestroy();
}

void FReplicationProtocolManager::InternalDestroyReplicationProtocol(const FReplicationProtocol* Protocol)
{
	if (Protocol)
	{
		UE_LOG_PROTOCOLMANAGER(Verbose, TEXT("FReplicationProtocolManager::InternalDestroyReplicationProtocol Destroyed protocol %s with ProtocolId:0x%x"), ToCStr(Protocol->DebugName), Protocol->ProtocolIdentifier);

		// Remove tracked descriptors
		for (const FReplicationStateDescriptor* Descriptor : MakeArrayView(Protocol->ReplicationStateDescriptors, Protocol->ReplicationStateCount))
		{
			DescriptorToProtocolMap.RemoveSingle(Descriptor, Protocol);
			Descriptor->Release();
		}
		FMemory::Free(const_cast<FReplicationProtocol*>(Protocol));
	}
}

void FReplicationProtocolManager::InternalDeferDestroyReplicationProtocol(const FReplicationProtocol* Protocol)
{
	PendingDestroyProtocols.Add(Protocol);
}

void FReplicationProtocolManager::PruneProtocolsPendingDestroy()
{
	for (auto It = PendingDestroyProtocols.CreateIterator(); It; ++It)
	{
		const FReplicationProtocol* Protocol = *It;
		if (Protocol->RefCount == 0)
		{
			InternalDestroyReplicationProtocol(Protocol);
			It.RemoveCurrent();
		}
	}
}

void FReplicationProtocolManager::InvalidateDescriptor(const FReplicationStateDescriptor* InvalidatedReplicationStateDescriptor)
{
	// Destroy all protocols that referenced the descriptor (or ensure that they are still valid)
	TArray<const FReplicationProtocol*, TInlineAllocator<32>> InvalidProtocols;

	// Find protocols using the descriptor being invalidated
	for (auto It = DescriptorToProtocolMap.CreateConstKeyIterator(InvalidatedReplicationStateDescriptor); It; ++It)
	{
		InvalidProtocols.Add(It.Value());
	}

	// Destroy them
	for (const FReplicationProtocol* Protocol : MakeArrayView(InvalidProtocols))
	{
		DestroyReplicationProtocol(Protocol);
	}
}

FReplicationProtocolManager::~FReplicationProtocolManager()
{
	// Cleanup protocols
	for (auto& It : ProtocolToInfoMap)
	{
		InternalDestroyReplicationProtocol(It.Value.Protocol);
	}	

	for (const FReplicationProtocol* Protocol : PendingDestroyProtocols)
	{
		InternalDestroyReplicationProtocol(Protocol);
	}
}


}
