// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "UObject/ObjectKey.h"

#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"

namespace UE::Net::Private
{

struct FCreateReplicationProtocolParameters
{
	const UObject* TemplateKey = nullptr;
	bool bValidateProtocolId = false;
	bool bHasTemplateKey = true;
	int32 TypeStatsIndex = INDEX_NONE;
};

class FReplicationProtocolManager
{
public:
	~FReplicationProtocolManager();

	/* Create protocol from registered fragment data with provided Id, verification is optional */
	IRISCORE_API const FReplicationProtocol* CreateReplicationProtocol(const FReplicationProtocolIdentifier ProtocolId, const FReplicationFragments& Fragments, const TCHAR* DebugName, const FCreateReplicationProtocolParameters& Params = FCreateReplicationProtocolParameters());

	/* Get an existing replication protocol */
	IRISCORE_API const FReplicationProtocol* GetReplicationProtocol(FReplicationProtocolIdentifier ProtocolId, FObjectKey TemplateKey);

	/** Iterate over all protocols matching the ProtocolId, mostly used by debug functionality with no sideeffects */
	template<typename T>
	void ForEachProtocol(FReplicationProtocolIdentifier ProtocolId, T&& Functor) const;

	/** Iterate over all protocols. */
	template<typename T>
	void ForEachProtocol(T&& Functor) const;
	
	/* Destroy existing replication protocol */
	IRISCORE_API void DestroyReplicationProtocol(const FReplicationProtocol* ReplicationProtocol);

	/** 
	 * Create instance protocol from registered fragment data
	 * Lifetime either the same as the lifetime of a remotely created instance (we are the client) or controlled by	calls to the bridge api to replicate an object or not
	 * @param ObjectTraits Traits that are passed by the object instance itself
	 */
	IRISCORE_API static FReplicationInstanceProtocol* CreateInstanceProtocol(const FReplicationFragments& Fragments, UE::Net::EReplicationFragmentTraits ObjectTraits);
	IRISCORE_API static void DestroyInstanceProtocol(FReplicationInstanceProtocol*);

	IRISCORE_API void InvalidateDescriptor(const FReplicationStateDescriptor* InvalidatedReplicationStateDescriptor);

public:

	/** Calculate protocol Identifier from registered fragment data */
	IRISCORE_API static FReplicationProtocolIdentifier CalculateProtocolIdentifier(const FReplicationFragments& Fragments);

	/** Validate that a existing protocol matches the FragmentList of an instance, returns true of it is a match */
	IRISCORE_API static bool ValidateReplicationProtocol(const FReplicationProtocol*, const FReplicationFragments& Fragments, bool bLogFragmentErrors=true);

	/** Print the names of the fragments and their hash */
	IRISCORE_API static void FragmentListToString(FStringBuilderBase& StringBuilder, const FReplicationFragments& Fragments);

private:
	void InternalDestroyReplicationProtocol(const FReplicationProtocol* Protocol);
	void InternalDeferDestroyReplicationProtocol(const FReplicationProtocol* Protocol);
	void PruneProtocolsPendingDestroy();

	struct FRegisteredProtocolInfo
	{
		const FReplicationProtocol* Protocol;
		FObjectKey TemplateKey;
	
		bool operator==(const FRegisteredProtocolInfo& Other) const { return Protocol == Other.Protocol && TemplateKey == Other.TemplateKey; };
	};

	TMultiMap<FReplicationProtocolIdentifier, FRegisteredProtocolInfo> RegisteredProtocols;
	TMap<const FReplicationProtocol*, FRegisteredProtocolInfo> ProtocolToInfoMap;
	
	// We use the pointer as key, we have full control over lifetime of descriptors so this should not be a problem
	TMultiMap<const FReplicationStateDescriptor*, const FReplicationProtocol*> DescriptorToProtocolMap;
	TArray<const FReplicationProtocol*> PendingDestroyProtocols;
};

template<typename T>
void FReplicationProtocolManager::ForEachProtocol(FReplicationProtocolIdentifier ProtocolId, T&& Functor) const
{
	// Find protocols using the descriptor being invalidated
	for (auto It = RegisteredProtocols.CreateConstKeyIterator(ProtocolId); It; ++It)
	{
		const FRegisteredProtocolInfo& Info = It.Value();
		Functor(Info.Protocol, Info.TemplateKey);
	}
}

template<typename T>
void FReplicationProtocolManager::ForEachProtocol(T&& Functor) const
{
	for (auto It = RegisteredProtocols.CreateConstIterator(); It; ++It)
	{
		const FRegisteredProtocolInfo& Info = It.Value();
		Functor(Info.Protocol, Info.TemplateKey);
	}
}

}
