// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ReplicationReader.h"

#include "Algo/RemoveIf.h"

#include "HAL/IConsoleManager.h"

#include "Iris/Core/BitTwiddling.h"
#include "Iris/Core/IrisDebugging.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"

#include "Iris/DataStream/DataStream.h"

#include "Iris/IrisConfig.h"

#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineManager.h"
#include "Iris/ReplicationSystem/DequantizeAndApplyHelper.h"
#include "Iris/ReplicationSystem/NetBlob/NetObjectBlobHandler.h"
#include "Iris/ReplicationSystem/NetBlob/PartialNetObjectAttachmentHandler.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ObjectReferenceTypes.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"

#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/ObjectNetSerializer.h"

#include "Misc/CoreMisc.h"
#include "Misc/ScopeExit.h"

#include "Net/Core/Trace/NetDebugName.h"
#include "Net/Core/Trace/NetTrace.h"

#include "ProfilingDebugging/CsvProfiler.h"

#include "UObject/Class.h"

#if UE_NET_ENABLE_REPLICATIONREADER_LOG
#	define UE_LOG_REPLICATIONREADER(Format, ...)  UE_LOG(LogIris, Log, Format, ##__VA_ARGS__)
#	define UE_LOG_REPLICATIONREADER_CONN(Format, ...)  UE_LOG(LogIris, Log, TEXT("Conn: %u ") Format, Parameters.ConnectionId, ##__VA_ARGS__)
#else
#	define UE_LOG_REPLICATIONREADER(...)
#	define UE_LOG_REPLICATIONREADER_CONN(Format, ...)
#endif

#define UE_LOG_REPLICATIONREADER_CONN_WARNING(Format, ...) UE_LOG(LogIris, Warning, TEXT("Conn: %u ") Format, Parameters.ConnectionId, ##__VA_ARGS__)

CSV_DECLARE_CATEGORY_EXTERN(IrisClient);

namespace UE::Net::Private
{

static bool bUseResolvingHandleCache = true;
static FAutoConsoleVariableRef CVarUseResolvingHandleCache(
	TEXT("net.Iris.UseResolvingHandleCache"),
	bUseResolvingHandleCache,
	TEXT("Enable the use of a hot and cold cache when resolving unresolved caches to reduce the time spent resolving references."));

static int32 HotResolvingLifetimeMS = 1000;
static FAutoConsoleVariableRef CVarHotResolvingLifetimeMS(
	TEXT("net.Iris.HotResolvingLifetimeMS"),
	HotResolvingLifetimeMS,
	TEXT("An unresolved reference is considered hot if it was created within this many milliseconds, and cold otherwise."));

static int32 ColdResolvingRetryTimeMS = 200;
static FAutoConsoleVariableRef CVarColdResolvingRetryTimeMS(
	TEXT("net.Iris.ColdResolvingRetryTimeMS"),
	ColdResolvingRetryTimeMS,
	TEXT("Resolve unresolved cold references after this many milliseconds."));

static bool bExecuteReliableRPCsBeforeApplyState = true;
static FAutoConsoleVariableRef CVarExecuteReliableRPCsBeforeApplyState(
		TEXT("net.Iris.ExecuteReliableRPCsBeforeApplyState"),
		bExecuteReliableRPCsBeforeApplyState,
		TEXT("If true and Iris runs in backwards compatibility mode then reliable RPCs will be executed before we apply state data on the target object unless we first need to spawn the object."));

static bool bDispatchUnresolvedPreviouslyReceivedChanges = false;
static FAutoConsoleVariableRef CvarDispatchUnresolvedPreviouslyReceivedChanges(
	TEXT("net.Iris.DispatchUnresolvedPreviouslyReceivedChanges"),
	bDispatchUnresolvedPreviouslyReceivedChanges,
	TEXT("Whether to include previously received changes with unresolved object references to data received this frame when applying state data. This can call rep notify functions to be called despite being unchanged. Default is false."));

static bool bRemapDynamicObjects = true;
static FAutoConsoleVariableRef CvarRemapDynamicObjects(
		TEXT("net.Iris.RemapDynamicObjects"),
		bRemapDynamicObjects,
		TEXT("Allow remapping of dynamic objects on the receiving end. This allows properties previously pointing to a particular object to be updated if the object is re-created. Default is true."));

static bool bResolvedObjectsDispatchDebugging = false;
static FAutoConsoleVariableRef CvarResolvedObjectsDispatchDebugging(
	TEXT("net.Iris.ResolvedObjectsDispatchDebugging"),
	bResolvedObjectsDispatchDebugging,
	TEXT("Debug logging of resolved object state dispatching. Default is false."));

static int32 QueuedBatchTimeoutWarningInterval = 30;
static FAutoConsoleVariableRef CVarQueuedBatchTimeoutWarningInterval(
	TEXT("net.Iris.QueuedBatch.TimeoutWarningInterval"),
	QueuedBatchTimeoutWarningInterval,
	TEXT("Number of ticks between checks for queuedbatches timeout."));

static float QueuedBatchTimeoutSeconds = 30.f;
static FAutoConsoleVariableRef CVarQueuedBatchTimeoutSeconds(
	TEXT("net.Iris.QueuedBatch.TimeoutSeconds"),
	QueuedBatchTimeoutSeconds,
	TEXT("Time in seconds to wait for queued batches to flush before logging a warning."));

static float QueuedBatchWarningRepeatTime = 10.f;
static FAutoConsoleVariableRef CVarQueuedBatchWarningRepeatTime(
	TEXT("net.Iris.QueuedBatch.WarningRepeatTime"),
	QueuedBatchWarningRepeatTime,
	TEXT("How long to wait before warning again."));

static int32 MaxMustBeMappedHandleArray = 16;
static FAutoConsoleVariableRef CVarMaxMustBeMappedHandleArray(
	TEXT("net.Iris.QueuedBatch.MaxMustBeMappedHandleArray"),
	MaxMustBeMappedHandleArray,
	TEXT("Maximum number of unresolved references that will be included with the QueuedBatch timeout error message."));

static bool bGracefullyHandleReachingEndOfBitstream = false;
static FAutoConsoleVariableRef CvarGracefullyHandleReachingEndOfBitstream(
		TEXT("net.Iris.ReplicationReader.GracefullyHandleReachingEndOfBitstream"),
		bGracefullyHandleReachingEndOfBitstream,
		TEXT("Gracefully handle reaching end of the ReplicationReader bitstream prior to reading as many object batches as expected. Default is true."));

#if UE_NET_ASYNCLOADING_DEBUG
static bool bForceObjectsToStall = false;
static FAutoConsoleVariableRef CVarForceObjectsToStall(
	TEXT("net.Iris.AsyncLoading.ForceObjectsToStall"),
	bForceObjectsToStall,
	TEXT("Force objects in the DebugObjectsForcedToStall list to never resolve their reference and simulate long async loading delays."));
#endif

static const FName NetError_FailedToFindAttachmentQueue("Failed to find attachment queue");

//------------------------------------------------------------------------

class FResolveAndCollectUnresolvedAndResolvedReferenceCollector
{
public:
	typedef FNetReferenceCollector::FReferenceInfoArray FReferenceInfoArray;

	void CollectReferences(FObjectReferenceCache& ObjectReferenceCache,
		const FNetObjectResolveContext& ResolveContext,
		bool bInIncludeInitState, 
		const FNetBitArrayView* ChangeMask,
		uint8* RESTRICT InternalBuffer, const FReplicationProtocol* Protocol)
	{
		bIncludeInitState = bInIncludeInitState;

		// Setup context
		FNetSerializationContext Context;
		Context.SetChangeMask(ChangeMask);
		Context.SetIsInitState(bInIncludeInitState);

		FNetReferenceCollector Collector;
		FReplicationProtocolOperationsInternal::CollectReferences(Context, Collector, InternalBuffer, Protocol);

		// Iterate over result and process results
		for (const FNetReferenceCollector::FReferenceInfo& Info : MakeArrayView(Collector.GetCollectedReferences()))
		{
			if (!ObjectReferenceCache.ResolveObjectReference(Info.Reference, ResolveContext))
			{
				UnresolvedReferenceInfos.Add(Info);
			}
			else
			{
				ResolvedReferenceInfos.Add(Info);
			}		
		}		
	}

	void Reset()
	{ 
		UnresolvedReferenceInfos.Reset();
		ResolvedReferenceInfos.Reset();
	}

	const FReferenceInfoArray& GetResolvedReferences() const { return ResolvedReferenceInfos; }
	const FReferenceInfoArray& GetUnresolvedReferences() const { return UnresolvedReferenceInfos; }
	bool IsInitStateIncluded() const { return bIncludeInitState; }

private:
	FReferenceInfoArray UnresolvedReferenceInfos;
	FReferenceInfoArray ResolvedReferenceInfos;
	bool bIncludeInitState = false;
};

//------------------------------------------------------------------------

struct FReplicationReader::FDispatchObjectInfo
{
	FInternalNetRefIndex InternalIndex = InvalidInternalNetRefIndex;
	FChangeMaskStorageOrPointer ChangeMaskOrPointer;
	bool bIsInitialState : 1 = false;
	bool bHasState : 1 = false;
	bool bHasAttachments : 1 = false;

	/** When true we will call EndReplication on the object after reading all the bunch */
	bool bWantsDeferredEndReplication : 1 = false;

	/** When true we will immediately call EndReplication on the object while still reading data instead of waiting for the entire bunch to be read */
	bool bWantsImmediateEndReplication : 1 = false;

	/** When true we will call EndReplication on all it's subobjects */
	bool bCallEndReplicationOnSubObjects : 1 = false;

	/** When true we'll tell the RootObject's factory that this subobject was instantiated */
	bool bShouldCallSubObjectCreatedFromReplication : 1 = false;

	/** The reason we received about why this object is no longer replicated */
	EInternalDetachReason InternalDetachReason = EInternalDetachReason::Normal;
};

//------------------------------------------------------------------------

// Helper class to deal with management of ObjectsToDispatch allocations from our temporary allocator
class FReplicationReader::FObjectsToDispatchArray
{
public:

	FObjectsToDispatchArray(uint32 InitialCapacity, FMemStackBase& Allocator)
	: ObjectsToDispatchCount(0U)
	, Capacity(InitialCapacity + ObjectsToDispatchSlackCount)
	{			
		ObjectsToDispatch = new (Allocator) FDispatchObjectInfo[Capacity];
	}

	void Grow(uint32 Count, FMemStackBase& Allocator)
	{
		if (Capacity < (ObjectsToDispatchCount + Count))
		{
			Capacity = ObjectsToDispatchCount + Count + ObjectsToDispatchSlackCount;
			FDispatchObjectInfo* NewObjectsToDispatch = new (Allocator) FDispatchObjectInfo[Capacity];

			// If we already had allocated data we need to copy the old elements
			if (ObjectsToDispatchCount)
			{
				FPlatformMemory::Memcpy(NewObjectsToDispatch, ObjectsToDispatch, ObjectsToDispatchCount*sizeof(FDispatchObjectInfo));
			}
			ObjectsToDispatch = NewObjectsToDispatch;
		}		
	}

	FDispatchObjectInfo& AddPendingDispatchObjectInfo(FMemStackBase& Allocator)
	{
		Grow(1, Allocator);
		ObjectsToDispatch[ObjectsToDispatchCount] = FDispatchObjectInfo();

		return ObjectsToDispatch[ObjectsToDispatchCount];
	}

	void CommitPendingDispatchObjectInfo()
	{
		checkSlow(ObjectsToDispatchCount < Capacity);
		++ObjectsToDispatchCount;
	}

	uint32 Num() const { return ObjectsToDispatchCount; }
	TArrayView<FDispatchObjectInfo> GetObjectsToDispatch() { return MakeArrayView(ObjectsToDispatch, ObjectsToDispatchCount); }

private:

	FDispatchObjectInfo* ObjectsToDispatch;
	uint32 ObjectsToDispatchCount;
	uint32 Capacity;		
};

//------------------------------------------------------------------------

FReplicationReader::FReplicatedObjectInfo::FReplicatedObjectInfo()
: InternalIndex(InvalidInternalNetRefIndex)
, Value(0U)
{
	FMemory::Memzero(StoredBaselines);
	LastStoredBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
	PrevStoredBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
}

bool FReplicationReader::FReplicatedObjectInfo::RemoveUnresolvedHandleCount(FNetRefHandle RefHandle)
{
	int16* HandleCount = UnresolvedHandleCount.Find(RefHandle);
	if (ensureMsgf(HandleCount != nullptr, TEXT("Unresolved handle counter could not be found for %s"), ToCStr(RefHandle.ToString())))
	{
		int16& HandleCountRef = (*HandleCount);

		ensure(HandleCountRef > 0);

		HandleCountRef--;

		if (HandleCountRef <= 0)
		{
			UnresolvedHandleCount.Remove(RefHandle);
			return true;
		}
	}

	return false;
}

bool FReplicationReader::FReplicatedObjectInfo::RemoveResolvedDynamicHandleCount(FNetRefHandle RefHandle)
{
	int16* HandleCount = ResolvedDynamicHandleCount.Find(RefHandle);
	if (ensureMsgf(HandleCount != nullptr, TEXT("Resolved dynamic handle counter could not be found for% s"), ToCStr(RefHandle.ToString())))
	{
		int16& HandleCountRef = (*HandleCount);

		ensure(HandleCountRef > 0);

		HandleCountRef--;

		if (HandleCountRef <= 0)
		{
			ResolvedDynamicHandleCount.Remove(RefHandle);
			return true;
		}
	}

	return false;
}

FReplicationReader::FReplicationReader()
: TempLinearAllocator()
, TempChangeMaskAllocator(&TempLinearAllocator)
, ReplicationSystemInternal(nullptr)
, NetRefHandleManager(nullptr)
, StateStorage(nullptr)
, ObjectsToDispatchArray(nullptr)
, NetBlobHandlerManager(nullptr)
, NetObjectBlobType(InvalidNetBlobType)
, DelayAttachmentsWithUnresolvedReferences(IConsoleManager::Get().FindConsoleVariable(TEXT("net.DelayUnmappedRPCs"), false /* bTrackFrequentCalls */))
{
}

FReplicationReader::~FReplicationReader()
{
	checkf(ReplicatedObjects.IsEmpty(), TEXT("Possible leak detected in FReplicationReader. Nothing should be registered after Deinit()"));
	checkf(PendingBatchHolder.IsEmpty(), TEXT("Possible leak detected in FReplicationReader. Nothing should be registered after Deinit()"));
}

void FReplicationReader::Init(const FReplicationParameters& InParameters)
{
	// Store copy of parameters
	Parameters = InParameters;

	// Cache internal systems
	ReplicationSystemInternal = Parameters.ReplicationSystem->GetReplicationSystemInternal();
	NetRefHandleManager = &ReplicationSystemInternal->GetNetRefHandleManager();
	StateStorage = &ReplicationSystemInternal->GetReplicationStateStorage();
	NetBlobHandlerManager = &ReplicationSystemInternal->GetNetBlobHandlerManager();
	ObjectReferenceCache = &ReplicationSystemInternal->GetObjectReferenceCache();
	ReplicationBridge = Parameters.ReplicationSystem->GetReplicationBridge();

	// Init resolve context
	ResolveContext.ConnectionId = InParameters.ConnectionId;
	ResolveContext.RemoteNetTokenStoreState = Parameters.ReplicationSystem->GetNetTokenStore()->GetRemoteNetTokenStoreState(InParameters.ConnectionId);
	ResolveContext.ReplicationSystem = Parameters.ReplicationSystem;

	// Find out if there's a PartialNetObjectAttachmentHandler so we can re-assemble split blobs
	if (const UPartialNetObjectAttachmentHandler* Handler = ReplicationSystemInternal->GetNetBlobManager().GetPartialNetObjectAttachmentHandler())
	{
		FNetObjectAttachmentsReaderInitParams InitParams;
		InitParams.PartialNetObjectAttachmentHandler = Handler;
		Attachments.Init(InitParams);
	}

	if (const UNetBlobHandler* Handler = ReplicationSystemInternal->GetNetBlobManager().GetNetObjectBlobHandler())
	{
		NetObjectBlobType = Handler->GetNetBlobType();
	}

	// reserve index 0
	StartReplication(ObjectIndexForOOBAttachment);
}

void FReplicationReader::Deinit()
{
	for (auto BatchIt = PendingBatchHolder.CreateConstIterator(); BatchIt; ++BatchIt)
	{
		const FPendingBatchData* PendingBatchData = BatchIt.Value().Get();
		UE_LOGF(LogIris, Warning, "FReplicationReader::Deinit NetHandle %ls has %d unprocessed data batches", *PendingBatchData->Owner.ToString(), PendingBatchData->QueuedDataChunks.Num());

		// Make sure to release all references that we are holding on to
		if (ObjectReferenceCache)
		{
			for (const FNetRefHandle& RefHandle : PendingBatchData->ResolvedReferences)
			{
				ObjectReferenceCache->RemoveTrackedQueuedBatchObjectReference(RefHandle);
			}
		}		
	}

	PendingBatchHolder.Empty();

	// Cleanup any allocation stored in the per object info
	for (auto& ObjectIt : ReplicatedObjects)
	{
		CleanupObjectData(ObjectIt.Value);
	}
	ReplicatedObjects.Empty();
}

// Read incomplete handle
FNetRefHandle FReplicationReader::ReadNetRefHandleId(FNetSerializationContext& Context, FNetBitStreamReader& Reader) const
{
	UE_NET_TRACE_NAMED_OBJECT_SCOPE(ReferenceScope, FNetRefHandle::GetInvalid(), *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);

	const uint64 NetId = ReadPackedUint64(&Reader);
	FNetRefHandle RefHandle = FNetRefHandleManager::MakeNetRefHandleFromId(NetId);

	UE_NET_TRACE_SET_SCOPE_OBJECTID(ReferenceScope, RefHandle);

	if (RefHandle.GetId() != NetId)
	{
		Context.SetError(GNetError_InvalidNetHandle);
		return FNetRefHandle::GetInvalid();
	}

	return RefHandle;
}
	
void FReplicationReader::ReadStreamDebugFeatures(FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	StreamDebugFeatures = ReadReplicationDataStreamDebugFeatures(Reader);

#if !UE_NET_REPLICATIONDATASTREAM_DEBUG
	// If datastream debug features isn't enabled in this build the debug functions won't do any useful work and will cause bitstream errors.
	if (!ensure(StreamDebugFeatures == EReplicationDataStreamDebugFeatures::None))
	{
		UE_LOGF(LogIris, Error, "StreamDebugFeatures enabled on sending side despite receiving side not being able to use them.");
		Context.SetError(GNetError_InvalidValue);
		return;
	}
#endif
}

uint32 FReplicationReader::ReadObjectsPendingDestroy(FNetSerializationContext& Context)
{
	return ReadRootObjectsPendingDestroy(Context);
}

uint32 FReplicationReader::ReadRootObjectsPendingDestroy(FNetSerializationContext& Context)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	UE_NET_TRACE_SCOPE(RootObjectsPendingDestroy, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	// Read how many destroyed objects we have
	constexpr uint32 DestroyObjectBitCount = 16U;
	constexpr uint32 MaxDestroyObjectCount = (1U << DestroyObjectBitCount) - 1U;
	const uint32 ObjectsToRead = Reader.ReadBits(DestroyObjectBitCount);
	
	if (Context.HasErrorOrOverflow())
	{
		return 0;
	}

	for (uint32 It = 0; It < ObjectsToRead; ++It)
	{
		UE_NET_TRACE_NAMED_OBJECT_SCOPE(DestroyedObjectScope, FNetRefHandle::GetInvalid(), Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

		FNetRefHandle IncompleteHandle = ReadNetRefHandleId(Context, Reader);

		const EInternalDetachReason DetachReason = (EInternalDetachReason)Context.GetBitStreamReader()->ReadBits(GetDetachReasonBitsNeeded());

		if (Context.HasErrorOrOverflow())
		{
			break;
		}

		if (FPendingBatchData* PendingBatchData = PendingBatchHolder.Find(IncompleteHandle))
		{
			EnqueueEndReplication(PendingBatchData, IncompleteHandle, DetachReason);
			continue;
		}

		{
			// Resolve handle and destroy using bridge
			const FInternalNetRefIndex InternalIndex = NetRefHandleManager->GetInternalIndex(IncompleteHandle);
			if (InternalIndex != InvalidInternalNetRefIndex)
			{
				UE_NET_TRACE_SET_SCOPE_OBJECTID(DestroyedObjectScope, IncompleteHandle);

				// Defer EndReplication until after applying state data
				FDispatchObjectInfo& Info = ObjectsToDispatchArray->AddPendingDispatchObjectInfo(TempLinearAllocator);

				Info.bWantsDeferredEndReplication = true;
				Info.bCallEndReplicationOnSubObjects = true;
				Info.InternalIndex = InternalIndex;
				Info.bIsInitialState = false;
				Info.bHasState = false;
				Info.bHasAttachments = false;
				Info.bShouldCallSubObjectCreatedFromReplication = false;
				Info.InternalDetachReason = DetachReason;

				UE_LOGF(LogIris, Verbose, "%s deferred end replication for RootObject: %ls", __FUNCTION__, *NetRefHandleManager->PrintObjectFromIndex(InternalIndex));

				// Mark for dispatch
				ObjectsToDispatchArray->CommitPendingDispatchObjectInfo();
			}
			else
			{
				// If we did not find the object or associated bridge, the packet that would have created the object may have been lost.
				UE_LOGF(LogIris, Verbose, "FReplicationReader::Read Tried to destroy object %ls. This can occur if the server sends destroy for an object that has not yet been confirmed as created.", *IncompleteHandle.ToString());
				// Remove from broken list, so that we can try again if the server asks us to instantiate to object again.
				BrokenObjects.RemoveSwap(IncompleteHandle);
			}
		}
	}

	return ObjectsToRead;
}

FReplicationReader::FReplicatedObjectInfo& FReplicationReader::StartReplication(uint32 InternalIndex)
{
	check(!ReplicatedObjects.Contains(InternalIndex));

	// Create ReadObjectInfo
	FReplicatedObjectInfo& ObjectInfo = ReplicatedObjects.Emplace(InternalIndex);
	ObjectInfo = FReplicatedObjectInfo();
	ObjectInfo.InternalIndex = InternalIndex;

	// Allocate changemask (if needed)
	if (InternalIndex != 0U)
	{		
		const FNetRefHandleManager::FReplicatedObjectData& Data = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);
		ObjectInfo.ChangeMaskBitCount = Data.Protocol->ChangeMaskBitCount;

		// Alloc and init changemask
		FNetBitArrayView ChangeMask = FChangeMaskStorageOrPointer::AllocAndInitBitArray(ObjectInfo.UnresolvedChangeMaskOrPointer, ObjectInfo.ChangeMaskBitCount, PersistentChangeMaskAllocator);
	}

	return ObjectInfo;
}

FReplicationReader::FReplicatedObjectInfo* FReplicationReader::GetReplicatedObjectInfo(uint32 InternalIndex)
{
	return ReplicatedObjects.Find(InternalIndex);
}

const FReplicationReader::FReplicatedObjectInfo* FReplicationReader::GetReplicatedObjectInfo(uint32 InternalIndex) const
{
	return ReplicatedObjects.Find(InternalIndex);
}

void FReplicationReader::CleanupObjectData(FReplicatedObjectInfo& ObjectInfo)
{
	// Remove from pending resolve
	if (ObjectInfo.InternalIndex != 0U)
	{
		FChangeMaskStorageOrPointer::Free(ObjectInfo.UnresolvedChangeMaskOrPointer, ObjectInfo.ChangeMaskBitCount, PersistentChangeMaskAllocator);
	}

	// Release stored baselines
	if (ObjectInfo.LastStoredBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
	{
		StateStorage->FreeBaseline(ObjectInfo.InternalIndex, ObjectInfo.StoredBaselines[ObjectInfo.LastStoredBaselineIndex]);
	}
	if (ObjectInfo.PrevStoredBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
	{
		StateStorage->FreeBaseline(ObjectInfo.InternalIndex, ObjectInfo.StoredBaselines[ObjectInfo.PrevStoredBaselineIndex]);
	}
}

void FReplicationReader::EndReplication(FInternalNetRefIndex InternalIndex, EInternalDetachReason InternalDetachReason)
{
	if (InternalIndex == ObjectIndexForOOBAttachment)
	{
		ensure(false);
		return;
	}

	FReplicatedObjectInfo* ObjectInfo = ReplicatedObjects.Find(InternalIndex);
	if (!ObjectInfo)
	{
		return;
	}

	CleanupReferenceTracking(ObjectInfo);
	Attachments.DropAllAttachments(ENetObjectAttachmentType::Normal, InternalIndex);

	FBridgeSerialization::DestroyNetObjectFromRemote(ReplicationBridge, InternalIndex, InternalDetachReason);

	CleanupObjectData(*ObjectInfo);

	ReplicatedObjects.Remove(InternalIndex);
}

void FReplicationReader::DeserializeObjectStateDelta(FNetSerializationContext& Context, uint32 InternalIndex, FDispatchObjectInfo& Info, FReplicatedObjectInfo& ObjectInfo, const FNetRefHandleManager::FReplicatedObjectData& ObjectData, uint32& OutNewBaselineIndex)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	uint32 BaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
	BaselineIndex = Reader.ReadBits(FDeltaCompressionBaselineManager::BaselineIndexBitCount);
	if (BaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
	{ 
		const bool bIsNewBaseline = Reader.ReadBool();

		if (Reader.IsOverflown())
		{
			UE_LOGF(LogIris, Error, "FReplicationReader::DeserializeObjectStateDelta Bitstream corrupted.");
			return;
		}

		if (bIsNewBaseline)
		{
			OutNewBaselineIndex = (BaselineIndex + 1) % FDeltaCompressionBaselineManager::MaxBaselineCount;
		}

		// If we are compressing against the LastStoredBaselineIndex we can release older baselines to reduce memory overhead
		if (!bIsNewBaseline && BaselineIndex == ObjectInfo.LastStoredBaselineIndex && ObjectInfo.PrevStoredBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
		{
			StateStorage->FreeBaseline(InternalIndex, ObjectInfo.StoredBaselines[ObjectInfo.PrevStoredBaselineIndex]);
			ObjectInfo.StoredBaselines[ObjectInfo.PrevStoredBaselineIndex] = nullptr;
			ObjectInfo.PrevStoredBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
		}

		check(ObjectInfo.StoredBaselines[BaselineIndex]);

		UE_NET_TRACE_SCOPE(DeltaCompressed, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		FReplicationProtocolOperations::DeserializeWithMaskDelta(Context, Info.ChangeMaskOrPointer.GetPointer(ObjectInfo.ChangeMaskBitCount), ObjectData.ReceiveStateBuffer, ObjectInfo.StoredBaselines[BaselineIndex], ObjectData.Protocol);
	}
	else
	{
		const uint32 NewBaselineIndex = Reader.ReadBits(FDeltaCompressionBaselineManager::BaselineIndexBitCount);
		if (Reader.IsOverflown())
		{
			UE_LOGF(LogIris, Error, "FReplicationReader::DeserializeObjectStateDelta Bitstream corrupted.");
			return;
		}
		OutNewBaselineIndex = NewBaselineIndex;
		FReplicationProtocolOperations::DeserializeWithMask(Context, Info.ChangeMaskOrPointer.GetPointer(ObjectInfo.ChangeMaskBitCount), ObjectData.ReceiveStateBuffer, ObjectData.Protocol);
	}
}

FPendingBatchData* FReplicationReader::UpdateUnresolvedMustBeMappedReferences(FNetRefHandle OwnerHandle, TArray<FNetRefHandle>& MustBeMappedReferences, EIrisAsyncLoadingPriority InIrisAsyncLoadingPriority)
{
	FPendingBatchData* PendingBatch = PendingBatchHolder.Find(OwnerHandle);

	// Take the priority if it was sent with the MustBeMapped reference list
	EIrisAsyncLoadingPriority OwnerAsyncLoadingPriority = InIrisAsyncLoadingPriority != EIrisAsyncLoadingPriority::Invalid ? InIrisAsyncLoadingPriority : EIrisAsyncLoadingPriority::Default;

	// If we already have a pending batch, let's append the previous references to this new list.
	if (PendingBatch)
	{
		for (const FNetRefHandle PreviousRef : PendingBatch->PendingMustBeMappedReferences)
		{
			MustBeMappedReferences.AddUnique(PreviousRef);
		}

		if (PendingBatch->IrisAsyncLoadingPriority != EIrisAsyncLoadingPriority::Invalid)
		{
			OwnerAsyncLoadingPriority = PendingBatch->IrisAsyncLoadingPriority;
		}
	}

	// Resolve
	TArray<FNetRefHandle, TInlineAllocator<4>> Unresolved;
	TArray<TPair<FNetRefHandle, UObject*>, TInlineAllocator<4>> ResolvedReferencesToTrack;

	Unresolved.Reserve(MustBeMappedReferences.Num());
	ResolvedReferencesToTrack.Reserve(MustBeMappedReferences.Num());

	// Assign the priority of the current object
	ResolveContext.AsyncLoadingPriority = ConvertAsyncLoadingPriority(OwnerAsyncLoadingPriority);
	ON_SCOPE_EXIT
	{
		// Revert to an invalid priority to detect paths that are not setting the priority
		ResolveContext.AsyncLoadingPriority = INDEX_NONE;
	};

	for (FNetRefHandle Handle : MustBeMappedReferences)
	{
		UObject* ResolvedObject = nullptr;

		// TODO: Report broken status in same call to avoid map lookup
		ENetObjectReferenceResolveResult ResolveResult = ObjectReferenceCache->ResolveObjectReference(FObjectReferenceCache::MakeNetObjectReference(Handle), ResolveContext, ResolvedObject);

#if UE_NET_ASYNCLOADING_DEBUG
		if (bForceObjectsToStall && FAsyncLoadingSimulator::IsObjectStalled(ReplicationBridge, OwnerHandle))
		{
			Unresolved.Add(Handle);
			continue;
		}
#endif

		if (EnumHasAnyFlags(ResolveResult, ENetObjectReferenceResolveResult::HasUnresolvedMustBeMappedReferences) && !ObjectReferenceCache->IsNetRefHandleBroken(Handle, true))
		{
			// The MustBeMapped reference does not exist yet
			Unresolved.Add(Handle);
		}
		else if (ResolveResult == ENetObjectReferenceResolveResult::None)
		{
			// The reference exists
			ResolvedReferencesToTrack.Emplace(Handle, ResolvedObject);
		}
	}

	// We still have references that are not loaded
	if (Unresolved.Num())
	{
		// We must create a new batch
		if (PendingBatch == nullptr)
		{
			PendingBatch = PendingBatchHolder.FindOrCreate(OwnerHandle);
		}

		// Update the list of references we are waiting on
		PendingBatch->PendingMustBeMappedReferences = MoveTemp(Unresolved);
	}
	else if (PendingBatch)
	{
		// Clear all references as they are all resolved now
		PendingBatch->PendingMustBeMappedReferences.Reset();
	}

	// If we resolved more references, add them to tracking list
	if (PendingBatch)
	{
		for (TPair<FNetRefHandle, UObject*>& NetRefHandleObjectPair : ResolvedReferencesToTrack)
		{
			if (!PendingBatch->ResolvedReferences.Contains(NetRefHandleObjectPair.Key))
			{
				PendingBatch->ResolvedReferences.Add(NetRefHandleObjectPair.Key);
				ObjectReferenceCache->AddTrackedQueuedBatchObjectReference(NetRefHandleObjectPair.Key, NetRefHandleObjectPair.Value);
			}
		}
	}

	return PendingBatch;
}

bool FReplicationReader::DoesParentExist(FNetRefHandle ParentHandle) const
{
	if (!ParentHandle.IsValid())
	{
		// If the handle we received is somehow invalid, consider it as existing to prevent objects from being blocked infinitely.
		return true;
	}

	return NetRefHandleManager->IsNetRefHandleAssigned(ParentHandle);
}

uint32 FReplicationReader::ReadObjectsInBatchWithoutSizes(FNetSerializationContext& Context, FNetRefHandle IncompleteHandle, bool bHasBatchOwnerData, uint32 BatchEndBitPosition)
{
	uint32 ReadObjectCount = 0;
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	// If the batch owner had state, we read it now
	if (bHasBatchOwnerData)
	{
		ReadObjectInBatch(Context, IncompleteHandle, false);
		if (Context.HasErrorOrOverflow())
		{
			return 0U;
		}
		++ReadObjectCount;
	}

	if (!ensure(Reader.GetPosBits() <= BatchEndBitPosition))
	{
		Context.SetError(GNetError_BitStreamOverflow);
		return 0U;
	}

	// ReadSubObjects 
	while (Reader.GetPosBits() < BatchEndBitPosition)
	{
		ReadObjectInBatch(Context, IncompleteHandle, true);
		if (Context.HasErrorOrOverflow())
		{
			return 0U;
		}
		++ReadObjectCount;
	}

	return ReadObjectCount;
}

uint32 FReplicationReader::ReadObjectsInBatchWithSizes(FNetSerializationContext& Context, FNetRefHandle IncompleteHandle, const bool bHasBatchOwnerData, uint32 BatchEndBitPosition)
{
#if !UE_NET_REPLICATIONDATASTREAM_DEBUG
	return 0;
#else
	const uint32 BatchSize = BatchEndBitPosition - Context.GetBitStreamReader()->GetPosBits();
	FNetBitStreamReader BatchReader = Context.GetBitStreamReader()->CreateSubstream(BatchSize);

	FNetSerializationContext ObjectContext;
	FNetBitStreamReader ObjectReader;

	bool bIsSubObject = !bHasBatchOwnerData;
	uint32 ReadObjectCount = 0;
	while (BatchReader.GetBitsLeft() > 0U)
	{
		{
			UE_NET_TRACE_SCOPE(BatchSize, BatchReader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
			const uint32 ObjectSize = BatchReader.ReadBits(Parameters.NumBitsUsedForHugeObjectBatchSize);
			ObjectReader = BatchReader.CreateSubstream(ObjectSize);

			// If we were unable to create a substream of the correct size we have bitstream corruption.
			if (ObjectReader.GetBitsLeft() != ObjectSize)
			{
				BatchReader.DiscardSubstream(ObjectReader);
				Context.GetBitStreamReader()->DiscardSubstream(BatchReader);
				Context.SetError(GNetError_BitStreamError);
				break;
			}
		}

		ObjectContext = Context.MakeSubContext(&ObjectReader);
		ReadObjectInBatch(ObjectContext, IncompleteHandle, bIsSubObject);

		// Check for error
		if (ObjectContext.HasErrorOrOverflow())
		{
			BatchReader.DiscardSubstream(ObjectReader);
			Context.GetBitStreamReader()->DiscardSubstream(BatchReader);
			Context.SetError(ObjectContext.GetError(), ObjectReader.IsOverflown());
			break;
		}

		// Check for underflow
		if (ObjectReader.GetBitsLeft() != 0U)
		{
			BatchReader.DiscardSubstream(ObjectReader);
			Context.GetBitStreamReader()->DiscardSubstream(BatchReader);
			Context.SetError(GNetError_BitStreamError);
			UE_LOGF(LogIris, Error, "FReplicationReader::ReadObjectsInBatchWithSizes Bitstream underflow after reading replicated object with Handle: %ls. %ls", ToCStr(ObjectContext.GetErrorHandleContext().ToString()), ToCStr(ObjectContext.PrintReadJournal()));
			break;
		}

		BatchReader.CommitSubstream(ObjectReader);

		++ReadObjectCount;
		// Only the first batch can be the subobject owner. The rest will be subobjects.
		bIsSubObject = true;
	}

	if (Context.HasErrorOrOverflow())
	{
		return 0U;
	}
	else
	{
		Context.GetBitStreamReader()->CommitSubstream(BatchReader);
		return ReadObjectCount;
	}
#endif
}

uint32 FReplicationReader::ReadObjectsInBatch(FNetSerializationContext& Context, FNetRefHandle IncompleteHandle, bool bHasBatchOwnerData, uint32 BatchEndBitPosition)
{
#if UE_NET_REPLICATIONDATASTREAM_DEBUG
	if (EnumHasAnyFlags(StreamDebugFeatures, EReplicationDataStreamDebugFeatures::BatchSizePerObject))
	{
		return ReadObjectsInBatchWithSizes(Context, IncompleteHandle, bHasBatchOwnerData, BatchEndBitPosition);
	}
	else
#endif
	{
		return ReadObjectsInBatchWithoutSizes(Context, IncompleteHandle, bHasBatchOwnerData, BatchEndBitPosition);
	}
}

uint32 FReplicationReader::ReadObjectBatch(FNetSerializationContext& Context, uint32 ReadObjectFlags)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	UE_NET_TRACE_SCOPE(Batch, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	UE_ADD_READ_JOURNAL_ENTRY(Context, TEXT("ReadObjectBatch"));

	// Special handling for destruction infos
	if (const bool bIsDestructionInfo = Reader.ReadBool())
	{
		FReplicationBridgeSerializationContext BridgeContext(Context, Parameters.ConnectionId, true);

		// For destruction infos we inline the exports
		FForceInlineExportScope ForceInlineExportScope(Context.GetInternalContext());
		FBridgeSerialization::ReadAndExecuteDestructionInfoFromRemote(ReplicationBridge, BridgeContext);

		ReadSentinel(Context, TEXT("DestructionInfo"));

		if (Context.HasErrorOrOverflow())
		{
			UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Failed to read destruction info. \n%ls", *Context.PrintReadJournal());
			Context.SetError(GNetError_BitStreamError);
			return 0U;
		}

		return 1U;
	}
	
	if (!ReadSentinel(Context, TEXT("Object")))
	{
		UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Failed to read object sentinel. \n%ls", *Context.PrintReadJournal());
		Context.SetError(GNetError_BitStreamError);
		return 0U;
	}

	uint32 ObjectsReadInBatch = 0U;
	
	// A batch starts with (RefHandleId | BatchSize | bHasBatchObjectData | bHasExports | bHasCreationDependencyHandles | ?CreationParentHandleList)

	// If the batch has exports we must to seek to the end of the batch to read and process exports before reading/processing batch data
	const FNetRefHandle IncompleteHandle = ReadNetRefHandleId(Context, Reader);

	uint32 BatchSize = 0U;
	// Read Batch size
	{
		const uint32 NumBitsUsedForBatchSize = (ReadObjectFlags & EReadObjectFlag::ReadObjectFlag_IsReadingHugeObjectBatch) == 0U ? Parameters.NumBitsUsedForBatchSize : Parameters.NumBitsUsedForHugeObjectBatchSize;

		UE_NET_TRACE_SCOPE(BatchSize, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		BatchSize = Reader.ReadBits(NumBitsUsedForBatchSize);
	}

	// Store the current handle if we encounter errors
	Context.SetErrorHandleContext(IncompleteHandle);

	if (Context.HasErrorOrOverflow() || BatchSize > Reader.GetBitsLeft())
	{
		Context.SetError(GNetError_InvalidValue);
		return 0U;
	}

	// This either marks the end of the data associated with this batch or the offset in the stream where exports are stored.
	const uint32 BatchEndOrStartOfExportsPos = Reader.GetPosBits() + BatchSize;

	// Do we have state data or attachments for the owner of the batch?
	const bool bHasBatchOwnerData = Reader.ReadBool();

	// Do we have exports or not?
	const bool bHasExports = Reader.ReadBool();

	// Read the optional creation dependency handles
	TArray<FNetRefHandle, TInlineAllocator<4>> CreationDependentParents;
	{
		bool bHasCreationDependencyHandles = Reader.ReadBool();
		while (bHasCreationDependencyHandles)
		{
			CreationDependentParents.Emplace(ReadNetRefHandle(Context));
			bHasCreationDependencyHandles = Reader.ReadBool();

			//UE_LOGF(LogIris, Verbose, "Reading CreationDependency for child: %ls. Dependent Parent: %ls %ls", *IncompleteHandle.ToString(), *CreationDependentParents.Last().ToString(), DoesParentExist(CreationDependentParents.Last())?TEXT("Exists"):TEXT("NotFound"));
		}
	}

	uint32 ReadObjectCount = 0U;

	// First we need to read exports, they are stored at the end of the batch
	uint32 BatchEndPos = BatchEndOrStartOfExportsPos;

	TempMustBeMappedReferences.Reset();
	EIrisAsyncLoadingPriority IrisAsyncLoadingPriority = EIrisAsyncLoadingPriority::Invalid;

	if (bHasExports)
	{
		const uint32 ReturnPos = Reader.GetPosBits();

		// Seek to the export section
		Reader.Seek(BatchEndPos);

		// Read exports and any must be mapped references
		ObjectReferenceCache->ReadExports(IncompleteHandle, Context, &TempMustBeMappedReferences, IrisAsyncLoadingPriority);
		if (Context.HasErrorOrOverflow())
		{
			UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Failed to read exports for handle: %ls.\n%ls", *IncompleteHandle.ToString(), *Context.PrintReadJournal());
			return 0U;
		}

		// Update BatchEndPos if we successfully read exports
		BatchEndPos = Reader.GetPosBits();

		// Seek back to state data
		Reader.Seek(ReturnPos);
	}

	// Skip over broken objects
	const bool bIsBroken = BrokenObjects.FindByKey(IncompleteHandle) != nullptr;
	if (bIsBroken)
	{
		UE_NET_TRACE_OBJECT_SCOPE(IncompleteHandle, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		UE_NET_TRACE_SCOPE(SkippedData, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

		Reader.Seek(BatchEndPos);
		return 0U;
	}

	// False if the object has a creation dependency and one of it's parent does not exist yet
	bool bDoAllParentsExist = true;

	if (CreationDependentParents.Num() > 0)
	{
		for (FNetRefHandle ParentHandle : CreationDependentParents)
		{
			const bool bParentExists = DoesParentExist(ParentHandle);

			// Check if its a broken handle
			if (!bParentExists && BrokenObjects.FindByKey(ParentHandle))
			{
				// The parent will never resolve so the object can never resolve also. Mark it broken so we don't queue up it's data indefinitely.

				//TODO: Test and support an object removing a broken dependency and becoming resolveable again.

				UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject RootObject: %ls has a creation dependency on a broken parent: %ls. RootObject will be marked broken too", *IncompleteHandle.ToString(), *ParentHandle.ToString());
					
				BrokenObjects.AddUnique(IncompleteHandle);
					
				UE_NET_TRACE_OBJECT_SCOPE(IncompleteHandle, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
				UE_NET_TRACE_SCOPE(SkippedData, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

				Reader.Seek(BatchEndPos);
				return 0U;
			}

			bDoAllParentsExist = bDoAllParentsExist && bParentExists;
		}
	}

	FPendingBatchData* PendingBatchData = nullptr; 
	
	if (ObjectReferenceCache->ShouldAsyncLoad())
	{
		// Check if this object has pending must be mapped references that must be resolved before we can process the data.
		PendingBatchData = UpdateUnresolvedMustBeMappedReferences(IncompleteHandle, TempMustBeMappedReferences, IrisAsyncLoadingPriority);
	}
	
	// Normally we will only receive a single state/attachment update for an object in a single packet
	// There are however a special case if we allow multiple HO-states to be in flight for the same object and they get assembled and dispatched during procssing of the same packet.
	// In that case we want to dispatch them separately to maintain attachment/state application order.
	bool bForcePendingBatchDataForHugeObject = false;
	if (ReadObjectFlags & EReadObjectFlag::ReadObjectFlag_IsReadingHugeObjectBatch)
	{
		if (!ObjectsReadFromHugeObjects.Contains(IncompleteHandle))
		{
			ObjectsReadFromHugeObjects.Add(IncompleteHandle);
		}
		else
		{
			bForcePendingBatchDataForHugeObject = true;
		}
	}
		
	// Force create pending batch data if we have multiple incoming hugeobject batches for the same object or if we have unfulfilled creation dependencies.
	const bool bForceCreatePendingBatchData = bForcePendingBatchDataForHugeObject || !bDoAllParentsExist;
	if (bForceCreatePendingBatchData)
	{
		PendingBatchData = PendingBatchHolder.FindOrCreate(IncompleteHandle);
	}

	// If the object's data cannot be processed and must be queued for later
	if (PendingBatchData)
	{
		// TODO: Test and support a case where the parents changed and now exist but we still have queued data waiting on the previous parents.
		if (!bDoAllParentsExist)
		{
			PendingBatchData->CreationDependentParents = CreationDependentParents;
		}

		UE_LOGF(LogIris, Verbose, "FReplicationReader::ReadObjectBatch Handle %ls will be defered as it has %d unresolved references and/or %d dependent parents not yet created.", 
			*IncompleteHandle.ToString(), PendingBatchData->PendingMustBeMappedReferences.Num(), PendingBatchData->CreationDependentParents.Num());

		PendingBatchData->IrisAsyncLoadingPriority = IrisAsyncLoadingPriority != EIrisAsyncLoadingPriority::Invalid ? IrisAsyncLoadingPriority : EIrisAsyncLoadingPriority::Default;

		UE_NET_TRACE_OBJECT_SCOPE(IncompleteHandle, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		UE_NET_TRACE_SCOPE(QueuedBatch, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

		// Enqueue BatchData
		FQueuedDataChunk DataChunk;
		
		const uint32 NumDataBits = BatchEndOrStartOfExportsPos - Reader.GetPosBits();
		const uint32 NumDataWords = (NumDataBits + 31U) / 32U;

		DataChunk.NumBits = NumDataBits;
		ensureMsgf((uint32)DataChunk.NumBits == NumDataBits, TEXT("NumDataBits: %u does not fit inside FQueuedDataChunk::NumBits bitfield"), NumDataBits);

		DataChunk.StorageOffset = PendingBatchData->DataChunkStorage.Num();
		DataChunk.bHasBatchOwnerData = bHasBatchOwnerData;
		DataChunk.bIsEndReplicationChunk = false;
		DataChunk.StreamDebugFeatures = StreamDebugFeatures;

		// Make sure we have space
		PendingBatchData->DataChunkStorage.AddUninitialized(NumDataWords);
	
		// Store batch data
		Reader.ReadBitStream(PendingBatchData->DataChunkStorage.GetData() + DataChunk.StorageOffset, DataChunk.NumBits);

		if (Context.HasErrorOrOverflow())
		{
			// Log error, this is something we cannot recover from.
			UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Failed to read object batch data chunk for handle:%ls \n%ls", ToCStr(IncompleteHandle.ToString()), *Context.PrintReadJournal());

			return 0U;
		}

		PendingBatchData->QueuedDataChunks.Add(MoveTemp(DataChunk));
	}
	// Everything is good let's process the received data
	else
	{
		ReadObjectCount = ReadObjectsInBatch(Context, IncompleteHandle, bHasBatchOwnerData, BatchEndOrStartOfExportsPos);

		if (Context.HasErrorOrOverflow())
		{
			if (Context.GetError() == GNetError_BrokenNetHandle)
			{
				ReplicationBridge->SendErrorWithNetRefHandle(UE::Net::ENetRefHandleError::ReplicationDisabled, IncompleteHandle, Parameters.ConnectionId, {}, Context.GetErrorContext());

				// Log error and try to recover, if get more incoming data for an object in the broken state we will skip it.
				UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Failed to read object batch handle: %ls skipping batch data", ToCStr(IncompleteHandle.ToString()));
					
				BrokenObjects.AddUnique(IncompleteHandle);

				Context.ResetErrorContext();

				UE_NET_TRACE_OBJECT_SCOPE(IncompleteHandle, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
				UE_NET_TRACE_SCOPE(SkippedData, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

				// Skip this batch
				Reader.Seek(BatchEndPos);
			}
			
			return 0U;
		}
	}

	if (!ensure(Reader.GetPosBits() == BatchEndOrStartOfExportsPos))
	{
		UE_LOGF(LogIris, Error, "FReplicationReader::ReadObjectsInBatch Did not read the expected number of bits when reading batch: %ls. \n%ls", *IncompleteHandle.ToString(), *Context.PrintReadJournal());
		Context.SetError(GNetError_BitStreamOverflow, true);
		return 0U;
	}

	// Skip to the end as we already have read any exports
	Reader.Seek(BatchEndPos);

	return ReadObjectCount;
}

void FReplicationReader::ReadObjectInBatch(FNetSerializationContext& Context, FNetRefHandle BatchHandle, bool bIsSubObject)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	const FNetRefHandle IncompleteHandle = !bIsSubObject ? BatchHandle : ReadNetRefHandleId(Context, Reader);
	
	// Read replicated destroy header if necessary. We don't know the internal index yet so can't do the more appropriate check IsObjectIndexForOOBAttachment.
	const bool bReadReplicatedDestroyHeader = IncompleteHandle.IsValid();
	EReplicatedDestroyHeaderFlags ReplicatedDestroyHeaderFlags = EReplicatedDestroyHeaderFlags::None;
	if (bReadReplicatedDestroyHeader)
	{
		ReplicatedDestroyHeaderFlags = (EReplicatedDestroyHeaderFlags)Reader.ReadBits(GetDestroyHeaderFlagsBitCount());
	}
	 

	const bool bHasState = Reader.ReadBool();
	if (bHasState)
	{
		if (!ReadSentinel(Context, TEXT("HasState")))
		{
			UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Failed to read replicated object with Handle: %ls. Error '%ls'. %ls", ToCStr(IncompleteHandle.ToString()), (Context.HasError() ? ToCStr(Context.GetError().ToString()) : TEXT("BitStream Overflow")), ToCStr(Context.PrintReadJournal()));
			return;
		}
	}

	uint32 NewBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;

	const bool bIsInitialState = bHasState && Reader.ReadBool();
	
	bool bShouldCallSubObjectCreatedFromReplication = false;
	uint32 InternalIndex = ObjectIndexForOOBAttachment;

	UE_NET_TRACE_OBJECT_SCOPE(IncompleteHandle, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	// Store the current handle in case we encounter errors
	if (IncompleteHandle != BatchHandle)
	{
		Context.SetErrorHandleContext(IncompleteHandle);
	}

	//UE_LOG_REPLICATIONREADER(TEXT("FReplicationReader::Read Object with %s InitialState: %u"), *IncompleteHandle.ToString(), bIsInitialState ? 1u : 0u);

	bool bHasErrors = false;
	bool bIsReplicatedDestroyForInvalidObject = false;

	// Read creation data
	Context.SetIsInitState(bIsInitialState);
	if (bIsInitialState)
	{
		UE_NET_TRACE_SCOPE(CreationInfo, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

		UE_ADD_READ_JOURNAL_ENTRY(Context, TEXT("ReadCreationInfo"));

		// SubObject data for initial state
		FNetRefHandle RootObjectOfSubObject;
		if (bIsSubObject)
		{
			// The owner is the same as the Batch owner
			const FNetRefHandle IncompleteOwnerHandle = BatchHandle;
				
			FInternalNetRefIndex RootObjectInternalIndex = NetRefHandleManager->GetInternalIndex(IncompleteOwnerHandle);
			if (Reader.IsOverflown() || RootObjectInternalIndex == InvalidInternalNetRefIndex)
			{
				UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Invalid subobjectowner handle. %ls", ToCStr(IncompleteOwnerHandle.ToString()));
				const FName& NetError = (Reader.IsOverflown() ? GNetError_BitStreamOverflow : GNetError_InvalidNetHandle);
				Context.SetError(NetError);

				bHasErrors = true;
				goto ErrorHandling;
			}

			RootObjectOfSubObject = NetRefHandleManager->GetReplicatedObjectDataNoCheck(RootObjectInternalIndex).RefHandle;
		}

		const bool bIsDeltaCompressed = Reader.ReadBool();
		if (bIsDeltaCompressed)
		{
			UE_LOG_REPLICATIONREADER(TEXT("DeltaCompression is enabled for Handle %s"), *IncompleteHandle.ToString());
			NewBaselineIndex = Reader.ReadBits(FDeltaCompressionBaselineManager::BaselineIndexBitCount);
		}
		
		// We got a read error
		if (Reader.IsOverflown() || !IncompleteHandle.IsValid())
		{
			UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Bitstream corrupted.");
			const FName& NetError = (Reader.IsOverflown() ? GNetError_BitStreamOverflow : GNetError_BitStreamError);
			Context.SetError(NetError);

			bHasErrors = true;
			goto ErrorHandling;
		}
	
		// Get Bridge
		FReplicationBridgeSerializationContext BridgeContext(Context, Parameters.ConnectionId);
		const FReplicationBridgeCreateNetRefHandleResult CreateResult = FBridgeSerialization::CreateNetRefHandleFromRemote(ReplicationBridge, RootObjectOfSubObject, IncompleteHandle, BridgeContext);
		FNetRefHandle NetRefHandle = CreateResult.NetRefHandle;
		if (!NetRefHandle.IsValid())
		{	
			UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Unable to create handle for %ls.", *IncompleteHandle.ToString());

			// Mark error, but do not mark the bitstream as overflown as we want to handle this error.
			Context.SetError(GNetError_BrokenNetHandle, false);

			bHasErrors = true;
			goto ErrorHandling;
		}

		// If this handle is considered unresolved, add it to the hot cache to force a resolve.
		RemoveFromUnresolvedCache(NetRefHandle);

		InternalIndex = NetRefHandleManager->GetInternalIndex(NetRefHandle);
		FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);

		// Cache the creation flags given by the factory
		ObjectData.bAllowDestroyInstanceFromRemote = EnumHasAnyFlags(CreateResult.Flags, EReplicationBridgeCreateNetRefHandleResultFlags::AllowDestroyInstanceFromRemote);
		bShouldCallSubObjectCreatedFromReplication = EnumHasAnyFlags(CreateResult.Flags, EReplicationBridgeCreateNetRefHandleResultFlags::ShouldCallSubObjectCreatedFromReplication);
		
		FReplicatedObjectInfo& ObjectInfo = StartReplication(InternalIndex);

		ObjectInfo.bIsDeltaCompressionEnabled = bIsDeltaCompressed;

#if IRIS_CLIENT_PROFILER_ENABLE
		if (UE::Net::FClientProfiler::IsCapturing())
		{
			UObject* Object = NetRefHandleManager->GetReplicatedObjectInstance(InternalIndex);
			if (Object)
			{
				UE::Net::FClientProfiler::RecordObjectCreate(Object->GetClass()->GetFName(), bIsSubObject);
			}
		}
#endif
	}
	else
	{
		bHasErrors = Context.HasErrorOrOverflow();
		UE_CLOGF(bHasErrors, LogIris, Error, "FReplicationReader::ReadObject ErrorOrOverFlow after reading object header. Overflow: %c Error: %ls Bit stream bits left: %u position: %u %ls", TEXT("YN")[Context.HasError()], ToCStr(Context.GetError().ToString()), Reader.GetBitsLeft(), Reader.GetPosBits(), *Context.PrintReadJournal())

		if (bHasErrors || !IncompleteHandle.IsValid())
		{
			InternalIndex = ObjectIndexForOOBAttachment;
		}
		else
		{
			// If we get back an invalid internal index then either the object has been deleted or there's bitstream corruption.
			InternalIndex = NetRefHandleManager->GetInternalIndex(IncompleteHandle);

			if (InternalIndex == InvalidInternalNetRefIndex)
			{
				if (!EnumHasAnyFlags(ReplicatedDestroyHeaderFlags, EReplicatedDestroyHeaderFlags::IsSubObject))
				{
					bHasErrors = true;
					UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Handle %ls not bound to any InternalIndex", *IncompleteHandle.ToString());
				}
				else
				{
					// If this is a subobject that is being destroyed this was no error as we send destroy info for unconfirmed object
					bIsReplicatedDestroyForInvalidObject = true;
				}
			}
		}
	}

	if (bHasErrors)
	{
		Context.SetError(GNetError_InvalidNetHandle);
		goto ErrorHandling;
	}

	// Read state data and attachments
	{
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);

		UE_ADD_READ_JOURNAL_ENTRY(Context, ObjectData.Protocol ? ObjectData.Protocol->DebugName->Name : TEXT("OOB"));

		// Add entry in our received data as we postpone state application until we have received all data in order to be able to properly resolve references
		FDispatchObjectInfo& Info = ObjectsToDispatchArray->AddPendingDispatchObjectInfo(TempLinearAllocator);

		// Update info based on ReplicatedDestroyHeader
		const bool bIsTearOff = EnumHasAnyFlags(ReplicatedDestroyHeaderFlags, EReplicatedDestroyHeaderFlags::TearOff);
		const bool bIsForSubObject = EnumHasAnyFlags(ReplicatedDestroyHeaderFlags, EReplicatedDestroyHeaderFlags::IsSubObject);
		const bool bDestroySubObject = EnumHasAnyFlags(ReplicatedDestroyHeaderFlags, EReplicatedDestroyHeaderFlags::DestroySubObject);

		if (bIsTearOff)
		{
			Info.bWantsDeferredEndReplication = true;

			// TornOff objects don't go through the destruction flow on the sending side.
			// So no detach reason was serialized in the writer.
			Info.InternalDetachReason = EInternalDetachReason::TornOff;
		}
		else if (bIsForSubObject)
		{
			// TODO: Find real subobject reason
			Info.InternalDetachReason = EInternalDetachReason::Normal;
			if (bDestroySubObject)
			{
				Info.bWantsImmediateEndReplication = true;
				Info.bWantsDeferredEndReplication = false;
			}
			else
			{
				Info.bWantsDeferredEndReplication = true;
			}
		}

		if (bIsInitialState)
		{
			Info.bShouldCallSubObjectCreatedFromReplication = bShouldCallSubObjectCreatedFromReplication;
		}

		if (bHasState)
		{
			UE_ADD_READ_JOURNAL_ENTRY(Context, TEXT("State"));

			if (IsObjectIndexForOOBAttachment(InternalIndex) || bIsReplicatedDestroyForInvalidObject)
			{
				bHasErrors = true;
				UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Bitstream corrupted. Getting state when not expecting state data.");
				Context.SetError(GNetError_BitStreamError);
				goto ErrorHandling;
			}

			FReplicatedObjectInfo* ObjectInfo = GetReplicatedObjectInfo(InternalIndex);
			checkSlow(ObjectInfo);

			const uint32 ChangeMaskBitCount = ObjectData.Protocol->ChangeMaskBitCount;

			// Temporary changemask
			FChangeMaskStorageOrPointer::Alloc(Info.ChangeMaskOrPointer, ChangeMaskBitCount, TempChangeMaskAllocator);

			if (bIsInitialState)
			{
				FReplicationProtocolOperations::DeserializeInitialStateWithMask(Context, Info.ChangeMaskOrPointer.GetPointer(ChangeMaskBitCount), ObjectData.ReceiveStateBuffer, ObjectData.Protocol);
			}
			else
			{
				if (ObjectInfo->bIsDeltaCompressionEnabled)
				{
					DeserializeObjectStateDelta(Context, InternalIndex, Info, *ObjectInfo, ObjectData, NewBaselineIndex);
				}
				else
				{
					FReplicationProtocolOperations::DeserializeWithMask(Context, Info.ChangeMaskOrPointer.GetPointer(ChangeMaskBitCount), ObjectData.ReceiveStateBuffer, ObjectData.Protocol);
				}
			}

			if (!ReadSentinel(Context, TEXT("HasStateEnd")))
			{
				bHasErrors = true;
				goto ErrorHandling;
			}

			// Should we store a new baseline?
			if (NewBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
			{
				// This object uses delta compression, store the last received state as a baseline with the specified index
				UE_LOG_REPLICATIONREADER(TEXT("Storing new baselineindex: %u for (:%u) Handle %s"), NewBaselineIndex, InternalIndex, *ObjectData.RefHandle.ToString());

				check(NewBaselineIndex < FDeltaCompressionBaselineManager::MaxBaselineCount);
				if (ObjectInfo->StoredBaselines[NewBaselineIndex])
				{
					// Clone into already allocated state, unfortunately we have to free dynamic state
					FReplicationProtocolOperations::FreeDynamicState(Context, ObjectInfo->StoredBaselines[NewBaselineIndex], ObjectData.Protocol);
					FReplicationProtocolOperationsInternal::CloneQuantizedState(Context, ObjectInfo->StoredBaselines[NewBaselineIndex], ObjectData.ReceiveStateBuffer, ObjectData.Protocol);
				}
				else
				{
					// Allocate new baseline and initialize from current RecvState
					ObjectInfo->StoredBaselines[NewBaselineIndex] = StateStorage->AllocBaseline(InternalIndex, EReplicationStateType::CurrentRecvState);
				}
	
				// Make sure that PrevStoredBaselineIndex is not set to the same as the NewBaselineIndex
				const uint32 OldPrevStoredBaselineIndex = ObjectInfo->PrevStoredBaselineIndex;
				const uint32 NewPrevStoredBaselineIndex = NewBaselineIndex != ObjectInfo->LastStoredBaselineIndex ? ObjectInfo->LastStoredBaselineIndex : FDeltaCompressionBaselineManager::InvalidBaselineIndex;
				ObjectInfo->PrevStoredBaselineIndex = NewPrevStoredBaselineIndex;
				if (NewPrevStoredBaselineIndex == FDeltaCompressionBaselineManager::InvalidBaselineIndex && NewPrevStoredBaselineIndex != OldPrevStoredBaselineIndex)
				{
					uint8* PrevBaseline = ObjectInfo->StoredBaselines[OldPrevStoredBaselineIndex];
					ObjectInfo->StoredBaselines[OldPrevStoredBaselineIndex] = nullptr;
					StateStorage->FreeBaseline(InternalIndex, PrevBaseline);
				}
				ObjectInfo->LastStoredBaselineIndex = NewBaselineIndex;
			}
		}

		const bool bHasAttachments = Reader.ReadBool();
		ENetObjectAttachmentType AttachmentType = ENetObjectAttachmentType::Normal;
		if (bHasAttachments)
		{
			UE_ADD_READ_JOURNAL_ENTRY(Context, TEXT("Attachments"));

			if (IsObjectIndexForOOBAttachment(InternalIndex))
			{
				bHasErrors = bIsReplicatedDestroyForInvalidObject;
				UE_CLOGF(bHasErrors, LogIris, Error, "FReplicationReader::ReadObject Bitstream corrupted. Reading attachments when this was a destroy info message.");

				if (!bHasErrors)
				{
					const uint32 ReadAttachmentType = Reader.ReadBits(NetObjectAttachmentTypeBitCount);
					AttachmentType = (ENetObjectAttachmentType)ReadAttachmentType;
					bHasErrors = (!Parameters.bAllowReceivingAttachmentsFromRemoteObjectsNotInScope && AttachmentType == ENetObjectAttachmentType::OutOfBand);
					UE_CLOGF(bHasErrors, LogIris, Error, "FReplicationReader::ReadObject Bitstream corrupted. Reading OutOfBand attachment for object not in scope.");
				}

				if (bHasErrors)
				{
					Context.SetError(GNetError_InvalidNetHandle);
					goto ErrorHandling;
				}
			}

			Attachments.Deserialize(Context, AttachmentType, InternalIndex, ObjectData.RefHandle);
		}

		if (Context.HasErrorOrOverflow())
		{
			bHasErrors = true;
			UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject ErrorOrOverflow after reading bitstream.");
			goto ErrorHandling;
		}

		// Fill in ReadObjectInfo, we must skip objects that has not been created and HugeObjects as they are not added to the dispatch list until they are fully assembled
		const bool bShouldCommitPendingDispatchObjectInfo = !(AttachmentType == ENetObjectAttachmentType::HugeObject || AttachmentType == ENetObjectAttachmentType::DebugObject) && !bIsReplicatedDestroyForInvalidObject;
		if (bShouldCommitPendingDispatchObjectInfo)
		{
			Info.InternalIndex = InternalIndex;
			Info.bIsInitialState = bIsInitialState;
			Info.bHasState = bHasState;
			Info.bHasAttachments = bHasAttachments;

			ObjectsToDispatchArray->CommitPendingDispatchObjectInfo();
		}
	}
	
ErrorHandling:
	if (bHasErrors)
	{
		Context.SetErrorHandleContext(IncompleteHandle);
		UE_LOGF(LogIris, Error, "FReplicationReader::ReadObject Failed to read replicated object with Handle: %ls. Error '%ls'. %ls", *IncompleteHandle.ToString(), (Context.HasError() ? ToCStr(Context.GetError().ToString()) : TEXT("BitStream Overflow")), *Context.PrintReadJournal());
	}
}

// Update reference tracking maps for the current object. It is assumed the ObjectReferenceTracker do no include duplicates for a given key.
void FReplicationReader::UpdateObjectReferenceTracking_Fast(FReplicatedObjectInfo* ReplicationInfo, FNetBitArrayView ChangeMask, bool bIncludeInitState, FResolvedNetRefHandlesArray& OutNewResolvedRefHandles, const FObjectReferenceTracker& NewUnresolvedReferences, const FObjectReferenceTracker& NewMappedDynamicReferences)
{
	IRIS_PROFILER_SCOPE(FReplicationReader_UpdateObjectReferenceTracking);

	/*
	 * As we store references per changemask we need to construct a set of all unresolved references and
	 * compare with the new set of unresolved references. The new set is found by first updating the 
	 * references that were found in the changemask.
	 */
	{
		// Try to avoid dynamic allocations during the update of the UnresolvedObjectReferences.
		ReplicationInfo->UnresolvedObjectReferences.Reserve(ReplicationInfo->UnresolvedObjectReferences.Num() + NewUnresolvedReferences.Num());
		ReplicationInfo->UnresolvedHandleCount.Reserve(ReplicationInfo->UnresolvedObjectReferences.Num() + NewUnresolvedReferences.Num());

		// Replace each entry in UnresolvedObjectReferences for the given changemask
		auto UpdateUnresolvedReferencesForChange = [ReplicationInfo, &NewUnresolvedReferences, &OutNewResolvedRefHandles, this](uint32 ChangeBit)
		{
			FObjectReferenceTracker& UnresolvedObjectReferences = ReplicationInfo->UnresolvedObjectReferences;

			bool bUnresolvedHandleCountShrink = false;

			for (FObjectReferenceTracker::TKeyIterator It = UnresolvedObjectReferences.CreateKeyIterator(ChangeBit); It; ++It)
			{
				const FNetRefHandle RefHandle = It.Value();

				const bool bInNewUnresolved = (NewUnresolvedReferences.FindPair(ChangeBit, RefHandle) != nullptr);
				if (!bInNewUnresolved)
				{
					It.RemoveCurrent();

					if (ReplicationInfo->RemoveUnresolvedHandleCount(RefHandle))
					{
						bUnresolvedHandleCountShrink = true;
							
						// Store new resolved handles so we can update partially resolved references properly
						OutNewResolvedRefHandles.Add(RefHandle);

						// Remove from tracking
						const uint32 OwnerInternalIndex = ReplicationInfo->InternalIndex;
						UnresolvedHandleToDependents.Remove(RefHandle, OwnerInternalIndex);
						RemoveFromUnresolvedCache(RefHandle);
						UE_LOGF(LogIris, Verbose, "FReplicationReader::UpdateObjectReferenceTracking Removing unresolved reference %ls for %ls (OwnerInternalIndex=%d)", ToCStr(RefHandle.ToString()), ToCStr(NetRefHandleManager->GetNetRefHandleFromInternalIndex(OwnerInternalIndex).ToString()), OwnerInternalIndex);
					}
				}
			}

			for (FObjectReferenceTracker::TConstKeyIterator It = NewUnresolvedReferences.CreateConstKeyIterator(ChangeBit); It; ++It)
			{
				const FNetRefHandle RefHandle = It.Value();
				const bool bInCurrUnresolved = (UnresolvedObjectReferences.FindPair(ChangeBit, RefHandle) != nullptr);

				if (!bInCurrUnresolved)
				{
					UnresolvedObjectReferences.Add(ChangeBit, RefHandle);

					int16& HandleCount = ReplicationInfo->UnresolvedHandleCount.FindOrAdd(RefHandle, 0);

					HandleCount++;

					// Add to tracking
					const uint32 OwnerInternalIndex = ReplicationInfo->InternalIndex;

					// Calling TMultiMap::Add() as this is O(1) performance as oppossed to TMultiMap::AddUnique() which is O(n). This does 
					// mean that UnresolvedHandleToDependents can contain duplicate (RefHandle, OwnerInternalIndex) pairs but is handled
					// gracefully by the rest of the code. It does mean that TMultiMap::Remove() must be called instead of TMultiMap::RemoveSingle()
					// to remove all pairs.
					UnresolvedHandleToDependents.Add(RefHandle, OwnerInternalIndex);
					UE_LOGF(LogIris, Verbose, "FReplicationReader::UpdateObjectReferenceTracking Adding unresolved reference %ls for %ls (OwnerInternalIndex=%d)", ToCStr(RefHandle.ToString()), ToCStr(NetRefHandleManager->GetNetRefHandleFromInternalIndex(OwnerInternalIndex).ToString()), OwnerInternalIndex);
				}
			}

			if (bUnresolvedHandleCountShrink)
			{
				ReplicationInfo->UnresolvedHandleCount.Shrink();
			}
		};

		ChangeMask.ForAllSetBits(UpdateUnresolvedReferencesForChange);
		if (bIncludeInitState)
		{
			UpdateUnresolvedReferencesForChange(FakeInitChangeMaskOffset);
		}

		// Update ReplicationInfo with the status of unresolved references
		ReplicationInfo->bHasUnresolvedReferences = ReplicationInfo->UnresolvedHandleCount.Num() > 0;
		ReplicationInfo->bHasUnresolvedInitialReferences = ReplicationInfo->UnresolvedObjectReferences.Find(FakeInitChangeMaskOffset) != nullptr;
	}

	// Update tracking for resolved dynamic references
	if (bRemapDynamicObjects)
	{
		// Try to avoid dynamic allocations during the update of the ResolvedDynamicObjectReferences.
		ReplicationInfo->ResolvedDynamicObjectReferences.Reserve(ReplicationInfo->ResolvedDynamicObjectReferences.Num() + NewMappedDynamicReferences.Num());
		ReplicationInfo->ResolvedDynamicHandleCount.Reserve(ReplicationInfo->ResolvedDynamicObjectReferences.Num() + NewMappedDynamicReferences.Num());

		// Replace each entry in ResolvedDynamicObjectReferences for the given changemask
		auto UpdateResolvedReferencesForChange = [ReplicationInfo, &NewMappedDynamicReferences, this](uint32 ChangeBit)
		{
			FObjectReferenceTracker& ResolvedDynamicObjectReferences = ReplicationInfo->ResolvedDynamicObjectReferences;

			bool bResolvedDynamicObjectReferenceShrink = false;

			for (FObjectReferenceTracker::TKeyIterator It = ResolvedDynamicObjectReferences.CreateKeyIterator(ChangeBit); It; ++It)
			{
				const FNetRefHandle RefHandle = It.Value();

				const bool bInNewResolved = (NewMappedDynamicReferences.FindPair(ChangeBit, RefHandle) != nullptr);
				if (!bInNewResolved)
				{
					It.RemoveCurrent();

					if (ReplicationInfo->RemoveResolvedDynamicHandleCount(RefHandle))
					{
						bResolvedDynamicObjectReferenceShrink = true;

						// Remove from tracking
						const uint32 OwnerInternalIndex = ReplicationInfo->InternalIndex;
						ResolvedDynamicHandleToDependents.Remove(RefHandle, OwnerInternalIndex);
						UE_LOGF(LogIris, Verbose, "FReplicationReader::UpdateObjectReferenceTracking Removing resolved dynamic reference %ls for %ls", ToCStr(RefHandle.ToString()), ToCStr(NetRefHandleManager->GetNetRefHandleFromInternalIndex(OwnerInternalIndex).ToString()));
					}
				}
			}

			for (FObjectReferenceTracker::TConstKeyIterator It = NewMappedDynamicReferences.CreateConstKeyIterator(ChangeBit); It; ++It)
			{
				const FNetRefHandle RefHandle = It.Value();
				const bool bInCurrResolved = (ResolvedDynamicObjectReferences.FindPair(ChangeBit, RefHandle) != nullptr);

				if (!bInCurrResolved)
				{
					ResolvedDynamicObjectReferences.Add(ChangeBit, RefHandle);

					int16& HandleCount = ReplicationInfo->ResolvedDynamicHandleCount.FindOrAdd(RefHandle, 0);

					HandleCount++;

					// Add to tracking
					const uint32 OwnerInternalIndex = ReplicationInfo->InternalIndex;

					// Calling TMultiMap::Add() as this is O(1) performance as oppossed to TMultiMap::AddUnique() which is O(n). This does 
					// mean that UnresolvedHandleToDependents can contain duplicate (RefHandle, OwnerInternalIndex) pairs but is handled
					// gracefully by the rest of the code. It does mean that TMultiMap::Remove() must be called instead of TMultiMap::RemoveSingle()
					// to remove all pairs.
					ResolvedDynamicHandleToDependents.Add(RefHandle, OwnerInternalIndex);
					UE_LOGF(LogIris, Verbose, "FReplicationReader::UpdateObjectReferenceTracking Adding resolved dynamic reference %ls for %ls", ToCStr(RefHandle.ToString()), ToCStr(NetRefHandleManager->GetNetRefHandleFromInternalIndex(OwnerInternalIndex).ToString()));
				}
			}

			if (bResolvedDynamicObjectReferenceShrink)
			{
				ReplicationInfo->ResolvedDynamicHandleCount.Shrink();
			}
		};

		ChangeMask.ForAllSetBits(UpdateResolvedReferencesForChange);
		// Intentionally leaving out init state. It seems weird to call rep notifies and update init only properties after
		// the initial state has already been applied.
	}
}

void FReplicationReader::RemoveUnresolvedObjectReferenceInReplicationInfo(FReplicatedObjectInfo* ReplicationInfo, FNetRefHandle Handle)
{
	for (FObjectReferenceTracker::TIterator It = ReplicationInfo->UnresolvedObjectReferences.CreateIterator(); It; ++It)
	{
		const FNetRefHandle RefHandle = It->Value;
		if (RefHandle == Handle)
		{
			It.RemoveCurrent();

			ReplicationInfo->RemoveUnresolvedHandleCount(RefHandle);
		}
	}
}

void FReplicationReader::RemoveResolvedObjectReferenceInReplicationInfo(FReplicatedObjectInfo* ReplicationInfo, FNetRefHandle Handle)
{
	for (FObjectReferenceTracker::TIterator It = ReplicationInfo->ResolvedDynamicObjectReferences.CreateIterator(); It; ++It)
	{
		const FNetRefHandle RefHandle = It->Value;
		if (RefHandle == Handle)
		{
			It.RemoveCurrent();

			ReplicationInfo->RemoveResolvedDynamicHandleCount(RefHandle);
		}
	}
}

bool FReplicationReader::MoveResolvedObjectReferenceToUnresolvedInReplicationInfo(FReplicatedObjectInfo* ReplicationInfo, FNetRefHandle UnresolvableHandle)
{
	bool bFoundHandle = false;
	bool bHasUnresolvedReferences = ReplicationInfo->bHasUnresolvedReferences;
	bool bHasUnresolvedInitialReferences = ReplicationInfo->bHasUnresolvedInitialReferences;
	FNetBitArrayView UnresolvedChangeMask = FChangeMaskUtil::MakeChangeMask(ReplicationInfo->UnresolvedChangeMaskOrPointer, ReplicationInfo->ChangeMaskBitCount);
	FObjectReferenceTracker& UnresolvedObjectReferences = ReplicationInfo->UnresolvedObjectReferences;
	TMap<FNetRefHandle, int16>& UnresolvedHandleCount = ReplicationInfo->UnresolvedHandleCount;
	TMap<FNetRefHandle, int16>& ResolvedDynamicHandleCount = ReplicationInfo->ResolvedDynamicHandleCount;
	for (FObjectReferenceTracker::TIterator It = ReplicationInfo->ResolvedDynamicObjectReferences.CreateIterator(); It; ++It)
	{
		const FNetRefHandle RefHandle = It->Value;
		if (RefHandle == UnresolvableHandle)
		{
			bFoundHandle = true;

			const uint32 ChangemaskOffset = It->Key;
			if (ChangemaskOffset == FakeInitChangeMaskOffset)
			{
				bHasUnresolvedInitialReferences = true;
			}
			else
			{
				bHasUnresolvedReferences = true;
				UnresolvedChangeMask.SetBit(ChangemaskOffset);
			}

			// At this point we'd like to skip iteration to the next key as a handle can only be found once per changemask.
			It.RemoveCurrent();

			ReplicationInfo->RemoveResolvedDynamicHandleCount(UnresolvableHandle);

			// This handle should only have existed once in the ResolvedDynamicObjectReferences map and should not be able to
			// already exist in the UnresolvedObjectReferences map, so no need to call AddUnique.
			UnresolvedObjectReferences.Add(ChangemaskOffset, UnresolvableHandle);
			
			int16& UnresolvableHandleCount = UnresolvedHandleCount.FindOrAdd(UnresolvableHandle, 0);
			UnresolvableHandleCount++;

			UE_LOGF(LogIris, Verbose, "FReplicationReader::MoveResolvedObjectReferenceToUnresolvedInReplicationInfo Moving from resolved to unresolved reference %ls for %ls", ToCStr(UnresolvableHandle.ToString()), ToCStr(NetRefHandleManager->GetNetRefHandleFromInternalIndex(ReplicationInfo->InternalIndex).ToString()));
		}
	}

	ReplicationInfo->bHasUnresolvedInitialReferences = bHasUnresolvedInitialReferences;
	ReplicationInfo->bHasUnresolvedReferences = bHasUnresolvedReferences;

	return bFoundHandle;
}

// Remove all references for object
void FReplicationReader::CleanupReferenceTracking(FReplicatedObjectInfo* ObjectInfo)
{
	const uint32 ObjectIndex = ObjectInfo->InternalIndex;

	// Remove from unresolved references
	for (FObjectReferenceTracker::ElementType Element : ObjectInfo->UnresolvedObjectReferences)
	{
		// Remove from tracking
		FNetRefHandle Handle = Element.Value;
		UnresolvedHandleToDependents.Remove(Handle, ObjectIndex);
		RemoveFromUnresolvedCache(Handle);
		UE_LOGF(LogIris, Verbose, "FReplicationReader::CleanupReferenceTracking Removing unresolved reference %ls for %ls", *Handle.ToString(), *(NetRefHandleManager->GetNetRefHandleFromInternalIndex(ObjectIndex).ToString()));
	}
	ObjectInfo->UnresolvedObjectReferences.Reset();
	ObjectInfo->UnresolvedHandleCount.Reset();

	// Remove from resolved dynamic references
	for (FObjectReferenceTracker::ElementType Element : ObjectInfo->ResolvedDynamicObjectReferences)
	{
		// Remove from tracking
		const FNetRefHandle Handle = Element.Value;
		ResolvedDynamicHandleToDependents.Remove(Handle, ObjectIndex);
		UE_LOGF(LogIris, Verbose, "FReplicationReader::CleanupReferenceTracking Removing resolved dynamic reference %ls for %ls", *Handle.ToString(), *(NetRefHandleManager->GetNetRefHandleFromInternalIndex(ObjectIndex).ToString()));
	}
	ObjectInfo->ResolvedDynamicObjectReferences.Reset();
	ObjectInfo->ResolvedDynamicHandleCount.Reset();

	// Remove from attachment resolve
	ObjectsWithAttachmentPendingResolve.Remove(ObjectIndex);
}

void FReplicationReader::BuildUnresolvedChangeMaskAndUpdateObjectReferenceTracking(const FResolveAndCollectUnresolvedAndResolvedReferenceCollector& Collector, FNetBitArrayView CollectorChangeMask, FReplicatedObjectInfo* ReplicationInfo, FNetBitArrayView& OutUnresolvedChangeMask, FResolvedNetRefHandlesArray& OutNewResolvedRefHandles)
{
	OutUnresolvedChangeMask.ClearAllBits();
	bool bHasUnresolvedInitReferences = false;

	UnresolvedReferencesCache.Reset();
	MappedDynamicReferencesCache.Reset();
	
	for (const auto& RefInfo : Collector.GetUnresolvedReferences())
	{
		const FNetSerializerChangeMaskParam& ChangeMaskInfo = RefInfo.ChangeMaskInfo;
		if (ChangeMaskInfo.BitCount)
		{
			OutUnresolvedChangeMask.SetBit(ChangeMaskInfo.BitOffset);
		}
		else
		{
			bHasUnresolvedInitReferences = true;
		}

		const uint32 BitOffset = (ChangeMaskInfo.BitCount > 0U ? ChangeMaskInfo.BitOffset : FakeInitChangeMaskOffset);
		UnresolvedReferencesCache.AddUnique(BitOffset, RefInfo.Reference.GetRefHandle());
	}

	for (const auto& RefInfo : Collector.GetResolvedReferences())
	{
		if (RefInfo.Reference.GetRefHandle().IsDynamic())
		{
			const uint32 BitOffset = (RefInfo.ChangeMaskInfo.BitCount > 0U ? RefInfo.ChangeMaskInfo.BitOffset : FakeInitChangeMaskOffset);
			MappedDynamicReferencesCache.AddUnique(BitOffset, RefInfo.Reference.GetRefHandle());
		}
	}

	// Update object specific
	UpdateObjectReferenceTracking_Fast(ReplicationInfo, CollectorChangeMask, Collector.IsInitStateIncluded(), OutNewResolvedRefHandles, UnresolvedReferencesCache, MappedDynamicReferencesCache);
}

void FReplicationReader::ResolveAndDispatchUnresolvedReferencesForObject(FNetSerializationContext& Context, uint32 InternalIndex)
{
	IRIS_PROFILER_SCOPE(FReplicationReader_ResolveAndDispatchUnresolvedReferencesForObject);

	FReplicatedObjectInfo* ReplicationInfo = GetReplicatedObjectInfo(InternalIndex);
	// Unexpected. Get more info.
	if (!ReplicationInfo)
	{
		static bool bHasLogged = false;
		UE_CLOGF(!bHasLogged, LogIris, Error, "Trying to resolve references for non-existing object ( InternalIndex: %u )", InternalIndex);
		bHasLogged = true;
		ensure(false);
		return;
	}

	const bool bObjectHasAttachments = ReplicationInfo->bHasAttachments;
	const bool bObjectHasReferences = ReplicationInfo->bHasUnresolvedInitialReferences | ReplicationInfo->bHasUnresolvedReferences;

	ENetObjectAttachmentDispatchFlags AttachmentDispatchedFlags = ENetObjectAttachmentDispatchFlags::None;
	if (bObjectHasReferences)
	{
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ReplicationInfo->InternalIndex);
		UE_LOGF(LogIris, VeryVerbose, "ResolveAndDispatchUnresolvedReferencesForObject %ls RefHandle %ls", ObjectData.Protocol->DebugName->Name, ToCStr(ObjectData.RefHandle.ToString()));
		const uint32 ChangeMaskBitCount = ReplicationInfo->ChangeMaskBitCount;
		
		// Try to resolve references and collect unresolved references
		const bool bOldHasUnresolvedInitReferences = ReplicationInfo->bHasUnresolvedInitialReferences;

		FNetBitArrayView UnresolvedChangeMask = FChangeMaskUtil::MakeChangeMask(ReplicationInfo->UnresolvedChangeMaskOrPointer, ChangeMaskBitCount);
	
		// Need a temporary changemask for the unresolved changes due to UnresolvedChangeMask being written to by BuildUnresolvedChangeMaskAndUpdateObjectReferenceTracking
		FChangeMaskStorageOrPointer TempChangeMaskOrPointer;
		FNetBitArrayView TempChangeMask = FChangeMaskStorageOrPointer::AllocAndInitBitArray(TempChangeMaskOrPointer, ChangeMaskBitCount, TempChangeMaskAllocator);
		FNetBitArrayView TempUnresolvedChangeMask = TempChangeMask;
		TempUnresolvedChangeMask.Copy(UnresolvedChangeMask);

		// Try to resolve references and collect resolved and references still pending resolve
		FResolveAndCollectUnresolvedAndResolvedReferenceCollector Collector;
		Collector.CollectReferences(*ObjectReferenceCache, ResolveContext, ReplicationInfo->bHasUnresolvedInitialReferences, &UnresolvedChangeMask, ObjectData.ReceiveStateBuffer, ObjectData.Protocol);

		// We need to track previously unresolved NetRefHandles that now are resolvable
		FResolvedNetRefHandlesArray NewResolvedRefHandles;

		// Build UnresolvedChangeMask from collected data and update replication info
		BuildUnresolvedChangeMaskAndUpdateObjectReferenceTracking(Collector, TempUnresolvedChangeMask, ReplicationInfo, UnresolvedChangeMask, NewResolvedRefHandles);

		// Re-purpose temp changemask for members that has resolved references.
		TempChangeMask.ClearAllBits();
		FNetBitArrayView ResolvedChangeMask = TempChangeMask;

		// Merge in partially resolved changes
		bool bHasResolvedInitReferences = false;
		if (NewResolvedRefHandles.Num())
		{
			for (const FNetReferenceCollector::FReferenceInfo& ReferenceInfo : Collector.GetResolvedReferences())
			{
				const FNetRefHandle& MatchRefHandle = ReferenceInfo.Reference.GetRefHandle();
				if (NewResolvedRefHandles.ContainsByPredicate([&MatchRefHandle](const FNetRefHandle& RefHandle) { return RefHandle == MatchRefHandle;} ))
				{
					const FNetSerializerChangeMaskParam& ChangeMaskInfo = ReferenceInfo.ChangeMaskInfo;
					if (ChangeMaskInfo.BitCount)
					{
						ResolvedChangeMask.SetBit(ChangeMaskInfo.BitOffset);
					}
					else
					{
						// If we had old unresolved init dependencies we need to include the init state when we update references
						bHasResolvedInitReferences = bOldHasUnresolvedInitReferences;
					}
				}
			}
		}

		if (ObjectData.InstanceProtocol)
		{
			// Apply resolved references, this is a blunt tool as we currently push out full dirty properties rather than only the resolved references
			if (ResolvedChangeMask.IsAnyBitSet() || bHasResolvedInitReferences)
			{
				if (bObjectHasAttachments && bExecuteReliableRPCsBeforeApplyState && !bHasResolvedInitReferences)
				{
					AttachmentDispatchedFlags = ENetObjectAttachmentDispatchFlags::Reliable;
					ResolveAndDispatchAttachments(Context, ReplicationInfo, ENetObjectAttachmentDispatchFlags::Reliable);
				}

				Context.SetIsInitState(bHasResolvedInitReferences);

				FDequantizeAndApplyParameters Params;
				Params.Allocator = &TempLinearAllocator;
				Params.ChangeMaskData = ResolvedChangeMask.GetData();
				Params.UnresolvedReferencesChangeMaskData = ReplicationInfo->bHasUnresolvedReferences ? ReplicationInfo->UnresolvedChangeMaskOrPointer.GetPointer(ChangeMaskBitCount) : nullptr;
				Params.InstanceProtocol = ObjectData.InstanceProtocol;
				Params.Protocol = ObjectData.Protocol;
				Params.SrcObjectStateBuffer = ObjectData.ReceiveStateBuffer;
				Params.bHasUnresolvedInitReferences = ReplicationInfo->bHasUnresolvedInitialReferences;

				if (bResolvedObjectsDispatchDebugging && UE_LOG_ACTIVE(LogIris, VeryVerbose))
				{
					uint32 CurrentChangeMaskBitOffset = 0;
					for (const FReplicationStateDescriptor* StateDescriptor : MakeArrayView(ObjectData.Protocol->ReplicationStateDescriptors, ObjectData.Protocol->ReplicationStateCount))
					{
						if (ResolvedChangeMask.IsAnyBitSet(CurrentChangeMaskBitOffset, StateDescriptor->ChangeMaskBitCount))
						{
							for (uint32 MemberIt = 0, MemberEndIt = StateDescriptor->MemberCount; MemberIt != MemberEndIt; ++MemberIt)
							{
								const FReplicationStateMemberChangeMaskDescriptor* MemberChangeMaskDesc = StateDescriptor->MemberChangeMaskDescriptors + MemberIt;
								if (ResolvedChangeMask.IsAnyBitSet(CurrentChangeMaskBitOffset + MemberChangeMaskDesc->BitOffset, MemberChangeMaskDesc->BitCount))
								{
									if (const FProperty* MemberProperty = StateDescriptor->MemberProperties[MemberIt])
									{
										UE_LOGF(LogIris, VeryVerbose, "ResolvedChangeMask State %ls Property %ls", ToCStr(StateDescriptor->DebugName->Name), ToCStr(MemberProperty->GetName()));
									}
								}
							}
						}

						CurrentChangeMaskBitOffset += StateDescriptor->ChangeMaskBitCount;
					}
				}

				FReplicationInstanceOperations::DequantizeAndApply(Context, Params);
			}
		}
		else
		{
			// $IRIS: $TODO: Figure out how to handle this, currently we do not crash but we probably want to
			// handle this properly by accumulating changemask for later instantiation
			UE_LOGF(LogIris, Verbose, "Cannot dispatch state data for not instantiated %ls", *ObjectData.RefHandle.ToString());
		}
	}

	// Dispatch attachment and enqueue for later resolving
	if (bObjectHasAttachments)
	{
		// If we haven't dispatched reliable attachments for this object then do so now in addition to unreliable attachments.
		const ENetObjectAttachmentDispatchFlags AttachmentDispatchFlags = ENetObjectAttachmentDispatchFlags::Unreliable | (AttachmentDispatchedFlags ^ ENetObjectAttachmentDispatchFlags::Reliable);
		ResolveAndDispatchAttachments(Context, ReplicationInfo, AttachmentDispatchFlags);
	}
}

// Dispatch all data received for the frame, this includes trying to resolve object references
void FReplicationReader::DispatchStateData(FNetSerializationContext& Context)
{
	IRIS_PROFILER_SCOPE(FReplicationReader::DispatchStateData);

	// In order to execute PostNetRecv/PostRepNotifies after we have applied the actual state 
	// we need to cache some information during dispatch and execute the logic in multiple passes
	// Note: Currently all objects received in the packet are treated a a single batch
	struct FPostDispatchObjectInfo
	{
		FReplicatedObjectInfo* ReplicationInfo;
		FDispatchObjectInfo* Info;
		FDequantizeAndApplyHelper::FContext* DequantizeAndApplyContext;
		ENetObjectAttachmentDispatchFlags AttachmentDispatchedFlags;		
	};

	// Allocate temporary space for post dispatch
	FPostDispatchObjectInfo* PostDispatchObjectInfos = new (TempLinearAllocator) FPostDispatchObjectInfo[ObjectsToDispatchArray->Num()];
	uint32 NumObjectsPendingPostDistpatch = 0U;

	// Function to flush all objects pending post dispatch
	auto FlushPostDispatchForBatch = [PostDispatchObjectInfos, &NumObjectsPendingPostDistpatch, this, &Context]()
	{
		// Dispatch replicated subobject destroy here to behave as close as possible to subobject replication in ActorChannel.
		// Added to solve issues with assumptions made by blueprint logic when replacing a component with another of the same type
		for (FPostDispatchObjectInfo& PostDispatchObjectInfo : MakeArrayView(PostDispatchObjectInfos, NumObjectsPendingPostDistpatch))
		{
			FDispatchObjectInfo& Info = *PostDispatchObjectInfo.Info;
			if (Info.bWantsImmediateEndReplication)
			{
				EndReplication(Info.InternalIndex, Info.InternalDetachReason);
			}
		}

		// When all received states have been applied we invoke PostReplicate and RepNotifies
		for (FPostDispatchObjectInfo& PostDispatchObjectInfo : MakeArrayView(PostDispatchObjectInfos, NumObjectsPendingPostDistpatch))
		{
			FDispatchObjectInfo& Info = *PostDispatchObjectInfo.Info;

			// Execute legacy post replicate functions
			if (Info.bHasState && PostDispatchObjectInfo.DequantizeAndApplyContext)
			{
				Context.SetIsInitState(Info.bIsInitialState);
				FDequantizeAndApplyHelper::CallLegacyPostApplyFunctions(PostDispatchObjectInfo.DequantizeAndApplyContext, Context);
			}
		}

		// In the last pass, RPC's and cleanup cached data
		for (FPostDispatchObjectInfo& PostDispatchObjectInfo : MakeArrayView(PostDispatchObjectInfos, NumObjectsPendingPostDistpatch))
		{
			FDispatchObjectInfo& Info = *PostDispatchObjectInfo.Info;

			// If the object was created this frame it's initial state is now applied
			if (Info.bHasState && Info.bIsInitialState)
			{
				FBridgeSerialization::PostApplyInitialState(ReplicationBridge, Info.InternalIndex);
			}

			// Dispatch attachment and enqueue for later resolving
			if (Info.bHasAttachments)
			{
				// If we haven't dispatched reliable attachments for this object then do so now in addition to unreliable attachments.
				const ENetObjectAttachmentDispatchFlags AttachmentDispatchFlags = ENetObjectAttachmentDispatchFlags::Unreliable | (PostDispatchObjectInfo.AttachmentDispatchedFlags ^ ENetObjectAttachmentDispatchFlags::Reliable);
				ResolveAndDispatchAttachments(Context, PostDispatchObjectInfo.ReplicationInfo, AttachmentDispatchFlags);
			}

			// Cleanup temporary state data
			if (PostDispatchObjectInfo.DequantizeAndApplyContext)
			{
				FDequantizeAndApplyHelper::Deinitialize(PostDispatchObjectInfo.DequantizeAndApplyContext);
			}
		}

		NumObjectsPendingPostDistpatch = 0U;
	};

	// In order to properly execute legacy callbacks we need to batch apply state data for owner/subobjects
	FInternalNetRefIndex LastDispatchedRootInternalIndex = 0U;
	
	// Dispatch and apply received state data
	for (FDispatchObjectInfo& Info : ObjectsToDispatchArray->GetObjectsToDispatch())
	{
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(Info.InternalIndex);

		// Before starting a potentially new batch we want to flush rpc's and legacy callbacks belonging to the previous batch
		const FInternalNetRefIndex RootInternalIndex = ObjectData.SubObjectRootIndex == InvalidInternalNetRefIndex ? Info.InternalIndex : ObjectData.SubObjectRootIndex;
		if (RootInternalIndex != LastDispatchedRootInternalIndex && NumObjectsPendingPostDistpatch)
		{
			FlushPostDispatchForBatch();
		}
		LastDispatchedRootInternalIndex = RootInternalIndex;
		
		// This must be done after FlushPostDispatchForBatch as ReplicatedObjects could be modified
		FReplicatedObjectInfo* ReplicationInfo = GetReplicatedObjectInfo(Info.InternalIndex);
		CA_ASSUME(ReplicationInfo != nullptr);

		FPostDispatchObjectInfo PostDispatchObjectInfo;
		PostDispatchObjectInfo.ReplicationInfo = ReplicationInfo;
		PostDispatchObjectInfo.Info = &Info;
		PostDispatchObjectInfo.DequantizeAndApplyContext = nullptr;
		PostDispatchObjectInfo.AttachmentDispatchedFlags = ENetObjectAttachmentDispatchFlags::None;

		// For SubObjects we call this method after applying state data for the owner, in order to remain backwards compatible.
		if (Info.bShouldCallSubObjectCreatedFromReplication)
		{
			if (ObjectData.SubObjectRootIndex != InvalidInternalNetRefIndex)
			{
				FBridgeSerialization::SubObjectCreatedFromReplication(ReplicationBridge, ObjectData.SubObjectRootIndex, ObjectData.RefHandle);
			}
		}

		// If we are running in backwards compatibility mode, execute Reliable RPC`s before applying state data unless object is already created.
		if (Info.bHasAttachments && bExecuteReliableRPCsBeforeApplyState && !Info.bIsInitialState)
		{
			PostDispatchObjectInfo.AttachmentDispatchedFlags |= ENetObjectAttachmentDispatchFlags::Reliable;
			ResolveAndDispatchAttachments(Context, PostDispatchObjectInfo.ReplicationInfo, ENetObjectAttachmentDispatchFlags::Reliable);
			// Update if we have attachments or not since we might have processed all of them in the first pass.
			Info.bHasAttachments = PostDispatchObjectInfo.ReplicationInfo->bHasAttachments;
		}

		// If we have any object references we want to update any unresolved ones, including previously unresolved references
		if (Info.bHasState)
		{

			const uint32 ChangeMaskBitCount = ReplicationInfo->ChangeMaskBitCount;

			FNetBitArrayView ChangeMask = FChangeMaskUtil::MakeChangeMask(Info.ChangeMaskOrPointer, ChangeMaskBitCount);
			// If we have pending unresolved changes we include them as well 
			FNetBitArrayView UnresolvedChangeMask = FChangeMaskUtil::MakeChangeMask(ReplicationInfo->UnresolvedChangeMaskOrPointer, ChangeMaskBitCount);

			FChangeMaskStorageOrPointer ChangeMaskForResolveAllocation;
			FNetBitArrayView ChangeMaskForResolve;
			const bool bHadUnresolvedReferences = ReplicationInfo->bHasUnresolvedReferences;
			if (bHadUnresolvedReferences)
			{
				if (bDispatchUnresolvedPreviouslyReceivedChanges)
				{
					// Combine the changemask with the unresolved changemask so that result is used for the apply operation as well.
					ChangeMask.Combine(UnresolvedChangeMask, FNetBitArrayView::OrOp);
					ChangeMaskForResolve = ChangeMask;
				}
				else
				{
					// Memory for the changemask allocation will be freed when the TempLinearAllocator is reset via FMemMark scope. TempChangeMaskAllocator uses TempLinearAllocator.
					ChangeMaskForResolveAllocation.Alloc(ChangeMaskForResolveAllocation, ChangeMaskBitCount, TempChangeMaskAllocator);
					ChangeMaskForResolve = MakeNetBitArrayView(ChangeMaskForResolveAllocation.GetPointer(ChangeMaskBitCount), ChangeMaskBitCount, FNetBitArrayView::NoResetNoValidate);
					ChangeMaskForResolve.Set(ChangeMask, FNetBitArrayView::OrOp, UnresolvedChangeMask);
				}
			}
			else
			{
				ChangeMaskForResolve = ChangeMask;
			}

			// Collect all unresolvable references, including old pending references
			FResolveAndCollectUnresolvedAndResolvedReferenceCollector Collector;
			Collector.CollectReferences(*ObjectReferenceCache, ResolveContext, Info.bIsInitialState || (ReplicationInfo->bHasUnresolvedInitialReferences!=0U), &ChangeMaskForResolve, ObjectData.ReceiveStateBuffer, ObjectData.Protocol);

			// If we have or had any object references we need to track them and update the unresolved mask
			if (bHadUnresolvedReferences || Collector.GetUnresolvedReferences().Num() > 0 || Collector.GetResolvedReferences().Num() > 0)
			{
				FChangeMaskStorageOrPointer ChangeMaskForPrevUnresolvedAllocation;
				FNetBitArrayView PrevUnresolvedChangeMask;

				// If we're avoiding dispatching state we didn't receive and we didn't resolve anything for we need to figure out what got resolves and combine that with the received changemask.
				const bool bMergeResolvedReferencesWithChangeMask = bHadUnresolvedReferences && !bDispatchUnresolvedPreviouslyReceivedChanges;
				if (bMergeResolvedReferencesWithChangeMask)
				{
					ChangeMaskForPrevUnresolvedAllocation.Alloc(ChangeMaskForPrevUnresolvedAllocation, ChangeMaskBitCount, TempChangeMaskAllocator);
					PrevUnresolvedChangeMask = MakeNetBitArrayView(ChangeMaskForPrevUnresolvedAllocation.GetPointer(ChangeMaskBitCount), ChangeMaskBitCount, FNetBitArrayView::NoResetNoValidate);
					PrevUnresolvedChangeMask.Copy(UnresolvedChangeMask);
				}

				// We need to track previously unresolved NetRefHandles that now are resolvable
				FResolvedNetRefHandlesArray NewResolvedRefHandles;

				BuildUnresolvedChangeMaskAndUpdateObjectReferenceTracking(Collector, ChangeMaskForResolve, ReplicationInfo, UnresolvedChangeMask, NewResolvedRefHandles);
			
				// Allow resolved changes to be part of the state to be applied.
				if (bMergeResolvedReferencesWithChangeMask)
				{
					// Merge in no longer unresolved changes
					ChangeMask.CombineMultiple(FNetBitArrayView::OrOp, PrevUnresolvedChangeMask, FNetBitArrayView::AndNotOp, UnresolvedChangeMask);

					// Merge in partially resolved changes
					if (NewResolvedRefHandles.Num())
					{
						for (const FNetReferenceCollector::FReferenceInfo& ReferenceInfo : Collector.GetResolvedReferences())
						{
							const FNetRefHandle& MatchRefHandle = ReferenceInfo.Reference.GetRefHandle();
							if (NewResolvedRefHandles.ContainsByPredicate([&MatchRefHandle](const FNetRefHandle& RefHandle) { return RefHandle == MatchRefHandle;} ))
							{
								ChangeMask.SetBit(ReferenceInfo.ChangeMaskInfo.BitOffset);
							}
						}
					}
				}
			}

			// Apply state data
			if (ObjectData.InstanceProtocol)
			{
				Context.SetIsInitState(Info.bIsInitialState);

				FDequantizeAndApplyParameters Params;
				Params.Allocator = &TempLinearAllocator;
				Params.ChangeMaskData = Info.ChangeMaskOrPointer.GetPointer(ChangeMaskBitCount);
				Params.UnresolvedReferencesChangeMaskData = ReplicationInfo->bHasUnresolvedReferences ? ReplicationInfo->UnresolvedChangeMaskOrPointer.GetPointer(ChangeMaskBitCount) : nullptr;
				Params.InstanceProtocol = ObjectData.InstanceProtocol;
				Params.Protocol = ObjectData.Protocol;
				Params.SrcObjectStateBuffer = ObjectData.ReceiveStateBuffer;
				Params.bHasUnresolvedInitReferences = ReplicationInfo->bHasUnresolvedInitialReferences;

				// Dequantize state data, call PreReplicate and apply received state
				PostDispatchObjectInfo.DequantizeAndApplyContext = FDequantizeAndApplyHelper::Initialize(Context, Params);
				FDequantizeAndApplyHelper::ApplyAndCallLegacyPreApplyFunction(PostDispatchObjectInfo.DequantizeAndApplyContext, Context);
			}
			else
			{
				// $IRIS: $TODO: Figure out how to handle this, currently we do not crash but we probably want to
				// handle this properly by accumulating changemask for later instantiation
				UE_LOGF(LogIris, Verbose, "Cannot dispatch state data for not instantiated %ls", *(ObjectData.RefHandle.ToString()));
			}
		}

		// Add to post dispatch
		PostDispatchObjectInfos[NumObjectsPendingPostDistpatch++] = PostDispatchObjectInfo;
	}

	FlushPostDispatchForBatch();
}

void FReplicationReader::ResolveAndDispatchUnresolvedReferences()
{
	IRIS_PROFILER_SCOPE(FReplicationReader_ResolveAndDispatchUnresolvedReferences);
	CSV_SCOPED_TIMING_STAT(IrisClient, ResolveAndDispatchUnresolvedReferences);

	// Setup context for dispatch
	FInternalNetSerializationContext InternalContext;
	FInternalNetSerializationContext::FInitParameters InternalContextInitParams;
	InternalContextInitParams.ReplicationSystem = Parameters.ReplicationSystem;
	InternalContextInitParams.ObjectResolveContext = ResolveContext;
	InternalContextInitParams.PackageMap = ReplicationSystemInternal->GetIrisObjectReferencePackageMap();
	InternalContext.Init(InternalContextInitParams);

	FNetSerializationContext Context;
	Context.SetLocalConnectionId(ResolveContext.ConnectionId);
	Context.SetInternalContext(&InternalContext);

	// Currently we brute force this by iterating over all handles pending resolve and update all objects pending resolve
	VisitedUnresolvedHandles.Reset();
	InternalObjectsToResolve.Reset();

	const uint32 CurrTimeMS = static_cast<uint32>(FPlatformTime::Seconds() * 1000.0f);
	const uint32 HotLifetimeMS = HotResolvingLifetimeMS > 0 ? static_cast<uint32>(HotResolvingLifetimeMS) : 0;
	const uint32 ColdRetryTimeMS = ColdResolvingRetryTimeMS > 0 ? static_cast<uint32>(ColdResolvingRetryTimeMS) : 0;

	for (const TPair<FNetRefHandle, uint32>& It : UnresolvedHandleToDependents)
	{
		const FNetRefHandle& Handle = It.Key;

		if (!VisitedUnresolvedHandles.Contains(Handle))
		{
			// Determine if the handle should be resolved.
			if (bUseResolvingHandleCache)
			{
				// If the handle is in the hot cache it should be resolved every time ResolveAndDispatchUnresolvedReferences() is called
				// and will be moved to the cold cache after a fixed period of time.
				if (const uint32* LifetimeMS = HotUnresolvedHandleCache.Find(Handle))
				{
					if ((CurrTimeMS - *LifetimeMS) > HotLifetimeMS)
					{
						HotUnresolvedHandleCache.Remove(Handle);
						ColdUnresolvedHandleCache.Add(Handle);
					}
				}
				// If the handle is in the cold cache it will only be resolved at a fixed interval and will remain in this cache indefinitely.
				else if (uint32* LastResolvedMS = ColdUnresolvedHandleCache.Find(Handle))
				{
					if ((CurrTimeMS - *LastResolvedMS) < ColdRetryTimeMS)
					{
						continue;
					}

					*LastResolvedMS = CurrTimeMS;
				}
				// If the handle is in neither the hot or cold cache, put it in the hot cache.
				else
				{
					HotUnresolvedHandleCache.Add(Handle, CurrTimeMS);
				}
			}

			// Only check this handle once per call.
			VisitedUnresolvedHandles.Add(Handle);
		}
	}

	for (FNetRefHandle Handle : VisitedUnresolvedHandles)
	{
		// Only make sense to update dependant objects if handle is resolvable
		if (ObjectReferenceCache->ResolveObjectReferenceHandle(Handle, ResolveContext) != nullptr)
		{
			for (auto It = UnresolvedHandleToDependents.CreateConstKeyIterator(Handle); It; ++It)
			{
				InternalObjectsToResolve.Add(It.Value());
			}
		}
	}

	// Add in any handles with pending attachments to resolve
	InternalObjectsToResolve.Append(ObjectsWithAttachmentPendingResolve);

	// Try to resolve objects with updated references
	for (FInternalNetRefIndex InternalIndex : InternalObjectsToResolve)
	{

#if UE_NET_ASYNCLOADING_DEBUG
		// If we want to fake async loading, test if the owner can resolve objects here
		if (bForceObjectsToStall && FAsyncLoadingSimulator::HasStalledObjects(ReplicationBridge))
		{
			const FInternalNetRefIndex OwnerIndex = InternalIndex;
			const FNetRefHandle OwnerHandle = NetRefHandleManager->GetNetRefHandleFromInternalIndex(OwnerIndex);
			if (FAsyncLoadingSimulator::IsObjectStalled(ReplicationBridge, OwnerHandle))
			{
				continue;
			}
		}
#endif

		ResolveAndDispatchUnresolvedReferencesForObject(Context, InternalIndex);
	}

#if IRIS_CLIENT_PROFILER_ENABLE
	CSV_CUSTOM_STAT(IrisClient, HotUnresolvedHandleCache, HotUnresolvedHandleCache.Num(), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(IrisClient, ColdUnresolvedHandleCache, ColdUnresolvedHandleCache.Num(), ECsvCustomStatOp::Set);

	CSV_CUSTOM_STAT(IrisClient, UnresolvedHandlesToResolve, VisitedUnresolvedHandles.Num(), ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(IrisClient, UnresolvedObjectsToResolve, InternalObjectsToResolve.Num(), ECsvCustomStatOp::Accumulate);

	const int32 TotalCacheSize = static_cast<int32>(
		HotUnresolvedHandleCache.GetAllocatedSize() +
		ColdUnresolvedHandleCache.GetAllocatedSize() +
		VisitedUnresolvedHandles.GetAllocatedSize() +
		InternalObjectsToResolve.GetAllocatedSize());
	CSV_CUSTOM_STAT(IrisClient, UnresolvedHandleBufferSizes, TotalCacheSize, ECsvCustomStatOp::Set);
#endif

	if (NumHandlesPendingResolveLastUpdate != VisitedUnresolvedHandles.Num() || ObjectsWithAttachmentPendingResolve.Num() > 0)
	{
		UE_LOG_REPLICATIONREADER(TEXT("FReplicationReader::ResolveAndDispatchUnresolvedReferences NetHandles pending: %u Attachments pending: %u)"), VisitedUnresolvedHandles.Num(), ObjectsWithAttachmentPendingResolve.Num());
		NumHandlesPendingResolveLastUpdate = VisitedUnresolvedHandles.Num();
	}
}


void FReplicationReader::UpdateUnresolvableReferenceTracking()
{
	constexpr uint32 AssumedMaxDependentCount = 256;
	TArray<uint32, TInlineAllocator<AssumedMaxDependentCount>> Dependents;

	// Naively go through every object pending destroy, see if it's dynamic and update dependent's unresolved tracking
	for (const FNetRefHandleManager::FPendingDestroyInfo& PendingDestroyInfo : NetRefHandleManager->GetObjectsPendingDestroy())
	{
		FInternalNetRefIndex InternalIndex = PendingDestroyInfo.InternalIndex;

		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);
		const FNetRefHandle DestroyedHandle = ObjectData.RefHandle;
		if (!DestroyedHandle.IsDynamic())
		{
			continue;
		}

		// Also update dependents for subobject references of this dynamic root.
		// When a dynamic root is destroyed, references to its subobjects should also become unresolvable.
		TArray<FNetRefHandle> SubObjectHandles;
		ObjectReferenceCache->GetTrackedSubObjectHandles(DestroyedHandle, SubObjectHandles);

		// For torn off objects or actually destroyed objects we want to remove both resolved and unresolved references to it as it will never be replicated again.
		const bool bIsPermanentlyDestroyed = ObjectData.bTearOff || (ObjectData.InternalDetachReason == EInternalDetachReason::Normal || ObjectData.InternalDetachReason == EInternalDetachReason::TornOff);

		if (bIsPermanentlyDestroyed)
		{
			constexpr bool bMaintainOrder = false;
			Dependents.Reset();
			UnresolvedHandleToDependents.MultiFind(DestroyedHandle, Dependents, bMaintainOrder);
			UnresolvedHandleToDependents.Remove(DestroyedHandle);
			RemoveFromUnresolvedCache(DestroyedHandle);
			for (const uint32 DependentObjectIndex : Dependents)
			{
				FReplicatedObjectInfo* ReplicationInfo = GetReplicatedObjectInfo(DependentObjectIndex);
				if (ensureMsgf(ReplicationInfo != nullptr, TEXT("Unable to find torn off unresolved replicated object info for %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalIndex)))
				{
					RemoveUnresolvedObjectReferenceInReplicationInfo(ReplicationInfo, DestroyedHandle);
				}
			}
		}

		// Helper to update resolved references for a handle - moves them to unresolved or removes them based on tear-off state
		auto UpdateResolvedReferencesForHandle = [this, &Dependents, &bIsPermanentlyDestroyed, InternalIndex](const FNetRefHandle& Handle)
		{
			constexpr bool bMaintainOrder = false;
			Dependents.Reset();
			ResolvedDynamicHandleToDependents.MultiFind(Handle, Dependents, bMaintainOrder);
			if (Dependents.Num() > 0)
			{
				ResolvedDynamicHandleToDependents.Remove(Handle);
				// For torn off objects or actually destroyed objects we want to remove both resolved and unresolved references to it as it will never be replicated again.
				if (bIsPermanentlyDestroyed)
				{
					for (const uint32 DependentObjectIndex : Dependents)
					{
						FReplicatedObjectInfo* ReplicationInfo = GetReplicatedObjectInfo(DependentObjectIndex);
						if (ensureMsgf(ReplicationInfo != nullptr, TEXT("Unable to find torn off resolved replicated object info for %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalIndex)))
						{
							RemoveResolvedObjectReferenceInReplicationInfo(ReplicationInfo, Handle);
						}
					}
				}
				else
				{
					for (const uint32 DependentObjectIndex : Dependents)
					{
						FReplicatedObjectInfo* ReplicationInfo = GetReplicatedObjectInfo(DependentObjectIndex);
						if (ensureMsgf(ReplicationInfo != nullptr, TEXT("Unable to find resolved replicated object info for %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalIndex)))
						{
							if (MoveResolvedObjectReferenceToUnresolvedInReplicationInfo(ReplicationInfo, Handle))
							{
								UnresolvedHandleToDependents.Add(Handle, DependentObjectIndex);
							}
						}
					}
				}
			}
		};

		// For any previously resolved handles make sure to move them to unresolved status.
		UpdateResolvedReferencesForHandle(DestroyedHandle);

		// If we had stable / static subobject references remove them as well.
		for (const FNetRefHandle& SubObjectHandle : SubObjectHandles)
		{
			UpdateResolvedReferencesForHandle(SubObjectHandle);
		}
	}

	// Track stats
	UE_NET_TRACE_FRAME_STATSCOUNTER(Parameters.ReplicationSystem->GetId(), ReplicationReader.UnresolvedHandleToDependents, (uint32)UnresolvedHandleToDependents.Num(), ENetTraceVerbosity::Trace);
	UE_NET_TRACE_FRAME_STATSCOUNTER(Parameters.ReplicationSystem->GetId(), ReplicationReader.ResolvedDynamicHandleToDependents, (uint32)ResolvedDynamicHandleToDependents.Num(), ENetTraceVerbosity::Trace);
}

void FReplicationReader::DispatchEndReplication(FNetSerializationContext& Context)
{
	for (const FDispatchObjectInfo& Info : ObjectsToDispatchArray->GetObjectsToDispatch())
	{
		if (Info.bWantsDeferredEndReplication)
		{
			// Detach and destroy object
			EndReplication(Info.InternalIndex, Info.InternalDetachReason);

			if (Info.bCallEndReplicationOnSubObjects)
			{
				for (FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(Info.InternalIndex))
				{
					// End replication for all subobjects and destroy dynamic ones
					EndReplication(SubObjectIndex, Info.InternalDetachReason);
				}
			}
		}
	}
}

void FReplicationReader::ReadObjects(FNetSerializationContext& Context, uint32 ObjectBatchCountToRead, uint32 ReadObjectFlags)
{
	IRIS_PROFILER_SCOPE(ReplicationReader_ReadObjects);

	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();
	
	const uint32 OriginalObjectBatchCountToRead = ObjectBatchCountToRead;
	while (ObjectBatchCountToRead && !Context.HasErrorOrOverflow() && (!bGracefullyHandleReachingEndOfBitstream || (Reader.GetBitsLeft() > 0)))
	{
		ReadObjectBatch(Context, ReadObjectFlags);
		--ObjectBatchCountToRead;
	}

	UE_CLOGF(Context.HasErrorOrOverflow(), LogIris, Error, "Overflow: %c Error: %ls Bit stream bits left: %u position: %u %ls", TEXT("YN")[Context.HasError()], ToCStr(Context.GetError().ToString()), Reader.GetBitsLeft(), Reader.GetPosBits(), *Context.PrintReadJournal());

	if (ensure(!Context.HasErrorOrOverflow()))
	{
		if (bGracefullyHandleReachingEndOfBitstream)
		{
			// ObjectBatchCountToRead should be zero at this point otherwise there's a problem on the writing side.
			UE_CLOGF(ObjectBatchCountToRead > 0, LogIris, Error, "Reached end of bitstream prior to reading all expected objects. %u/%u left to read. PacketId: %u %ls", ObjectBatchCountToRead, OriginalObjectBatchCountToRead, Context.GetPacketId(), *Context.PrintReadJournal());
			ensure(ObjectBatchCountToRead == 0);
		}
	}
}

void FReplicationReader::ProcessHugeObjectAttachment(FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& Attachment)
{
	if (Attachment->GetCreationInfo().Type != NetObjectBlobType)
	{
		Context.SetError(GNetError_UnsupportedNetBlob);
		return;
	}

	IRIS_PROFILER_SCOPE(FReplicationReader_ProcessHugeObjectAttachment)

	FNetTraceCollector* HugeObjectTraceCollector = nullptr;
#if UE_NET_TRACE_ENABLED
	FNetTraceCollector HugeObjectTraceCollectorOnStack;
	HugeObjectTraceCollector = &HugeObjectTraceCollectorOnStack;
#endif

	const FNetObjectBlob& NetObjectBlob = *static_cast<FNetObjectBlob*>(Attachment.GetReference());

	FNetBitStreamReader HugeObjectReader;
	HugeObjectReader.InitBits(NetObjectBlob.GetRawData().GetData(), NetObjectBlob.GetRawDataBitCount());
	FNetSerializationContext HugeObjectSerializationContext = Context.MakeSubContext(&HugeObjectReader);
	HugeObjectSerializationContext.SetTraceCollector(HugeObjectTraceCollector);
	UE_RESET_READ_JOURNAL(HugeObjectSerializationContext);
	UE_ADD_READ_JOURNAL_ENTRY(HugeObjectSerializationContext, TEXT("HugeObject"));

	UE_NET_TRACE_NAMED_SCOPE(HugeObjectTraceScope, HugeObjectState, HugeObjectReader, HugeObjectTraceCollector, ENetTraceVerbosity::Trace);

	ON_SCOPE_EXIT
	{
#if UE_NET_TRACE_ENABLED
		UE_NET_TRACE_EXIT_NAMED_SCOPE(HugeObjectTraceScope);

		// Append huge object state at end of stream.
		if (FNetTraceCollector* TraceCollector = Context.GetTraceCollector())
		{
			FNetBitStreamReader& Reader = *Context.GetBitStreamReader();
			// Inject after all other trace events
			FNetTrace::FoldTraceCollector(TraceCollector, HugeObjectTraceCollector, GetBitStreamPositionForNetTrace(Reader) + DebugTraceOffset);
			DebugTraceOffset += NetObjectBlob.GetRawDataBitCount();
		}
#endif
	};

	// Find out how many objects to read so we can reserve object dispatch infos.
	FNetObjectBlob::FHeader HugeObjectHeader = {};
	FNetObjectBlob::DeserializeHeader(HugeObjectSerializationContext, HugeObjectHeader);
	ReadStreamDebugFeatures(HugeObjectSerializationContext);
	if (HugeObjectSerializationContext.HasErrorOrOverflow() || HugeObjectHeader.ObjectCount < 1U)
	{
		if (!Context.HasError())
		{
			UE_LOGF(LogIris, Error, "Failed to process HugeObject payload");
			Context.SetError(GNetError_BitStreamError);
			return;
		}
	}

	// Reserve space for more dispatch infos as needed, we allocate some extra to account for subobjects etc
	ObjectsToDispatchArray->Grow(HugeObjectHeader.ObjectCount + ObjectsToDispatchSlackCount, TempLinearAllocator);

	const uint32 ReadObjectFlags = EReadObjectFlag::ReadObjectFlag_IsReadingHugeObjectBatch;
	ReadObjects(HugeObjectSerializationContext, HugeObjectHeader.ObjectCount, ReadObjectFlags);
	if (HugeObjectSerializationContext.HasErrorOrOverflow())
	{
		Context.SetError(GNetError_BitStreamError);
		return;
	}
}

bool FReplicationReader::EnqueueEndReplication(FPendingBatchData* PendingBatchData, FNetRefHandle NetRefHandleToEndReplication, EInternalDetachReason DetachReason)
{
	UE_LOGF(LogIris, Verbose, "FReplicationReader::EnqueueEndReplication for %ls since %ls has queued batches | Reason: %ls", *NetRefHandleToEndReplication.ToString(), *PendingBatchData->Owner.ToString(), LexToString(DetachReason));

	/* Template of the extra data we serialize in the chunk.
	struct FEndReplicationData
	{
		FNetRefHandle EndReplicationHandle;
		EInternalDetachReason Reason : GetDetachReasonBitsNeeded;
	}
	*/
	constexpr uint32 MaxNumDataBits = (sizeof(FNetRefHandle) * 8U) + GetDetachReasonBitsNeeded();
	constexpr uint32 NumDataWords = (MaxNumDataBits + 31U) / 32U;

	static_assert(MaxNumDataBits < (1U << 30 /*bitsize of FQueuedDataChunk::NumBits*/), "Bits needed don't fit inside FQueuedDataChunk::NumBits");

	// Enqueue BatchData
	FQueuedDataChunk DataChunk;

	DataChunk.NumBits = MaxNumDataBits;
	DataChunk.StorageOffset = PendingBatchData->DataChunkStorage.Num();
	DataChunk.bHasBatchOwnerData = false;
	DataChunk.bIsEndReplicationChunk = true;

	// Make sure we have space
	PendingBatchData->DataChunkStorage.AddUninitialized(NumDataWords);

	FNetBitStreamWriter Writer;
	Writer.InitBytes(PendingBatchData->DataChunkStorage.GetData() + DataChunk.StorageOffset, NumDataWords*sizeof(uint32));

	// Write data to be parsed by ProcessQueuedBatches
	WriteUint64(&Writer, NetRefHandleToEndReplication.GetId());
	Writer.WriteBits((uint32)DetachReason, GetDetachReasonBitsNeeded());

	Writer.CommitWrites();
	ensure(Writer.GetPosBits() <= MaxNumDataBits);

	// Set chunk's actual bit count
	DataChunk.NumBits = Writer.GetPosBits();

	if (Writer.IsOverflown())
	{
		UE_LOGF(LogIris, Error, "Failed to EnqueueEndReplication for %ls, Should never occur unless size of NetRefHandle has been increased.", *NetRefHandleToEndReplication.ToString());
		ensure(false);

		return false;
	}

	PendingBatchData->QueuedDataChunks.Add(MoveTemp(DataChunk));

	return true;
}

void FReplicationReader::RemoveFromUnresolvedCache(const FNetRefHandle Handle)
{
	if (bUseResolvingHandleCache && !UnresolvedHandleToDependents.Contains(Handle))
	{
		HotUnresolvedHandleCache.Remove(Handle);
		ColdUnresolvedHandleCache.Remove(Handle);
	}
}

void FReplicationReader::ProcessQueuedBatches()
{
	UE_NET_TRACE_FRAME_STATSCOUNTER(Parameters.ReplicationSystem->GetId(), ReplicationReader.PendingQueuedBatches, PendingBatchHolder.Num(), ENetTraceVerbosity::Trace);

	if (PendingBatchHolder.IsEmpty())
	{
		//Nothing to do.
		return;
	}

	// Setup context for dispatch
	FInternalNetSerializationContext InternalContext;
	FInternalNetSerializationContext::FInitParameters InternalContextInitParams;
	InternalContextInitParams.ReplicationSystem = Parameters.ReplicationSystem;
	InternalContextInitParams.ObjectResolveContext = ResolveContext;
	InternalContextInitParams.PackageMap = ReplicationSystemInternal->GetIrisObjectReferencePackageMap();
	InternalContext.Init(InternalContextInitParams);

	FNetBitStreamReader Reader;
	FNetSerializationContext Context(&Reader);
	Context.SetLocalConnectionId(ResolveContext.ConnectionId);
	Context.SetInternalContext(&InternalContext);
	Context.SetNetBlobReceiver(&ReplicationSystemInternal->GetNetBlobHandlerManager());

	// List of objects that were processed
	TQueue<FNetRefHandle, EQueueMode::SingleThreaded> ObjectsProcessed;

	// Map of dependent parent handles and the pending batch owners that are waiting on them.
	TMap<FNetRefHandle /*CreationDependentParent*/, TArray<FNetRefHandle, TInlineAllocator<16>>/* OwnersWaitingOnParent*/> WaitedOnParentHandles;

	auto ProcessBatch = [&, this](FPendingBatchData* PendingBatchData, FNetRefHandle OwnerHandle)
	{
		UE_LOGF(LogIris, Verbose, "ProcessQueuedBatches processing %d queued batches for Handle %ls ", PendingBatchData->QueuedDataChunks.Num(), *OwnerHandle.ToString());

		// Reset for each batch
		Reader = FNetBitStreamReader();
		Context.ResetErrorContext();
		Context.SetErrorHandleContext(OwnerHandle);
		UE_RESET_READ_JOURNAL(Context);
		UE_ADD_READ_JOURNAL_ENTRY(Context, TEXT("ProcessQueuedBatches"));

		// Process batched data & dispatch data
		for (const FQueuedDataChunk& CurrentChunk : PendingBatchData->QueuedDataChunks)
		{
			Reader.InitBits(PendingBatchData->DataChunkStorage.GetData() + CurrentChunk.StorageOffset, CurrentChunk.NumBits);

			// Chunks marked as bIsEndReplicationChunk are dispatched immediately as we do not know if the next chunk tries to re-create the instance
			if (CurrentChunk.bIsEndReplicationChunk)
			{
				// Read data stored for objects ending replication, 
				// this can be the batch root or a subobject owned by the batched root.

				/* Template of the extra data we serialized in the chunk.
				struct FEndReplicationData
				{
					FNetRefHandle EndReplicationHandle;
					EInternalDetachReason DetachReason : GetDetachReasonBitsNeeded;
				}
				*/

				const uint64 NetRefHandleIdToEndReplication = ReadUint64(&Reader);
				const FNetRefHandle NetRefHandleToEndReplication = FNetRefHandleManager::MakeNetRefHandleFromId(NetRefHandleIdToEndReplication);

				const EInternalDetachReason DetachReason = (EInternalDetachReason)Reader.ReadBits(GetDetachReasonBitsNeeded());

				if (Reader.IsOverflown())
				{
					ensureMsgf(false, TEXT("Error while reading end replication data. Should not happen"));
					break;
				}

				// End replication for object
				const uint32 InternalIndex = NetRefHandleManager->GetInternalIndex(NetRefHandleToEndReplication);
				if (InternalIndex != InvalidInternalNetRefIndex)
				{
					EndReplication(InternalIndex, DetachReason);

					for (const FInternalNetRefIndex SubObjectIndex : NetRefHandleManager->GetSubObjects(InternalIndex))
					{
						// End replication for all subobjects and destroy dynamic ones
						EndReplication(SubObjectIndex, DetachReason);
					}
				}

				// Remove from broken list
				BrokenObjects.RemoveSwap(NetRefHandleToEndReplication);

				UE_LOGF(LogIris, Verbose, "FReplicationReader::ProcessQueuedBatches EndReplication for %ls while processing queued batches for %ls", *NetRefHandleToEndReplication.ToString(), *OwnerHandle.ToString());
				continue;
			}

			// Skip over broken objects, we still process remaining chunk if the object has been destroyed.
			const bool bIsBroken = BrokenObjects.Contains(OwnerHandle);
			if (bIsBroken)
			{
				continue;
			}

			// Read and process chunk as it was a received packet for now at least
			FMemMark TempAllocatorScope(TempLinearAllocator);

			// We need to set this up to store temporary dispatch data, the array will grow if needed
			FObjectsToDispatchArray TempObjectsToDispatchArray(ObjectsToDispatchSlackCount, TempLinearAllocator);

			// Need to set this pointer as we are dealing with temporary linear allocations
			ObjectsToDispatchArray = &TempObjectsToDispatchArray;

			// Use whatver StreamDebugFeatures were set when receiving the chunk
			StreamDebugFeatures = CurrentChunk.StreamDebugFeatures;

			// $IRIS: $TODO: Implement special dispatch to defer RepNotifies if we are processing multiple batches for the same object.
			ReadObjectsInBatch(Context, OwnerHandle, CurrentChunk.bHasBatchOwnerData, CurrentChunk.NumBits);

			if (Context.HasErrorOrOverflow())
			{
				BrokenObjects.AddUnique(OwnerHandle);

				// Log error  if get more incoming data for an object in the broken state we will skip it.
				UE_LOGF(LogIris, Error, "FReplicationReader::ProcessQueuedBatches Failed to process object batch handle: %ls skipping batch data. bIsEndReplication: %d", ToCStr(OwnerHandle.ToString()), int(CurrentChunk.bIsEndReplicationChunk));

				if (Context.GetError() == GNetError_BrokenNetHandle)
				{
					ReplicationBridge->SendErrorWithNetRefHandle(UE::Net::ENetRefHandleError::ReplicationDisabled, OwnerHandle, Parameters.ConnectionId, {}, Context.GetErrorContext());

					Context.ResetErrorContext();
				}
			}
			
			// Apply received data and resolve dependencies
			DispatchStateData(Context);

			// Resolve
			ResolveAndDispatchUnresolvedReferences();

			// EndReplication for all objects in the batch that should no longer replicate
			DispatchEndReplication(Context);

			// Drop temporary data
			ObjectsToDispatchArray = nullptr;
		}
	
		// Make sure to release all references that we hold on to
		for (const FNetRefHandle& RefHandle : PendingBatchData->ResolvedReferences)
		{
			ObjectReferenceCache->RemoveTrackedQueuedBatchObjectReference(RefHandle);
		}
	};

	for (auto BatchIt = PendingBatchHolder.CreateConstIterator(); BatchIt; ++BatchIt)
	{
		const FPendingBatchDataPtr& DataPtr = BatchIt.Value();
		FPendingBatchData* PendingBatchData = DataPtr.Get();
		check(PendingBatchData);

		const FNetRefHandle OwnerHandle = PendingBatchData->Owner;

		// Try to resolve remaining must be mapped references
		{
			TempMustBeMappedReferences.Reset();
			UpdateUnresolvedMustBeMappedReferences(OwnerHandle, TempMustBeMappedReferences, PendingBatchData->IrisAsyncLoadingPriority);
		}

		const bool bAllReferencesResolved = PendingBatchData->PendingMustBeMappedReferences.IsEmpty();

		// Look if all CreationDependencies are resolved
		int32 NumUnknownParents = 0;
		for (const FNetRefHandle CreationDependentParent : PendingBatchData->CreationDependentParents)
		{
			if (!DoesParentExist(CreationDependentParent))
			{
				++NumUnknownParents;

				// Add the owner to the wait list if only creation dependencies are left
				if (bAllReferencesResolved)
				{
					auto& WaitedOnList = WaitedOnParentHandles.FindOrAdd(CreationDependentParent);
					WaitedOnList.Add(OwnerHandle);
				}
			}
		}
		
		const bool bAllParentsExist = NumUnknownParents == 0;

		//UE_LOGF(LogIris, Display, "ProcessQueuedBatches found %ls to have %d unresolved references and %d non-existing parent", 
		//	*OwnerHandle.ToString(), PendingBatchData.PendingMustBeMappedReferences.Num(), NumUnknownParents);

		// If we are no longer waiting on references or on creation parents, let's process the data!
		if (bAllParentsExist && bAllReferencesResolved)
		{
			ProcessBatch(PendingBatchData, OwnerHandle);
			ObjectsProcessed.Enqueue(OwnerHandle);
		}
		// Batch is still pending something, track how long its been waiting
		else
		{
			// Check how long we've been blocked only every 30 ticks
			++PendingBatchData->PendingBatchTryProcessCount;

			if ((QueuedBatchTimeoutWarningInterval > 0) && (PendingBatchData->PendingBatchTryProcessCount > QueuedBatchTimeoutWarningInterval) && FPlatformProperties::RequiresCookedData() && !FPlatformMisc::IsDebuggerPresent())
			{
				const double CurrentTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64());// Different to FPlatformTime::Seconds on some platforms due to GetSecondsTimeOffset

				// Log a warning and tell the server about the blockage every X seconds (first after 30secs, again every 10secs after)
				if (CurrentTime >= PendingBatchData->NextWarningTimeout)
				{
					const double BlockedTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - PendingBatchData->PendingBatchStartCycles);

					if (!bAllParentsExist)
					{
						UE_LOGF(LogIris, Warning, "FReplicationWriter::ProcessQueuedBatches: Replication is blocked for longer than normal (%f secs) for %ls, Waiting on %d/%d parents to be created."
							   , BlockedTime
							   , *NetRefHandleManager->PrintObjectFromNetRefHandle(OwnerHandle)
							   , NumUnknownParents
							   , PendingBatchData->CreationDependentParents.Num());

						TArray<FNetRefHandle> BlockedParents;
						for (const FNetRefHandle CreationDependentParent : PendingBatchData->CreationDependentParents)
						{
							const bool bParentExists = DoesParentExist(CreationDependentParent);

							UE_LOGF(LogIris, Warning, "FReplicationWriter::ProcessQueuedBatches: %ls CreationDependent parent: %ls .", bParentExists?TEXT("Found"):TEXT("Waiting"), *NetRefHandleManager->PrintObjectFromNetRefHandle(CreationDependentParent));

							if (!bParentExists)
							{
								BlockedParents.Add(CreationDependentParent);
							}
						}

						ReplicationBridge->SendErrorWithNetRefHandle(ENetRefHandleError::BlockedByCreationParents, OwnerHandle, Parameters.ConnectionId, BlockedParents);
					}

					if (!bAllReferencesResolved)
					{
						TArray<FNetRefHandle> PendingMustBeMapped;
						PendingMustBeMapped.Reserve(MaxMustBeMappedHandleArray);

						for (const FNetRefHandle& AsyncLoadingRef : PendingBatchData->PendingMustBeMappedReferences)
						{
							if (PendingMustBeMapped.Num() >= MaxMustBeMappedHandleArray)
							{
								// Waiting on too many assets already, ignore the rest
								break;
							}

							// Ignore this asset if we recently warned about it
							const double LastWarningTime = BlockedMustBeMappedLastWarningTime.FindOrAdd(AsyncLoadingRef, 0.0);
							if (LastWarningTime == 0.0 || (CurrentTime - LastWarningTime) > QueuedBatchWarningRepeatTime)
							{
								PendingMustBeMapped.Emplace(AsyncLoadingRef);

								BlockedMustBeMappedLastWarningTime[AsyncLoadingRef] = CurrentTime;
							}
						}

						// Warn and tell server about the blockage
						if (!PendingMustBeMapped.IsEmpty())
						{
							const FString BlockedObjectName = *NetRefHandleManager->PrintObjectFromNetRefHandle(OwnerHandle);

							UE_LOGF(LogIris, Warning, "FReplicationWriter::ProcessQueuedBatches: Replication is blocked for longer than normal for: %ls, QueuedBunches: %i, PendingNetRefHandleResolves: %i, BlockedTime: %f"
								   , *BlockedObjectName
								   , PendingBatchData->QueuedDataChunks.Num()
								   , PendingBatchData->PendingMustBeMappedReferences.Num()
								   , BlockedTime);

							for (const FNetRefHandle& NetRefHandle : PendingBatchData->PendingMustBeMappedReferences)
							{
								UE_LOGF(LogIris, Log, "  NetRefHandle Pending resolve [%ls]", *ObjectReferenceCache->DescribeObjectReference(FObjectReferenceCache::MakeNetObjectReference(NetRefHandle), ResolveContext));
							}

							#if IRIS_CLIENT_PROFILER_ENABLE
							FClientProfiler::RecordBlockedReplication(*BlockedObjectName, PendingBatchData->PendingMustBeMappedReferences.Num(), (float)BlockedTime);
							#endif

							ReplicationBridge->SendErrorWithNetRefHandle(ENetRefHandleError::BlockedByMustBeMapped, OwnerHandle, Parameters.ConnectionId, PendingMustBeMapped);
						}
					}

					PendingBatchData->NextWarningTimeout = CurrentTime + QueuedBatchWarningRepeatTime;
				}

				// Reset counter
				PendingBatchData->PendingBatchTryProcessCount = 0;
			}
		}
	}

	FNetRefHandle ProcessedObject;
	while (ObjectsProcessed.Dequeue(ProcessedObject))
	{
		UE_LOGF(LogIris, Verbose, "Clean-up processed pending object: %ls", *NetRefHandleManager->PrintObjectFromNetRefHandle(ProcessedObject));

		// Delete the previously processed batch data for this object
		PendingBatchHolder.Remove(ProcessedObject);

		const bool bIsWaitedOn = WaitedOnParentHandles.Contains(ProcessedObject);
		
		// This newly created object had others waiting on it, let's check if they can be processed now
		if (bIsWaitedOn)
		{
			const auto& WaitingOwners = WaitedOnParentHandles.FindChecked(ProcessedObject);

			for (FNetRefHandle WaitingOwner : WaitingOwners)
			{
				//UE_LOGF(LogIris, Verbose, "Pending object: %ls was waiting on: %ls", *WaitingOwner.ToString() ,*NetRefHandleManager->PrintObjectFromNetRefHandle(ProcessedObject));

				if (FPendingBatchData* PendingBatchData = PendingBatchHolder.Find(WaitingOwner))
				{
					// Make sure all it's parent do exist
					bool bAllParentsExist = true;
					for (const FNetRefHandle CreationDependentParent : PendingBatchData->CreationDependentParents)
					{
						bAllParentsExist = DoesParentExist(CreationDependentParent);
						if (!bAllParentsExist)
						{
							break;
						}
					}

					// No reasons to wait anymore, process it's queued data
					if (bAllParentsExist)
					{
						ProcessBatch(PendingBatchData, WaitingOwner);
						ObjectsProcessed.Enqueue(WaitingOwner);
					}
				}
			}
		}
	}
}

void FReplicationReader::ProcessHugeObject(FNetSerializationContext& Context, ENetObjectAttachmentType AttachmentType)
{
	if (!Attachments.HasUnprocessedAttachments(AttachmentType, ObjectIndexForOOBAttachment))
	{
		return;
	}

	FNetObjectAttachmentReceiveQueue* AttachmentQueue = Attachments.GetQueue(AttachmentType, ObjectIndexForOOBAttachment);
	while (const TRefCountPtr<FNetBlob>* Attachment = AttachmentQueue->PeekReliable())
	{
		ProcessHugeObjectAttachment(Context, *Attachment);
		AttachmentQueue->PopReliable();
		if (Context.HasError())
		{
			return;
		}
	}
	while (const TRefCountPtr<FNetBlob>* Attachment = AttachmentQueue->PeekUnreliable())
	{
		ProcessHugeObjectAttachment(Context, *Attachment);
		AttachmentQueue->PopUnreliable();
		if (Context.HasError())
		{
			return;
		}
	}
}

void FReplicationReader::Read(FNetSerializationContext& Context)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	// Setup internal context
	FInternalNetSerializationContext InternalContext;
	FInternalNetSerializationContext::FInitParameters InternalContextInitParams;
	InternalContextInitParams.ReplicationSystem = Parameters.ReplicationSystem;
	InternalContextInitParams.PackageMap = ReplicationSystemInternal->GetIrisObjectReferencePackageMap();
	InternalContextInitParams.ObjectResolveContext = ResolveContext;
	InternalContext.Init(InternalContextInitParams);
	
	Context.SetLocalConnectionId(Parameters.ConnectionId);
	Context.SetInternalContext(&InternalContext);
	Context.SetNetBlobReceiver(&ReplicationSystemInternal->GetNetBlobHandlerManager());

	UE_NET_TRACE_SCOPE(ReplicationData, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	ReadStreamDebugFeatures(Context);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	FMemMark TempAllocatorScope(TempLinearAllocator);

	// Sanity check received object count
	const uint32 MaxObjectBatchCountToRead = 8192U;
	const uint32 ReceivedObjectBatchCountToRead = Reader.ReadBits(16);
	uint32 ObjectBatchCountToRead = ReceivedObjectBatchCountToRead;

	if (Reader.IsOverflown() || ObjectBatchCountToRead >= MaxObjectBatchCountToRead)
	{
		const FName& NetError = (Reader.IsOverflown() ? GNetError_BitStreamOverflow : GNetError_BitStreamError);
		Context.SetError(NetError);

		return;
	}

	if (ObjectBatchCountToRead == 0)
	{
		return;
	}

	// Allocate tracking info for objects we receive this packet from temporary allocator
	// We need to set this up to store temporary dispatch data, the array will grow if needed
	FObjectsToDispatchArray TempObjectsToDispatchArray(ObjectBatchCountToRead + ObjectsToDispatchSlackCount, TempLinearAllocator);

	// Need to set this pointer as we are dealing with temporary linear allocations
	ObjectsToDispatchArray = &TempObjectsToDispatchArray;

	uint32 DestroyedObjectCount = ReadObjectsPendingDestroy(Context);

	ObjectBatchCountToRead -= DestroyedObjectCount;

	// Nothing more to do or we failed and should disconnect
	if (Context.HasErrorOrOverflow() || (ObjectBatchCountToRead == 0 && ObjectsToDispatchArray->Num() == 0))
	{
		return;
	}

	const uint32 ReadObjectFlags = 0U;
	ReadObjects(Context, ObjectBatchCountToRead, ReadObjectFlags);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	// Assemble and deserialize huge object if present
	ProcessHugeObject(Context, ENetObjectAttachmentType::HugeObject);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	// Assemble and deserialize debug object if present
	ProcessHugeObject(Context, ENetObjectAttachmentType::DebugObject);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	// Reset 
	ObjectsReadFromHugeObjects.Reset();
	DebugTraceOffset = 0U;

	// Stats
	UE_NET_TRACE_PACKET_STATSCOUNTER(Parameters.ReplicationSystem->GetId(), Parameters.ConnectionId, ReplicationReader.ReadObjectBatchCount, ReceivedObjectBatchCountToRead, ENetTraceVerbosity::Trace);
	UE_NET_TRACE_PACKET_STATSCOUNTER(Parameters.ReplicationSystem->GetId(), Parameters.ConnectionId, ReplicationReader.ReadObjectsAndSubObjectsToDispatchCount, ObjectsToDispatchArray->Num(), ENetTraceVerbosity::Trace);

	// Apply received data and resolve dependencies
	DispatchStateData(Context);
	if (Context.HasErrorOrOverflow())
	{
		return;
	}

	// Resolve
	ResolveAndDispatchUnresolvedReferences();

	// EndReplication for all objects that should no longer replicate
	DispatchEndReplication(Context);

	// Drop temporary dispatch data
	ObjectsToDispatchArray = nullptr;
}

void FReplicationReader::ResolveAndDispatchAttachments(FNetSerializationContext& Context, FReplicatedObjectInfo* ReplicationInfo, ENetObjectAttachmentDispatchFlags DispatchFlags)
{
	if (DispatchFlags == ENetObjectAttachmentDispatchFlags::None)
	{
		return;
	}

	Context.GetInternalContext()->RPCDoS = RPCDoS;

	// Cache configurables before processing attachments
	const bool bDispatchReliableAttachments = EnumHasAnyFlags(DispatchFlags, ENetObjectAttachmentDispatchFlags::Reliable);
	const bool bDispatchUnreliableAttachments = EnumHasAnyFlags(DispatchFlags, ENetObjectAttachmentDispatchFlags::Unreliable);
	const bool bCanDelayAttachments = Parameters.bAllowDelayingAttachmentsWithUnresolvedReferences && (DelayAttachmentsWithUnresolvedReferences != nullptr && DelayAttachmentsWithUnresolvedReferences->GetInt() > 0);
	const uint32 InternalIndex = ReplicationInfo->InternalIndex;

	/**
	 * This code path handles all cases where the initial state has already been applied. An object can have multiple entries in ObjectsPendingResolve.
	 * Reliable attachments will be dispatched if they can be resolved or if CVarDelayUnmappedRPCs is <= 0. Unreliable but ordered attachments will always be dispatched.
	 */
	bool bHasUnresolvedReferences = false;
	const ENetObjectAttachmentType AttachmentType = (IsObjectIndexForOOBAttachment(InternalIndex) ? ENetObjectAttachmentType::OutOfBand : ENetObjectAttachmentType::Normal);
	if (FNetObjectAttachmentReceiveQueue* AttachmentQueue = Attachments.GetQueue(AttachmentType, InternalIndex))
	{
		if (bDispatchReliableAttachments)
		{
			while (const TRefCountPtr<FNetBlob>* Attachment = AttachmentQueue->PeekReliable())
			{
				// Delay reliable attachments with unresolved pending references
				const bool bIsReliable = EnumHasAnyFlags(Attachment->GetReference()->GetCreationInfo().Flags, ENetBlobFlags::Reliable);
				if (bIsReliable && bCanDelayAttachments)
				{
					bool bDelayRpc = false;

					FNetReferenceCollector Collector;
					Attachment->GetReference()->CollectObjectReferences(Context, Collector);

					// Check status of references, as we already should have queued up any unmapped references at the batch level, it should be enough to only check if we have any unresolved references pending async load.
					// NOTE: Behavior is slightly different between Iris and old replication system due to the fact that Iris processes incoming packet data prior to dispatching received stats and RPC:s, that means that 
					// we expect to be able to resolve all dynamic references contained in the same data packet and does not delay the RPC until the next tick to solve that as the old system does. 
					// The difference is that the old system might be able to resolve incoming dynamic references from later packets processed for the same tick, but as this is far from guaranteed we currently do not try to mimic this.
					for (const FNetReferenceCollector::FReferenceInfo& Info : MakeArrayView(Collector.GetCollectedReferences()))
					{
						if (ObjectReferenceCache->IsNetRefHandlePending(Info.Reference.GetRefHandle(), PendingBatchHolder))
						{
							bDelayRpc = true;
							break;
						}
					}

					if (bDelayRpc)
					{
						const FReplicationStateDescriptor* Descriptor = Attachment->GetReference()->GetReplicationStateDescriptor();

						UE_LOGF(LogIris, Verbose, "Delaying Attachment - %ls for InternalIndex %u.", (Descriptor != nullptr ? ToCStr(Descriptor->DebugName) : TEXT("N/A")), InternalIndex);
						break;
					}
				}

				NetBlobHandlerManager->OnNetBlobReceived(Context, *reinterpret_cast<const TRefCountPtr<FNetBlob>*>(Attachment));
				AttachmentQueue->PopReliable();

				if (Context.HasError())
				{
					return;
				}
			}
		}

		if (bDispatchUnreliableAttachments)
		{
			while (const TRefCountPtr<FNetBlob>* Attachment = AttachmentQueue->PeekUnreliable())
			{
				NetBlobHandlerManager->OnNetBlobReceived(Context, *Attachment);
				AttachmentQueue->PopUnreliable();

				if (Context.HasError())
				{
					return;
				}
			}
		}
		
		if (AttachmentQueue->IsSafeToDestroy())
		{
			// N.B. AttachmentQueue is no longer valid after this call
			Attachments.DropAllAttachments(AttachmentType, InternalIndex);
		}
		else
		{
			bHasUnresolvedReferences = AttachmentQueue->HasUnprocessed();
		}
	}
	else
	{
		// Should not get here, if we do something is out of sync and we should disconnect
		Context.SetError(NetError_FailedToFindAttachmentQueue);
		ensure(AttachmentType == ENetObjectAttachmentType::OutOfBand);
	}

	// Update tracking of objects with attachments pending resolve
	if (bHasUnresolvedReferences && !ReplicationInfo->bHasAttachments)
	{
		ObjectsWithAttachmentPendingResolve.Add(InternalIndex);
	}
	else if (!bHasUnresolvedReferences && ReplicationInfo->bHasAttachments)
	{
		ObjectsWithAttachmentPendingResolve.RemoveSwap(InternalIndex);
	}
	ReplicationInfo->bHasAttachments = bHasUnresolvedReferences;
}

bool FReplicationReader::ReadSentinel(FNetSerializationContext& Context, const TCHAR* DebugName)
{
	bool bSuccess = true;

#if UE_NET_REPLICATIONDATASTREAM_DEBUG
	if (EnumHasAnyFlags(StreamDebugFeatures, EReplicationDataStreamDebugFeatures::Sentinels))
	{
		bSuccess = ReadAndVerifySentinelBits(Context.GetBitStreamReader(), DebugName, 8U);
		if (!bSuccess)
		{
			Context.SetError(GNetError_BitStreamError);
		}
	}
#endif

	return bSuccess;
}

FString FReplicationReader::PrintObjectInfo(FInternalNetRefIndex ObjectIndex, FNetRefHandle NetRefHandle) const
{
	const FReplicatedObjectInfo* ObjectInfo = GetReplicatedObjectInfo(ObjectIndex);
	if (!ObjectInfo)
	{
		return FString::Printf(TEXT("No object info for (InternalIndex: %u)"), ObjectIndex);
	}

	TStringBuilder<512> InfoBuilder;

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectIndex);
	const bool bIsSubObject = ObjectData.SubObjectRootIndex != InvalidInternalNetRefIndex;
	InfoBuilder.Appendf(TEXT("Status info: 0x%llx (%s)"), ObjectInfo->Value, bIsSubObject?TEXT("SubObject"):TEXT("RootObject"));

	const FInternalNetRefIndex RootInternalIndex = bIsSubObject ? ObjectData.SubObjectRootIndex : ObjectIndex;
	const FNetRefHandle RootObjectHandle = bIsSubObject ? NetRefHandleManager->GetNetRefHandleFromInternalIndex(RootInternalIndex) : NetRefHandle;
	if (const FPendingBatchData* PendingBatchData = PendingBatchHolder.Find(RootObjectHandle))
	{
		InfoBuilder.Appendf(TEXT("| PendingBatches: QueuedChunks: %d | PendingReferences(%d): "),PendingBatchData->QueuedDataChunks.Num(), PendingBatchData->PendingMustBeMappedReferences.Num());

		for (const FNetRefHandle& PendingHandle : PendingBatchData->PendingMustBeMappedReferences)
		{
			InfoBuilder << PendingHandle.ToString() << ", ";
		}

		InfoBuilder.Appendf(TEXT(" | CreationDependentParents(%d): "), PendingBatchData->CreationDependentParents.Num());
		for (FNetRefHandle CreationDependentParent : PendingBatchData->CreationDependentParents)
		{
			InfoBuilder << CreationDependentParent.ToString() << ", ";
		}
	}
	else
	{
		InfoBuilder << TEXT("| NoPendingBatches");
	}

	return InfoBuilder.ToString();
}

//------------------------------------------------------------------------
// FPendingBatchHolder
//------------------------------------------------------------------------

FPendingBatchData* FPendingBatchHolder::CreatePendingBatch(FNetRefHandle Owner)
{
	// TODO: Would be more efficient to base this off FReplicationSystem::GetElapsedTime() instead.
	const uint64 CurrentTime = FPlatformTime::Cycles64();
	
	FPendingBatchData* NewData(new FPendingBatchData());
	
	NewData->Owner = Owner;
	NewData->PendingBatchStartCycles = CurrentTime;
	NewData->NextWarningTimeout = FPlatformTime::ToSeconds64(CurrentTime) + QueuedBatchTimeoutSeconds;

	PendingBatches.Add(Owner, FPendingBatchDataPtr(NewData));
	
	return NewData;
}

} // end namespace UE::Net::Private
