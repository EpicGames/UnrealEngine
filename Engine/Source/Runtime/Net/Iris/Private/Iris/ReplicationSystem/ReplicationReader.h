// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This class will never be included from public headers
#include "Iris/ReplicationSystem/AttachmentReplication.h"
#include "Iris/ReplicationSystem/ChangeMaskUtil.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/ReplicationSystem/PendingBatchData.h"
#include "Iris/ReplicationSystem/ReplicationDataStreamDebug.h"
#include "Iris/ReplicationSystem/ReplicationTypes.h"
#include "Containers/Map.h"
#include "Misc/MemStack.h"

// Forward declaration
class UObjectReplicationBridge;
class FRPCDoSDetection;

enum class EIrisAsyncLoadingPriority : uint8;

namespace UE::Net
{
	class FNetSerializationContext;
	class FNetTokenStoreState;
	class FReplicationStateStorage;
	class FNetBitStreamReader;
	class FNetBitStreamWriter;

	namespace Private
	{
		class FReplicationSystemInternal;
		class FResolveAndCollectUnresolvedAndResolvedReferenceCollector;
		class FNetBlobHandlerManager;
		class FNetRefHandleManager;

		enum class EInternalDetachReason : uint8;
	}
}

namespace UE::Net::Private
{

enum class ENetObjectAttachmentDispatchFlags : uint32
{
	None = 0,
	Reliable = 1U << 0U,
	Unreliable = Reliable << 1U,
};
ENUM_CLASS_FLAGS(ENetObjectAttachmentDispatchFlags);

/** Deals with Incoming replication Data and dispatch */
class FReplicationReader
{
public:
	FReplicationReader();
	~FReplicationReader();

	// Init
	void Init(const FReplicationParameters& Parameters);
	void Deinit();

	void SetRPCDoS(FRPCDoSDetection* InRPCDoS) { RPCDoS = InRPCDoS; }

	// Read incoming replication data
	void Read(FNetSerializationContext& Context);
	
	// Mark objects pending destroy as unresolvable.
	void UpdateUnresolvableReferenceTracking();

	// Process queued batches
	void ProcessQueuedBatches();

	[[nodiscard]] FString PrintObjectInfo(FInternalNetRefIndex ObjectIndex, FNetRefHandle NetRefHandle) const;

	// Get the number of unresolved handles we are tracking
	uint32 GetNumTrackedUnresolvedHandleToDependents() const
	{
		return UnresolvedHandleToDependents.Num();
	}

	// Get the number of resolved dynamic handles we are tracking
	uint32 GetNumTrackedResolvedDynamicHandleToDependents() const
	{
		return ResolvedDynamicHandleToDependents.Num();
	}

private:
	// ChangeMaskOffset -> FNetRefHandle
	enum EConstants : uint32
	{
		FakeInitChangeMaskOffset = 0xFFFFFFFFU,
	};
	typedef TMultiMap<uint32, FNetRefHandle> FObjectReferenceTracker;
	typedef TArray<FNetRefHandle, TInlineAllocator<32>> FResolvedNetRefHandlesArray;

	struct FReplicatedObjectInfo
	{
		FReplicatedObjectInfo();

		// We accumulate unresolved changes in this changemask
		FChangeMaskStorageOrPointer UnresolvedChangeMaskOrPointer;

		/* In order to be able to do partial updates the changemask bit is stored with the reference.
		 * That also means a reference can have many entries, but at most one per changemask bit. */
		FObjectReferenceTracker UnresolvedObjectReferences;
		FObjectReferenceTracker ResolvedDynamicObjectReferences;

		// These maps provide a O(1) lookup for the number of handles referenced in 
		// UnresolvedObjectReferences and ResolvedDynamicObjectReferences.
		TMap<FNetRefHandle, int16> UnresolvedHandleCount;
		TMap<FNetRefHandle, int16> ResolvedDynamicHandleCount;

		// Baselines
		uint8* StoredBaselines[2];

		uint32 InternalIndex;							// InternalIndex
		union
		{
			uint32 Value;
			struct	 
			{
				uint32 ChangeMaskBitCount : 16;					// This is cached to avoid having to look it up in the protocol		
				uint32 bHasUnresolvedReferences : 1;			// Do we have unresolved references in the changemask?
				uint32 bHasUnresolvedInitialReferences : 1;		// Do we have unresolved initial only references
				uint32 bHasAttachments : 1;
				uint32 bDestroy : 1;
				uint32 bTearOff : 1;
				uint32 bIsDeltaCompressionEnabled : 1;
				uint32 LastStoredBaselineIndex : 2;				// Last stored baseline, as soon as we receive data compressed against the baseline we know that we can release older baselines
				uint32 PrevStoredBaselineIndex : 2;				// Previous stored baselines index
				uint32 Padding : 7;
			};
		};

		bool RemoveUnresolvedHandleCount(FNetRefHandle RefHandle);
		bool RemoveResolvedDynamicHandleCount(FNetRefHandle RefHandle);
	};

	// Temporary Data to dispatch
	struct FDispatchObjectInfo;

	enum : uint32
	{
		ObjectIndexForOOBAttachment = 0U,
		// Try to avoid reallocations for dispatch in the case we need to process a huge object
		ObjectsToDispatchSlackCount = 32U,
	};

	bool IsObjectIndexForOOBAttachment(uint32 InternalIndex) const { return InternalIndex == ObjectIndexForOOBAttachment; }

	// Read index part of handle
	FNetRefHandle ReadNetRefHandleId(FNetSerializationContext& Context, FNetBitStreamReader& Reader) const;

	enum EReadObjectFlag : unsigned
	{
		ReadObjectFlag_IsReadingHugeObjectBatch = 1U,
	};

	// Read a new or updated object
	uint32 ReadObjectBatch(FNetSerializationContext& Context, uint32 ReadObjectFlags);

	// Read object or subobject
	void ReadObjectInBatch(FNetSerializationContext& Context, FNetRefHandle BatchHandle, bool bIsSubObject);

	uint32 ReadObjectsInBatch(FNetSerializationContext& Context, FNetRefHandle InCompleteHandle, bool bHasBatchOwnerData, uint32 BatchEndBitPosition);
	uint32 ReadObjectsInBatchWithoutSizes(FNetSerializationContext& Context, FNetRefHandle InCompleteHandle, bool bHasBatchOwnerData, uint32 BatchEndBitPosition);
	uint32 ReadObjectsInBatchWithSizes(FNetSerializationContext& Context, FNetRefHandle InCompleteHandle, bool bHasBatchOwnerData, uint32 BatchEndBitPosition);


	// Read stream debug features. Use StreamDebugFeatures to see if a feature is enabled.
	void ReadStreamDebugFeatures(FNetSerializationContext& Context);
	// Read objects pending destroy. This will call ReadObjectsAndSubObjectsPendingDestroy or ReadRootObjectsPendingDestroy depending on sending side settings.
	uint32 ReadObjectsPendingDestroy(FNetSerializationContext& Context);

	// Receive only root object destruction and destroy all existing subobjects with them atomically.
	uint32 ReadRootObjectsPendingDestroy(FNetSerializationContext& Context);

	// Read state data for all incoming objects
	void ReadObjects(FNetSerializationContext& Context, uint32 ObjectCountToRead, uint32 ReadObjectFlags);

	// Process a single huge object attachment
	void ProcessHugeObjectAttachment(FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& Attachment);

	// Assemble and deserialize huge object if present for the given attachmenttype.
	void ProcessHugeObject(FNetSerializationContext& Context, ENetObjectAttachmentType AttachmentType);

	// Resolve and dispatch unresolved references
	void ResolveAndDispatchUnresolvedReferencesForObject(FNetSerializationContext& Context, uint32 InternalIndex);

	// Resolve and dispatch unresolved references
	void ResolveAndDispatchUnresolvedReferences();

	// Dispatch all data received for the frame, this includes trying to resolve object references
	void DispatchStateData(FNetSerializationContext& Context);

	// Dispatch resolved attachments
	void ResolveAndDispatchAttachments(FNetSerializationContext& Context, FReplicatedObjectInfo* ReplicationInfo, ENetObjectAttachmentDispatchFlags DispatchFlags);

	// End replication for all objects that the server has told us to destroy or tear off
	void DispatchEndReplication(FNetSerializationContext& Context);

	// Create tracking info for the object with the given InternalInfo
	FReplicatedObjectInfo& StartReplication(uint32 InternalIndex);

	// Remove tracking info for the object with InternalIndex
	void EndReplication(FInternalNetRefIndex InternalIndex, EInternalDetachReason DetachReason);

	// Free any data allocated per object
	void CleanupObjectData(FReplicatedObjectInfo& ObjectInfo);

	// Lookup the tracking info for the object with IntnernalIndex
	FReplicatedObjectInfo* GetReplicatedObjectInfo(uint32 InternalIndex);
	const FReplicatedObjectInfo* GetReplicatedObjectInfo(uint32 InternalIndex) const;

	// Update reference tracking maps for the current object
	void UpdateObjectReferenceTracking_Fast(FReplicatedObjectInfo* ReplicationInfo, FNetBitArrayView ChangeMask, bool bIncludeInitState, FResolvedNetRefHandlesArray& OutNewResolvedRefHandles, const FObjectReferenceTracker& NewUnresolvedReferences, const FObjectReferenceTracker& NewMappedDynamicReferences);

	// Remove all references for object
	void CleanupReferenceTracking(FReplicatedObjectInfo* ObjectInfo);

	// Update ReplicationInfo and OutUnresolvedChangeMask based on data collected by the Collector
	void BuildUnresolvedChangeMaskAndUpdateObjectReferenceTracking(const FResolveAndCollectUnresolvedAndResolvedReferenceCollector& Collector, FNetBitArrayView CollectorChangeMask, FReplicatedObjectInfo* ReplicationInfo, FNetBitArrayView& OutUnresolvedChangeMask, FResolvedNetRefHandlesArray& OutNewResolvedRefHandles);

	void RemoveUnresolvedObjectReferenceInReplicationInfo(FReplicatedObjectInfo* ReplicationInfo, FNetRefHandle Handle);
	void RemoveResolvedObjectReferenceInReplicationInfo(FReplicatedObjectInfo* ReplicationInfo, FNetRefHandle Handle);

	// A previously resolved dynamic reference is now unresolvable. The ReplicationInfo needs to be updated to reflect this.
	// Returns true if the reference was found.
	bool MoveResolvedObjectReferenceToUnresolvedInReplicationInfo(FReplicatedObjectInfo* ReplicationInfo, FNetRefHandle UnresolvableHandle);

	void DeserializeObjectStateDelta(FNetSerializationContext& Context, uint32 InternalIndex, FDispatchObjectInfo& Info, FReplicatedObjectInfo& ObjectInfo, const FNetRefHandleManager::FReplicatedObjectData& ObjectData, uint32& OutNewBaselineIndex);

	// If async loading is enabled this function will verify if we can resolve all PendingMustBeMappedReferences
	FPendingBatchData* UpdateUnresolvedMustBeMappedReferences(FNetRefHandle OwnerHandle, TArray<FNetRefHandle>& MustBeMappedReferences, EIrisAsyncLoadingPriority InIrisAsyncLoadingPriority);

	// Returns true if the object assigned to this handle has been instantiated locally
	bool DoesParentExist(FNetRefHandle ObjectHandle) const;

	// If we are queuing data for a batch we must also defer calls to EndReplication
	// This method writes this method in the form of a QueuedChunk
	bool EnqueueEndReplication(FPendingBatchData* PendingBatchData, FNetRefHandle NetRefHandleToEndReplication, EInternalDetachReason DetachReason);

	// Remove a handle from the hot and cold unresolved caches used by ResolveAndDispatchUnresolvedReferences(). If this handle is marked as unresolved
	// again, it will be added to the hot cache.
	void RemoveFromUnresolvedCache(const FNetRefHandle Handle);

	// Reads and verifies the sentinel was read properly. If an error is detected the context will have an error set. The function will return true if the sentinel matches, false otherwise.
	bool ReadSentinel(FNetSerializationContext& Context, const TCHAR* DebugName);
	
private:

	class FObjectsToDispatchArray;

	FReplicationParameters Parameters;

	FMemStackBase TempLinearAllocator;
	FMemStackChangeMaskAllocator TempChangeMaskAllocator;

	FGlobalChangeMaskAllocator PersistentChangeMaskAllocator;

	FReplicationSystemInternal* ReplicationSystemInternal;
	FNetRefHandleManager* NetRefHandleManager;
	FReplicationStateStorage* StateStorage;
	UObjectReplicationBridge* ReplicationBridge;

	// A cache holding unresolved handles that should be resolved each time ResolveAndDispatchUnresolvedReferences() is called.
	TMap<FNetRefHandle, uint32> HotUnresolvedHandleCache;

	// A cache holding unresolved handles that should be resolved by ResolveAndDispatchUnresolvedReferences() at fixed intervals.
	TMap<FNetRefHandle, uint32> ColdUnresolvedHandleCache;

	// Temporary buffers used by ResolveAndDispatchUnresolvedReferences().
	TSet<FNetRefHandle> VisitedUnresolvedHandles;
	TSet<FInternalNetRefIndex> InternalObjectsToResolve;

	// We track some data about incoming objects
	// Stored in a map for now
	TMap<uint32, FReplicatedObjectInfo> ReplicatedObjects;

	// Temporary data valid during receive
	FObjectsToDispatchArray* ObjectsToDispatchArray;

	// We need to keep some data around for objects with pending dependencies
	// For now just use array and brute force the updates
	TArray<uint32> ObjectsWithAttachmentPendingResolve;

	// Track all objects waiting for this handle to be resolvable
	TMultiMap<FNetRefHandle, FInternalNetRefIndex /*OwnerInternalIndex*/> UnresolvedHandleToDependents;
	
	// Track all objects with a dynamic handle in case it becomes unresolvable
	TMultiMap<FNetRefHandle, uint32> ResolvedDynamicHandleToDependents;

	// We do not expect to have many objects in this state
	FPendingBatchHolder PendingBatchHolder;

	// We do not expect many objects to be broken
	TArray<FNetRefHandle> BrokenObjects;

	// Keep track of the last time we warned about an object blocked by this must be mapped reference
	// Prevents spamming errors for multiple objects all waiting on the same asset
	TMap<FNetRefHandle, double> BlockedMustBeMappedLastWarningTime;

	// Used during receive and processing of pending batches
	TArray<FNetRefHandle> TempMustBeMappedReferences;

	// Used during receive to track objects read from hugeobject path
	TArray<FNetRefHandle> ObjectsReadFromHugeObjects;

	// Preallocate the arrays used by BuildUnresolvedChangeMaskAndUpdateObjectReferenceTracking() to 
	// avoid memory allocations during the frame.
	FObjectReferenceTracker UnresolvedReferencesCache;
	FObjectReferenceTracker MappedDynamicReferencesCache;

	FNetBlobHandlerManager* NetBlobHandlerManager;
	FNetBlobType NetObjectBlobType;
	FNetObjectAttachmentsReader Attachments;
	FObjectReferenceCache* ObjectReferenceCache;
	FNetObjectResolveContext ResolveContext;

	IConsoleVariable const* DelayAttachmentsWithUnresolvedReferences;
	uint32 NumHandlesPendingResolveLastUpdate = 0U;


	uint32 DebugTraceOffset = 0U;

	// Features that can aid in tracking down bitstream errors etc.
	EReplicationDataStreamDebugFeatures StreamDebugFeatures = EReplicationDataStreamDebugFeatures::None;

	FRPCDoSDetection* RPCDoS = nullptr;
};

}
