// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Iris/ReplicationSystem/ChangeMaskUtil.h"
#include "Net/Core/Misc/ResizableCircularQueue.h"

namespace UE::Net::Private
{

class FReplicationRecord
{
public:
	// We should get away with 64k records in flight given a packet window of 256 and an average of 256 replicated object destroys, objects and subobjects per packet
	typedef uint16 ReplicationRecordIndex;
	static constexpr ReplicationRecordIndex InvalidReplicationRecordIndex = 65535U;
	// The MaxReplicationRecordCount should account for at least 256 packets with N records per replicated object or destroyed object. With lots of destroyed objects you could end up with maybe 300-400 records for a single packet.
	static constexpr ReplicationRecordIndex MaxReplicationRecordCount = InvalidReplicationRecordIndex;

	static constexpr uint32 ObjectIndexBitCount = 20U;
	static constexpr uint32 ReplicatedObjectStateBitCount = 5U;

	// Circular queue, indexing into record infos
	struct FRecord
	{
		uint16 RecordCount;
	};

	struct FSubObjectRecord
	{
		struct FSubObjectInfo
		{
			uint32 Index : ObjectIndexBitCount;
			uint32 ReplicatedObjectState : ReplicatedObjectStateBitCount;
		};

		TArray<FSubObjectInfo> SubObjectInfos;
	};

	// $IRIS: implement some some sort of low overhead chunked fifo array allocator for changemasks used for the replication record
	// They tend to be relatively short lived and are always allocated and freed in the same order.
	// for smaller changemasks we use inlined storage.

	// We want to keep this state as small as possible as we will have many of them
	struct FRecordInfo
	{
		FChangeMaskStorageOrPointer ChangeMaskOrPtr;	// used for ChangeMask storage or a pointer to the changemask (could be repurposed to always be an index to save space)
		uint32 Index : ObjectIndexBitCount;				// Index in the ReplicationInfo array, effectively identifying the object
		uint32 ReplicatedObjectState : ReplicatedObjectStateBitCount; // Encode EReplicatedObjectState using as few bits as we can
		uint32 HasChangeMask : 1;						// Do we have a changeMask? 
		uint32 HasAttachments : 1;						// If this flag is set there's an associated AttachmentRecord
		uint32 WroteTearOff : 1;						// If this flag is set, we wrote TearOff
		uint32 WroteDestroySubObject : 1;				// If this flag is set, we wrote DestroySubObject
		uint32 NewBaselineIndex : 2;					// This is a new Baseline pending ack
		uint32 HasSubObjectRecord : 1;					// If this flag is set there's an associated SubObjectRecord
		ReplicationRecordIndex NextIndex;				// Points to next older record index
		uint16 Padding : 16;
	};

	static_assert(sizeof(FRecordInfo) == 16, "Expected sizeof FRecordInfo to be 16 bytes");
	static_assert(sizeof(ReplicationRecordIndex) == 2, "Need to remove or adjust Padding field in FRecordInfo");

public:

	// Simple index based linked list used to track in flight data
	struct FRecordInfoList
	{
		inline FRecordInfoList();

		ReplicationRecordIndex LastRecordIndex;		// Index of the last written index for this object, used to chain replication infos
		ReplicationRecordIndex FirstRecordIndex;	// First index in flight (oldest), used to quickly be able to iterate over all changes in flight
	};

	// Pop RecordInfo from record and remove it from the provided RecordInfoList
	inline void PopInfoAndRemoveFromList(FRecordInfoList& RecordList);

	// Push RecordInfo to record and add it to the provided RecordInfoList
	inline void PushInfoAndAddToList(FRecordInfoList& RecordList, const FRecordInfo& RecordInfo, uint64 AttachmentRecord = 0);

	// Push RecordInfo to record and add it to the provided RecordInfoList. Call this when there's an associated SubObjectRecord.
	inline void PushInfoAndAddToList(FRecordInfoList& RecordList, const FRecordInfo& RecordInfo, const FSubObjectRecord& SubObjectRecord);

	// Reset a RecordInfoList
	inline void ResetList(FRecordInfoList& RecordList);

public:
	inline FReplicationRecord();

	// Get the info for the provided index
	inline FRecordInfo* GetInfoForIndex(ReplicationRecordIndex Index);
	inline const FRecordInfo* GetInfoForIndex(ReplicationRecordIndex Index) const;

	// PeekInfo. If the info indicates there's an attachment record one must call DequeueAttachmentRecord(). If the info indicates there's a SubObjectRecord one must call DequeueSubObjectRecord().
	inline const FRecordInfo& PeekInfo() const { return RecordInfos.Peek(); }
	inline const FRecordInfo& PeekInfoAtOffset(uint32 Offset) const { return RecordInfos.PeekAtOffset(Offset); }
	inline const uint32 GetInfoCount() const { return static_cast<uint32>(RecordInfos.Count()); }
	inline const uint32 GetUnusedInfoCount() const { return static_cast<uint32>(MaxReplicationRecordCount) - static_cast<uint32>(RecordInfos.Count()); }

	// If the info from PeekInfo() indicates there's an attachment record one needs to call this function as well.
	uint64 DequeueAttachmentRecord();

	// If the info from PeekInfo() indicates there's a subobject record one needs to call this function as well.
	FSubObjectRecord DequeueSubObjectRecord();

	// FrontIndex
	inline ReplicationRecordIndex GetFrontIndex() const { return FrontIndex; }

	// Push a record, currently the record is simply a count of how mane RecordInfos we stored for the record
	inline void PushRecord(uint16 InfoCount);

	// Pop a record
	inline uint16 PopRecord();

	inline const uint32 PeekRecordAtOffset(uint32 Offset) const { return Record.PeekAtOffset(Offset); }

	// Num Records
	inline const uint32 GetRecordCount() const { return static_cast<uint32>(Record.Count()); }
	
private:

	// Push Info to queue, the index of the info will be returned, as long as the info is valid it can be retrieved by the index
	inline ReplicationRecordIndex PushInfo(const FRecordInfo& Info);

	// PopInfo
	inline void PopInfo();

private:
	// Storage for RecordInfos
	TResizableCircularQueue<FRecordInfo> RecordInfos;

	// Storages for the Record for each packet
	TResizableCircularQueue<uint16> Record;

	// Storage for attachment records
	TResizableCircularQueue<uint64> AttachmentRecords;

	// Storage for minimalistic subobject records, used by for example object destruction.
	TResizableCircularQueue<FSubObjectRecord> SubObjectRecords;

	// Current Index at the oldest Record in the queue, this is used to do relative indexing into the queue when linking pushed records
	ReplicationRecordIndex FrontIndex;
};

FReplicationRecord::FReplicationRecord()
: RecordInfos(1024)
, Record(256)
, AttachmentRecords(64)
, SubObjectRecords(128)
, FrontIndex(0u)
{
}

FReplicationRecord::FRecordInfo* FReplicationRecord::GetInfoForIndex(ReplicationRecordIndex Index)
{
	if (Index != InvalidReplicationRecordIndex)
	{
		const uint32 Offset = uint32(MaxReplicationRecordCount + Index - FrontIndex) % uint32(MaxReplicationRecordCount);
		return &RecordInfos.PokeAtOffsetNoCheck(Offset);
	}

	return nullptr;
}

const FReplicationRecord::FRecordInfo* FReplicationRecord::GetInfoForIndex(ReplicationRecordIndex Index) const
{
	if (Index != InvalidReplicationRecordIndex)
	{
		const uint32 Offset = uint32(MaxReplicationRecordCount + Index - FrontIndex) % uint32(MaxReplicationRecordCount);
		return &RecordInfos.PeekAtOffsetNoCheck(Offset);
	}

	return nullptr;
}

inline uint64 FReplicationRecord::DequeueAttachmentRecord()
{
	const uint64 AttachmentRecord = AttachmentRecords.Peek();
	AttachmentRecords.Pop();
	return AttachmentRecord;
}

inline FReplicationRecord::FSubObjectRecord FReplicationRecord::DequeueSubObjectRecord()
{
	FSubObjectRecord SubObjectRecord = MoveTemp(SubObjectRecords.Poke());
	SubObjectRecords.Pop();
	return SubObjectRecord;
}

void FReplicationRecord::PushRecord(uint16 InfoCount)
{
	Record.Enqueue(InfoCount);
}

uint16 FReplicationRecord::PopRecord()
{
	check(Record.Count());

	const uint16 InfoCount = Record.Peek();
	Record.Pop();

	return InfoCount;
}

FReplicationRecord::ReplicationRecordIndex FReplicationRecord::PushInfo(const FRecordInfo& Info)
{ 
	const SIZE_T CurrentInfoCount = RecordInfos.Count();

	check(CurrentInfoCount < MaxReplicationRecordCount);

	FRecordInfo& NewInfo = RecordInfos.EnqueueDefaulted_GetRef();
	NewInfo = Info;
	NewInfo.NextIndex = InvalidReplicationRecordIndex;

	return (FrontIndex + CurrentInfoCount) % MaxReplicationRecordCount;
}

void FReplicationRecord::PopInfo()
{ 
	FrontIndex = (FrontIndex + 1U) % MaxReplicationRecordCount;
	RecordInfos.Pop();
};

void FReplicationRecord::PopInfoAndRemoveFromList(FRecordInfoList& RecordList)
{
	// This is the record at the front of the Record.
	const ReplicationRecordIndex RecordIndex = GetFrontIndex();
	const FRecordInfo& RecordInfo = PeekInfo();

	// unlink
	RecordList.FirstRecordIndex = RecordInfo.NextIndex;
	RecordList.LastRecordIndex = (RecordList.LastRecordIndex == RecordIndex) ? InvalidReplicationRecordIndex : RecordList.LastRecordIndex;

	PopInfo();
}

void FReplicationRecord::PushInfoAndAddToList(FRecordInfoList& RecordList, const FRecordInfo& RecordInfo, uint64 AttachmentRecord)
{
	FReplicationRecord::ReplicationRecordIndex NewIndex = PushInfo(RecordInfo);
	if (RecordInfo.HasAttachments)
	{
		AttachmentRecords.Enqueue(AttachmentRecord);
	}

	if (FRecordInfo* LastRecord = GetInfoForIndex(RecordList.LastRecordIndex))
	{
		// Link already in flight record to this newIndex
		LastRecord->NextIndex = NewIndex;
		RecordList.LastRecordIndex = NewIndex;
	}
	else
	{
		// If this is the FirstRecord we update it as well.
		RecordList.FirstRecordIndex = NewIndex;
		RecordList.LastRecordIndex = NewIndex;
	}
}

void FReplicationRecord::PushInfoAndAddToList(FRecordInfoList& RecordList, const FRecordInfo& RecordInfo, const FSubObjectRecord& SubObjectRecord)
{
	FReplicationRecord::ReplicationRecordIndex NewIndex = PushInfo(RecordInfo);
	if (RecordInfo.HasSubObjectRecord)
	{
		SubObjectRecords.Enqueue(SubObjectRecord);
	}

	if (FRecordInfo* LastRecord = GetInfoForIndex(RecordList.LastRecordIndex))
	{
		// Link already in flight record to this newIndex
		LastRecord->NextIndex = NewIndex;
		RecordList.LastRecordIndex = NewIndex;
	}
	else
	{
		// If this is the FirstRecord we update it as well.
		RecordList.FirstRecordIndex = NewIndex;
		RecordList.LastRecordIndex = NewIndex;
	}
}

void FReplicationRecord::ResetList(FRecordInfoList& RecordList)
{
	RecordList.FirstRecordIndex = InvalidReplicationRecordIndex;
	RecordList.LastRecordIndex = InvalidReplicationRecordIndex;
}

FReplicationRecord::FRecordInfoList::FRecordInfoList()
: LastRecordIndex(InvalidReplicationRecordIndex)
, FirstRecordIndex(InvalidReplicationRecordIndex)
{
}

}
