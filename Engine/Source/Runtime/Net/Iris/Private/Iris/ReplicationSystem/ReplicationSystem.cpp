// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/Core/IrisCsv.h"
#include "Iris/Core/IrisDebugging.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisMemoryTracker.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/NetObjectReference.h"

#include "Iris/ReplicationSystem/ObjectReplicationBridgeConfig.h"

#include "Net/Core/Connection/NetEnums.h"
#include "Net/Core/Misc/NetConditionGroupManager.h"
#include "Net/Core/NetToken/NetToken.h"
#include "Net/Core/Trace/NetTrace.h"

#include "Iris/ReplicationState/ReplicationStateUtil.h"

#include "Iris/ReplicationSystem/ChangeMaskCache.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationSystem/ReplicationReader.h"
#include "Iris/ReplicationSystem/ReplicationSystemDelegates.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"
#include "Iris/ReplicationSystem/ReplicationTypes.h"
#include "Iris/ReplicationSystem/ReplicationWriter.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"

#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilterDefinitions.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFiltering.h"

#include "Iris/DataStream/DataStreamManager.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"

#include "Iris/Stats/NetStats.h"
#include "Iris/Stats/NetStatsContext.h"

#include "Iris/Metrics/NetMetrics.h"

#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "UObject/RemoteExecutor.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplicationSystem)

namespace ReplicationSystemCVars
{

static bool bForcePruneBeforeUpdate = false;
static FAutoConsoleVariableRef CVarForcePruneBeforeUpdate(TEXT("net.Iris.ForcePruneBeforeUpdate"), bForcePruneBeforeUpdate, TEXT("Verify integrity of all tracked instances at the start of every update."));

static bool bAllowAttachmentSendPolicyFlags = true;
static FAutoConsoleVariableRef CVarAllowAttachmentSendPolicyFlags(TEXT("net.Iris.Attachments.AllowSendPolicyFlags"), bAllowAttachmentSendPolicyFlags, TEXT("Allow use of ENetObjectAttachmentSendPolicyFlags to specify behavior of RPCs."));

static bool bOnlyResetDirtinessForQuantizedObjects = true;
static FAutoConsoleVariableRef CVarOnlyResetDirtinessForQuantizedObjects(TEXT("net.Iris.OnlyResetDirtinessForQuantizedObjects"), bOnlyResetDirtinessForQuantizedObjects, TEXT("Only Reset Dirtiness For QuantizedObjects, optimization that only resets dirtiness for objects actually considered dirty."));

static bool bSetFilterSetsRequiresFrequentLocationUpdates = true;
static FAutoConsoleVariableRef CVarSetFilterSetsRequiresFrequentLocationUpdates(
	TEXT("net.Iris.SetFilterSetsRequiresFrequentLocationUpdates"),
	bSetFilterSetsRequiresFrequentLocationUpdates,
	TEXT("Call OptionallySetObjectRequiresFrequentWorldLocationUpdate from SetFilter in case the new filter is spatial and the object needs frequent updates. ")
	TEXT("This being enabled is the intended behavior, only disable in case of an unintended regression."));

}

static_assert(!UE::Net::ENetConnectionTraits::None, "UE::Net::ENetConnectionTraits::None must be zero to avoid header include in ReplicationSystem.h");

namespace UE::Net::Private
{

class FReplicationSystemImpl
{
public:
	TMap<FObjectKey, ENetObjectAttachmentSendPolicyFlags> AttachmentSendPolicyFlags;
	UReplicationSystem* ReplicationSystem;
	FNetTokenStore* NetTokenStore;
	FReplicationSystemInternal ReplicationSystemInternal;
	FReplicationSystemDelegates Delegates;
	uint64 IrisDebugHelperDummy = 0U;
	FNetObjectGroupHandle NotReplicatedNetObjectGroupHandle;
	FNetObjectGroupHandle AsyncStopReplicationNetObjectGroupHandle;
	FNetObjectGroupHandle NetGroupOwnerNetObjectGroupHandle;
	FNetObjectGroupHandle NetGroupReplayNetObjectGroupHandle;
	FNetBitArray ConnectionsPendingPostTickDispatchSend;
	EReplicationSystemSendPass CurrentSendPass = EReplicationSystemSendPass::Invalid;

	inline static const FName MetricNameTotalRootObjects = "TotalRootObjects";
	inline static const FName MetricNameTotalSubObjects = "TotalSubObjects";
	inline static const FName MetricNameAvgPendingObjectCount = "AvgPendingObjectCount";
	inline static const FName MetricNameMaxPendingObjectCount = "MaxPendingObjectCount";
	inline static const FName MetricNameAvgPendingDependentObjectCount= "AvgPendingDependentObjectCount";
	inline static const FName MetricNameMaxPendingDependentObjectCount = "MaxPendingDependentObjectCount";
	inline static const FName MetricNameAvgHugeObjectSendQueue = "AvgHugeObjectSendQueue";
	inline static const FName MetricNameMaxHugeObjectSendQueue = "MaxHugeObjectSendQueue";
	bool bHadValidConnectionsLastUpdate = false;

	explicit FReplicationSystemImpl(UReplicationSystem* InReplicationSystem, const UReplicationSystem::FReplicationSystemParams& Params)
	: ReplicationSystem(InReplicationSystem)
	, NetTokenStore(Params.NetTokenStore)
	, ReplicationSystemInternal(
		FReplicationSystemInternalInitParams(
		{ 
			.ReplicationSystemId = InReplicationSystem->GetId(),
			.MaxReplicatedObjectCount = Params.MaxReplicatedObjectCount,
			.NetChunkedArrayCount = Params.PreAllocatedMemoryBuffersObjectCount,
			.MaxReplicationWriterObjectCount = Params.MaxReplicationWriterObjectCount,
			.bUseRemoteObjectReferences = Params.bUseRemoteObjectReferences,
			.bAllowParallelTasks = Params.bAllowParallelTasks,
			.bAllowMinimalUpdateIfNoConnections = Params.bAllowMinimalUpdateIfNoConnections,
			.bUseSharedObjectReferenceCache = (Params.ProxyType == UE::Net::EProxyType::Backend)
		}))
	{
		const bool bIsProxyBackend = (Params.ProxyType == UE::Net::EProxyType::Backend);
		ensureMsgf(!bIsProxyBackend || (bIsProxyBackend && !InReplicationSystem->IsServer()), TEXT("A replication system configured as a proxy backend is expected to be a client connecting to a remote server."));
	}

	~FReplicationSystemImpl()
	{
	}

	void InitDefaultFilteringGroups()
	{
		NotReplicatedNetObjectGroupHandle = ReplicationSystem->CreateGroup(FName("NotReplicated"));
		check(NotReplicatedNetObjectGroupHandle.IsNotReplicatedNetObjectGroup());
		ReplicationSystem->AddExclusionFilterGroup(NotReplicatedNetObjectGroupHandle);

		AsyncStopReplicationNetObjectGroupHandle = ReplicationSystem->CreateGroup(FName("AsyncStopReplication"));
		check(AsyncStopReplicationNetObjectGroupHandle.IsAsyncStopReplicationNetObjectGroup());
		ReplicationSystem->AddExclusionFilterGroup(AsyncStopReplicationNetObjectGroupHandle);
		
		// Setup SubObjectFiltering groups
		NetGroupOwnerNetObjectGroupHandle = ReplicationSystem->GetOrCreateSubObjectFilter(UE::Net::NetGroupOwner);
		check(NetGroupOwnerNetObjectGroupHandle.IsNetGroupOwnerNetObjectGroup());

		NetGroupReplayNetObjectGroupHandle = ReplicationSystem->GetOrCreateSubObjectFilter(UE::Net::NetGroupReplay);
		check(NetGroupReplayNetObjectGroupHandle.IsNetGroupReplayNetObjectGroup());
	}

	void Init(const UReplicationSystem::FReplicationSystemParams& Params)
	{
#if !UE_BUILD_SHIPPING
		IrisDebugHelperDummy = UE::Net::IrisDebugHelper::Init();
#endif
		const uint32 ReplicationSystemId = ReplicationSystem->GetId();

		// Verify that we got a NetTokenStore and that it is configured as we expect.
		bool bHasValidNetTokenStore = ensureMsgf(Params.NetTokenStore, TEXT("ReplicationSystem cannot be initialized without a valid NetTokenStore"));
		bHasValidNetTokenStore = bHasValidNetTokenStore && ensureMsgf(Params.NetTokenStore->GetDataStore<FStringTokenStore>(), TEXT("ReplicationSystem cannot be initialized without a StringTokenStore"));
		bHasValidNetTokenStore = bHasValidNetTokenStore && ensureMsgf(Params.NetTokenStore->GetDataStore<FNameTokenStore>(), TEXT("ReplicationSystem cannot be initialized without a NameTokenStore"));

		if (!bHasValidNetTokenStore)
		{
			LowLevelFatalError(TEXT("Cannot initialize ReplicationSystem with invalid NetTokenStore"));
			return;
		}

		FNetRefHandleManager& NetRefHandleManager = ReplicationSystemInternal.GetNetRefHandleManager();
		{
			FNetRefHandleManager::FInitParams NetRefHandleManagerInitParams;
			NetRefHandleManagerInitParams.ReplicationSystemId = ReplicationSystemId;
			NetRefHandleManagerInitParams.MaxActiveObjectCount = Params.MaxReplicatedObjectCount;
			NetRefHandleManagerInitParams.InternalNetRefIndexInitSize = Params.InitialNetObjectListCount;
			NetRefHandleManagerInitParams.InternalNetRefIndexGrowSize = Params.NetObjectListGrowCount;
			NetRefHandleManagerInitParams.NetChunkedArrayCount = Params.PreAllocatedMemoryBuffersObjectCount;
			NetRefHandleManagerInitParams.InitialNetRefHandleIndex = Params.InitialNetRefHandleIndex;
			NetRefHandleManager.Init(NetRefHandleManagerInitParams);

			NetRefHandleManager.GetOnMaxInternalNetRefIndexIncreasedDelegate().AddRaw(this, &FReplicationSystemImpl::OnMaxInternalNetRefIndexIncreased);
			NetRefHandleManager.GetOnInternalNetRefIndicesFreedDelegate().AddRaw(this, &FReplicationSystemImpl::OnInternalNetRefIndicesFreed);
		}

		// Note that Params.MaxReplicatedObjectCount was just a suggestion for the NetRefHandleManager.
		// From here systems must rely on the NetRefHandleManager configuration.
		const uint32 AbsoluteMaxObjectCount =  NetRefHandleManager.GetMaxActiveObjectCount();
		const uint32 CurrentMaxInternalNetRefIndex = NetRefHandleManager.GetCurrentMaxInternalNetRefIndex();

		// DirtyNetObjectTracking is only needed when object replication is allowed
		if (Params.bAllowObjectReplication)
		{
			// $IRIS TODO: Need object ID range. Currently abusing hardcoded values from FNetRefHandleManager
			FDirtyNetObjectTrackerInitParams DirtyNetObjectTrackerInitParams;

			DirtyNetObjectTrackerInitParams.NetRefHandleManager = &NetRefHandleManager;
			DirtyNetObjectTrackerInitParams.ReplicationSystemId = ReplicationSystemId;
			DirtyNetObjectTrackerInitParams.MaxInternalNetRefIndex = CurrentMaxInternalNetRefIndex;

			ReplicationSystemInternal.InitDirtyNetObjectTracker(DirtyNetObjectTrackerInitParams);
		}

		{
			FReplicationStateStorage& StateStorage = ReplicationSystemInternal.GetReplicationStateStorage();
			FReplicationStateStorageInitParams InitParams;
			InitParams.ReplicationSystem = ReplicationSystem;
			InitParams.NetRefHandleManager = &NetRefHandleManager;
			InitParams.MaxObjectCount = AbsoluteMaxObjectCount;
			InitParams.MaxInternalNetRefIndex = CurrentMaxInternalNetRefIndex;
			InitParams.MaxConnectionCount = ReplicationSystemInternal.GetConnections().GetMaxConnectionCount();
			InitParams.MaxDeltaCompressedObjectCount = Params.MaxDeltaCompressedObjectCount;
			StateStorage.Init(InitParams);
		}

		{
			FNetObjectGroups& Groups = ReplicationSystemInternal.GetGroups();
			FNetObjectGroupInitParams InitParams = {};
			InitParams.NetRefHandleManager = &NetRefHandleManager;
			InitParams.MaxInternalNetRefIndex = CurrentMaxInternalNetRefIndex;
			InitParams.MaxGroupCount = Params.MaxNetObjectGroupCount;

			Groups.Init(InitParams);
		}

		{
			FReplicationStateDescriptorRegistry& Registry = ReplicationSystemInternal.GetReplicationStateDescriptorRegistry();
			FReplicationStateDescriptorRegistryInitParams InitParams = {};
			InitParams.ProtocolManager = &ReplicationSystemInternal.GetReplicationProtocolManager();

			Registry.Init(InitParams);
		}		

		{
			FWorldLocations& WorldLocations = ReplicationSystemInternal.GetWorldLocations();
			FWorldLocationsInitParams InitParams;
			InitParams.MaxInternalNetRefIndex = CurrentMaxInternalNetRefIndex;
			InitParams.ReplicationSystem = ReplicationSystem;
			WorldLocations.Init(InitParams);
		}
	
		{
			FDeltaCompressionBaselineInvalidationTracker& DeltaCompressionBaselineInvalidationTracker = ReplicationSystemInternal.GetDeltaCompressionBaselineInvalidationTracker();
			FDeltaCompressionBaselineInvalidationTrackerInitParams InitParams;
			InitParams.BaselineManager = &ReplicationSystemInternal.GetDeltaCompressionBaselineManager();
			InitParams.MaxInternalNetRefIndex = CurrentMaxInternalNetRefIndex;
			DeltaCompressionBaselineInvalidationTracker.Init(InitParams);
		}

		{
			FDeltaCompressionBaselineManager& DeltaCompressionBaselineManager = ReplicationSystemInternal.GetDeltaCompressionBaselineManager();
			FDeltaCompressionBaselineManagerInitParams InitParams;
			InitParams.BaselineInvalidationTracker = &ReplicationSystemInternal.GetDeltaCompressionBaselineInvalidationTracker();
			InitParams.Connections = &ReplicationSystemInternal.GetConnections();
			InitParams.NetRefHandleManager = &NetRefHandleManager;
			InitParams.ReplicationStateStorage = &ReplicationSystemInternal.GetReplicationStateStorage();
			InitParams.MaxNetObjectCount = AbsoluteMaxObjectCount;
			InitParams.MaxInternalNetRefIndex = CurrentMaxInternalNetRefIndex;
			InitParams.MaxDeltaCompressedObjectCount = Params.MaxDeltaCompressedObjectCount;
			InitParams.ReplicationSystem = ReplicationSystem;
			DeltaCompressionBaselineManager.Init(InitParams);
		}

		{
			FReplicationFiltering& ReplicationFiltering = ReplicationSystemInternal.GetFiltering();
			FReplicationFilteringInitParams InitParams;
			InitParams.ReplicationSystem = ReplicationSystem;
			InitParams.Connections = &ReplicationSystemInternal.GetConnections();
			InitParams.NetRefHandleManager = &NetRefHandleManager;
			InitParams.Groups = &ReplicationSystemInternal.GetGroups();
			InitParams.MaxInternalNetRefIndex = CurrentMaxInternalNetRefIndex;
			InitParams.MaxGroupCount = Params.MaxNetObjectGroupCount;
			ReplicationFiltering.Init(InitParams);
		}

		InitDefaultFilteringGroups();

		{
			FReplicationConditionals& ReplicationConditionals = ReplicationSystemInternal.GetConditionals();
			FReplicationConditionalsInitParams InitParams = {};
			InitParams.NetRefHandleManager = &NetRefHandleManager;
			InitParams.ReplicationConnections = &ReplicationSystemInternal.GetConnections();
			InitParams.ReplicationFiltering = &ReplicationSystemInternal.GetFiltering();
			InitParams.NetObjectGroups = &ReplicationSystemInternal.GetGroups();
			InitParams.BaselineInvalidationTracker = &ReplicationSystemInternal.GetDeltaCompressionBaselineInvalidationTracker();
			InitParams.MaxInternalNetRefIndex = CurrentMaxInternalNetRefIndex;
			InitParams.MaxConnectionCount = ReplicationSystemInternal.GetConnections().GetMaxConnectionCount();
			ReplicationConditionals.Init(InitParams);
		}

		{
			FReplicationPrioritization& ReplicationPrioritization = ReplicationSystemInternal.GetPrioritization();
			FReplicationPrioritizationInitParams InitParams;
			InitParams.ReplicationSystem = ReplicationSystem;
			InitParams.Connections = &ReplicationSystemInternal.GetConnections();
			InitParams.NetRefHandleManager = &NetRefHandleManager;
			InitParams.MaxInternalNetRefIndex = CurrentMaxInternalNetRefIndex;
			ReplicationPrioritization.Init(InitParams);
		}

		ReplicationSystemInternal.SetReplicationBridge(Params.ReplicationBridge);

		// Init replication bridge
		ReplicationSystemInternal.GetReplicationBridge()->Initialize(ReplicationSystem);

		{
			FObjectReferenceCacheParams InitParams;
			InitParams.bAuthority = ReplicationSystem->IsServer();

			const bool bIsUsingSharedObjectReferenceCache = &ReplicationSystemInternal.GetObjectReferenceCache() == FReplicationSystemFactory::GetSharedObjectReferenceCache().Get();
			if (!bIsUsingSharedObjectReferenceCache)
			{
				InitParams.ObjectToNetRefHandleSerial = Params.ObjectToNetRefHandleSerial;
				InitParams.NetRefHandleSerialToObject = Params.NetRefHandleSerialToObject;
			}
			else
			{
				const bool bDelegatesAreBound = Params.ObjectToNetRefHandleSerial.IsBound() || Params.NetRefHandleSerialToObject.IsBound();
				check(!bDelegatesAreBound);
				UE_CLOGF(bDelegatesAreBound, LogIris, Error, "Cannot bind FNetRefHandle mapping delegates when using a shared object reference cache.");
			}

			ReplicationSystemInternal.GetObjectReferenceCache().Init(InitParams);
		}

		// Init custom packagemap we use for capturing references for backwards compatible NetSerializers
		{
			UIrisObjectReferencePackageMap* ObjectReferencePackageMap = NewObject<UIrisObjectReferencePackageMap>();
			ObjectReferencePackageMap->AddToRoot();
			ReplicationSystemInternal.SetIrisObjectReferencePackageMap(ObjectReferencePackageMap);
		}

		{
			FNetBlobManager& BlobManager = ReplicationSystemInternal.GetNetBlobManager();
			FNetBlobManagerInitParams InitParams = {};
			InitParams.ReplicationSystem = ReplicationSystem;
			InitParams.bSendAttachmentsWithObject = ReplicationSystem->IsServer();
			BlobManager.Init(InitParams);
		}

		if (Params.ForwardNetRPCCallDelegate.IsBound())
		{
			ReplicationSystemInternal.GetForwardNetRPCCallMulticastDelegate().Add(Params.ForwardNetRPCCallDelegate);
		}

		ConnectionsPendingPostTickDispatchSend.Init(ReplicationSystemInternal.GetConnections().GetMaxConnectionCount());

		{
			FNetTypeStats& NetStats = ReplicationSystemInternal.GetNetTypeStats();
			FNetTypeStats::FInitParams InitParams;
			InitParams.NetRefHandleManager = &NetRefHandleManager;
			NetStats.Init(InitParams);
		}
	}

	void Deinit()
	{
		ReplicationSystemInternal.GetPrioritization().Deinit();
		ReplicationSystemInternal.GetFiltering().Deinit();
		ReplicationSystemInternal.GetConnections().Deinit();
		ReplicationSystemInternal.GetDeltaCompressionBaselineManager().Deinit();
		ReplicationSystemInternal.GetReplicationStateStorage().Deinit();

		if (ReplicationSystemInternal.IsDirtyNetObjectTrackerInitialized())
		{
			ReplicationSystemInternal.GetDirtyNetObjectTracker().Deinit();
		}

		// Reset replication bridge
		ReplicationSystemInternal.GetReplicationBridge()->Deinitialize();

		if (UIrisObjectReferencePackageMap* ObjectReferencePackageMap = ReplicationSystemInternal.GetIrisObjectReferencePackageMap())
		{
			ObjectReferencePackageMap->RemoveFromRoot();
			ObjectReferencePackageMap->MarkAsGarbage();
			ReplicationSystemInternal.SetIrisObjectReferencePackageMap(static_cast<UIrisObjectReferencePackageMap*>(nullptr));
		}

		ReplicationSystemInternal.GetNetRefHandleManager().GetOnMaxInternalNetRefIndexIncreasedDelegate().RemoveAll(this);
		ReplicationSystemInternal.GetNetRefHandleManager().GetOnInternalNetRefIndicesFreedDelegate().RemoveAll(this);
		ReplicationSystemInternal.GetNetRefHandleManager().Deinit();
	}

	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
	{
		ReplicationSystemInternal.GetReplicationStateStorage().OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
		ReplicationSystemInternal.GetGroups().OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
		ReplicationSystemInternal.GetWorldLocations().OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
		ReplicationSystemInternal.GetDeltaCompressionBaselineInvalidationTracker().OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
		ReplicationSystemInternal.GetDeltaCompressionBaselineManager().OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
		ReplicationSystemInternal.GetFiltering().OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
		ReplicationSystemInternal.GetConditionals().OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
		ReplicationSystemInternal.GetPrioritization().OnMaxInternalNetRefIndexIncreased(NewMaxInternalIndex);
	}

	void OnInternalNetRefIndicesFreed(const TConstArrayView<FInternalNetRefIndex>& FreedIndices)
	{
		ReplicationSystemInternal.GetFiltering().OnInternalNetRefIndicesFreed(FreedIndices);
		ReplicationSystemInternal.GetConditionals().OnInternalNetRefIndicesFreed(FreedIndices);
		ReplicationSystemInternal.GetObjectReferenceCache().OnInternalNetRefIndicesFreed(FreedIndices);
	}

	void StartPreSendUpdate()
	{
		// Block unsupported operations until SendUpdate is finished
		ReplicationSystemInternal.SetBlockFilterChanges(true);

		// Tell systems we are starting PreSendUpdate
		ReplicationSystemInternal.GetReplicationBridge()->OnStartPreSendUpdate();

		// Sync the state of the world at the beginning of PreSendUpdate.
		ReplicationSystemInternal.GetNetRefHandleManager().OnPreSendUpdate();
	}

	void CallPreSendUpdate(float DeltaSeconds)
	{
		ReplicationSystemInternal.GetReplicationBridge()->CallPreSendUpdate(DeltaSeconds);
	}

	void EndPostSendUpdate()
	{
		// Unblock operations
		ReplicationSystemInternal.SetBlockFilterChanges(false);

		ReplicationSystemInternal.GetChangeMaskCache().ResetCache();

		// Store the scope list for the next SendUpdate.
		ReplicationSystemInternal.GetNetRefHandleManager().OnPostSendUpdate();

		// Update handles pending tear-off/end-replication
		ReplicationSystemInternal.GetReplicationBridge()->UpdateHandlesPendingEndReplication();

		// Reset baseline invalidation
		ReplicationSystemInternal.GetDeltaCompressionBaselineInvalidationTracker().PostSendUpdate();

		// Reset dirty info list for the next frame
		ReplicationSystemInternal.GetWorldLocations().PostSendUpdate();

		// Tell systems we finished PostSendUpdate
		ReplicationSystemInternal.GetReplicationBridge()->OnPostSendUpdate();
	}

	void UpdateDirtyObjectList()
	{
		ReplicationSystemInternal.GetDirtyNetObjectTracker().UpdateDirtyNetObjects();
	}

	void UpdateDirtyListPostPoll()
	{
		// From here there shouldn't be any user code that calls public API functions.
		ReplicationSystemInternal.GetWorldLocations().LockDirtyInfoList(true);

		ReplicationSystemInternal.GetDirtyNetObjectTracker().UpdateAccumulatedDirtyList();
	}

	void UpdateWorldLocations()
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_UpdateWorldLocations);
		ReplicationSystemInternal.GetReplicationBridge()->CallUpdateInstancesWorldLocation();
	}

	void UpdateFiltering()
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_UpdateFiltering);
		LLM_SCOPE_BYTAG(Iris);

		FReplicationFiltering& Filtering = ReplicationSystemInternal.GetFiltering();
		Filtering.Filter();
	}

	void UpdateObjectScopes()
	{
		LLM_SCOPE_BYTAG(Iris);

		FReplicationFiltering& Filtering = ReplicationSystemInternal.GetFiltering();

		{
			IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_UpdateConnectionsScope);
		
			// Iterate over all valid connections and propagate updated scopes
			FReplicationConnections& Connections = ReplicationSystemInternal.GetConnections();
		
			auto UpdateConnectionScope = [&Filtering, &Connections](uint32 ConnectionId)
			{
				FReplicationConnection* Conn = Connections.GetConnection(ConnectionId);
				const FNetBitArrayView ObjectsInScope = Filtering.GetRelevantObjectsInScope(ConnectionId);
				Conn->ReplicationWriter->UpdateScope(ObjectsInScope);
			};

			const FNetBitArray& ValidConnections = Connections.GetValidConnections();
			ValidConnections.ForAllSetBits(UpdateConnectionScope);
		}
	}

	// Can run at any time between scoping and replication.
	void UpdateConditionals()
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_UpdateConditionals);

		FReplicationConditionals& Conditionals = ReplicationSystemInternal.GetConditionals();
		Conditionals.Update();
	}

	// Runs after filtering
	void UpdatePrioritization(const FNetBitArrayView& ReplicatingConnections)
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_UpdatePrioritization);
		LLM_SCOPE_BYTAG(Iris);

		const FNetBitArrayView RelevantObjects = ReplicationSystemInternal.GetNetRefHandleManager().GetRelevantObjectsInternalIndices();

		// Make a list of objects that were dirty and are also relevant
		FNetBitArray DirtyAndRelevantObjects(RelevantObjects.GetNumBits(), FNetBitArray::NoResetNoValidate);
		FNetBitArrayView DirtyAndRelevantObjectsView = MakeNetBitArrayView(DirtyAndRelevantObjects, FNetBitArray::NoResetNoValidate);

		const FNetBitArrayView AccumulatedDirtyObjects = ReplicationSystemInternal.GetDirtyNetObjectTracker().GetAccumulatedDirtyNetObjects();
		DirtyAndRelevantObjectsView.Set(RelevantObjects, FNetBitArray::AndOp, AccumulatedDirtyObjects);

		FReplicationPrioritization& Prioritization = ReplicationSystemInternal.GetPrioritization();
		Prioritization.Prioritize(ReplicatingConnections, DirtyAndRelevantObjectsView);
	}

	void PropagateDirtyChanges()
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_PropagateDirtyChanges);

		FReplicationConnections& Connections = ReplicationSystemInternal.GetConnections();
		const FChangeMaskCache& UpdatedChangeMasks = ReplicationSystemInternal.GetChangeMaskCache();

		// Iterate over connections and propagate dirty changemasks
		auto UpdateDirtyChangeMasks = [&Connections, &UpdatedChangeMasks](uint32 ConnectionId)
		{
			FReplicationConnection* Conn = Connections.GetConnection(ConnectionId);
			
			// Only update open connections, as closing connections are only
			// flushing reliable data and we shouldn't send new state data to them.
			if (!Conn->bIsClosing)
			{
				Conn->ReplicationWriter->UpdateDirtyChangeMasks(UpdatedChangeMasks);
			}
		};
		const FNetBitArray& ValidConnections = Connections.GetValidConnections();
		ValidConnections.ForAllSetBits(UpdateDirtyChangeMasks);
	}

	void QuantizeDirtyStateData()
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_QuantizeDirtyStateData);
		LLM_SCOPE_BYTAG(IrisState);

		FNetRefHandleManager& NetRefHandleManager = ReplicationSystemInternal.GetNetRefHandleManager();
		FChangeMaskCache& Cache = ReplicationSystemInternal.GetChangeMaskCache();

		uint32 QuantizedObjectCount = 0;
		
		// Prepare cache
		constexpr uint32 ReservedIndexCount = 2048;
		constexpr uint32 ReservedStorageCount = 16536;
		Cache.PrepareCache(ReservedIndexCount, ReservedStorageCount);

		// We use this ChangeMaskWriter to capture changemasks for all copied objects
		FNetBitStreamWriter ChangeMaskWriter;

		// Setup context
		FNetSerializationContext SerializationContext;
		FInternalNetSerializationContext InternalContext(ReplicationSystem);

		SerializationContext.SetInternalContext(&InternalContext);
		SerializationContext.SetNetStatsContext(ReplicationSystemInternal.GetNetTypeStats().GetNetStatsContext());

		// Copy the state data of objects that were dirty this frame.
		FNetBitArrayView DirtyObjectsToQuantize = NetRefHandleManager.GetDirtyObjectsToQuantize();

		auto QuantizeFunction = [&ChangeMaskWriter, &Cache, &NetRefHandleManager, &QuantizedObjectCount, &SerializationContext](uint32 DirtyIndex)
		{
			QuantizedObjectCount += FReplicationInstanceOperationsInternal::QuantizeObjectStateData(ChangeMaskWriter, Cache, NetRefHandleManager, SerializationContext, DirtyIndex);
		};

		DirtyObjectsToQuantize.ForAllSetBits(QuantizeFunction);
		// DirtyObjectsToQuantize is cleared in ResetObjectStateDirtiness

		const uint32 ReplicationSystemId = ReplicationSystem->GetId();
		UE_NET_TRACE_FRAME_STATSCOUNTER(ReplicationSystemId, ReplicationSystem.QuantizedObjectCount, QuantizedObjectCount, ENetTraceVerbosity::Trace);
	}

	void ExecuteQuantizeDirtyStateData(bool bTransact)
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
		if (bTransact && !AutoRTFM::IsTransactional())
		{
			static FName TransactionalWorkName("Iris FReplicationSystemImpl::QuantizeDirtyStateData");
			UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, [this]()
				{
					QuantizeDirtyStateData();
				});
		}
		else
#endif
		{
			QuantizeDirtyStateData();
		}
	}

	void ResetObjectStateDirtiness()
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_ResetObjectStateDirtiness);

		FNetRefHandleManager& NetRefHandleManager = ReplicationSystemInternal.GetNetRefHandleManager();

		// Clear the objects that got polled this frame
		const FNetBitArrayView PolledObjects = NetRefHandleManager.GetPolledObjectsInternalIndices();
		FNetBitArrayView DirtyObjectsToQuantize = NetRefHandleManager.GetDirtyObjectsToQuantize();

		// This is clearing the internal changemask
		if (ReplicationSystemCVars::bOnlyResetDirtinessForQuantizedObjects)
		{
			DirtyObjectsToQuantize.ForAllSetBits([&NetRefHandleManager](uint32 DirtyIndex)
			{
				FReplicationInstanceOperationsInternal::ResetObjectStateDirtiness(NetRefHandleManager, DirtyIndex);
			});
		}
		else
		{
			PolledObjects.ForAllSetBits([&NetRefHandleManager](uint32 DirtyIndex)
			{
				FReplicationInstanceOperationsInternal::ResetObjectStateDirtiness(NetRefHandleManager, DirtyIndex);
			});
		}

		DirtyObjectsToQuantize.ClearAllBits();

		ReplicationSystemInternal.GetDirtyNetObjectTracker().ReconcilePolledList(PolledObjects);

		// Reset the poll list.
		NetRefHandleManager.GetPolledObjectsInternalIndices().ClearAllBits();
	}

	void ProcessNetObjectAttachmentSendQueue(FNetBlobManager::EProcessMode ProcessMode)
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_ProcessNetObjectAttachmentSendQueue);

		FNetBlobManager& NetBlobManager = ReplicationSystemInternal.GetNetBlobManager();
		NetBlobManager.ProcessNetObjectAttachmentSendQueue(ProcessMode);
	}

	void ProcessOOBNetObjectAttachmentSendQueue()
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_ProcessOOBNetObjectAttachmentSendQueue);

		FNetBlobManager& NetBlobManager = ReplicationSystemInternal.GetNetBlobManager();
		NetBlobManager.ProcessOOBNetObjectAttachmentSendQueue(ConnectionsPendingPostTickDispatchSend);
	}

	void PreReceiveUpdate()
	{
		ReplicationSystemInternal.GetReplicationBridge()->PreReceiveUpdate();
	}

	void PostReceiveUpdate()
	{
		ReplicationSystemInternal.GetReplicationBridge()->PostReceiveUpdate();
	}

	void ResetNetObjectAttachmentSendQueue()
	{
		FNetBlobManager& NetBlobManager = ReplicationSystemInternal.GetNetBlobManager();
		NetBlobManager.ResetNetObjectAttachmentSendQueue();
	}

	bool ShouldUpdateForConnections()
	{
		if (ReplicationSystemInternal.GetInitParams().bAllowMinimalUpdateIfNoConnections)
		{
			// Make sure that we do a complete update after closing the last connection
			const bool bHasValidConnections = ReplicationSystemInternal.GetConnections().GetValidConnections().IsAnyBitSet();
			const bool bUpdateConnections = bHadValidConnectionsLastUpdate || bHasValidConnections;
			bHadValidConnectionsLastUpdate = bHasValidConnections;

			return bUpdateConnections;
		}
		else
		{
			return true;
		}
	}

	TWeakObjectPtr<UNetObjectFilter> CreateFilter(const FNetObjectFilterDefinition& Definition)
	{
		if (!ensure(!ReplicationSystemInternal.AreFilterChangesBlocked()))
		{
			UE_LOGF(
				LogIris,
				Error,
				"CreateFilter() called while filter changes are blocked; filter %ls not created",
				*Definition.FilterName.ToString());

			return nullptr;
		}

		return ReplicationSystemInternal.GetFiltering().CreateFilter(Definition);
	}

	void DestroyFilter(const FName Name)
	{
		if (ReplicationSystemInternal.AreFilterChangesBlocked())
		{
			UE_LOGF(LogIris, Warning, "DestroyFilter() called while filter changes are blocked; filter %ls not destroyed", *Name.ToString());
			return;
		}

		ReplicationSystemInternal.GetFiltering().DestroyFilter(Name);
	}

	void AddConnection(uint32 ConnectionId, UE::Net::ENetConnectionTraits NetConnectionTraits)
	{
		LLM_SCOPE_BYTAG(IrisConnection);

		FReplicationConnections& Connections = ReplicationSystemInternal.GetConnections();
		// Invalid or out of bounds ID?
		if (ConnectionId == UE::Net::InvalidConnectionId || ConnectionId >= Connections.GetMaxConnectionCount())
		{
			UE_LOGF(LogIris, Error, "UReplicationSystem::AddConnection called with a bad connection Id: %u. Max connection count: %u.", ConnectionId, Connections.GetMaxConnectionCount());
			return;
		}

		// Already registered?
		if (Connections.IsValidConnection(ConnectionId))
		{
			UE_LOGF(LogIris, Error, "UReplicationSystem::AddConnection called with already added connection Id: %u.", ConnectionId);
			return;
		}

		{
			Connections.AddConnection(ConnectionId, NetConnectionTraits);

			FReplicationConnection* Connection = Connections.GetConnection(ConnectionId);

			FReplicationParameters Params;
			Params.ReplicationSystem = ReplicationSystem;
			Params.PacketSendWindowSize = 256;
			Params.ConnectionId = ConnectionId;
			Params.MaxInternalNetRefIndex = ReplicationSystemInternal.GetNetRefHandleManager().GetCurrentMaxInternalNetRefIndex();
			Params.MaxReplicationWriterObjectCount = ReplicationSystemInternal.GetInitParams().MaxReplicationWriterObjectCount;

			/** 
			  * Currently we expect all objects to be replicated from server to client.
			  * That means we will have to support sending attachments such as RPCs from
			  * the client to the server, if the RPC is allowed to be sent in the first place.
			  */
			Params.bAllowSendingAttachmentsToObjectsNotInScope = !ReplicationSystem->IsServer();
			Params.bAllowReceivingAttachmentsFromRemoteObjectsNotInScope = true;

			// Delaying attachments with unresolved references on the server could cause massive queues of RPCs, potentially an OOM situation.
			Params.bAllowDelayingAttachmentsWithUnresolvedReferences = !ReplicationSystem->IsServer();

			// $IRIS TODO Replay connection specifics 
			Connection->ReplicationWriter = new FReplicationWriter();
			Connection->ReplicationReader = new FReplicationReader();

			Connection->ReplicationWriter->Init(Params);
			Connection->ReplicationReader->Init(Params);
		}

		{
			FReplicationConditionals& ReplicationConditionals = ReplicationSystemInternal.GetConditionals();
			ReplicationConditionals.AddConnection(ConnectionId, NetConnectionTraits);
		}

		{
			FReplicationFiltering& ReplicationFiltering = ReplicationSystemInternal.GetFiltering();
			ReplicationFiltering.AddConnection(ConnectionId, NetConnectionTraits);
		}

		{
			FReplicationPrioritization& ReplicationPrioritization = ReplicationSystemInternal.GetPrioritization();
			ReplicationPrioritization.AddConnection(ConnectionId, NetConnectionTraits);
		}

		{
			FDeltaCompressionBaselineManager& DC = ReplicationSystemInternal.GetDeltaCompressionBaselineManager();
			DC.AddConnection(ConnectionId, NetConnectionTraits);
		}

		FConnectionHandle ConnectionHandle(ConnectionId);
		Delegates.ConnectionAddedDelegate.Broadcast(ConnectionHandle);
	}

	void RemoveConnection(uint32 ConnectionId)
	{
		FReplicationConnections& Connections = ReplicationSystemInternal.GetConnections();
		if (!Connections.IsValidConnection(ConnectionId))
		{
			UE_LOGF(LogIris, Error, "UReplicationSystem::RemoveConnection called for connection ID that isn't added: %u.", ConnectionId);
		}

		{
			FDeltaCompressionBaselineManager& DC = ReplicationSystemInternal.GetDeltaCompressionBaselineManager();
			DC.RemoveConnection(ConnectionId);
		}

		{
			FReplicationPrioritization& ReplicationPrioritization = ReplicationSystemInternal.GetPrioritization();
			ReplicationPrioritization.RemoveConnection(ConnectionId);
		}

		{
			FReplicationFiltering& ReplicationFiltering = ReplicationSystemInternal.GetFiltering();
			ReplicationFiltering.RemoveConnection(ConnectionId);
		}

		{
			FReplicationConditionals& ReplicationConditionals = ReplicationSystemInternal.GetConditionals();
			ReplicationConditionals.RemoveConnection(ConnectionId);
		}

		{
			FObjectReferenceCache& ObjectReferenceCache = ReplicationSystemInternal.GetObjectReferenceCache();
			ObjectReferenceCache.RemoveConnection(ReplicationSystem, ConnectionId);
		}

		{
			Connections.RemoveConnection(ConnectionId);
		}

		FConnectionHandle ConnectionHandle(ConnectionId);
		Delegates.ConnectionRemovedDelegate.Broadcast(ConnectionHandle);
	}

	bool HasConnectionWithAnyTraits(ENetConnectionTraits NetConnectionTraits) const
	{
		const FReplicationConnections& Connections = ReplicationSystemInternal.GetConnections();
		return Connections.HasConnectionWithAnyTraits(NetConnectionTraits);
	}

	void UpdateDataStreams(UDataStream::FUpdateParameters& UpdateParameters)
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_UpdateDataStreams);

		FReplicationConnections& Connections = ReplicationSystemInternal.GetConnections();
		auto UpdateDataStreamsForConnection = [&Connections, &UpdateParameters](uint32 ConnectionId)
		{
			FReplicationConnection* Conn = Connections.GetConnection(ConnectionId);
			if (UDataStreamManager* DataStreamManager = Conn->DataStreamManager.Get())
			{
				Conn->DataStreamManager->Update(UpdateParameters);
			}
		};
		const FNetBitArray& ValidConnections = Connections.GetValidConnections();
		ValidConnections.ForAllSetBits(UpdateDataStreamsForConnection);
	}

	void UpdateUnresolvableReferenceTracking()
	{
		IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_UpdateUnresolvableReferenceTracking);

		UE_NET_TRACE_FRAME_STATSCOUNTER(ReplicationSystem->GetId(), ObjectReferenceCache.GetNumObjectsWithTrackedSubObjectHandles, ReplicationSystemInternal.GetObjectReferenceCache().GetNumObjectsWithTrackedSubObjectHandles(), ENetTraceVerbosity::Verbose);	

		FReplicationConnections& Connections = ReplicationSystemInternal.GetConnections();
		auto UpdateUnresolvableReferenceTracking = [&Connections](uint32 ConnectionId)
		{
			FReplicationConnection* Conn = Connections.GetConnection(ConnectionId);
			Conn->ReplicationReader->ProcessQueuedBatches();
			Conn->ReplicationReader->UpdateUnresolvableReferenceTracking();
		};
		const FNetBitArray& ValidConnections = Connections.GetValidConnections();
		ValidConnections.ForAllSetBits(UpdateUnresolvableReferenceTracking);
	}

	void CollectNetMetrics(UE::Net::FNetMetrics& OutNetMetrics) const
	{
		using namespace UE::Net;

		const FNetRefHandleManager& NetRefHandleManager = ReplicationSystemInternal.GetNetRefHandleManager();

		const uint32 TotalNetObjects = NetRefHandleManager.GetActiveObjectCount();
		const uint32 TotalSubObjects = NetRefHandleManager.GetSubObjectInternalIndicesView().CountSetBits();

		// Collect stats on total replicated objects
		OutNetMetrics.EmplaceMetric(MetricNameTotalRootObjects, FNetMetric(TotalNetObjects-TotalSubObjects));
		OutNetMetrics.EmplaceMetric(MetricNameTotalSubObjects, FNetMetric(TotalSubObjects));

		// Pending object and huge object stats
		const FReplicationStats& ReplicationStats = ReplicationSystemInternal.GetAccumulatedReplicationStats();
		double AvgPendingObjectCount;
		double AvgPendingDependentObjectCount;
		double AvgHugeObjectSendQueue;
		if (ReplicationStats.SampleCount)
		{
			double SampleCount = (double)ReplicationStats.SampleCount;
			AvgPendingObjectCount = (double)ReplicationStats.PendingObjectCount / SampleCount;
			AvgPendingDependentObjectCount = (double)ReplicationStats.PendingDependentObjectCount / SampleCount;
			AvgHugeObjectSendQueue = (double)ReplicationStats.HugeObjectSendQueue / SampleCount;
		}
		else
		{
			AvgPendingObjectCount = 0.0f;
			AvgPendingDependentObjectCount = 0.0f;
			AvgHugeObjectSendQueue = 0.0f;
		}
		OutNetMetrics.EmplaceMetric(MetricNameAvgPendingObjectCount, AvgPendingObjectCount);
		OutNetMetrics.EmplaceMetric(MetricNameMaxPendingObjectCount, ReplicationStats.MaxPendingObjectCount);
		OutNetMetrics.EmplaceMetric(MetricNameAvgPendingDependentObjectCount, AvgPendingDependentObjectCount);
		OutNetMetrics.EmplaceMetric(MetricNameMaxPendingDependentObjectCount, ReplicationStats.MaxPendingDependentObjectCount);
		OutNetMetrics.EmplaceMetric(MetricNameAvgHugeObjectSendQueue, AvgHugeObjectSendQueue);
		OutNetMetrics.EmplaceMetric(MetricNameMaxHugeObjectSendQueue, ReplicationStats.MaxHugeObjectSendQueue);
	}

	void ResetNetMetrics()
	{
		ReplicationSystemInternal.GetAccumulatedReplicationStats() = {};
	}
}; // end class FReplicationSystemImpl

void FReplicationSystemInternal::RemoveFromAllGroups(FInternalNetRefIndex ObjectInternalIndex)
{
	uint32 NumGroupMemberShips = 0;
	if (AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation. Filter condition on %s failed."), *NetRefHandleManager.PrintObjectFromIndex(ObjectInternalIndex));
		return;
	}

	// We copy the membership array as it is modified during removal
	TArray<FNetObjectGroupHandle> CopiedGroupHandles;
	Groups.GetGroupHandlesOfNetObject(ObjectInternalIndex, CopiedGroupHandles);

	for (FNetObjectGroupHandle GroupHandle : CopiedGroupHandles)
	{
		Groups.RemoveFromGroup(GroupHandle, ObjectInternalIndex);
		Filtering.NotifyObjectRemovedFromGroup(GroupHandle, ObjectInternalIndex);
	}	
}

} // end namespace UE::Net::Private

UReplicationSystem::UReplicationSystem()
: Super()
, Impl(nullptr)
, Id(~0U)
, PIEInstanceID(INDEX_NONE)
, bIsServer(0)
, bAllowObjectReplication(0)
, bDoCollectGarbage(0)
, bReplicateTransactionally(0)
{
}

void UReplicationSystem::Init(uint32 InId, const FReplicationSystemParams& Params)
{
	Id = InId;
	bIsServer = Params.bIsServer;
	bAllowObjectReplication = Params.bAllowObjectReplication;
	bAllowExecuteRPC = Params.bAllowExecuteRPC;
	bAllowExecuteRepNotify = Params.bAllowExecuteRepNotifies;
	bAlwaysSwapRolesOnReplication = Params.bAlwaysSwapRolesOnReplication;
	bReplicateTransactionally = Params.bReplicateTransactionally;
	ProxyType = Params.ProxyType;

	ReplicationBridge = Params.ReplicationBridge;

	FReplicationSystemParams LocalParams = Params;

	if (bIsServer)
	{
		InitializeNetRefHandleGenerator(LocalParams);
	}
	else
	{
		LocalParams.ObjectToNetRefHandleSerial.Unbind();
		LocalParams.NetRefHandleSerialToObject.Unbind();
	}

	Impl = MakePimpl<UE::Net::Private::FReplicationSystemImpl>(this, LocalParams);
	Impl->Init(LocalParams);

	PostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UReplicationSystem::PostGarbageCollection);
}

void UReplicationSystem::Shutdown()
{
	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGarbageCollectHandle);

	// Destroy impl
	Impl->Deinit();
	Impl.Reset();

	// Destroy bridge
	ReplicationBridge->MarkAsGarbage();
	ReplicationBridge = nullptr;
}

UReplicationSystem::~UReplicationSystem()
{
}

UE::Net::Private::FReplicationSystemInternal* UReplicationSystem::GetReplicationSystemInternal()
{
	return &Impl->ReplicationSystemInternal;
}

const UE::Net::Private::FReplicationSystemInternal* UReplicationSystem::GetReplicationSystemInternal() const
{
	return &Impl->ReplicationSystemInternal;
}

void UReplicationSystem::StartParallelPhase()
{ 
	GetReplicationSystemInternal()->StartParallelPhase();
}

void UReplicationSystem::StopParallelPhase()
{
	GetReplicationSystemInternal()->StopParallelPhase(); 
}

bool UReplicationSystem::IsInParallelPhase() const
{
	return GetReplicationSystemInternal()->IsInParallelPhase();
}

UE::Net::Private::FNetStatsContext* UReplicationSystem::AcquireStatsForSerializationContext()
{
	const bool bIsParallel = IsInParallelPhase();
	check(GetReplicationSystemInternal());
	UE::Net::Private::FNetTypeStats& NetTypeStats = GetReplicationSystemInternal()->GetNetTypeStats();
	UE::Net::Private::FNetStatsContext* NetStatsContext = bIsParallel ? NetTypeStats.AcquireChildNetStatsContext() : NetTypeStats.GetNetStatsContext();
	return NetStatsContext;
}

void UReplicationSystem::ReleaseStatsForSerializationContext(UE::Net::Private::FNetStatsContext* NetStatsContext)
{
	const bool bIsParallel = IsInParallelPhase();
	check(GetReplicationSystemInternal());
	UE::Net::Private::FNetTypeStats& NetTypeStats = GetReplicationSystemInternal()->GetNetTypeStats();
	if (bIsParallel && ensure(NetStatsContext))
	{
		NetTypeStats.ReleaseChildNetStatsContext(NetStatsContext);
	}
}

void UReplicationSystem::NetUpdate(float DeltaSeconds)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_NetUpdate);

	FReplicationSystemInternal& InternalSys = Impl->ReplicationSystemInternal;

	ElapsedTime += DeltaSeconds;

	ensureMsgf(Impl->CurrentSendPass == EReplicationSystemSendPass::Invalid, TEXT("PostSendUpdate was not called after the last Tick."));
	Impl->CurrentSendPass = EReplicationSystemSendPass::TickFlush;

	// $IRIS TODO. There may be some throttling of connections to tick that we should take into account.
	const FNetBitArrayView& ReplicatingConnections = MakeNetBitArrayView(Impl->ReplicationSystemInternal.GetConnections().GetValidConnections());

	#if UE_NET_IRIS_CSV_STATS
	{
		FNetSendStats& SendStats = InternalSys.GetSendStats();
		SendStats.Reset();
		SendStats.SetNumberOfReplicatingConnections(ReplicatingConnections.CountSetBits());

		FNetTypeStats& NetTypeStats = InternalSys.GetNetTypeStats();
		NetTypeStats.PreUpdateSetup();
	}
	#endif
	InternalSys.GetAccumulatedReplicationStats().Accumulate(InternalSys.GetTickReplicationStats());
	InternalSys.GetTickReplicationStats() = {};
	
	// Force a integrity check of all replicated instances
	if (bDoCollectGarbage || ReplicationSystemCVars::bForcePruneBeforeUpdate)
	{
		CollectGarbage();
	}

	// DataStream presend update, this is similar to channel tick()
	UDataStream::FUpdateParameters DataStreamUpdateParams = { .UpdateType = UDataStream::EUpdateType::PreSendUpdate };
	Impl->UpdateDataStreams(DataStreamUpdateParams);

	// If we no longer have any valid connections we can skip part of the update.
	const bool bUpdateConnectionSpecifics = Impl->ShouldUpdateForConnections();

	if (bAllowObjectReplication)
	{
		UE_NET_TRACE_FRAME_STATSCOUNTER(GetId(), ReplicationSystem.ReplicatedObjectCount, InternalSys.GetNetRefHandleManager().GetActiveObjectCount(), ENetTraceVerbosity::Verbose);

		// Tell systems we are starting PreSendUpdate
		Impl->StartPreSendUpdate();

		// Refresh the dirty objects we were told about.
		Impl->UpdateDirtyObjectList();

		// Update world locations. We need this to happen before both filtering and prioritization.
		Impl->UpdateWorldLocations();

		// Update filters, reduce the top-level scoped object list and set each connection's scope.
		Impl->UpdateFiltering();

		if (bUpdateConnectionSpecifics)
		{

			// Invoke any operations we need to do before copying state data
			Impl->CallPreSendUpdate(DeltaSeconds);

			// Finalize the dirty list with objects set dirty during the poll phase
			Impl->UpdateDirtyListPostPoll();

			// Update conditionals
			Impl->UpdateConditionals();

			// Quantize dirty state data. We need this to happen before both filtering and prioritization
			Impl->ExecuteQuantizeDirtyStateData(bReplicateTransactionally);

			// We must process all attachments to objects going out of scope before we update the scope
			Impl->ProcessNetObjectAttachmentSendQueue(FNetBlobManager::EProcessMode::ProcessObjectsGoingOutOfScope);

			// Update scope for all connections
			Impl->UpdateObjectScopes();

			// Propagate dirty changes to all connections
			Impl->PropagateDirtyChanges();
		}
	}

	// Forward attachments to the connections after scope update
	Impl->ProcessNetObjectAttachmentSendQueue(FNetBlobManager::EProcessMode::ProcessObjectsInScope);
	Impl->ResetNetObjectAttachmentSendQueue();

	if (bAllowObjectReplication && bUpdateConnectionSpecifics)
	{
		// Update object priorities
		Impl->UpdatePrioritization(ReplicatingConnections);

		// Delta compression preparations before send
		{
			FDeltaCompressionBaselineManagerPreSendUpdateParams UpdateParams;
			UpdateParams.ChangeMaskCache = &InternalSys.GetChangeMaskCache();
			InternalSys.GetDeltaCompressionBaselineManager().PreSendUpdate(UpdateParams);
		}
	}

	// Destroy objects pending destroy
	{
		Impl->UpdateUnresolvableReferenceTracking();
		InternalSys.GetNetRefHandleManager().DestroyObjectsPendingDestroy();
	}
}

void UReplicationSystem::TickPostReceive()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_TickPostReceive);

	ensureMsgf(Impl->CurrentSendPass == EReplicationSystemSendPass::Invalid, TEXT("PostSendUpdate was not called after the last Tick."));
	Impl->CurrentSendPass = EReplicationSystemSendPass::PostTickDispatch;

	// Forward attachments scheduled to use the OOBChannel and mark connections needing immediate send
	Impl->ProcessOOBNetObjectAttachmentSendQueue();
}

IRISCORE_API void UReplicationSystem::SendUpdate(TFunctionRef<void(TArrayView<uint32>)> SendFunction)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (!ensure(Impl->CurrentSendPass != EReplicationSystemSendPass::Invalid))
	{
		return;
	}

	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	const FNetBitArray& ReplicatingConnections = Connections.GetValidConnections();

	TArray<uint32, TInlineAllocator<128>> ConnectionToUpdate;
	
	if (Impl->CurrentSendPass == EReplicationSystemSendPass::TickFlush)
	{
		// This is currently handled when ticking NetDriver->NetConnection->Channels.

		ConnectionToUpdate.SetNum(ReplicatingConnections.CountSetBits());
		Connections.GetValidConnections().GetSetBitIndices(0U, ~0U, ConnectionToUpdate.GetData(), ConnectionToUpdate.Num());
	}
	else if (Impl->CurrentSendPass == EReplicationSystemSendPass::PostTickDispatch)
	{
		// We only need to send data to connections that has data to send in PostTickDispatch

		FNetBitArray::ForAllSetBits(Impl->ConnectionsPendingPostTickDispatchSend, ReplicatingConnections, FNetBitArray::AndOp, [&ConnectionToUpdate](uint32 ConnId)
		{ 
			ConnectionToUpdate.Add(ConnId);
		});
		Impl->ConnectionsPendingPostTickDispatchSend.ClearAllBits();
	}

	SendFunction(MakeArrayView(ConnectionToUpdate));
}

void UReplicationSystem::PostSendUpdate()
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_PostSendUpdate);

	if (!ensure(Impl->CurrentSendPass != EReplicationSystemSendPass::Invalid))
	{
		return;
	}

	// Most systems are only updated during the normal NetUpdate
	if (Impl->CurrentSendPass == EReplicationSystemSendPass::TickFlush)
	{
		if (bAllowObjectReplication)
		{
			Impl->ResetObjectStateDirtiness();
		}

		Impl->EndPostSendUpdate();

		if (bAllowObjectReplication)
		{
			FDeltaCompressionBaselineManagerPostSendUpdateParams UpdateParams;
			Impl->ReplicationSystemInternal.GetDeltaCompressionBaselineManager().PostSendUpdate(UpdateParams);
		}

		// Allow data streams to do end of tick work.
		UDataStream::FUpdateParameters DataStreamUpdateParams = { .UpdateType = UDataStream::EUpdateType::PostTickFlush };
		Impl->UpdateDataStreams(DataStreamUpdateParams);

#if UE_NET_IRIS_CSV_STATS
		{
			FNetSendStats& SendStats = Impl->ReplicationSystemInternal.GetSendStats();
			SendStats.ReportCsvStats();

			FNetTypeStats& TypeStats = Impl->ReplicationSystemInternal.GetNetTypeStats();
			TypeStats.ReportCSVStats();

			FReplicationStats& ReplicationStats = Impl->ReplicationSystemInternal.GetTickReplicationStats();
			ReplicationStats.ReportCSVStats();

			if (Impl->ReplicationSystemInternal.IsDirtyNetObjectTrackerInitialized())
			{
				Impl->ReplicationSystemInternal.GetDirtyNetObjectTracker().ReportCSVStats();
			}
		}
#endif

	}

	Impl->CurrentSendPass = EReplicationSystemSendPass::Invalid;
}

void UReplicationSystem::PreReceiveUpdate()
{
	Impl->PreReceiveUpdate();
}

void UReplicationSystem::PostReceiveUpdate()
{
	Impl->PostReceiveUpdate();
}

void UReplicationSystem::PostGarbageCollection()
{
	bDoCollectGarbage = 1U;
}

void UReplicationSystem::CollectGarbage()
{
	IRIS_CSV_PROFILER_SCOPE(Iris, ReplicationSystem_CollectGarbage);

	// Prune stale object instances before descriptors and protocols are pruned
	Impl->ReplicationSystemInternal.GetReplicationBridge()->CallPruneStaleObjects();
	Impl->ReplicationSystemInternal.GetReplicationStateDescriptorRegistry().PruneStaleDescriptors();

	bDoCollectGarbage = 0;
}

void UReplicationSystem::NotifyStreamingLevelUnload(const UObject* Level)
{
	Impl->ReplicationSystemInternal.GetReplicationBridge()->NotifyStreamingContainerUnload(Level);
}

void UReplicationSystem::PreSeamlessTravelGarbageCollect()
{
	UE_LOGF(LogIris, Verbose, "UReplicationSystem::PreSeamlessTravelGarbageCollect IsServer: %d", IsServer());
	Impl->ReplicationSystemInternal.PreSeamlessTravelGarbageCollect();
}

void UReplicationSystem::PostSeamlessTravelGarbageCollect()
{
	UE_LOGF(LogIris, Verbose, "UReplicationSystem::PostSeamlessTravelGarbageCollect IsServer: %d", IsServer());
	CollectGarbage();
	Impl->ReplicationSystemInternal.PostSeamlessTravelGarbageCollect();
}

void UReplicationSystem::InitializeNetRefHandleGenerator(UReplicationSystem::FReplicationSystemParams& Params)
{	
	using namespace UE::Net;

	if (Params.ProxyType != UE::Net::EProxyType::None)
	{
		return;
	}

	const UClass* GeneratorClass = UObjectReplicationBridgeConfig::GetConfig()->GetNetRefHandleGeneratorClass();
	if (!GeneratorClass)
	{
		return;
	}

	if (!GeneratorClass->IsChildOf<UNetRefHandleGenerator>())
	{
		return;
	}

	NetRefHandleGenerator = NewObject<UNetRefHandleGenerator>(this, GeneratorClass);
	if (!NetRefHandleGenerator)
	{
		UE_LOGF(LogIris, Error, "UReplicationSystem::InitializeNetRefHandleGenerator: Unable to instantiate class %ls", *GeneratorClass->GetName());
		return;
	}

	UE_LOGF(LogIris, Log, "UReplicationSystem::InitializeNetRefHandleGenerator: Using generator %ls", *GeneratorClass->GetName());

	UE::Net::FNetRefHandleGeneratorBindParams BindParams;
	BindParams.ObjectToNetRefHandleSerial = &Params.ObjectToNetRefHandleSerial;
	BindParams.NetRefHandleSerialToObject = &Params.NetRefHandleSerialToObject;

	UE_CLOGF(BindParams.ObjectToNetRefHandleSerial->IsBound(), LogIris, Warning, "UReplicationSystem::InitializeNetRefHandleGenerator: ObjectToNetRefHandleSerial is already set and will be overwritten.");
	UE_CLOGF(BindParams.NetRefHandleSerialToObject->IsBound(), LogIris, Warning, "UReplicationSystem::InitializeNetRefHandleGenerator: NetRefHandleSerialToObject is already set and will be overwritten.");

	NetRefHandleGenerator->Bind(BindParams);
}

void UReplicationSystem::AddConnection(uint32 ConnectionId, UE::Net::ENetConnectionTraits Traits)
{
	Impl->AddConnection(ConnectionId, Traits);
}

void UReplicationSystem::RemoveConnection(uint32 ConnectionId)
{
	Impl->RemoveConnection(ConnectionId);
}

bool UReplicationSystem::HasConnectionWithAnyTraits(UE::Net::ENetConnectionTraits NetConnectionTraits) const
{
	return Impl->HasConnectionWithAnyTraits(NetConnectionTraits);
}

bool UReplicationSystem::IsValidConnection(uint32 ConnectionId) const
{
	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	return Connections.GetConnection(ConnectionId) != nullptr;
}


void UReplicationSystem::SetConnectionGracefullyClosing(uint32 ConnectionId) const
{
	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	check(Connections.IsValidConnection(ConnectionId));

	Connections.SetConnectionIsClosing(ConnectionId);
}

void UReplicationSystem::SetReplicationEnabledForConnection(uint32 ConnectionId, bool bReplicationEnabled)
{
	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	UE::Net::Private::FReplicationConnection* Connection = Connections.GetConnection(ConnectionId);

	check(Connection);

	Connection->ReplicationWriter->SetReplicationEnabled(bReplicationEnabled);
}

void UReplicationSystem::SetReplicationView(uint32 ConnectionId, const UE::Net::FReplicationView& View)
{
	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	Connections.SetReplicationView(ConnectionId, View);
}

void UReplicationSystem::SetStaticPriority(FNetRefHandle Handle, float Priority)
{
	const UE::Net::FInternalNetRefIndex ObjectInternalIndex = Impl->ReplicationSystemInternal.GetNetRefHandleManager().GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return;
	}

	return Impl->ReplicationSystemInternal.GetPrioritization().SetStaticPriority(ObjectInternalIndex, Priority);
}

bool UReplicationSystem::SetPrioritizer(FNetRefHandle Handle, UE::Net::FNetObjectPrioritizerHandle Prioritizer)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (!Handle.IsValid())
	{
		return false;
	}

	FReplicationSystemInternal& ReplicationSystemInternal = Impl->ReplicationSystemInternal;
	const FInternalNetRefIndex ObjectInternalIndex = ReplicationSystemInternal.GetNetRefHandleManager().GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return false;
	}

	return ReplicationSystemInternal.GetPrioritization().SetPrioritizer(ObjectInternalIndex, Prioritizer);
}

UE::Net::FNetObjectPrioritizerHandle UReplicationSystem::GetPrioritizerHandle(const FName PrioritizerName) const
{
	return Impl->ReplicationSystemInternal.GetPrioritization().GetPrioritizerHandle(PrioritizerName);
}

UNetObjectPrioritizer* UReplicationSystem::GetPrioritizer(const FName PrioritizerName) const
{
	return Impl->ReplicationSystemInternal.GetPrioritization().GetPrioritizer(PrioritizerName);
}

const UE::Net::FNetTokenStore* UReplicationSystem::GetNetTokenStore() const
{
	return Impl->NetTokenStore;
}

UE::Net::FNetTokenStore* UReplicationSystem::GetNetTokenStore()
{
	return Impl->NetTokenStore;
}

UE::Net::FNetTokenResolveContext UReplicationSystem::GetNetTokenResolveContext(uint32 ConnectionId) const
{
	using namespace UE::Net;

	UE::Net::FNetTokenResolveContext NetTokenResolveContext;

	NetTokenResolveContext.NetTokenStore = Impl->NetTokenStore;
	NetTokenResolveContext.RemoteNetTokenStoreState = Impl->NetTokenStore->GetRemoteNetTokenStoreState(ConnectionId);

	return NetTokenResolveContext;
}

bool UReplicationSystem::RegisterNetBlobHandler(UNetBlobHandler* Handler)
{
	UE::Net::Private::FNetBlobManager& NetBlobManager = Impl->ReplicationSystemInternal.GetNetBlobManager();
	return NetBlobManager.RegisterNetBlobHandler(Handler);
}

/** Returns true if there exists a DataStreamDefinition for the provided Name */
bool UReplicationSystem::IsKnownDataStreamDefinition(FName Name) const
{
	return UDataStreamManager::IsKnownStreamDefinition(Name);
}

UDataStream* UReplicationSystem::OpenDataStream(uint32 ConnectionId, FName Name)
{
	if (!IsServer())
	{
		UE_LOGF(LogIris, Error, "DataStream %ls can only be opened from server", *Name.ToString());
		return nullptr;
	}

	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	if (!ensureMsgf(Connections.IsValidConnection(ConnectionId), TEXT("Invalid ConnectionId %u passed to UReplicationSystem::GetDataStream."), ConnectionId))
	{
		return nullptr;
	}

	UE::Net::Private::FReplicationConnection* Connection = Connections.GetConnection(ConnectionId);
	UDataStreamManager* DataStreamManager = Connection ? Connection->DataStreamManager.Get() : nullptr;
	if (ensureMsgf(DataStreamManager, TEXT("UReplicationSystem::OpenDataStream Trying to open datastream for not yet initialized connection %u"), ConnectionId))
	{
		ECreateDataStreamResult Result = DataStreamManager->CreateStream(Name);
		if (Result == ECreateDataStreamResult::Success)
		{
			return DataStreamManager->GetStream(Name);
		}
	}

	return nullptr;
}

UDataStream* UReplicationSystem::GetDataStream(uint32 ConnectionId, FName Name)
{
	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	if (!ensureMsgf(Connections.IsValidConnection(ConnectionId), TEXT("Invalid ConnectionId %u passed to UReplicationSystem::GetDataStream."), ConnectionId))
	{
		return nullptr;
	}

	UE::Net::Private::FReplicationConnection* Connection = Connections.GetConnection(ConnectionId);
	UDataStreamManager* DataStreamManager = Connection ? Connection->DataStreamManager.Get() : nullptr;
	if (ensureMsgf(DataStreamManager, TEXT("UReplicationSystem::GetDataStream Trying to get datastream for not yet initialized connection %u"), ConnectionId))
	{
		return DataStreamManager->GetStream(Name);
	}

	return nullptr;
}

const UDataStream* UReplicationSystem::GetDataStream(uint32 ConnectionId, FName Name) const
{
	const UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	if (!ensureMsgf(Connections.IsValidConnection(ConnectionId), TEXT("Invalid ConnectionId %u passed to UReplicationSystem::GetDataStream."), ConnectionId))
	{
		return nullptr;
	}

	const UE::Net::Private::FReplicationConnection* Connection = Connections.GetConnection(ConnectionId);
	const UDataStreamManager* DataStreamManager = Connection ? Connection->DataStreamManager.Get() : nullptr;
	if (ensureMsgf(DataStreamManager, TEXT("UReplicationSystem::GetDataStream Trying to get datastream for not yet initialized connection %u"), ConnectionId))
	{
		return DataStreamManager->GetStream(Name);
	}

	return nullptr;
}

bool UReplicationSystem::IsRelevantForConnection(FNetRefHandle Handle, uint32 ConnectionId) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	if (!ensureMsgf(Connections.IsValidConnection(ConnectionId), TEXT("Invalid ConnectionId %u passed to UReplicationSystem::IsRelevantForConnection."), ConnectionId))
	{
		return false;
	}

	const FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return false;
	}

	const FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();
	const FNetBitArrayView ObjectsInScope = Filtering.GetRelevantObjectsInScope(ConnectionId);

	const FReplicationConnection* Connection = Connections.GetConnection(ConnectionId);

	return ObjectsInScope.GetNumBits() > ObjectInternalIndex ? (ObjectsInScope.IsBitSet(ObjectInternalIndex) && Connection->ReplicationWriter->IsInReplicatingState(ObjectInternalIndex)) : false;
}

void UReplicationSystem::CloseDataStream(uint32 ConnectionId, FName Name)
{
	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	if (!ensureMsgf(Connections.IsValidConnection(ConnectionId), TEXT("Invalid ConnectionId %u passed to UReplicationSystem::CloseDataStream."), ConnectionId))
	{
		return;
	}

	UE::Net::Private::FReplicationConnection* Connection = Connections.GetConnection(ConnectionId);
	UDataStreamManager* DataStreamManager = Connection ? Connection->DataStreamManager.Get() : nullptr;
	if (ensureMsgf(DataStreamManager, TEXT("UReplicationSystem::CloseDataStream Trying to get datastream for not yet initialized connection %u"), ConnectionId))
	{
		DataStreamManager->CloseStream(Name);
		return;
	}
}

bool UReplicationSystem::QueueNetObjectAttachment(uint32 ConnectionId, const UE::Net::FNetObjectReference& TargetRef, const TRefCountPtr<UE::Net::FNetObjectAttachment>& Attachment)
{
	UE::Net::Private::FNetBlobManager& NetBlobManager = Impl->ReplicationSystemInternal.GetNetBlobManager();
	return NetBlobManager.QueueNetObjectAttachment(ConnectionId, TargetRef, Attachment);
}

bool UReplicationSystem::SendRPC(const UObject* RootObject, const UObject* SubObject, const UFunction* Function, const void* Parameters)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	TRACE_CPUPROFILER_EVENT_SCOPE(SendRPC_SendMulticastRPC);

	ENetObjectAttachmentSendPolicyFlags SendFlags = ENetObjectAttachmentSendPolicyFlags::None;
	if (ReplicationSystemCVars::bAllowAttachmentSendPolicyFlags)
	{
		if (ENetObjectAttachmentSendPolicyFlags* Flags = Impl->AttachmentSendPolicyFlags.Find(FObjectKey(Function)))
		{
			SendFlags = *Flags;
		}
	}

	FNetBlobManager::FSendRPCContext RPCContext = { .RootObject = RootObject, .SubObject = SubObject, .Function = Function };

	FNetBlobManager& NetBlobManager = Impl->ReplicationSystemInternal.GetNetBlobManager();
	return NetBlobManager.SendMulticastRPC(RPCContext, Parameters, SendFlags);
}

bool UReplicationSystem::SendRPC(uint32 ConnectionId, const UObject* RootObject, const UObject* SubObject, const UFunction* Function, const void* Parameters)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;
	TRACE_CPUPROFILER_EVENT_SCOPE(SendRPC_SendUnicastRPC);

	ENetObjectAttachmentSendPolicyFlags SendFlags = ENetObjectAttachmentSendPolicyFlags::None;
	if (ReplicationSystemCVars::bAllowAttachmentSendPolicyFlags)
	{
		if (ENetObjectAttachmentSendPolicyFlags* Flags = Impl->AttachmentSendPolicyFlags.Find(FObjectKey(Function)))
		{
			SendFlags = *Flags;
		}
	}
	
	FNetBlobManager::FSendRPCContext RPCContext = { .RootObject = RootObject, .SubObject = SubObject, .Function = Function };
	FNetBlobManager& NetBlobManager = Impl->ReplicationSystemInternal.GetNetBlobManager();
	return NetBlobManager.SendUnicastRPC(ConnectionId, RPCContext, Parameters, SendFlags);
}

bool UReplicationSystem::SetRPCSendPolicyFlags(const UFunction* Function, UE::Net::ENetObjectAttachmentSendPolicyFlags SendFlags)
{
	if (!Function)
	{	
		return false;
	}

	UE_LOGF(LogIris, Verbose, "SetRPCSendPolicyFlags %ls::%ls => %ls ", *GetNameSafe(Function->GetOuterUClass()), *GetNameSafe(Function), LexToString(SendFlags));

	Impl->AttachmentSendPolicyFlags.Add(FObjectKey(Function), SendFlags);
	return true;
}

void UReplicationSystem::ResetRPCSendPolicyFlags()
{
	Impl->AttachmentSendPolicyFlags.Reset();
}

void UReplicationSystem::InitDataStreamManager(uint32 ConnectionId, UDataStreamManager* DataStreamManager)
{
	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	Connections.InitDataStreamManager(GetId(), ConnectionId, DataStreamManager);
}

void UReplicationSystem::SetConnectionUserData(uint32 ConnectionId, UObject* InUserData)
{
	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	UE::Net::Private::FReplicationConnection* Connection = Connections.GetConnection(ConnectionId);
	
	check(Connection);

	Connection->UserData = InUserData;
}

UObject* UReplicationSystem::GetConnectionUserData(uint32 ConnectionId) const
{
	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	if (!ensureMsgf(Connections.IsValidConnection(ConnectionId), TEXT("Invalid ConnectionId %u passed to UReplicationSystem::GetConnectionUserData."), ConnectionId))
	{
		return nullptr;
	}

	if (UE::Net::Private::FReplicationConnection* Connection = Connections.GetConnection(ConnectionId))
	{
		return Connection->UserData.Get();
	}
	return nullptr;
}

void UReplicationSystem::SetConnectionRPCDoS(uint32 ConnectionId, FRPCDoSDetection* InRPCDoS)
{
	UE::Net::Private::FReplicationConnections& Connections = Impl->ReplicationSystemInternal.GetConnections();
	if (!ensureMsgf(Connections.IsValidConnection(ConnectionId), TEXT("Invalid ConnectionId %u passed to UReplicationSystem::SetConnectionRPCDoS."), ConnectionId))
	{
		return;
	}

	if (UE::Net::Private::FReplicationConnection* Connection = Connections.GetConnection(ConnectionId))
	{
		if (UE::Net::Private::FReplicationReader* Reader = Connection->ReplicationReader)
		{
			Reader->SetRPCDoS(InRPCDoS);
		}
	}
}

UObjectReplicationBridge* UReplicationSystem::GetReplicationBridge() const
{
	return ReplicationBridge;
}

bool UReplicationSystem::IsNetRefHandleAssigned(FNetRefHandle Handle) const
{
	return Handle.IsValid() && Impl->ReplicationSystemInternal.GetNetRefHandleManager().IsNetRefHandleAssigned(Handle);
}

const UE::Net::FReplicationProtocol* UReplicationSystem::GetReplicationProtocol(FNetRefHandle Handle) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();

	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return nullptr;
	}

	return NetRefHandleManager.GetReplicatedObjectDataNoCheck(ObjectInternalIndex).Protocol;
}

const UE::Net::FNetDebugName* UReplicationSystem::GetDebugName(FNetRefHandle Handle) const
{
	using namespace UE::Net;
	
	const FReplicationProtocol* Protocol = GetReplicationProtocol(Handle);
	return Protocol ? Protocol->DebugName : nullptr;
}

TWeakObjectPtr<UNetObjectFilter> UReplicationSystem::CreateFilter(const FNetObjectFilterDefinition& Definition)
{
	return Impl->CreateFilter(Definition);
}

void UReplicationSystem::DestroyFilter(const FName Name)
{
	Impl->DestroyFilter(Name);
}

void UReplicationSystem::SetOwningNetConnection(FNetRefHandle Handle, uint32 ConnectionId)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation. Filter condition on %s (%s) failed."), *GetNameSafe(NetRefHandleManager.GetReplicatedObjectInstance(ObjectInternalIndex)), *Handle.ToString());
		return;
	}

	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return;
	}

	FReplicationConditionals& Conditionals = Impl->ReplicationSystemInternal.GetConditionals();
	Conditionals.SetOwningConnection(ObjectInternalIndex, ConnectionId);

	FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();
	Filtering.SetOwningConnection(ObjectInternalIndex, ConnectionId);
}

uint32 UReplicationSystem::GetOwningNetConnection(FNetRefHandle Handle) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return UE::Net::InvalidConnectionId;
	}

	const FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();
	return Filtering.GetOwningConnection(ObjectInternalIndex);
}

bool UReplicationSystem::SetFilter(FNetRefHandle Handle, UE::Net::FNetObjectFilterHandle Filter, FName FilterConfigProfile /*= NAME_None*/)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return false;
	}

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation. Filter condition on %s (%s) failed."), *GetNameSafe(NetRefHandleManager.GetReplicatedObjectInstance(ObjectInternalIndex)), *Handle.ToString());
		return false;
	}

	FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();
	const bool SetFilterResult = Filtering.SetFilter(ObjectInternalIndex, Filter, FilterConfigProfile);

	if (ReplicationSystemCVars::bSetFilterSetsRequiresFrequentLocationUpdates)
	{
		// If the new filter is spatial, the object might now require world location updates.
		UObjectReplicationBridge* Bridge = Impl->ReplicationSystemInternal.GetReplicationBridge();
		const bool bWantsToBeDormant = Bridge->GetObjectWantsToBeDormant(Handle);
		Bridge->OptionallySetObjectRequiresFrequentWorldLocationUpdate(Handle, !bWantsToBeDormant);
	}

	return SetFilterResult;
}

UE::Net::FNetObjectFilterHandle UReplicationSystem::GetFilterHandle(const FName FilterName) const
{
	return Impl->ReplicationSystemInternal.GetFiltering().GetFilterHandle(FilterName);
}

UNetObjectFilter* UReplicationSystem::GetFilter(const FName FilterName) const
{
	return Impl->ReplicationSystemInternal.GetFiltering().GetFilter(FilterName);
}

FName UReplicationSystem::GetFilterName(UE::Net::FNetObjectFilterHandle Filter) const
{
	return Impl->ReplicationSystemInternal.GetFiltering().GetFilterName(Filter);
}

UE::Net::FNetObjectGroupHandle UReplicationSystem::GetOrCreateSubObjectFilter(FName GroupName)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetObjectGroups& Groups = Impl->ReplicationSystemInternal.GetGroups();
	FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();

	FNetObjectGroupHandle GroupHandle = Groups.FindGroupHandle(GroupName);
	if (GroupHandle.IsValid())
	{
		check(Filtering.IsSubObjectFilterGroup(GroupHandle));
		return GroupHandle;
	}

	GroupHandle = Groups.CreateGroup(GroupName);
	if (GroupHandle.IsValid())
	{
		Filtering.AddSubObjectFilter(GroupHandle);
	}
	return GroupHandle;
}

UE::Net::FNetObjectGroupHandle UReplicationSystem::GetSubObjectFilterGroupHandle(FName GroupName) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetObjectGroups& Groups = Impl->ReplicationSystemInternal.GetGroups();
	FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();

	FNetObjectGroupHandle GroupHandle = Groups.FindGroupHandle(GroupName);
	if (GroupHandle.IsValid())
	{
		if (ensureMsgf(Filtering.IsSubObjectFilterGroup(GroupHandle), TEXT("UReplicationSystem::GetSubObjectFilterGroupHandle Trying to lookup NetObjectGroupHandle for NetGroup %s that is not a subobject filter"), *GroupName.ToString()))
		{
			return GroupHandle;
		}
	}
	return FNetObjectGroupHandle();
}

void UReplicationSystem::SetSubObjectFilterStatus(FName GroupName, UE::Net::FConnectionHandle ConnectionHandle, UE::Net::ENetFilterStatus ReplicationStatus)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (UE::Net::IsSpecialNetConditionGroup(GroupName))
	{
		ensureMsgf(false, TEXT("UReplicationSystem::SetSubObjectFilterStatus Cannot SetSubObjectFilterStatus for special NetGroup %s"), *GroupName.ToString());
		return;
	}

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation."));
		return;
	}

	FNetObjectGroupHandle GroupHandle = GetSubObjectFilterGroupHandle(GroupName);
	if (GroupHandle.IsValid())
	{
		FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();
		Filtering.SetSubObjectFilterStatus(GroupHandle, ConnectionHandle, ReplicationStatus);

		FReplicationConditionals& Conditionals = Impl->ReplicationSystemInternal.GetConditionals();
		Conditionals.MarkLifeTimeConditionalsDirtyForObjectsInGroup(GroupHandle);
	}
}

void UReplicationSystem::RemoveSubObjectFilter(FName GroupName)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation."));
		return;
	}

	FNetObjectGroups& Groups = Impl->ReplicationSystemInternal.GetGroups();
	FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();

	FNetObjectGroupHandle GroupHandle = GetSubObjectFilterGroupHandle(GroupName);
	if (GroupHandle.IsValid())
	{
		Filtering.RemoveSubObjectFilter(GroupHandle);
		Groups.DestroyGroup(GroupHandle);
	}
}

UE::Net::FNetObjectGroupHandle UReplicationSystem::CreateGroup(FName GroupName)
{
	LLM_SCOPE_BYTAG(Iris);

	return Impl->ReplicationSystemInternal.GetGroups().CreateGroup(GroupName);
}

void UReplicationSystem::AddToGroup(FNetObjectGroupHandle GroupHandle, FNetRefHandle Handle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// Early out if this is invalid group
	if (!ensure(IsValidGroup(GroupHandle)))
	{
		return;
	}

	LLM_SCOPE_BYTAG(Iris);

	FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation. Filter condition on %s (%s) failed."), *GetNameSafe(NetRefHandleManager.GetReplicatedObjectInstance(ObjectInternalIndex)), *Handle.ToString());
		return;
	}

	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return;
	}
	
	FNetObjectGroups& Groups = Impl->ReplicationSystemInternal.GetGroups();
	FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();	

	Groups.AddToGroup(GroupHandle, ObjectInternalIndex);
	Filtering.NotifyObjectAddedToGroup(GroupHandle, ObjectInternalIndex);
}

void UReplicationSystem::RemoveFromGroup(FNetObjectGroupHandle GroupHandle, FNetRefHandle Handle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// Early out if this is invalid group
	if (!ensure(IsValidGroup(GroupHandle)))
	{
		return;
	}

	FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);
	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation. Filter condition on %s (%s) failed."), *GetNameSafe(NetRefHandleManager.GetReplicatedObjectInstance(ObjectInternalIndex)), *Handle.ToString());
		return;
	}

	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return;
	}

	FNetObjectGroups& Groups = Impl->ReplicationSystemInternal.GetGroups();
	FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();

	Groups.RemoveFromGroup(GroupHandle, ObjectInternalIndex);
	Filtering.NotifyObjectRemovedFromGroup(GroupHandle, ObjectInternalIndex);
}

void UReplicationSystem::RemoveFromAllGroups(FNetRefHandle Handle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);

	Impl->ReplicationSystemInternal.RemoveFromAllGroups(ObjectInternalIndex);
}

bool UReplicationSystem::IsInGroup(FNetObjectGroupHandle GroupHandle, FNetRefHandle Handle) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	// Early out if this is invalid group
	if (!IsValidGroup(GroupHandle))
	{
		return false;
	}

	const FNetObjectGroups& Groups = Impl->ReplicationSystemInternal.GetGroups();
	const FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();

	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);

	return Groups.Contains(GroupHandle, ObjectInternalIndex);
}

bool UReplicationSystem::IsValidGroup(FNetObjectGroupHandle GroupHandle) const
{
	const UE::Net::Private::FNetObjectGroups& Groups = Impl->ReplicationSystemInternal.GetGroups();

	return GroupHandle.IsValid() && Groups.IsValidGroup(GroupHandle);
}

void UReplicationSystem::DestroyGroup(FNetObjectGroupHandle GroupHandle)
{
	// Early out if this is invalid or reserved group
	if (!ensure(IsValidGroup(GroupHandle) || GroupHandle.IsReservedNetObjectGroup()))
	{
		return;
	}

	using namespace UE::Net;
	using namespace UE::Net::Private;

	FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();	
	FNetObjectGroups& Groups = Impl->ReplicationSystemInternal.GetGroups();

	Filtering.RemoveGroupFilter(GroupHandle);
	Filtering.RemoveSubObjectFilter(GroupHandle);

	Groups.DestroyGroup(GroupHandle);
}

UE::Net::FNetObjectGroupHandle UReplicationSystem::FindGroup(FName GroupName) const
{
	const UE::Net::Private::FNetObjectGroups& Groups = Impl->ReplicationSystemInternal.GetGroups();

	return Groups.FindGroupHandle(GroupName);
}

UE::Net::FNetObjectGroupHandle UReplicationSystem::GetNotReplicatedNetObjectGroup() const
{
	return Impl->NotReplicatedNetObjectGroupHandle;
}

UE::Net::FNetObjectGroupHandle UReplicationSystem::GetAsyncStopReplicationNetObjectGroup() const
{
	return Impl->AsyncStopReplicationNetObjectGroupHandle;
}

UE::Net::FNetObjectGroupHandle UReplicationSystem::GetNetGroupOwnerNetObjectGroup() const
{
	return Impl->NetGroupOwnerNetObjectGroupHandle;
}

UE::Net::FNetObjectGroupHandle UReplicationSystem::GetNetGroupReplayNetObjectGroup() const
{
	return Impl->NetGroupReplayNetObjectGroupHandle;
}

bool UReplicationSystem::AddExclusionFilterGroup(FNetObjectGroupHandle GroupHandle)
{
	// Early out if this is invalid group
	if (!ensure(IsValidGroup(GroupHandle)))
	{
		return false;
	}

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation."));
		return false;
	}

	UE::Net::Private::FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();	
	return Filtering.AddExclusionFilterGroup(GroupHandle);
}

bool UReplicationSystem::AddInclusionFilterGroup(FNetObjectGroupHandle GroupHandle)
{
	// Early out if this is invalid group
	if (!ensure(IsValidGroup(GroupHandle)))
	{
		return false;
	}

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation."));
		return false;
	}

	UE::Net::Private::FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();	
	return Filtering.AddInclusionFilterGroup(GroupHandle);
}

void UReplicationSystem::RemoveGroupFilter(FNetObjectGroupHandle GroupHandle)
{
	// Early out if this is invalid group
	if (!ensure(IsValidGroup(GroupHandle)))
	{
		return;
	}

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation."));
		return;
	}

	UE::Net::Private::FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();	
	Filtering.RemoveGroupFilter(GroupHandle);
}

void UReplicationSystem::SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId, UE::Net::ENetFilterStatus ReplicationStatus)
{
	// Early out if this is invalid group
	if (!ensure(IsValidGroup(GroupHandle)))
	{
		return;
	}

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation."));
		return;
	}

	UE::Net::Private::FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();	
	Filtering.SetGroupFilterStatus(GroupHandle, ConnectionId, ReplicationStatus);
}

void UReplicationSystem::SetProxyConnection(uint32 ConnectionId, bool bEnabled)
{
	Impl->ReplicationSystemInternal.GetConditionals().SetAutonomousAsSimulated(ConnectionId, bEnabled);
}

void UReplicationSystem::SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, const UE::Net::FNetBitArray& Connections, UE::Net::ENetFilterStatus ReplicationStatus)
{
	// Early out if this is invalid group
	if (!ensure(IsValidGroup(GroupHandle)))
	{
		return;
	}

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation."));
		return;
	}

	UE::Net::Private::FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();	
	Filtering.SetGroupFilterStatus(GroupHandle, UE::Net::MakeNetBitArrayView(Connections), ReplicationStatus);
}

void UReplicationSystem::SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, UE::Net::ENetFilterStatus ReplicationStatus)
{
	// Early out if this is invalid group
	if (!ensure(IsValidGroup(GroupHandle)))
	{
		return;
	}

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation."));
		return;
	}

	UE::Net::Private::FReplicationFiltering& Filtering = Impl->ReplicationSystemInternal.GetFiltering();	
	Filtering.SetGroupFilterStatus(GroupHandle, ReplicationStatus);
}

bool UReplicationSystem::SetReplicationConditionConnectionFilter(FNetRefHandle Handle, UE::Net::EReplicationCondition Condition, uint32 ConnectionId, bool bEnable)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return false;
	}

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting filter conditions is not yet supported during this operation. Filter condition on %s (%s) failed."), *GetNameSafe(NetRefHandleManager.GetReplicatedObjectInstance(ObjectInternalIndex)), *Handle.ToString());
		return false;
	}

	FReplicationConditionals& Conditionals = Impl->ReplicationSystemInternal.GetConditionals();
	return Conditionals.SetConditionConnectionFilter(ObjectInternalIndex, Condition, ConnectionId, bEnable);
}

bool UReplicationSystem::SetReplicationCondition(FNetRefHandle Handle, UE::Net::EReplicationCondition Condition, bool bEnable)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return false;
	}

	FReplicationConditionals& Conditionals = Impl->ReplicationSystemInternal.GetConditionals();
	return Conditionals.SetCondition(ObjectInternalIndex, Condition, bEnable);
}

void UReplicationSystem::SetDeltaCompressionStatus(FNetRefHandle Handle, UE::Net::ENetObjectDeltaCompressionStatus Status)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	FReplicationSystemInternal& InternalSys = Impl->ReplicationSystemInternal;
	FNetRefHandleManager& NetRefHandleManager = InternalSys.GetNetRefHandleManager();
	const FInternalNetRefIndex ObjectInternalIndex = NetRefHandleManager.GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return;
	}

	if (Impl->ReplicationSystemInternal.AreFilterChangesBlocked())
	{
		ensureMsgf(false, TEXT("Setting delta compression is not yet supported during this operation. Filter condition on %s (%s) failed."), *GetNameSafe(NetRefHandleManager.GetReplicatedObjectInstance(ObjectInternalIndex)), *Handle.ToString());
		return;
	}

	FDeltaCompressionBaselineManager& DC = InternalSys.GetDeltaCompressionBaselineManager();
	return DC.SetDeltaCompressionStatus(ObjectInternalIndex, Status);
}

void UReplicationSystem::SetIsDebugObject(FNetRefHandle Handle)
{
#if UE_NET_IRIS_ALLOW_DEBUG_DATA
	UE::Net::Private::FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();	
	NetRefHandleManager.SetIsDebugObject(Handle);
#else
	ensureMsgf(false, TEXT("SetIsDebugObject is only supported in non-shipping builds."));
#endif
}

void UReplicationSystem::SetIsNetTemporary(FNetRefHandle Handle)
{
	UE::Net::Private::FNetRefHandleManager& NetRefHandleManager = Impl->ReplicationSystemInternal.GetNetRefHandleManager();
	if (ensure(NetRefHandleManager.IsLocalNetRefHandle(Handle)))
	{
		// Set the object to not propagate changed states
		NetRefHandleManager.SetShouldPropagateChangedStates(Handle, false);
	}
}

void UReplicationSystem::TearOffNextUpdate(FNetRefHandle Handle)
{
	constexpr EEndReplicationFlags DestroyFlags = EEndReplicationFlags::TearOff | EEndReplicationFlags::ClearNetPushId;
	if (const uint32 InternalObjectIndex = Impl->ReplicationSystemInternal.GetNetRefHandleManager().GetInternalIndex(Handle))
	{
		Impl->ReplicationSystemInternal.GetReplicationBridge()->AddPendingEndReplication(Handle, InternalObjectIndex, DestroyFlags);
	}
}

void UReplicationSystem::ForceNetUpdate(FNetRefHandle Handle)
{
	if (const uint32 InternalObjectIndex = Impl->ReplicationSystemInternal.GetNetRefHandleManager().GetInternalIndex(Handle))
	{
		UE::Net::Private::ForceNetUpdate(GetId(), InternalObjectIndex);
	}
}

void UReplicationSystem::MarkDirty(FNetRefHandle Handle)
{
	if (const uint32 InternalObjectIndex = Impl->ReplicationSystemInternal.GetNetRefHandleManager().GetInternalIndex(Handle))
	{
		 UE::Net::Private::MarkNetObjectStateDirty(GetId(), InternalObjectIndex);
	}
}

uint32 UReplicationSystem::GetMaxConnectionCount() const
{
	return Impl->ReplicationSystemInternal.GetConnections().GetMaxConnectionCount();
}

void UReplicationSystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UReplicationSystem* This = CastChecked<UReplicationSystem>(InThis);
	if (This->Impl.IsValid())
	{
		This->Impl->ReplicationSystemInternal.GetNetRefHandleManager().AddReferencedObjects(Collector);
		This->Impl->ReplicationSystemInternal.GetObjectReferenceCache().AddReferencedObjects(Collector, This);
	}
	Super::AddReferencedObjects(InThis, Collector);
}

const UE::Net::FWorldLocations& UReplicationSystem::GetWorldLocations() const
{
	return Impl->ReplicationSystemInternal.GetWorldLocations();
}

void UReplicationSystem::SetCullDistanceOverride(FNetRefHandle Handle, float CullDistance)
{
	const UE::Net::FInternalNetRefIndex ObjectInternalIndex = Impl->ReplicationSystemInternal.GetNetRefHandleManager().GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return;
	}

	UE_LOGF(LogIrisNetCull, Verbose, "UReplicationSystem::SetCullDistanceOverride: %ls will be overridden to %f. Previous cull distance: %f",
		*Impl->ReplicationSystemInternal.GetNetRefHandleManager().PrintObjectFromIndex(ObjectInternalIndex),
		CullDistance,
		Impl->ReplicationSystemInternal.GetWorldLocations().GetCullDistance(ObjectInternalIndex));

	const bool bSuccess = Impl->ReplicationSystemInternal.GetWorldLocations().SetCullDistanceOverride(ObjectInternalIndex, CullDistance);

	ensureMsgf(bSuccess, TEXT("SetCullDistanceOverride failed for %s (cull distance: %f). The object does not use the world location cache."), 
		*Impl->ReplicationSystemInternal.GetNetRefHandleManager().PrintObjectFromIndex(ObjectInternalIndex), CullDistance);
}

void UReplicationSystem::ClearCullDistanceOverride(FNetRefHandle Handle)
{
	const UE::Net::FInternalNetRefIndex ObjectInternalIndex = Impl->ReplicationSystemInternal.GetNetRefHandleManager().GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return;
	}

	const bool bWasCullDistanceOverriden = Impl->ReplicationSystemInternal.GetWorldLocations().ClearCullDistanceOverride(ObjectInternalIndex);
	
	UE_CLOGF(bWasCullDistanceOverriden, LogIrisNetCull, Verbose, "UReplicationSystem::ClearCullDistanceOverride: %ls is no longer overridden. Now using cull distance: %f",
		*Impl->ReplicationSystemInternal.GetNetRefHandleManager().PrintObjectFromNetRefHandle(Handle),
		Impl->ReplicationSystemInternal.GetWorldLocations().GetCullDistance(ObjectInternalIndex));
}

float UReplicationSystem::GetCullDistance(FNetRefHandle Handle, float DefaultValue) const
{
	const UE::Net::FInternalNetRefIndex ObjectInternalIndex = Impl->ReplicationSystemInternal.GetNetRefHandleManager().GetInternalIndex(Handle);
	if (ObjectInternalIndex == UE::Net::InvalidInternalNetRefIndex)
	{
		return DefaultValue;
	}

	if (!Impl->ReplicationSystemInternal.GetWorldLocations().HasInfoForObject(ObjectInternalIndex))
	{
		// This replicated object did not register world information
		return DefaultValue;
	}

	return Impl->ReplicationSystemInternal.GetWorldLocations().GetCullDistance(ObjectInternalIndex);
}

void UReplicationSystem::ReportProtocolMismatch(uint64 NetRefHandleId, uint32 ConnectionId, const TArray<uint8>& ClientCDOStateBytes)
{
	using namespace UE::Net::Private;
	const FNetRefHandle NetRefHandle = FNetRefHandleManager::MakeNetRefHandle(NetRefHandleId, GetId());

	Impl->ReplicationSystemInternal.GetReplicationBridge()->OnProtocolMismatchReported(NetRefHandle, ConnectionId, ClientCDOStateBytes);
}

void UReplicationSystem::ReportErrorWithNetRefHandle(UE::Net::ENetRefHandleError ErrorType, uint64 NetRefHandleId, uint32 ConnectionId, const TArray<uint64>& ExtraNetRefHandles, const FString& DiagnosticMessage)
{
	using namespace UE::Net::Private;
	const FNetRefHandle NetRefHandle = FNetRefHandleManager::MakeNetRefHandle(NetRefHandleId, GetId());
	TArray<FNetRefHandle> ExtraHandles;
	ExtraHandles.Reserve(ExtraNetRefHandles.Num());
	for (uint64 HandleId : ExtraNetRefHandles)
	{
		ExtraHandles.Emplace(FNetRefHandleManager::MakeNetRefHandle(HandleId, GetId()));
	}

	Impl->ReplicationSystemInternal.GetReplicationBridge()->OnErrorWithNetRefHandleReported(ErrorType, NetRefHandle, ConnectionId, ExtraHandles, DiagnosticMessage);
}

void UReplicationSystem::CollectNetMetrics(UE::Net::FNetMetrics& OutNetMetrics) const
{
	Impl->CollectNetMetrics(OutNetMetrics);
}

void UReplicationSystem::ResetNetMetrics()
{
	Impl->ResetNetMetrics();
}

UE::Net::FReplicationSystemDelegates& UReplicationSystem::GetDelegates()
{
	return Impl->Delegates;
}

bool UReplicationSystem::IsUsingRemoteObjectReferences() const
{
	return Impl->ReplicationSystemInternal.GetInitParams().bUseRemoteObjectReferences;
}

#pragma region ReplicationSystemFactory

namespace UE::Net
{
	
// FReplicationSystemFactory
FReplicationSystemFactory::FReplicationSystemArray FReplicationSystemFactory::ReplicationSystems;
TSharedPtr<UE::Net::Private::FObjectReferenceCache> FReplicationSystemFactory::SharedObjectReferenceCache;

FReplicationSystemCreatedDelegate& FReplicationSystemFactory::GetReplicationSystemCreatedDelegate()
{
	static FReplicationSystemCreatedDelegate Delegate;
	return Delegate;
}

FReplicationSystemDestroyedDelegate& FReplicationSystemFactory::GetReplicationSystemDestroyedDelegate()
{
	static FReplicationSystemDestroyedDelegate Delegate;
	return Delegate;
}

TSharedPtr<UE::Net::Private::FObjectReferenceCache> FReplicationSystemFactory::GetSharedObjectReferenceCache()
{
	if (!SharedObjectReferenceCache)
	{
		SharedObjectReferenceCache = MakeShared<UE::Net::Private::FObjectReferenceCache>();

		// It's assumed that the shared object reference cache will be used by client replication systems.
		UE::Net::Private::FObjectReferenceCacheParams InitParams;
		SharedObjectReferenceCache->Init(InitParams);
	}

	return SharedObjectReferenceCache;
}

UReplicationSystem* FReplicationSystemFactory::CreateReplicationSystem(const UReplicationSystem::FReplicationSystemParams& Params)
{
	LLM_SCOPE_BYTAG(IrisInitialization);

	if (!Params.ReplicationBridge)
	{
		UE_LOGF(LogIris, Error, "Cannot create ReplicationSystem without a ReplicationBridge");
		return nullptr;
	}

	if (static_cast<uint32>(ReplicationSystems.Num()) < FNetRefHandle::MaxReplicationSystemCount)
	{
		int32 NewSystemIndex = ReplicationSystems.Find(nullptr);
		if (NewSystemIndex == INDEX_NONE)
		{
			NewSystemIndex = ReplicationSystems.Add(nullptr);
		}

		UReplicationSystem* ReplicationSystem = NewObject<UReplicationSystem>();
		ReplicationSystems[NewSystemIndex] = ReplicationSystem;
		ReplicationSystem->AddToRoot();

		const uint32 ReplicationSystemId = static_cast<uint32>(NewSystemIndex);

		UE_LOGF(LogIris, Display, "Iris ReplicationSystem[%i]: %ls (0x%p) is created", ReplicationSystemId, *ReplicationSystem->GetName(), ReplicationSystem);

		ReplicationSystem->Init(ReplicationSystemId, Params);

		if (GetReplicationSystemCreatedDelegate().IsBound())
		{
			GetReplicationSystemCreatedDelegate().Broadcast(ReplicationSystem);
		}

		return ReplicationSystem;
	}

	LowLevelFatalError(TEXT("Too many ReplicationSystems have already been created (%llu)"), FNetRefHandle::MaxReplicationSystemCount);
	return nullptr;
}

void FReplicationSystemFactory::DestroyReplicationSystem(UReplicationSystem* System)
{
	if (System == nullptr)
	{
		return;
	}

	const uint32 Id = System->GetId();

	UE_LOGF(LogIris, Display, "Iris ReplicationSystem[%i]: %ls (0x%p) is about to be destroyed", Id, *System->GetName(), System);

	if (Id < static_cast<uint32>(ReplicationSystems.Num()))
	{
		ReplicationSystems[Id] = nullptr;

		// Remove all null entries from the end of the array
		const int32 LastValidIndex = ReplicationSystems.FindLastByPredicate([](const UReplicationSystem* It)
		{
			return It != nullptr;
		});

		const int32 MaxValidSystemNum = LastValidIndex == INDEX_NONE ? 0 : LastValidIndex + 1;
		ReplicationSystems.SetNum(MaxValidSystemNum, EAllowShrinking::No);
	}

	if (GetReplicationSystemDestroyedDelegate().IsBound())
	{
		GetReplicationSystemDestroyedDelegate().Broadcast(System);
	}

	if (SharedObjectReferenceCache.IsValid())
	{
		SharedObjectReferenceCache->ReplicationSystemDestroyed(System);
	}

	System->Shutdown();
	System->RemoveFromRoot();
	System->MarkAsGarbage();

	// If this is the last replication system, reset the shared object reference cache.
	if (ReplicationSystems.IsEmpty())
	{
		SharedObjectReferenceCache.Reset();
	}
}

TArrayView<UReplicationSystem*> FReplicationSystemFactory::GetAllReplicationSystems()
{
	return TArrayView<UReplicationSystem*>(ReplicationSystems);
}

FScopedNetStatsContext::FScopedNetStatsContext(UReplicationSystem* const InReplicationSystem)
{
	if (ensure(InReplicationSystem))
	{
		StoredNetStatsContext = InReplicationSystem->AcquireStatsForSerializationContext();
		StoredReplicationSystem = InReplicationSystem;
		bWasAcquiredInParallelPhase = InReplicationSystem->IsInParallelPhase();
	}
}

FScopedNetStatsContext::~FScopedNetStatsContext()
{
	if (ensure(StoredReplicationSystem))
	{
		ensureMsgf(bWasAcquiredInParallelPhase == StoredReplicationSystem->IsInParallelPhase(), TEXT("ScopedNetStatsContext lifetime mismatch - %s"), bWasAcquiredInParallelPhase ? TEXT("Expected to be destroyed in ParallelPhase") : TEXT("Expected to be destroyed outside of ParallelPhase"));
	
		StoredReplicationSystem->ReleaseStatsForSerializationContext(StoredNetStatsContext);
		StoredNetStatsContext = nullptr;
	}
}

Private::FNetStatsContext* FScopedNetStatsContext::GetRawPtr()
{
	return StoredNetStatsContext;
}

} //end namespace UE::Net

#pragma endregion
