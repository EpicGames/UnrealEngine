// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Iris/ReplicationState/ReplicationStateStorage.h"
#include "Iris/ReplicationSystem/DirtyNetObjectTracker.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ChangeMaskCache.h"
#include "Iris/ReplicationSystem/Conditionals/ReplicationConditionals.h"
#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineManager.h"
#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineInvalidationTracker.h"
#include "Iris/ReplicationSystem/Filtering/ReplicationFiltering.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectGroups.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Iris/ReplicationSystem/ReplicationConnections.h"
#include "Iris/ReplicationSystem/Prioritization/ReplicationPrioritization.h"
#include "Iris/ReplicationSystem/ReplicationProtocolManager.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlobManager.h"
#include "Net/Core/NetToken/NetToken.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/StringTokenStore.h"
#include "Iris/ReplicationSystem/NameTokenStore.h"
#include "Iris/ReplicationSystem/WorldLocations.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorRegistry.h"
#include "Iris/Stats/NetStats.h"

class UIrisObjectReferencePackageMap;

namespace UE::Net::Private
{

struct FReplicationSystemInternalInitParams
{
	uint32 ReplicationSystemId;
	uint32 MaxReplicatedObjectCount;
	uint32 NetChunkedArrayCount;
	uint32 MaxReplicationWriterObjectCount;
	bool bUseRemoteObjectReferences;
	bool bAllowParallelTasks;
	bool bAllowMinimalUpdateIfNoConnections;
	bool bUseSharedObjectReferenceCache;
};

class FReplicationSystemInternal
{
public:
	explicit FReplicationSystemInternal(const FReplicationSystemInternalInitParams& Params)
	: NetRefHandleManager(ReplicationProtocolManager)
	, InternalInitParams(Params)
	, DirtyNetObjectTracker()
	, ReplicationBridge(nullptr)
	, IrisObjectReferencePackageMap(nullptr)
	, Id(Params.ReplicationSystemId)
	, bAllowParallelTasks(Params.bAllowParallelTasks)
	{
		if (Params.bUseSharedObjectReferenceCache)
		{
			ObjectReferenceCache = FReplicationSystemFactory::GetSharedObjectReferenceCache();
		}
		
		if (!ObjectReferenceCache)
		{
			ObjectReferenceCache = MakeShared<FObjectReferenceCache>();
		}
	}

	FReplicationProtocolManager& GetReplicationProtocolManager() { return ReplicationProtocolManager; }

	FNetRefHandleManager& GetNetRefHandleManager() { return NetRefHandleManager; }
	const FNetRefHandleManager& GetNetRefHandleManager() const { return NetRefHandleManager; }

	void InitDirtyNetObjectTracker(const struct FDirtyNetObjectTrackerInitParams& Params) { DirtyNetObjectTracker.Init(Params); }
	bool IsDirtyNetObjectTrackerInitialized() const { return DirtyNetObjectTracker.IsInit(); }
	FDirtyNetObjectTracker& GetDirtyNetObjectTracker() { checkf(DirtyNetObjectTracker.IsInit(), TEXT("Not allowed to access the DirtyNetObjectTracker unless object replication is enabled.")); return DirtyNetObjectTracker; }

	FReplicationStateDescriptorRegistry& GetReplicationStateDescriptorRegistry() { return ReplicationStateDescriptorRegistry; }

	FReplicationStateStorage& GetReplicationStateStorage() { return ReplicationStateStorage; }

	FObjectReferenceCache& GetObjectReferenceCache() { return *ObjectReferenceCache; }

	void SetReplicationBridge(UObjectReplicationBridge* InReplicationBridge) { ReplicationBridge = InReplicationBridge; }
	UObjectReplicationBridge* GetReplicationBridge() const { return ReplicationBridge; }
	UObjectReplicationBridge* GetReplicationBridge(FNetRefHandle Handle) const { return ReplicationBridge; }

	void SetIrisObjectReferencePackageMap(UIrisObjectReferencePackageMap* InIrisObjectReferencePackageMap) { IrisObjectReferencePackageMap = InIrisObjectReferencePackageMap; }
	UIrisObjectReferencePackageMap* GetIrisObjectReferencePackageMap() { return IrisObjectReferencePackageMap; }

	FChangeMaskCache& GetChangeMaskCache() { return ChangeMaskCache; }

	FReplicationConnections& GetConnections() { return Connections; }
	const FReplicationConnections& GetConnections() const { return Connections; }

	FReplicationFiltering& GetFiltering() { return Filtering; }
	const FReplicationFiltering& GetFiltering() const { return Filtering; }

	FNetObjectGroups& GetGroups() { return Groups; }

	FReplicationConditionals& GetConditionals() { return Conditionals; }

	FReplicationPrioritization& GetPrioritization() { return Prioritization; }

	FNetBlobManager& GetNetBlobManager() { return NetBlobManager; }
	FNetBlobHandlerManager& GetNetBlobHandlerManager() { return NetBlobManager.GetNetBlobHandlerManager(); }
	const FNetBlobHandlerManager& GetNetBlobHandlerManager() const { return NetBlobManager.GetNetBlobHandlerManager(); }

	FWorldLocations& GetWorldLocations() { return WorldLocations; }

	FDeltaCompressionBaselineManager& GetDeltaCompressionBaselineManager() { return DeltaCompressionBaselineManager; }
	FDeltaCompressionBaselineInvalidationTracker& GetDeltaCompressionBaselineInvalidationTracker() { return DeltaCompressionBaselineInvalidationTracker; }

	FNetTypeStats& GetNetTypeStats() { return TypeStats; }

	const FReplicationSystemInternalInitParams& GetInitParams() const { return InternalInitParams; }

	FNetSendStats& GetSendStats()
	{ 
		return SendStats;
	}

	FReplicationStats& GetTickReplicationStats()
	{
		return TickReplicationStats;
	}

	const FReplicationStats& GetTickReplicationStats() const
	{
		return TickReplicationStats;
	}

	FReplicationStats& GetAccumulatedReplicationStats()
	{
		return AccumulatedReplicationStats;
	}

	const FReplicationStats& GetAccumulatedReplicationStats() const
	{
		return AccumulatedReplicationStats;
	}

	FForwardNetRPCCallMulticastDelegate& GetForwardNetRPCCallMulticastDelegate()
	{
		return ForwardNetRPCCallMulticastDelegate;
	}

	void SetBlockFilterChanges(bool bBlock) { bBlockFilterChanges = bBlock; }

	bool AreFilterChangesBlocked() const { return bBlockFilterChanges; }

	bool AreParallelTasksAllowed() const { return bAllowParallelTasks; }

	/** Used by subsystems such as ObjectPoller to indicate when we're running tasks simultaneously and need to be thread-safe
		Must be called in-order and exclusively. We currently do not support simultaneous parallel phases (e.g running Write tasks whilst Polling is running) */
	void StartParallelPhase() 
	{ 
		check(bAllowParallelTasks);
		check(!bIsInParallelPhase);
		bIsInParallelPhase = true;
		TypeStats.SetIsInParallelPhase(true);
	}

	void StopParallelPhase()
	{ 
		check(bAllowParallelTasks); 
		check(bIsInParallelPhase); 
		bIsInParallelPhase = false; 
		TypeStats.SetIsInParallelPhase(false); 
	}

	bool IsInParallelPhase() const
	{
		return bIsInParallelPhase;
	}

	void PreSeamlessTravelGarbageCollect()
	{
		GetReplicationBridge()->PreSeamlessTravelGarbageCollect();
	}

	void PostSeamlessTravelGarbageCollect()
	{
		GetReplicationBridge()->PostSeamlessTravelGarbageCollect();
	}
	#if UE_SUPPORT_PARALLEL_IRIS
	void AcquireTickReplicationStatsWriteLock()
	{
		TickReplicationStatsMutex.WriteLock();
	}
		
	void AcquireTickReplicationStatsReadLock()
	{
		TickReplicationStatsMutex.ReadLock();
	}

	void ReleaseTickReplicationStatsWriteLock()
	{
		TickReplicationStatsMutex.WriteUnlock();
	}

	void ReleaseTickReplicationStatsReadLock()
	{
		TickReplicationStatsMutex.ReadUnlock();
	}
#endif // UE_SUPPORT_PARALLEL_IRIS

void RemoveFromAllGroups(UE::Net::FInternalNetRefIndex ObjectInternalIndex);

private:
	FReplicationProtocolManager ReplicationProtocolManager;
	FNetRefHandleManager NetRefHandleManager;
	FReplicationSystemInternalInitParams InternalInitParams;
	FDirtyNetObjectTracker DirtyNetObjectTracker;
	FReplicationStateStorage ReplicationStateStorage;
	FReplicationStateDescriptorRegistry ReplicationStateDescriptorRegistry;
	UObjectReplicationBridge* ReplicationBridge;
	UIrisObjectReferencePackageMap* IrisObjectReferencePackageMap;
	FChangeMaskCache ChangeMaskCache;
	FReplicationConnections Connections;
	FReplicationFiltering Filtering;
	FNetObjectGroups Groups;
	FReplicationConditionals Conditionals;
	FReplicationPrioritization Prioritization;
	TSharedPtr<FObjectReferenceCache> ObjectReferenceCache;
	FNetBlobManager NetBlobManager;
	FWorldLocations WorldLocations;
	FDeltaCompressionBaselineManager DeltaCompressionBaselineManager;
	FDeltaCompressionBaselineInvalidationTracker DeltaCompressionBaselineInvalidationTracker;
	FNetSendStats SendStats;
	FNetTypeStats TypeStats;
	FReplicationStats TickReplicationStats = {};
	FReplicationStats AccumulatedReplicationStats = {};
	FForwardNetRPCCallMulticastDelegate ForwardNetRPCCallMulticastDelegate;
	uint32 Id;

#if UE_SUPPORT_PARALLEL_IRIS
	/** Mutex which protects access to TickReplicationStats during parallel phases*/
	FTransactionallySafeRWLock TickReplicationStatsMutex;
#endif // UE_SUPPORT_PARALLEL_IRIS

	/** When true this prevents any changes to the filter system. Enabled during times where adding filter options is unsupported. */
	bool bBlockFilterChanges = false;

public:
	/**
	*   When true, allow subsystems to run parallel workloads, such as the PollAndCopy step running several asynchronous tasks to speed up game thread execution time.
	*   Only supported when bIsServer = true and bAllowObjectReplication = true
	*/
	bool bAllowParallelTasks = false;

	/** Is true whilst a phase is running parallel tasks. If bAllowParallelTasks is false, this can never be true. */
	bool bIsInParallelPhase = false;
};

}
