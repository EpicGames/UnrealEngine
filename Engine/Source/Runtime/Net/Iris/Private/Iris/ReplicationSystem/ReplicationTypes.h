// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Core/BitTwiddling.h"

#include "Net/Core/NetBitArray.h"

#include "Misc/EnumClassFlags.h"

class UReplicationSystem;

namespace UE::Net
{
	typedef uint32 FInternalNetRefIndex;
}

namespace UE::Net::Private
{

struct FReplicationParameters
{
	FInternalNetRefIndex MaxInternalNetRefIndex = 0;
	uint32 MaxReplicationWriterObjectCount = 0;
	uint32 PacketSendWindowSize = 0;
	uint32 ConnectionId = 0;
	UReplicationSystem* ReplicationSystem = nullptr;
	bool bAllowSendingAttachmentsToObjectsNotInScope = false;
	bool bAllowReceivingAttachmentsFromRemoteObjectsNotInScope = false;
	bool bAllowDelayingAttachmentsWithUnresolvedReferences = false;
	uint32 SmallObjectBitThreshold = 160U; // Number of bits remaining in a packet for us to consider trying to serialize a replicated object
	uint32 MaxFailedSmallObjectCount = 10U;	// Number of objects that we try to serialize after an initial stream overflow to fill up a packet, this can improve bandwidth usage but comes at a cpu cost
	uint32 NumBitsUsedForBatchSize = 16U;
	uint32 NumBitsUsedForHugeObjectBatchSize = 32U;
};

enum class EReplicatedDestroyHeaderFlags : uint32
{
	None				= 0U,
	TearOff				= 1U << 0U,
	IsSubObject    		= TearOff << 1U,
	DestroySubObject	= IsSubObject << 1U,
	BitCount			= 3U //update as necessary
};

consteval uint32 GetDestroyHeaderFlagsBitCount()
{
	return (uint32)EReplicatedDestroyHeaderFlags::BitCount;
}

ENUM_CLASS_FLAGS(EReplicatedDestroyHeaderFlags);

}
