// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/ReplicationWriter.h"

#include "Containers/Array.h"

#include "HAL/IConsoleManager.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"

#include "Iris/IrisConfigInternal.h"

#include "Iris/ReplicationSystem/IrisCreationFlowLog.h"

#include "Iris/PacketControl/PacketNotification.h"

#include "Iris/ReplicationSystem/DeltaCompression/DeltaCompressionBaselineManager.h"

#include "Iris/ReplicationSystem/Filtering/ReplicationFiltering.h"

#include "Iris/ReplicationSystem/NetBlob/NetObjectBlobHandler.h"
#include "Iris/ReplicationSystem/NetBlob/PartialNetObjectAttachmentHandler.h"

#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"

#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/ObjectNetSerializer.h"

#include "Iris/Stats/NetStatsContext.h"

#include "Misc/ScopeExit.h"

#include "Net/Core/Trace/NetDebugName.h"
#include "Net/Core/Trace/NetTrace.h"

#include "ProfilingDebugging/CsvProfiler.h"

#include <algorithm>
#include <cmath> // std::nextafter

#if UE_NET_ENABLE_REPLICATIONWRITER_LOG
#	define UE_LOG_REPWRITER(Verbosity, Format, ...)  UE_LOG(LogIris, Verbosity, TEXT("Conn: %u ") Format, Parameters.ConnectionId, ##__VA_ARGS__)
#else
#	define UE_LOG_REPWRITER(...)
#endif

#define UE_LOG_REPWRITER_WARNING(Format, ...) UE_LOG(LogIris, Warning, TEXT("Conn: %u ") Format, Parameters.ConnectionId, ##__VA_ARGS__)
#define UE_LOG_REPWRITER_ERROR(Format, ...) UE_LOG(LogIris, Error, TEXT("Conn: %u ") Format, Parameters.ConnectionId, ##__VA_ARGS__)
#define UE_CLOG_REPLICATIONWRITER_WARNING(Condition, Format, ...)  UE_CLOG(Condition, LogIris, Warning, TEXT("Conn: %u ") Format, Parameters.ConnectionId, ##__VA_ARGS__)

#define UE_NET_ENFORCE_CREATION_DEPENDENCIES_ON_SERVER 0

namespace UE::Net::Private
{

static bool bWarnAboutDroppedAttachmentsToObjectsNotInScope = false;
static FAutoConsoleVariableRef CVarWarnAboutDroppedAttachmentsToObjectsNotInScope(
	TEXT("net.Iris.WarnAboutDroppedAttachmentsToObjectsNotInScope"),
	bWarnAboutDroppedAttachmentsToObjectsNotInScope,
	TEXT("Warn when attachments are dropped due to object not in scope. Default is false."
	));

/*
 * net.Iris.ReplicationWriterMaxHugeObjectsInTransit
 * There's a tradeoff mainly between the connection characteristics to support and normal object replication scheduling when tweaking this value.
 * On one hand you don't want to end up stalling object replication because the top priority objects are huge. So you want to be able to keep replicating huge objects during the maximum latency, including latency variation, and packet loss scenarios 
 * you want to provide the best experience possible for. On the other hand object deletion cannot be performed once the object is in the huge object queue. Consider this and how long time it will take to replicate the huge object queue depending on the average payload of a huge object.
 */
static int32 GReplicationWriterMaxHugeObjectsInTransit = 16;
static FAutoConsoleVariableRef CVarReplicationWriterMaxHugeObjectsInTransit(TEXT("net.Iris.ReplicationWriterMaxHugeObjectsInTransit"), GReplicationWriterMaxHugeObjectsInTransit,
	TEXT("How many very large objects, one whose payload doesn't fit in a single packet, is allowed to be scheduled for send. Needs to be at least 1."));

/*
 * net.Iris.ReplicationWriterMaxDestroyObjectsPerFrame
 * Limit the number of root objects to be destroyed per frame to relieve pressure on the ReplicationRecord count and potentially for client performance reasons. The number includes both regular object destruction as well as destruction infos.
 * Without limiting several hundred objects could fit a packet which would exhaust the replication records long before filling the packet window.
 */
static int32 GReplicationWriterMaxDestroyObjectsPerFrame = 150;
static FAutoConsoleVariableRef CVarReplicationWriterMaxDestroyObjectsPerFrame(TEXT("net.Iris.ReplicationWriterMaxDestroyObjectsPerFrame"), GReplicationWriterMaxDestroyObjectsPerFrame,
	TEXT("How many objects can be replicated for destroy per frame. The number is for regular destruction and destruction infos combined. A value less or equal to zero means unlimited."));

/*
 * net.Iris.ReplicationWriterReplicationRecordStarvationThreshold
 * When the number of ReplicationRecords left subceeds this number the ReplicationWriter will limit what is replicated in a packet to a minimum, effectively limiting replication to OOB attachments and huge objects.
 */
static int32 GReplicationWriterReplicationRecordStarvationThreshold = 1000;
static FAutoConsoleVariableRef CVarReplicationWriterReplicationRecordStarvationThreshold(TEXT("net.Iris.ReplicationWriterReplicationRecordStarvationThreshold"), GReplicationWriterReplicationRecordStarvationThreshold,
	TEXT("How many ReplicationRecords need to be left in order to proceed with replication as normal. Below this threshold replication will be limited to OOB attachments and huge objects until there are more ReplicationRecords available."));

static bool bValidateObjectsWithDirtyChanges = true;
static FAutoConsoleVariableRef CvarValidateObjectsWithDirtyChanges(TEXT("net.Iris.ReplicationWriter.ValidateObjectsWithDirtyChanges"), bValidateObjectsWithDirtyChanges, TEXT("Ensure that we don't try to mark invalid objects as dirty when they shouldn't."));

// net.Iris.ReplicationWriter.WriteBatchSizePerObject
static bool bDebugBatchSizePerObjectEnabled = false;
static FAutoConsoleVariableRef CvarWriteBatchSizePerObject(TEXT("net.Iris.ReplicationWriter.WriteBatchSizePerObject"), bDebugBatchSizePerObjectEnabled, TEXT("Write batch size per object. Helps tracking down bitstream errors. Requires code to be compiled with UE_NET_REPLICATIONDATASTREAM_DEBUG to be enabled."));

// net.Iris.ReplicationWriter.WriteSentinels
static bool bDebugSentinelsEnabled = false;
static FAutoConsoleVariableRef CvarWriteSentinels(TEXT("net.Iris.ReplicationWriter.WriteSentinels"), bDebugSentinelsEnabled, TEXT("Write sentinels at carefully chosen points in the stream. Helps tracking down bitstream errors. Requires code to be compiled with UE_NET_REPLICATIONDATASTREAM_DEBUG to be enabled."));

// Allow warning if object has been prevented from sending for a long time.
static int32 GReplicationWriterCannotSendWarningInterval = 256;
static FAutoConsoleVariableRef CVarReplicationWriterCannotSendWarningInterval(TEXT("net.Iris.ReplicationWriterCannotSendWarningInterval"), GReplicationWriterCannotSendWarningInterval,
	TEXT("Warn if we are prevented from Sending for more than a certain number of updates, setting it to 0 will disable warning"));

// Drop unsent ordered unreliable attachments at the end of the tick, 
// -1 = Allow them to be queued for next tick, 
// 0  = Always drop at the end of the tick
// >0 = Drop and log the count if we have more queued ordered unreliable than GMaxUnsentOrderedUnreliableAttachmentAtEndOfTick
static int32 GMaxUnsentOrderedUnreliableAttachmentAtEndOfTick = -1;
static FAutoConsoleVariableRef CvarMaxUnsentOrderedUnreliableAttachmentAtEndOfTick(TEXT("net.Iris.ReplicationWriter.MaxUnsentOrderedUnreliableAttachmentAtEndOfTick"), GMaxUnsentOrderedUnreliableAttachmentAtEndOfTick, TEXT("Drop unsent ordered unreliable attachments at the end of the tick, -1 means that that we allow them to be queued for next tick, 0 or greater means that we will drop if we have more queued unreliable attachment than the set value at the end of the tick"));

static const FName NetError_ObjectStateTooLarge("Object state is too large to be split.");

const TCHAR* FReplicationWriter::LexToString(const EReplicatedObjectState State)
{
	static const TCHAR* Names[] = {
		TEXT("Invalid"),
		TEXT("AttachmentToObjectNotInScope"),
		TEXT("HugeObject"),
		TEXT("DebugObject"),
		TEXT("PendingCreate"),
		TEXT("WaitOnCreateConfirmation"),
		TEXT("Created"),
		TEXT("WaitOnFlush"),
		TEXT("PendingTearOff"),
		TEXT("SubObjectPendingDestroy"),
		TEXT("CancelPendingDestroy"),
		TEXT("PendingDestroy"),
		TEXT("WaitOnDestroyConfirmation"),
		TEXT("Destroyed"),
		TEXT("PermanentlyDestroyed"),
	};
	static_assert(UE_ARRAY_COUNT(Names) == uint32(EReplicatedObjectState::Count), "Missing names for one or more values of EReplicatedObjectState.");

	return State < EReplicatedObjectState::Count ? Names[(uint32)State] : TEXT("");
}


// Default allocator for changemasks
static FGlobalChangeMaskAllocator s_DefaultChangeMaskAllocator;

// Helper class to process all ReplicationInfos for a record
struct TReplicationRecordHelper
{
	typedef FReplicationWriter::FReplicationInfo FReplicationInfo;
	typedef FReplicationRecord::FRecordInfoList FRecordInfoList;
	typedef FReplicationWriter::EReplicatedObjectState EReplicatedObjectState;

	TArray<FReplicationInfo>& ReplicationInfos;
	TArray<FReplicationRecord::FRecordInfoList>& ReplicationInfosRecordInfoLists;
	FReplicationRecord* ReplicationRecord;

	TReplicationRecordHelper(TArray<FReplicationInfo>& InReplicationInfos, TArray<FReplicationRecord::FRecordInfoList>& InReplicationInfosRecordInfoLists, FReplicationRecord* InReplicationRecordRecord)
	: ReplicationInfos(InReplicationInfos)
	, ReplicationInfosRecordInfoLists(InReplicationInfosRecordInfoLists)
	, ReplicationRecord(InReplicationRecordRecord)
	{
	}
	
	template <typename T>
	void Process(uint32 RecordInfoCount, T&& Functor)
	{
		for (uint32 It = 0; It < RecordInfoCount; ++It)
		{
			const FReplicationRecord::FRecordInfo& RecordInfo = ReplicationRecord->PeekInfo();
			FNetObjectAttachmentsWriter::FReliableReplicationRecord AttachmentRecord(RecordInfo.HasAttachments ? ReplicationRecord->DequeueAttachmentRecord() : uint64(0));
			FReplicationInfo& Info = ReplicationInfos[RecordInfo.Index];
			FReplicationRecord::FRecordInfoList& RecordInfoList = ReplicationInfosRecordInfoLists[RecordInfo.Index];

			// We Need to cache this as the the ReplicationInfo might be invalidated by the functor
			const uint32 ChangeMaskBitCount = Info.ChangeMaskBitCount;

			// Invoke function
			Functor(RecordInfo, Info, AttachmentRecord);

			// We must free any dynamic memory allocated in PushRecordInfo
			if (RecordInfo.HasChangeMask)
			{
				FChangeMaskStorageOrPointer::Free(RecordInfo.ChangeMaskOrPtr, ChangeMaskBitCount, s_DefaultChangeMaskAllocator);
			}

			// Construct a RecordInfo from each subobject info
			if (RecordInfo.HasSubObjectRecord)
			{
				FNetObjectAttachmentsWriter::FReliableReplicationRecord SubObjectAttachmentRecord(0);
				FReplicationRecord::FRecordInfo SubObjectRecordInfo = {.NewBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex};
				FReplicationRecord::FSubObjectRecord SubObjectRecord = ReplicationRecord->DequeueSubObjectRecord();
				for (FReplicationRecord::FSubObjectRecord::FSubObjectInfo SubObjectInfo : SubObjectRecord.SubObjectInfos)
				{
					FReplicationInfo& SubObjectReplicationInfo = ReplicationInfos[SubObjectInfo.Index];

					SubObjectRecordInfo.Index = SubObjectInfo.Index;
					SubObjectRecordInfo.ReplicatedObjectState = SubObjectInfo.ReplicatedObjectState;

					// Invoke function
					Functor(SubObjectRecordInfo, SubObjectReplicationInfo, SubObjectAttachmentRecord);
				}
			}

			// We must remove the record and unlink it
			// It is safe to call even if we have stopped replicating the object		
			ReplicationRecord->PopInfoAndRemoveFromList(RecordInfoList);
		}
	}
};

#if UE_NET_VALIDATE_REPLICATION_RECORD

static bool s_ValidateReplicationRecord(const FReplicationRecord* ReplicationRecord, uint32 MaxInternalIndexCount, bool bVerifyFirstRecord)
{
	if (ReplicationRecord->GetRecordCount() == 0U)
	{
		return true;
	}

	// validate count
	{
		uint32 TotalPushedInfos = 0U;
		for (uint32 It = 0U, EndIt = ReplicationRecord->GetRecordCount(); It < EndIt; ++It)
		{
			TotalPushedInfos += ReplicationRecord->PeekRecordAtOffset(It);
		}

		if (TotalPushedInfos != ReplicationRecord->GetInfoCount())
		{
			ensure(false);
			return false;
		}
	}
	
	// Verify last / first record
	const uint32 RecordInfoCount = bVerifyFirstRecord ? ReplicationRecord->PeekRecordAtOffset(0) : ReplicationRecord->PeekRecordAtOffset(ReplicationRecord->GetRecordCount() - 1);

	// check for duplicates
	{
		FNetBitArray BitArray;
		BitArray.Init(MaxInternalIndexCount);
		
		uint32 Offset = bVerifyFirstRecord ? 0U : ReplicationRecord->GetInfoCount() - RecordInfoCount;
		for (uint32 It = 0U; It < RecordInfoCount; ++It)
		{
 			const FReplicationRecord::FRecordInfo& RecordInfo = ReplicationRecord->PeekInfoAtOffset(It + Offset);
			// We allow multiple entries for the OOB attachments but do not expect multiple entries for normal replicated objects
			if (RecordInfo.Index != 0U && BitArray.GetBit(RecordInfo.Index))
			{
				ensure(false);
				return false;
			}
			BitArray.SetBit(RecordInfo.Index);
		}
	}

	return true;
}

#endif

FReplicationWriter::~FReplicationWriter()
{
	DiscardAllRecords();

	// Freeing the huge object queue needs to be done before calling StopAllReplication() in order to be able to free any changemask allocations.
	FreeHugeObjectSendQueues();
	
	StopAllReplication();
}

void FReplicationWriter::SetReplicationEnabled(bool bInReplicationEnabled)
{
	bReplicationEnabled = bInReplicationEnabled;
}

bool FReplicationWriter::IsReplicationEnabled() const
{
	return bReplicationEnabled;
}

bool FReplicationWriter::IsInReplicatingState(FInternalNetRefIndex ObjectInternalIndex) const
{
	if (!ReplicatedObjects.IsValidIndex(ObjectInternalIndex))
	{
		ensureMsgf(false, TEXT("Received invalid index: %u"), ObjectInternalIndex);
		return false;
	}

	// We let InvalidInternalNetRefIndex be accessed because it has an object state too.
	const FReplicationInfo& Info = GetReplicationInfo(ObjectInternalIndex);
	const EReplicatedObjectState CurrentState = Info.GetState();
	
	// We consider an object to be relevant while it's creation data is in flight and before we sent destruction data.
	return CurrentState >= EReplicatedObjectState::WaitOnCreateConfirmation && CurrentState <= EReplicatedObjectState::WaitOnFlush;
}

// $IRIS TODO : May need to introduce queue and send behaviors. For example one may want to send only with object.
// One may not want to send unless the object is replicated very soon etc.
bool FReplicationWriter::QueueNetObjectAttachments(FInternalNetRefIndex OwnerInternalIndex, FInternalNetRefIndex SubObjectInternalIndex, TArrayView<const TRefCountPtr<FNetBlob>> InAttachments, ENetObjectAttachmentSendPolicyFlags SendFlags)
{
	if (InAttachments.Num() <= 0)
	{
		ensureMsgf(false, TEXT("%s"), TEXT("QueueNetObjectAttachments expects at least one attachment."));
		return false;
	}

	const uint32 TargetIndex = SubObjectInternalIndex != InvalidInternalNetRefIndex ? SubObjectInternalIndex : OwnerInternalIndex;
	const bool bTargetObjectInScope = ObjectsInScope.GetBit(TargetIndex);
	if (!bTargetObjectInScope && !Parameters.bAllowSendingAttachmentsToObjectsNotInScope)
	{
		UE_CLOG_REPLICATIONWRITER_WARNING(bWarnAboutDroppedAttachmentsToObjectsNotInScope, TEXT("Dropping %s attachment due to object ( InternalIndex: %u ) not in scope."), (EnumHasAnyFlags(InAttachments[0]->GetCreationInfo().Flags, ENetBlobFlags::Reliable) ? TEXT("reliable") : TEXT("unreliable")), TargetIndex);
		return false;
	}
	
	const bool bScheduleUsingOOBChannel = EnumHasAnyFlags(SendFlags, ENetObjectAttachmentSendPolicyFlags::ScheduleAsOOB);
	if (bScheduleUsingOOBChannel)
	{
		// Route attachments flagged with ScheduleAsOOB through OOB channel only if we have started replicating the target.
		const EReplicatedObjectState ReplicationState = GetReplicationInfo(TargetIndex).GetState();
		if (ReplicationState < EReplicatedObjectState::WaitOnCreateConfirmation || ReplicationState >= EReplicatedObjectState::PendingDestroy)
		{
			UE_CLOG_REPLICATIONWRITER_WARNING(bWarnAboutDroppedAttachmentsToObjectsNotInScope, TEXT("Dropping attachment scheduled as ScheduleAsOOB due to object ( InternalIndex: %u ) not in replicated state."),  OwnerInternalIndex);
			return false;
		}
	}

	const uint32 AttachmentQueueIndex = (bTargetObjectInScope && !bScheduleUsingOOBChannel) ? TargetIndex : ObjectIndexForOOBAttachment;
	const ENetObjectAttachmentType AttachmentType = ((bTargetObjectInScope && !bScheduleUsingOOBChannel) ? ENetObjectAttachmentType::Normal : ENetObjectAttachmentType::OutOfBand);
	if (!Attachments.Enqueue(AttachmentType, AttachmentQueueIndex, InAttachments))
	{
		return false;
	}

	// We do not have to mark anything dirty as there's a special case for out of band attachments
	if (!IsObjectIndexForOOBAttachment(AttachmentQueueIndex))
	{
		FReplicationInfo& TargetInfo = GetReplicationInfo(AttachmentQueueIndex);
		TargetInfo.HasAttachments = 1;

		MarkObjectDirty(AttachmentQueueIndex, "QueueAttachment");

		if (OwnerInternalIndex != AttachmentQueueIndex)
		{
			MarkObjectDirty(OwnerInternalIndex, "QueueAttachment2");
			FReplicationInfo& OwnerInfo = GetReplicationInfo(OwnerInternalIndex);
			OwnerInfo.HasDirtySubObjects = 1;
		}
	}

	return true;
}

bool FReplicationWriter::AreAllReliableAttachmentsSentAndAcked() const
{
	const bool bHasUnprocessedReliables = ReplicationSystemInternal->GetNetBlobManager().HasAnyUnprocessedReliableAttachments();
	return !bHasUnprocessedReliables && Attachments.AreAllObjectsReliableSentAndAcked();
}

bool FReplicationWriter::HasDebugDataToSend() const
{
	return Attachments.HasUnsentAttachments(ENetObjectAttachmentType::DebugObject, ObjectIndexForOOBAttachment);
}

void FReplicationWriter::SetState(uint32 InternalIndex, EReplicatedObjectState NewState)
{
	FReplicationInfo& Info = GetReplicationInfo(InternalIndex);

	const EReplicatedObjectState CurrentState = Info.GetState();
	UE_LOG_REPWRITER(Verbose, TEXT("ReplicationWriter.SetState for ( InternalIndex: %u ) %s -> %s"), InternalIndex, LexToString(CurrentState), LexToString(NewState));

	switch (NewState)
	{
		case EReplicatedObjectState::PendingCreate:
		{
			ensureMsgf(CurrentState == EReplicatedObjectState::Invalid || CurrentState == EReplicatedObjectState::WaitOnCreateConfirmation, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;
		case EReplicatedObjectState::WaitOnCreateConfirmation:
		{
			ensureMsgf(CurrentState == EReplicatedObjectState::PendingCreate || CurrentState == EReplicatedObjectState::CancelPendingDestroy, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;
		case EReplicatedObjectState::Created:
		{
			ensureMsgf(CurrentState == EReplicatedObjectState::PendingCreate || CurrentState == EReplicatedObjectState::WaitOnCreateConfirmation || CurrentState == EReplicatedObjectState::CancelPendingDestroy || CurrentState == EReplicatedObjectState::WaitOnFlush, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;
		case EReplicatedObjectState::PendingTearOff:
		{
			ensureMsgf(CurrentState == EReplicatedObjectState::PendingTearOff || CurrentState == EReplicatedObjectState::WaitOnFlush || CurrentState == EReplicatedObjectState::WaitOnCreateConfirmation || CurrentState == EReplicatedObjectState::Created || CurrentState == EReplicatedObjectState::WaitOnDestroyConfirmation, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT " HasAttachments: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject, Info.HasAttachments);
		}
		break;
		case EReplicatedObjectState::SubObjectPendingDestroy:
		{
			ensureMsgf(CurrentState == EReplicatedObjectState::PendingDestroy || CurrentState == EReplicatedObjectState::SubObjectPendingDestroy || CurrentState == EReplicatedObjectState::WaitOnCreateConfirmation || CurrentState == EReplicatedObjectState::Created || CurrentState == EReplicatedObjectState::WaitOnFlush || CurrentState == EReplicatedObjectState::WaitOnDestroyConfirmation, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;
		case EReplicatedObjectState::WaitOnFlush:
		{
			ensureMsgf(CurrentState != EReplicatedObjectState::Invalid, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;
		case EReplicatedObjectState::PendingDestroy:
		{
			ensureMsgf(CurrentState != EReplicatedObjectState::Invalid, TEXT("Trying to set state %s when state is %s. (InternalIndex: %u) IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), 
					   LexToString(NewState), LexToString(CurrentState), InternalIndex, Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;
		case EReplicatedObjectState::WaitOnDestroyConfirmation:
		{
			ensureMsgf(CurrentState >= EReplicatedObjectState::PendingTearOff, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;
		case EReplicatedObjectState::CancelPendingDestroy:
		{
			ensureMsgf(CurrentState == EReplicatedObjectState::WaitOnDestroyConfirmation || CurrentState == EReplicatedObjectState::CancelPendingDestroy, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;
		case EReplicatedObjectState::Destroyed:
		{
			ensureMsgf(CurrentState == EReplicatedObjectState::WaitOnDestroyConfirmation || CurrentState == EReplicatedObjectState::PendingTearOff || CurrentState == EReplicatedObjectState::CancelPendingDestroy, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;
		case EReplicatedObjectState::PermanentlyDestroyed:
		{
			ensureMsgf(CurrentState == EReplicatedObjectState::Invalid || CurrentState == EReplicatedObjectState::WaitOnCreateConfirmation, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;
		case EReplicatedObjectState::Invalid:
		{
			ensureMsgf(CurrentState == EReplicatedObjectState::PermanentlyDestroyed || CurrentState == EReplicatedObjectState::Destroyed || CurrentState == EReplicatedObjectState::PendingCreate, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), LexToString(NewState), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
		}
		break;

		default:
			ensureMsgf(false, TEXT("Trying to set state %s when state is %s. Object: %s IsDestructionInfo: %" UINT64_FMT " IsSubObject: %" UINT64_FMT), ToCStr(FString::FromInt(int(NewState))), LexToString(CurrentState), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.IsDestructionInfo, Info.IsSubObject);
			break;
	};

	Info.State = (uint32)NewState;
}

FReplicationWriter::FReplicationWriter()
: HugeObjectSendQueue(TEXT("HugeObjectState"))
, DebugObjectSendQueue(TEXT("DebugObjectState"), /*bAllowMultipleBatchesPerRootObjectInTransit*/true)
{
}

void FReplicationWriter::Init(const FReplicationParameters& InParameters)
{
	// Store copy of parameters
	Parameters = InParameters;

	UE_LOGF(LogIris, Verbose, "ReplicationWriter: Configured with MaxInternalNetRefIndex=%d and MaxReplicationWriterObjectCount=%d.", 
		Parameters.MaxInternalNetRefIndex, Parameters.MaxReplicationWriterObjectCount);

	// Cache internal systems
	ReplicationSystemInternal = Parameters.ReplicationSystem->GetReplicationSystemInternal();
	NetRefHandleManager = &ReplicationSystemInternal->GetNetRefHandleManager();
	ReplicationBridge = Parameters.ReplicationSystem->GetReplicationBridge();
	BaselineManager = &ReplicationSystemInternal->GetDeltaCompressionBaselineManager();
	ObjectReferenceCache = &ReplicationSystemInternal->GetObjectReferenceCache();
	ReplicationFiltering = &ReplicationSystemInternal->GetFiltering();
	ReplicationConditionals = &ReplicationSystemInternal->GetConditionals();
	const FNetBlobManager* NetBlobManager = &ReplicationSystemInternal->GetNetBlobManager();
	PartialNetObjectAttachmentHandler = NetBlobManager->GetPartialNetObjectAttachmentHandler();
	NetObjectBlobHandler = NetBlobManager->GetNetObjectBlobHandler();
	NetTypeStats = &ReplicationSystemInternal->GetNetTypeStats();

	// See if we want to limit the amount of objects able to do property replication, otherwise follow the system max and grow as needed
	const uint32 MaxSupportedObjects = Parameters.MaxReplicationWriterObjectCount > 0 ? Parameters.MaxReplicationWriterObjectCount : Parameters.MaxInternalNetRefIndex;
	ReplicatedObjects.SetNumZeroed(MaxSupportedObjects);
	ReplicatedObjectsRecordInfoLists.SetNumZeroed(MaxSupportedObjects);
	SchedulingPriorities.SetNumZeroed(MaxSupportedObjects);
	
	SetNetObjectListsSize(Parameters.MaxInternalNetRefIndex);

	NetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().AddRaw(this, &FReplicationWriter::OnMaxInternalNetRefIndexIncreased);

	// Attachments init
	SetupReplicationInfoForAttachmentsToObjectsNotInScope();

	bReplicationEnabled = false;
}

void FReplicationWriter::Deinit()
{
	NetRefHandleManager->GetOnMaxInternalNetRefIndexIncreasedDelegate().RemoveAll(this);
}

void FReplicationWriter::SetNetObjectListsSize(FInternalNetRefIndex NewMaxInternalIndex)
{
	ObjectsPendingDestroy.SetNumBits(NewMaxInternalIndex);
	ObjectsWithDirtyChanges.SetNumBits(NewMaxInternalIndex);
	ObjectsInScope.SetNumBits(NewMaxInternalIndex);
	WriteContext.ObjectsWrittenThisPacket.SetNumBits(NewMaxInternalIndex);
}

void FReplicationWriter::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	// Only grow the objects if no limits were set
	if (Parameters.MaxReplicationWriterObjectCount == 0)
	{
		ReplicatedObjects.SetNumZeroed(NewMaxInternalIndex);
		ReplicatedObjectsRecordInfoLists.SetNumZeroed(NewMaxInternalIndex);
		SchedulingPriorities.SetNumZeroed(NewMaxInternalIndex);
	}

	SetNetObjectListsSize(NewMaxInternalIndex);
}

void FReplicationWriter::GetInitialChangeMask(ChangeMaskStorageType* ChangeMaskData, const FReplicationProtocol* Protocol)
{
	FNetBitArrayView ChangeMask(ChangeMaskData, Protocol->ChangeMaskBitCount, FNetBitArrayView::NoResetNoValidate);

	// Just fill with all dirty for now
	ChangeMask.SetAllBits();
}

void FReplicationWriter::StartReplication(uint32 InternalIndex)
{
	FReplicationInfo& Info = GetReplicationInfo(InternalIndex);

	if (!ensureMsgf(Info.GetState() == EReplicatedObjectState::Invalid, TEXT("Object ( InternalIndex: %u ) is in state %s in StartReplication."), InternalIndex, LexToString(Info.GetState())))
	{
		return;
	}

	if (InternalIndex == ObjectIndexForOOBAttachment || !NetRefHandleManager->GetAssignedInternalIndices().GetBit(InternalIndex))
	{
		UE_LOGF(LogIris, Error, "FReplicationWriter::StartReplication - Unexpected call to StartReplication for not assigned index ( InternalIndex: %u )", InternalIndex);
		ensure(false);
		return;
	}

	if (Attachments.HasUnsentAttachments(ENetObjectAttachmentType::Normal, InternalIndex))
	{
		UE_LOGF(LogIris, Error, "FReplicationWriter::StartReplication - Expected object %ls to not to have any queued up attachments", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex));
		ensure(false);
		Attachments.DropAllAttachments(ENetObjectAttachmentType::Normal, InternalIndex);
	}

	// Reset info
	Info = FReplicationInfo();
	Info.LastAckedBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
	Info.PendingBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;

	const bool bIsDestructionInfo = NetRefHandleManager->IsDestructionInfo(InternalIndex);	
	if (bIsDestructionInfo)
	{
		// Check status of original object about to be destroyed. Due to supporting sending reliable attachments for objects that have been created and destroyed in the same frame we may encounter a PendingCreate object which should be replicated.
		if (const uint32 OriginalInternalIndex = NetRefHandleManager->GetOriginalDestroyedStartupObjectIndex(InternalIndex))
		{
			const FReplicationInfo& OriginalInfo = GetReplicationInfo(OriginalInternalIndex);
			const EReplicatedObjectState OriginalObjectState = OriginalInfo.GetState();
			if ((OriginalObjectState != EReplicatedObjectState::Invalid) && 
				((OriginalInfo.IsCreationConfirmed && (OriginalObjectState != EReplicatedObjectState::WaitOnDestroyConfirmation)) || ((OriginalObjectState == EReplicatedObjectState::PendingCreate) && (OriginalInfo.FlushFlags != EFlushFlags::FlushFlags_None))))
			{
				// We do not need to send the destruction info so we mark it as PermanentlyDestroyed
				SetState(InternalIndex, EReplicatedObjectState::PermanentlyDestroyed);

				Info.IsDestructionInfo = 1U;
				Info.IsCreationConfirmed = 1U;

				NetRefHandleManager->AddNetObjectRef(InternalIndex);

				return;
			}
		}
	}

	const FNetRefHandleManager::FReplicatedObjectData& Data = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);	
	if (!Data.Protocol)
	{
		UE_LOGF(LogIris, Error, "FReplicationWriter::StartReplication - Unexpected call to StartReplication for object ( InternalIndex: %u ) with no ReplicationProtocol", InternalIndex);
		ensure(Data.Protocol);
		return;	
	}

	// Pending create
	SetState(InternalIndex, EReplicatedObjectState::PendingCreate);

	NetRefHandleManager->AddNetObjectRef(InternalIndex);

	Info.ChangeMaskBitCount = Data.Protocol->ChangeMaskBitCount;
	Info.HasDirtySubObjects = 1U;
	Info.IsSubObject = NetRefHandleManager->GetSubObjectInternalIndices().GetBit(InternalIndex);
	Info.HasDirtyChangeMask = 1U;
	Info.HasAttachments = 0U;
	Info.HasChangemaskFilter = EnumHasAnyFlags(Data.Protocol->ProtocolTraits, EReplicationProtocolTraits::HasConditionalChangeMask);
	Info.IsDestructionInfo = bIsDestructionInfo;
	Info.IsCreationConfirmed = 0U;
	Info.TearOff = Data.bTearOff;
	// If object is marked for flush we set the FlushFlags_FlushState in addition to the default ones.
	Info.FlushFlags = GetDefaultFlushFlags() | (Data.bFlush ? EFlushFlags::FlushFlags_FlushState : EFlushFlags::FlushFlags_None);
	Info.SubObjectPendingDestroy = 0U;
	Info.IsDeltaCompressionEnabled = BaselineManager->GetDeltaCompressionStatus(InternalIndex) == ENetObjectDeltaCompressionStatus::Allow ? 1U : 0U;
	Info.LastAckedBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
	Info.PendingBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
	Info.IsDebugObject = Data.bIsDebugObject;

	// Allocate storage for changemask (if needed)
	FChangeMaskStorageOrPointer::Alloc(Info.ChangeMaskOrPtr, Info.ChangeMaskBitCount, s_DefaultChangeMaskAllocator);

	// Get Initial ChangeMask
	GetInitialChangeMask(Info.GetChangeMaskStoragePointer(), Data.Protocol);

	// Reset record for object
	ReplicationRecord.ResetList(ReplicatedObjectsRecordInfoLists[InternalIndex]);

	// Set initial priority
	// Subobject are always set to have zero priority as they are replicated with owner
	// Currently we also do this for dependent objects to support objects with zero priority that should only replicate with parents
	SchedulingPriorities[InternalIndex] = (Data.IsDependentObject() || Info.IsSubObject) ? 0.f : CreatePriority;

	UE_LOG_REPWRITER(Verbose, TEXT("ReplicationWriter.StartReplication for ( InternalIndex: %u ) %s"), InternalIndex, ToCStr(Data.RefHandle.ToString()));
	UE_CLOGF_IRISCREATIONFLOW(!Info.IsSubObject, !Info.IsDestructionInfo && UE::Net::CreationFlowLog::ShouldEmitForConnection(NetRefHandleManager->GetReplicatedObjectInstance(InternalIndex), ReplicationFiltering->GetOwningConnection(InternalIndex), Parameters.ConnectionId), Verbose, "[PendingCreate] ObjectInfo=%ls Conn=%d", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), Parameters.ConnectionId);

	ObjectsWithDirtyChanges.SetBit(InternalIndex);

	// Subobject needs to mark its owner as dirty as the subobject could have been filtered out and now allowed to replicate again.
	if (Info.IsSubObject)
	{
		const uint32 RootObjectInternalIndex = NetRefHandleManager->GetRootObjectInternalIndexOfSubObject(InternalIndex);
		if (ensure(RootObjectInternalIndex != InvalidInternalNetRefIndex))
		{
			FReplicationInfo& OwnerInfo = ReplicatedObjects[RootObjectInternalIndex];
			if (ensureMsgf(OwnerInfo.GetState() != EReplicatedObjectState::Invalid && OwnerInfo.GetState() < EReplicatedObjectState::PendingDestroy, 
							TEXT("Unsupported state %s on root object of: %s"), LexToString(OwnerInfo.GetState()), *NetRefHandleManager->PrintObjectFromIndex(InternalIndex)))
			{
				ObjectsWithDirtyChanges.SetBit(RootObjectInternalIndex);
				OwnerInfo.HasDirtySubObjects = 1U;
			}
		}
	}
}

void FReplicationWriter::StopReplication(uint32 InternalIndex)
{
	FReplicationInfo& Info = GetReplicationInfo(InternalIndex);

	// Invalidate state
	SetState(InternalIndex, EReplicatedObjectState::Invalid);

	// Need to free allocated ChangeMask (if it is allocated)
	FChangeMaskStorageOrPointer::Free(Info.ChangeMaskOrPtr, Info.ChangeMaskBitCount, s_DefaultChangeMaskAllocator);
	
	Info.IsCreationConfirmed = 0U;

	// Remove from objects with dirty changes
	ObjectsWithDirtyChanges.ClearBit(InternalIndex);

	// Remove from pending destroy
	ObjectsPendingDestroy.ClearBit(InternalIndex);

	// Explicitly remove from objects in scope since we might call StopReplication from outside ScopeUpdate
	ObjectsInScope.ClearBit(InternalIndex);

	UE_LOG_REPWRITER(Verbose, TEXT("ReplicationWriter.StopReplication for ( InternalIndex: %u )"), InternalIndex);
	NetRefHandleManager->ReleaseNetObjectRef(InternalIndex);

	Attachments.DropAllAttachments(ENetObjectAttachmentType::Normal, InternalIndex);

	// Release baselines
    const bool bDestroyPendingBaseline = Info.PendingBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex;
    const bool bDestroyLastAckedBaseline = Info.PendingBaselineIndex != Info.LastAckedBaselineIndex && Info.LastAckedBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex;
    if (bDestroyPendingBaseline || bDestroyLastAckedBaseline)
    {
        UE_IRIS_PARALLEL_EXPR(FScopedBaselineAccess BaselineRW(BaselineManager, true));
        if (bDestroyPendingBaseline)
        {
            BaselineManager->DestroyBaseline(Parameters.ConnectionId, InternalIndex, Info.PendingBaselineIndex);
        }
        if (bDestroyLastAckedBaseline)
        {
            BaselineManager->DestroyBaseline(Parameters.ConnectionId, InternalIndex, Info.LastAckedBaselineIndex);
        }
    }

	Info.PendingBaselineIndex = Info.LastAckedBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;

#if UE_NET_ENABLE_REPLICATIONWRITER_CANNOT_SEND_WARNING
	if (Info.HasCannotSendInfo)
	{
		CannotSendInfos.Remove(InternalIndex);
		Info.HasCannotSendInfo = 0U;
	}
#endif
}

FReplicationWriter::FReplicationInfo& FReplicationWriter::GetReplicationInfo(uint32 InternalIndex)
{
	return ReplicatedObjects[InternalIndex];
}

const FReplicationWriter::FReplicationInfo& FReplicationWriter::GetReplicationInfo(uint32 InternalIndex) const
{
	return ReplicatedObjects[InternalIndex];
}

void FReplicationWriter::WriteNetRefHandleId(FNetSerializationContext& Context, FNetRefHandle Handle)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	UE_NET_TRACE_OBJECT_SCOPE(Handle, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
	WritePackedUint64(Writer, Handle.GetId());
}

uint32 FReplicationWriter::GetDefaultFlushFlags() const
{
	// By default we currently always flush if we have pending reliable attachments when EndReplication is called for a NetObject.
	return EFlushFlags::FlushFlags_FlushReliable;
}

uint32 FReplicationWriter::GetFlushStatus(uint32 InternalIndex, const FReplicationInfo& Info, uint32 FlushFlagsToTest) const
{
	uint32 FlushFlags = EFlushFlags::FlushFlags_None;

	if (FlushFlagsToTest == EFlushFlags::FlushFlags_None)
	{
		return EFlushFlags::FlushFlags_None;
	}

	if (Info.IsSubObject)
	{
		// If owner is PendingDestroy it means that this subobject has been filtered out and should not trigger a flush.
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);
		const FReplicationInfo& OwnerInfo = ReplicatedObjects[ObjectData.SubObjectRootIndex];
		if (OwnerInfo.GetState() == EReplicatedObjectState::PendingDestroy)
		{
			return EFlushFlags::FlushFlags_None;
		}
	}

	if (!!(FlushFlagsToTest & EFlushFlags::FlushFlags_FlushState) && (Info.HasDirtyChangeMask || HasInFlightStateChanges(InternalIndex, Info) || IsObjectPartOfActiveHugeObject(InternalIndex)))
	{
		FlushFlags |= EFlushFlags::FlushFlags_FlushState;
	}

	if (!!(FlushFlagsToTest & EFlushFlags::FlushFlags_FlushReliable) && !Attachments.IsAllReliableSentAndAcked(ENetObjectAttachmentType::Normal, InternalIndex))
	{
		FlushFlags |= EFlushFlags::FlushFlags_FlushReliable;
	}

	// Do we have a tear-off for the subobject in-flight?
	if (!!(FlushFlagsToTest & EFlushFlags::FlushFlags_FlushTornOffSubObjects) && (Info.IsSubObject && Info.TearOff && (Info.GetState() == EReplicatedObjectState::WaitOnDestroyConfirmation)))
	{
		FlushFlags |= EFlushFlags::FlushFlags_FlushTornOffSubObjects;
	}

	if (!Info.IsSubObject && FlushFlags != FlushFlagsToTest)
	{
		// Check status of SubObjects that are not filtered out as well.
		FReplicationConditionals::FSubObjectsToReplicateArray SubObjectsToReplicate;
		ReplicationConditionals->GetSubObjectsToReplicate(Parameters.ConnectionId, InternalIndex, SubObjectsToReplicate);

		for (uint32 SubObjectIndex : SubObjectsToReplicate)
		{
			const FReplicationInfo& SubObjectInfo = GetReplicationInfo(SubObjectIndex);
			FlushFlags |= GetFlushStatus(SubObjectIndex, SubObjectInfo, FlushFlagsToTest);

			if (FlushFlags == FlushFlagsToTest)
			{
				break;
			}
		}
	}

	return FlushFlags;
}

void FReplicationWriter::SetPendingDestroyOrSubObjectPendingDestroyState(uint32 InternalIndex, FReplicationInfo& Info)
{
	if (Info.IsSubObject)
	{
		// Subobject destroyed before its owner is explicitly replicated as state data.
		FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);
		// If owner is not pending destroy we mark the state of the SubObject to SubObjectPendingDestroy and mark owner as having dirty subobjects which will 
		// destroy the subobject using the replicated state path of the owner.
		FReplicationInfo& OwnerInfo = ReplicatedObjects[ObjectData.SubObjectRootIndex];
		if ((OwnerInfo.GetState() != EReplicatedObjectState::Invalid) && !ObjectsPendingDestroy.GetBit(ObjectData.SubObjectRootIndex))
		{
			MarkObjectDirty(ObjectData.SubObjectRootIndex, "SetPendingDestroyOrSubObjectPendingDestroyState");
			OwnerInfo.HasDirtySubObjects = 1U;

			SetState(InternalIndex, EReplicatedObjectState::SubObjectPendingDestroy);
			MarkObjectDirty(InternalIndex, "SetPendingDestroyOrSubObjectPendingDestroyState2");
			ObjectsPendingDestroy.SetBit(InternalIndex);
			Info.SubObjectPendingDestroy = 1U;
			
			// For clarity clear HasDirtyChangeMask as we do not intend to replicate state. The object itself needs to remain dirty in ObjectsWithDirtyChanges though.
			Info.HasDirtyChangeMask = 0U;
			// One cannot cancel a subobject destroy so we should not send any attachments for this subobject going forward
			Info.HasAttachments = 0U;

			ensure(!Info.TearOff || (GetFlushStatus(InternalIndex, Info, Info.FlushFlags) == FlushFlags_None));
			return;
		}
	}
	else if (Info.HasDirtySubObjects)
	{
		// If the owner is destroyed, all subobjects in the EReplicatedObjectState::SubObjectPendingDestroy state must also be marked as PendingDestroy as owner no longer will be replicated
		for (uint32 SubObjectIndex : NetRefHandleManager->GetSubObjects(InternalIndex))
		{
			FReplicationInfo& SubObjectInfo = GetReplicationInfo(SubObjectIndex);
			if (SubObjectInfo.GetState() == EReplicatedObjectState::SubObjectPendingDestroy)
			{
				SetState(SubObjectIndex, EReplicatedObjectState::PendingDestroy);
				SubObjectInfo.SubObjectPendingDestroy = 0U;
				ObjectsWithDirtyChanges.ClearBit(SubObjectIndex);
			}
		}
	}
				
	ObjectsPendingDestroy.SetBit(InternalIndex);
	ObjectsWithDirtyChanges.ClearBit(InternalIndex);
	SetState(InternalIndex, EReplicatedObjectState::PendingDestroy);				
	Info.HasDirtyChangeMask = 0U;
}

void FReplicationWriter::UpdateScope(const FNetBitArrayView& UpdatedScope)
{
	//IRIS_PROFILER_SCOPE(FReplicationWriter_ScopeUpdate);

	auto NewObjectFunctor = [this](uint32 Index)
	{
		// We can only start replicating an object that is not currently replicated
		FReplicationInfo& Info = GetReplicationInfo(Index);
		const EReplicatedObjectState State = Info.GetState();

		if (State == EReplicatedObjectState::Invalid)
		{
			StartReplication(Index);
		}
		else if (State == EReplicatedObjectState::WaitOnFlush)
		{			
			// If we are waiting on flush but are re-added to scope we reset flush flags to default.
			ObjectsPendingDestroy.ClearBit(Index);
			Info.FlushFlags = GetDefaultFlushFlags();
			SetState(Index, EReplicatedObjectState::Created);

			// If we have accumulated changes while WaitingOnFlush, we should send them now
			Info.SubObjectPendingDestroy = 0U;
			Info.HasDirtyChangeMask |= FNetBitArrayView(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount).IsAnyBitSet();
			ObjectsWithDirtyChanges.SetBitValue(Index, Info.HasDirtyChangeMask);
		}
		else if (State == EReplicatedObjectState::WaitOnCreateConfirmation)
		{
			// Need to restore as we might have been in case where we was pending destroy
			ObjectsPendingDestroy.ClearBit(Index);
			Info.FlushFlags = GetDefaultFlushFlags();

			// If we have accumulated changes while waiting on flush, we should send them now
			Info.SubObjectPendingDestroy = 0U;
			Info.HasDirtyChangeMask |= FNetBitArrayView(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount).IsAnyBitSet();
			ObjectsWithDirtyChanges.SetBitValue(Index, Info.HasDirtyChangeMask);
		}
		else if (State == EReplicatedObjectState::WaitOnDestroyConfirmation || State == EReplicatedObjectState::CancelPendingDestroy)
		{
			// Need to clear the pending destroy bit or else the object will be masked out of ObjectsInScope.
			// Keep the SubObjectPendingDestroy status as is until we know if the destroy packet was received or not.
			ObjectsPendingDestroy.ClearBit(Index);
			SetState(Index, EReplicatedObjectState::CancelPendingDestroy);
		}
		else if (State == EReplicatedObjectState::SubObjectPendingDestroy || State == EReplicatedObjectState::PendingDestroy)
		{
			// Object was waiting to be destroyed but should now resume replication.
			// If the object has been created we can go back to Created state, otherwise we go back to WaitOnCreateConfirmation
			SetState(Index, EReplicatedObjectState::WaitOnDestroyConfirmation);
			SetState(Index, EReplicatedObjectState::CancelPendingDestroy);
			SetState(Index, (Info.IsCreationConfirmed ? EReplicatedObjectState::Created : EReplicatedObjectState::WaitOnCreateConfirmation));

			Info.SubObjectPendingDestroy = 0U;
			Info.HasDirtyChangeMask |= FNetBitArrayView(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount).IsAnyBitSet();
			ObjectsWithDirtyChanges.SetBitValue(Index, Info.HasDirtyChangeMask);
			ObjectsPendingDestroy.ClearBit(Index);

			if (State == EReplicatedObjectState::SubObjectPendingDestroy)
			{
				// If owner is not pending destroy we mark it as dirty as appropriate
				FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(Index);
				FReplicationInfo& OwnerInfo = ReplicatedObjects[ObjectData.SubObjectRootIndex];
				if (OwnerInfo.GetState() < EReplicatedObjectState::PendingDestroy)
				{
					ensureMsgf(!bValidateObjectsWithDirtyChanges || OwnerInfo.GetState() != EReplicatedObjectState::Invalid, TEXT("Object ( InternalIndex: %u ) with Invalid state potentially marked dirty."), ObjectData.SubObjectRootIndex);
					ensureMsgf(!OwnerInfo.TearOff, TEXT("Parent is tearing off ( InternalIndex: %u ) currently in State: %s "), ObjectData.SubObjectRootIndex, LexToString(OwnerInfo.GetState()));
					OwnerInfo.HasDirtySubObjects |= Info.HasDirtyChangeMask;
					ObjectsWithDirtyChanges.SetBitValue(ObjectData.SubObjectRootIndex, ObjectsWithDirtyChanges.GetBit(ObjectData.SubObjectRootIndex) || Info.HasDirtyChangeMask);
				}
			}
			else if (!Info.IsSubObject)
			{
				// If there are subobjects pending destroy we should make sure they're once again resorting to getting destroyed via state replication.
				bool bHasSubObjectsPendingDestroy = false;
				for (uint32 SubObjectIndex : NetRefHandleManager->GetSubObjects(Index))
				{
					FReplicationInfo& SubObjectInfo = GetReplicationInfo(SubObjectIndex);
					if (SubObjectInfo.GetState() == EReplicatedObjectState::PendingDestroy)
					{
						SetState(SubObjectIndex, EReplicatedObjectState::SubObjectPendingDestroy);
						SubObjectInfo.SubObjectPendingDestroy = 1U;

						ObjectsWithDirtyChanges.SetBit(SubObjectIndex);

						bHasSubObjectsPendingDestroy = true;
					}
				}

				if (bHasSubObjectsPendingDestroy)
				{
					ObjectsWithDirtyChanges.SetBit(Index);
					Info.HasDirtySubObjects = 1U;
				}
			}
		}
		else
		{
			UE_LOG_REPWRITER(Verbose, TEXT("New object added to scope, Waiting to start replication for ( InternalIndex: %u ) currently in State: %s "), Index, LexToString(State));

			ensureMsgf(!ObjectsWithDirtyChanges.GetBit(Index) , TEXT("New object added to scope, Waiting to start replication for ( InternalIndex: %u ) currently in State: %s "), Index, LexToString(State));
			ensureMsgf(!Info.HasDirtyChangeMask, TEXT("New object added to scope, Waiting to start replication for ( InternalIndex: %u ) currently in State: %s "), Index, LexToString(State));
		}
	};

	auto DestroyedObjectFunctor = [this](uint32 Index) 
	{
		// Request object to be destroyed
		FReplicationInfo& Info = GetReplicationInfo(Index);

		// We handle objects marked for tear-off using the state update path.
		if (Info.TearOff)
		{
			return;
		}

		const EReplicatedObjectState State = Info.GetState();
		if (State < EReplicatedObjectState::PendingDestroy)
		{
			if (State == EReplicatedObjectState::PendingCreate)
			{
				// If we have no data to flush, we can stop replication now.
				const uint32 FlushFlags = GetFlushStatus(Index, Info, Info.FlushFlags);
				if (FlushFlags == FlushFlags_None || NetRefHandleManager->GetReplicatedObjectDataNoCheck(Index).bHasCachedCreationInfo == 0U)
				{
					StopReplication(Index);
				}
				else
				{
					// Mark for destroy.
					ObjectsPendingDestroy.SetBit(Index);
				}
			}
			else if (State == EReplicatedObjectState::CancelPendingDestroy)
			{
				// If we wanted to cancel the pending destroy but now want to destroy the object again we can resume waiting for the destroy.
				ObjectsPendingDestroy.SetBit(Index);
				SetState(Index, EReplicatedObjectState::WaitOnDestroyConfirmation);
			}
			else if (const uint32 FlushFlags = GetFlushStatus(Index, Info, Info.FlushFlags))
			{
				// Store info about what we need to flush
				Info.FlushFlags = FlushFlags;

				if (State != EReplicatedObjectState::WaitOnCreateConfirmation)
				{
					SetState(Index, EReplicatedObjectState::WaitOnFlush);

					// If we do not have any state data to flush we can clear the has dirty states flag
					if ((FlushFlags & FlushFlags_FlushState) == 0U)
					{
						Info.HasDirtyChangeMask = 0U;
					}
				}

				// Mark object as pending destroy so that we can poll the flush status in WriteObjectPendingDestroy
				ObjectsPendingDestroy.SetBit(Index);
			}
			else
			{
				SetPendingDestroyOrSubObjectPendingDestroyState(Index, Info);
			}
		}
		else if (State == EReplicatedObjectState::PermanentlyDestroyed)
		{
			StopReplication(Index);
		}
	};

	FNetBitArrayView CurrentScope = MakeNetBitArrayView(ObjectsInScope);
	const FNetBitArrayView SubObjects = NetRefHandleManager->GetSubObjectInternalIndicesView();

	// Process root objects first
	FNetBitArrayView::ForAllExclusiveBitsByPredicate(UpdatedScope, CurrentScope, NewObjectFunctor, DestroyedObjectFunctor, [SubObjects](uint32 InternalIndex)
		{
			return !SubObjects.GetBit(InternalIndex);
		}
	);

	// Process subobjects second
	FNetBitArrayView::ForAllExclusiveBitsByPredicate(UpdatedScope, CurrentScope, NewObjectFunctor, DestroyedObjectFunctor, [SubObjects](uint32 InternalIndex)
		{
			return SubObjects.GetBit(InternalIndex);
		}
	);

	CurrentScope.Copy(UpdatedScope);

	// No objects marked for destroy can be in scope
	ObjectsInScope.Combine(ObjectsPendingDestroy, FNetBitArrayBase::AndNotOp);
}

void FReplicationWriter::UpdateDirtyGlobalLifetimeConditionals(TArrayView<FInternalNetRefIndex> ObjectsWithDirtyConditionals)
{
	ensureMsgf(!Parameters.ReplicationSystem->GetReplicationSystemInternal()->IsInParallelPhase(), TEXT("UpdateDirtyGlobalLifetimeConditionals  should never be called from within a parallel phase"));

	for (FInternalNetRefIndex InternalObjectIndex : ObjectsWithDirtyConditionals)
	{
		FReplicationInfo& Info = ReplicatedObjects[InternalObjectIndex];
		if (Info.GetState() != EReplicatedObjectState::Invalid && Info.GetState() < EReplicatedObjectState::PendingDestroy)
		{
			if (Info.IsSubObject)
			{
				const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalObjectIndex);

				const FInternalNetRefIndex RootInternalObjectIndex = ObjectData.SubObjectRootIndex;
				FReplicationInfo& OwnerInfo = ReplicatedObjects[RootInternalObjectIndex];

				if ((OwnerInfo.GetState() != EReplicatedObjectState::Invalid && OwnerInfo.GetState() < EReplicatedObjectState::PendingDestroy))
				{
					UE_LOG_REPWRITER(Verbose, TEXT("UpdateDirtyGlobalLifetimeConditionals for - %s"), ToCStr(NetRefHandleManager->PrintObjectFromIndex(RootInternalObjectIndex)));
						
					// Better safe than sorry, we do not want to dirty something going out of scope.
					if (ObjectsInScope.GetBit(RootInternalObjectIndex))
					{
						MarkObjectDirty(RootInternalObjectIndex, "UpdateDirtyGlobalLifetimeConditionals");
					}
					OwnerInfo.HasDirtyConditionals = 1U;
					OwnerInfo.HasDirtySubObjects = 1U;
				}
			}
			else
			{
				UE_LOG_REPWRITER(Verbose, TEXT("UpdateDirtyGlobalLifetimeConditionals for - %s"), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalObjectIndex)));
	
				// Better safe than sorry, we do not want to dirty something going out of scope.
				if (ObjectsInScope.GetBit(InternalObjectIndex))
				{
					MarkObjectDirty(InternalObjectIndex, "UpdateDirtyGlobalLifetimeConditionals2");
				}
				Info.HasDirtyConditionals = 1U;
				Info.HasDirtySubObjects = 1U;
			}
		}
	}
}

void FReplicationWriter::InternalUpdateDirtyChangeMasks(const FChangeMaskCache& CachedChangeMasks, EFlushFlags ExtraFlushFlags, bool bMarkForTearOff)
{
	//IRIS_PROFILER_SCOPE(FReplicationWriter_UpdateDirtyChangeMasks);

	const uint32 MarkForTearOff = bMarkForTearOff ? 1U : 0U;
	const ChangeMaskStorageType* StoragePtr = CachedChangeMasks.Storage.GetData();

	for (const FChangeMaskCache::FCachedInfo& Entry : CachedChangeMasks.Indices)
	{
		FReplicationInfo& Info = ReplicatedObjects[Entry.InternalIndex];
		if (Info.GetState() == EReplicatedObjectState::Invalid)
		{
			continue;
		}

		// We want to accumulate dirty changes even if we are going out of scope in case we get re-added to scope before replication has ended.
		const bool bMarkScopedObjectDirty = ObjectsInScope.GetBit(Entry.InternalIndex);
		if (bMarkScopedObjectDirty)
		{
			MarkObjectDirty(Entry.InternalIndex, "UpdateDirtyChangeMasks");
		}

		if (Entry.bMarkSubObjectOwnerDirty == 0U)
		{
			ensure((!MarkForTearOff && !Info.TearOff) || !Info.SubObjectPendingDestroy);
			// Mark object for TearOff, that is that we will stop replication as soon as the tear-off is acknowledged
			if (MarkForTearOff)
			{
				const EReplicatedObjectState CurrentState = Info.GetState();
				// If an object was recently removed from scope for a connection then many objects can be in PendingDestroy. We need to move to WaitOnFlush to progress with tear off.
				if (CurrentState == EReplicatedObjectState::PendingDestroy)
				{
					SetState(Entry.InternalIndex, EReplicatedObjectState::WaitOnFlush);
					Info.FlushFlags = GetFlushStatus(Entry.InternalIndex, Info, static_cast<EFlushFlags>(Info.FlushFlags | ExtraFlushFlags));
				}
			}

			Info.TearOff |= MarkForTearOff;

			// Update flush flags
			Info.FlushFlags = Info.FlushFlags | ExtraFlushFlags;

			// Merge in dirty changes
			if (Entry.bHasDirtyChangeMask)
			{
				const uint32 ChangeMaskBitCount = Info.ChangeMaskBitCount;

				// Merge updated changes
				FNetBitArrayView Changes(Info.GetChangeMaskStoragePointer(), ChangeMaskBitCount);

				const FNetBitArrayView UpdatedChanges = MakeNetBitArrayView(StoragePtr + Entry.StorageOffset, ChangeMaskBitCount);
				Changes.Combine(UpdatedChanges, FNetBitArrayView::OrOp);

				// Mark changemask as dirty
				Info.HasDirtyChangeMask = bMarkScopedObjectDirty ? 1U : 0U;
			}

			if (bMarkForTearOff && !bMarkScopedObjectDirty)
			{
				if (Info.GetState() <= EReplicatedObjectState::PendingTearOff)
				{
					MarkObjectDirty(Entry.InternalIndex, "TearOffObjectNotInScope");
					Info.HasDirtyChangeMask = FNetBitArrayView(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount).IsAnyBitSet();

					// If we're a subobject and forced dirtied ourself we want the root object to be dirty as well
					if (Info.IsSubObject)
					{
						const FInternalNetRefIndex OwnerIndex = NetRefHandleManager->GetRootObjectInternalIndexOfSubObject(Entry.InternalIndex);
						FReplicationInfo& OwnerInfo = ReplicatedObjects[OwnerIndex];
						if (OwnerInfo.GetState() != EReplicatedObjectState::Invalid && OwnerInfo.GetState() < EReplicatedObjectState::PendingDestroy)
						{
							MarkObjectDirty(OwnerIndex, "TearOffObjectNotInScopeRoot");
							OwnerInfo.HasDirtySubObjects = 1U;
						}
					}
				}
			}
		}
		else
		{
			Info.HasDirtySubObjects = 1U;
		}
	}

	//UE_LOG_REPWRITER(VeryVerbose, TEXT("FReplicationWriter::UpdateDirtyChangeMasks() Updated %u Objects for ConnectionId:%u, ReplicationSystemId: %u."), CachedChangeMasks.Indices.Num(), Parameters.ConnectionId, Parameters.ReplicationSystem->GetId());	
}

const FNetBitArray& FReplicationWriter::GetObjectsRequiringPriorityUpdate() const
{
	return ObjectsWithDirtyChanges;
}

void FReplicationWriter::UpdatePriorities(const float* UpdatedPriorities)
{
	IRIS_PROFILER_SCOPE(FReplicationWriter_UpdatePriorities);

	auto UpdatePriority = [&LocalPriorities = SchedulingPriorities, UpdatedPriorities](uint32 Index)
	{
		LocalPriorities[Index] += UpdatedPriorities[Index];
	};

	ObjectsWithDirtyChanges.ForAllSetBits(UpdatePriority);
}

void FReplicationWriter::ScheduleDependentObjects(uint32 Index, float ParentPriority, TArray<float>& LocalPriorities, FScheduledObjectInfosArray& ScheduledObjectIndices)
{
	const float DependentObjectPriorityBump = UE_KINDA_SMALL_NUMBER;

	for (const FDependentObjectInfo& DependentObjectInfo : NetRefHandleManager->GetDependentObjectInfos(Index))
	{
		const FInternalNetRefIndex DependentInternalIndex = DependentObjectInfo.NetRefIndex;
		float UpdatedPriority = ParentPriority;

		if (ObjectsWithDirtyChanges.GetBit(DependentInternalIndex))
		{
			const FReplicationInfo& DependentInfo = this->GetReplicationInfo(DependentInternalIndex);

			const bool bReplicateBeforeParent =	(DependentObjectInfo.SchedulingHint == EDependentObjectSchedulingHint::ScheduleBeforeParent) || ((DependentObjectInfo.SchedulingHint == EDependentObjectSchedulingHint::ScheduleBeforeParentIfInitialState) && IsInitialState(DependentInfo.GetState()));

			if (bReplicateBeforeParent)
			{
				// Bump prio of dependent object to be scheduled before its parent.
				UpdatedPriority = FMath::Max(std::nextafter(ParentPriority, std::numeric_limits<float>::infinity()), LocalPriorities[DependentInternalIndex]);
				LocalPriorities[DependentInternalIndex] = UpdatedPriority;

				// Schedule it, it does not matter if we add it to the scheduled list multiple times
				ScheduledObjectIndices.Add({.Index = DependentInternalIndex, .SortKey = UpdatedPriority});
			}
		}
		
		// We go through all dependent objects here even though it might not be 100% correct, but it will make sure that we respect
		// the scheduling order hint at least in relation to the parent, but a dependent object might also end up replicating before its parent`s parent
		if (NetRefHandleManager->GetObjectsWithDependentObjectsInternalIndices().GetBit(DependentInternalIndex))
		{
			ScheduleDependentObjects(DependentInternalIndex, UpdatedPriority, LocalPriorities, ScheduledObjectIndices);
		}
	}
}

uint32 FReplicationWriter::ScheduleObjects(FScheduledObjectInfosArray& OutScheduledObjectIndices)
{
	IRIS_PROFILER_SCOPE(FReplicationWriter_ScheduleObjects);

	FScheduledObjectInfosArray& ScheduledObjectIndices = OutScheduledObjectIndices;

	// Special index is handled later.
	ObjectsWithDirtyChanges.ClearBit(ObjectIndexForOOBAttachment);

	const FNetBitArray& UpdatedObjects = ObjectsWithDirtyChanges;
	const FNetBitArray& SubObjects = NetRefHandleManager->GetSubObjectInternalIndices();

	auto FillIndexListFunc = [&ScheduledObjectIndices, this](uint32 Index)
	{
		const float UpdatedPriority = SchedulingPriorities[Index];

		FScheduleObjectInfo ScheduledObjectInfo = {.Index = Index,.SortKey = UpdatedPriority };
		ScheduledObjectInfo.Index = Index;
		ScheduledObjectInfo.SortKey = UpdatedPriority;

		if (UpdatedPriority >= FReplicationWriter::SchedulingThresholdPriority)
		{
			ScheduledObjectIndices.Add({.Index = Index,.SortKey = UpdatedPriority });

			// If we have dependent objects that needs to replicate before parent we need to schedule them as well.
			if (NetRefHandleManager->GetObjectsWithDependentObjectsInternalIndices().GetBit(Index))
			{
				ScheduleDependentObjects(Index, UpdatedPriority, SchedulingPriorities, ScheduledObjectIndices);
			}
		}
	};

	// Invoke functor for all updated objects that not are sub objects.
	FNetBitArray::ForAllSetBits(UpdatedObjects, SubObjects, FNetBitArray::AndNotOp, FillIndexListFunc);

	// we now have our list of objects to write.
	return ScheduledObjectIndices.Num();
}

uint32 FReplicationWriter::SortScheduledObjects(FScheduleObjectInfo* ScheduledObjectIndices, uint32 ScheduledObjectCount, uint32 StartIndex)
{
	check(ScheduledObjectCount > 0 && StartIndex <= ScheduledObjectCount);

	// Partial sort of scheduled objects
	{
		IRIS_PROFILER_SCOPE(FReplicationWriter_SortScheduledObjects);

		// We only need a partial sort of the highest priority objects as we wont be able to fit that much data in a packet anyway
		// $IRIS TODO: Implement and evaluate partial sort algorithm, currently we simply use std::partial_sort https://jira.it.epicgames.com/browse/UE-123444
		FScheduleObjectInfo* StartIt = ScheduledObjectIndices + StartIndex;
		FScheduleObjectInfo* EndIt = ScheduledObjectIndices + ScheduledObjectCount;
		FScheduleObjectInfo* SortIt = FMath::Min(StartIt + PartialSortObjectCount, EndIt);

		std::partial_sort(StartIt, SortIt, EndIt, [](const FScheduleObjectInfo& EntryA, const FScheduleObjectInfo& EntryB) { return EntryA.SortKey > EntryB.SortKey; });
	}

	return FMath::Min(ScheduledObjectCount - StartIndex, PartialSortObjectCount);
}

void FReplicationWriter::HandleDeliveredRecord(const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
{
	EReplicatedObjectState DeliveredState = (EReplicatedObjectState)RecordInfo.ReplicatedObjectState;
	EReplicatedObjectState CurrentState = Info.GetState();
	const uint32 InternalIndex = RecordInfo.Index;
	
	if (CurrentState == EReplicatedObjectState::Invalid)
	{
		UE_LOG_REPWRITER_WARNING(TEXT("FReplicationWriter::HandleDeliveredRecord - Warning Object ( InternalIndex: %u ) is invalid. DeliveredState %s WasDestroySubObject: %u"), InternalIndex, LexToString(DeliveredState), RecordInfo.WroteDestroySubObject)
		ensure(false);
		return;
	}
	
	// We confirmed a new baseline
	if (RecordInfo.NewBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
	{
		check(RecordInfo.NewBaselineIndex == Info.PendingBaselineIndex);

		// Destroy old baseline
		if (Info.LastAckedBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
		{			
			BaselineManager->DestroyBaseline(Parameters.ConnectionId, InternalIndex, Info.LastAckedBaselineIndex);
			UE_LOG_REPWRITER(VeryVerbose, TEXT("Destroyed old baseline %llu for ( InternalIndex: %u )"), Info.LastAckedBaselineIndex, InternalIndex);			
		}
		Info.LastAckedBaselineIndex = RecordInfo.NewBaselineIndex;
		Info.PendingBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;

		UE_LOG_REPWRITER(VeryVerbose, TEXT("Acknowledged baseline %u for ( InternalIndex: %u )"), RecordInfo.NewBaselineIndex, InternalIndex);
	}

	// Update state
	switch (DeliveredState)
	{
		case EReplicatedObjectState::WaitOnCreateConfirmation:
		{
			// if we are still waiting for CreateConfirmation
			if (CurrentState == EReplicatedObjectState::WaitOnCreateConfirmation)
			{
				// If this is a destruction info, just put it in the destroyed state
				if (Info.IsDestructionInfo)
				{
					SetState(InternalIndex, EReplicatedObjectState::PermanentlyDestroyed);
				}
				// If this object was teared off, it can now be considered as destroyed
				else if (RecordInfo.WroteTearOff)
				{
					// Must also mark owner dirty as it might have been waiting for a subobject flush
					if (Info.IsSubObject)
					{
						FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);

						FReplicationInfo& OwnerInfo = ReplicatedObjects[ObjectData.SubObjectRootIndex];
						if (OwnerInfo.GetState() != EReplicatedObjectState::Invalid && OwnerInfo.GetState() < EReplicatedObjectState::PendingDestroy)
						{
							MarkObjectDirty(ObjectData.SubObjectRootIndex, "HandleDeliveredRecordSubObjectTearOff");
							OwnerInfo.HasDirtySubObjects = 1U;
						}
					}

					SetState(InternalIndex, EReplicatedObjectState::PendingTearOff);
					SetState(InternalIndex, EReplicatedObjectState::Destroyed);
					StopReplication(InternalIndex);

					// Cleanup any filtered out subobjects that are still in PendingCreate
					for (uint32 SubObjectIndex : NetRefHandleManager->GetSubObjects(InternalIndex))
					{
						const FReplicationInfo& SubObjectInfo = GetReplicationInfo(SubObjectIndex);
						if (SubObjectInfo.GetState() == EReplicatedObjectState::PendingCreate && SubObjectInfo.IsFilteredOutSubObjectPendingCreate)
						{
							StopReplication(SubObjectIndex);
						}
					}
				}
				else
				{
					SetState(InternalIndex, EReplicatedObjectState::Created);
					UE_CLOGF_IRISCREATIONFLOW(!Info.IsSubObject, !Info.IsDestructionInfo && UE::Net::CreationFlowLog::ShouldEmitForConnection(NetRefHandleManager->GetReplicatedObjectInstance(InternalIndex), ReplicationFiltering->GetOwningConnection(InternalIndex), Parameters.ConnectionId), Verbose, "[CreationDelivered] ObjectInfo=%ls Conn=%d", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), Parameters.ConnectionId);

					// Tear-off is marked as a flush
					if (Info.TearOff)
					{
						UE_LOG_REPWRITER(VeryVerbose, TEXT("ReplicationWriter::HandleDeliveredRecord for ( InternalIndex: %u ) Waiting for flush before tearoff"), InternalIndex);
						SetState(InternalIndex, EReplicatedObjectState::WaitOnFlush);
					}
					// so are objects marked for destroy requiring flush
					else if (ObjectsPendingDestroy.GetBit(InternalIndex))
					{
						UE_LOG_REPWRITER(VeryVerbose, TEXT("ReplicationWriter::HandleDeliveredRecord for ( InternalIndex: %u ) Waiting for flush before destroy"), InternalIndex);
						SetState(InternalIndex, EReplicatedObjectState::WaitOnFlush);
					}
				}
			}
			Info.IsCreationConfirmed = 1U;
		}
		break;

		case EReplicatedObjectState::WaitOnDestroyConfirmation:
		{
			SetState(InternalIndex, EReplicatedObjectState::Destroyed);

			// It is now safe to stop tracking this object
			StopReplication(InternalIndex);

			if (RecordInfo.WroteTearOff)
			{
				// Cleanup any filtered out subobjects that are still in PendingCreate
				for (uint32 SubObjectIndex : NetRefHandleManager->GetSubObjects(InternalIndex))
				{
					const FReplicationInfo& SubObjectInfo = GetReplicationInfo(SubObjectIndex);
					if (SubObjectInfo.GetState() == EReplicatedObjectState::PendingCreate && SubObjectInfo.IsFilteredOutSubObjectPendingCreate)
					{
						StopReplication(SubObjectIndex);
					}
				}
			}

			if (CurrentState == EReplicatedObjectState::CancelPendingDestroy)
			{
				StartReplication(InternalIndex);
				ObjectsInScope.SetBit(InternalIndex);
			}
		}
		break;

		case EReplicatedObjectState::AttachmentToObjectNotInScope:
		{
			check(IsObjectIndexForOOBAttachment(InternalIndex));
			Attachments.ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Delivered, ENetObjectAttachmentType::OutOfBand, ObjectIndexForOOBAttachment, AttachmentRecord);
		}
		return;

		case EReplicatedObjectState::HugeObject:
		{
			check(IsObjectIndexForOOBAttachment(InternalIndex));
			Attachments.ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Delivered, ENetObjectAttachmentType::HugeObject, ObjectIndexForOOBAttachment, AttachmentRecord);

			HugeObjectSendQueue.AckObjects([this](const FHugeObjectContext& HugeObjectContext)
			{
				// If we've sent an entire huge objects we can ack everything in the payload and continue replicating this object using normal means.
				const FReplicationInfo& HugeObjectReplicationInfo = this->GetReplicationInfo(HugeObjectContext.RootObjectInternalIndex);
				for (const FObjectRecord& ObjectRecord : HugeObjectContext.BatchRecord.ObjectReplicationRecords)
				{
					FReplicationInfo& ReplicationInfo = this->GetReplicationInfo(ObjectRecord.Record.Index);
					const uint32 ChangeMaskBitCount = ReplicationInfo.ChangeMaskBitCount;
					this->HandleDeliveredRecord(ObjectRecord.Record, ReplicationInfo, ObjectRecord.AttachmentRecord);
					if (ObjectRecord.Record.HasChangeMask)
					{
						FChangeMaskStorageOrPointer::Free(ObjectRecord.Record.ChangeMaskOrPtr, ChangeMaskBitCount, s_DefaultChangeMaskAllocator);
					}
				}

				// We need to explicitly acknowledge exports made through the huge object batch
				this->NetExports->AcknowledgeBatchExports(HugeObjectContext.BatchExports);
			});
		}
		return;

		case EReplicatedObjectState::DebugObject:
		{
			check(IsObjectIndexForOOBAttachment(InternalIndex));
			Attachments.ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Delivered, ENetObjectAttachmentType::DebugObject, ObjectIndexForOOBAttachment, AttachmentRecord);

			DebugObjectSendQueue.AckObjects([this](const FHugeObjectContext& HugeObjectContext)
			{
				// If we've sent an entire huge objects we can ack everything in the payload and continue replicating this object using normal means.
				const FReplicationInfo& HugeObjectReplicationInfo = this->GetReplicationInfo(HugeObjectContext.RootObjectInternalIndex);
				for (const FObjectRecord& ObjectRecord : HugeObjectContext.BatchRecord.ObjectReplicationRecords)
				{
					FReplicationInfo& ReplicationInfo = this->GetReplicationInfo(ObjectRecord.Record.Index);
					const uint32 ChangeMaskBitCount = ReplicationInfo.ChangeMaskBitCount;
					this->HandleDeliveredRecord(ObjectRecord.Record, ReplicationInfo, ObjectRecord.AttachmentRecord);
					if (ObjectRecord.Record.HasChangeMask)
					{
						FChangeMaskStorageOrPointer::Free(ObjectRecord.Record.ChangeMaskOrPtr, ChangeMaskBitCount, s_DefaultChangeMaskAllocator);
					}
				}

				// We need to explicitly acknowledge exports made through the huge object batch
				this->NetExports->AcknowledgeBatchExports(HugeObjectContext.BatchExports);
			});
		}
		return;

		default:
		break;
	}

	// If we now can send more data from our backlog, we need to mark object as dirty and having dirty attachments.
	if (RecordInfo.HasAttachments)
	{
		bool bMarkHasAttachmentDirty = false;

		Attachments.ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Delivered, ENetObjectAttachmentType::Normal, InternalIndex, AttachmentRecord, bMarkHasAttachmentDirty);

		if (bMarkHasAttachmentDirty && CurrentState < EReplicatedObjectState::PendingDestroy)
		{
			// Mark object as having dirty changes
			MarkObjectDirty(InternalIndex, "HandleDeliveredRecordMarkHasAttachmentDirty");
			Info.HasAttachments = true;

			// Bump prio if necessary
			SchedulingPriorities[InternalIndex] = FPlatformMath::Max(FReplicationWriter::LostStatePriorityBump, SchedulingPriorities[InternalIndex]);

			// Must also mark owner dirty
			if (Info.IsSubObject)
			{
				FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);
				FReplicationInfo& OwnerInfo = ReplicatedObjects[ObjectData.SubObjectRootIndex];
				if (OwnerInfo.GetState() != EReplicatedObjectState::Invalid && OwnerInfo.GetState() < EReplicatedObjectState::PendingDestroy)
				{
					MarkObjectDirty(ObjectData.SubObjectRootIndex, "HandleDeliveredRecordMarkHasAttachmentDirty");
					OwnerInfo.HasDirtySubObjects = 1U;

					// Bump prio if necessary
					SchedulingPriorities[ObjectData.SubObjectRootIndex] = FPlatformMath::Max(FReplicationWriter::LostStatePriorityBump, SchedulingPriorities[ObjectData.SubObjectRootIndex]);
				}
			}
		}
	}

	// Must process WaitOnflush after attachments in order to correctly evaluate flush-status if needed
	if (Info.GetState() == EReplicatedObjectState::WaitOnFlush)
	{
		bool bStillPendingFlush = false;
		if ((RecordInfo.HasChangeMask || Info.HasDirtyChangeMask) && !!(Info.FlushFlags & EFlushFlags::FlushFlags_FlushState))
		{
			bStillPendingFlush |= (Info.HasDirtyChangeMask || HasInFlightStateChanges(ReplicationRecord.GetInfoForIndex(RecordInfo.NextIndex)) || IsObjectPartOfActiveHugeObject(InternalIndex));
		}

		if ((RecordInfo.HasAttachments || Info.HasAttachments) && !!(Info.FlushFlags & FlushFlags_FlushReliable))
		{
			bStillPendingFlush |= !Attachments.IsAllReliableSentAndAcked(ENetObjectAttachmentType::Normal, InternalIndex);
		}

		// This is a bit blunt as subobjects might be "acked" later but in this case it will be captured in WriteObjectsPendingDestroy
		if (!bStillPendingFlush && !Info.IsSubObject)
		{
			// Check status of SubObjects as well.
			for (uint32 SubObjectIndex : NetRefHandleManager->GetSubObjects(InternalIndex))
			{
				const FReplicationInfo& SubObjectInfo = GetReplicationInfo(SubObjectIndex);
				if (GetFlushStatus(SubObjectIndex, SubObjectInfo, Info.FlushFlags) != FlushFlags_None)
				{
					bStillPendingFlush = true;
					break;
				}
			}
		}

		if (!bStillPendingFlush)
		{
			if (Info.TearOff)
			{
				SetState(InternalIndex, EReplicatedObjectState::PendingTearOff);
				ObjectsWithDirtyChanges.SetBit(InternalIndex);

				// Must also mark owner dirty to make sure that we send the tearoff
				if (Info.IsSubObject)
				{
					FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);
					FReplicationInfo& OwnerInfo = ReplicatedObjects[ObjectData.SubObjectRootIndex];
					MarkObjectDirty(ObjectData.SubObjectRootIndex, "HandleDeliveredRecordTearOff");
					OwnerInfo.HasDirtySubObjects = 1U;
				}
			}
			else
			{
				SetPendingDestroyOrSubObjectPendingDestroyState(InternalIndex, Info);
			}
		}
	}
}

void FReplicationWriter::HandleDiscardedRecord(const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
{
	const EReplicatedObjectState DeliveredState = (EReplicatedObjectState)RecordInfo.ReplicatedObjectState;
	const uint32 InternalIndex = RecordInfo.Index;

	// There are a couple of special cases we need to handle. Regular attachments are ignored since they don't require special handling at the moment.
	switch (DeliveredState)
	{
		// If we need to handle attachments this should return rather than fallback on some default path like HandleDeliveredRecord.
		case EReplicatedObjectState::AttachmentToObjectNotInScope:
		{
		}
		return;

		case EReplicatedObjectState::HugeObject:
		{
			check(IsObjectIndexForOOBAttachment(InternalIndex));

			// Deal with it similar to if the entire state has been sent as we need to go through all records.
			Attachments.ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Discard, ENetObjectAttachmentType::HugeObject, ObjectIndexForOOBAttachment, AttachmentRecord);

			HugeObjectSendQueue.AckObjects([this](const FHugeObjectContext& HugeObjectContext)
			{
				const FReplicationInfo& HugeObjectReplicationInfo = this->GetReplicationInfo(HugeObjectContext.RootObjectInternalIndex);
				for (const FObjectRecord& ObjectRecord : HugeObjectContext.BatchRecord.ObjectReplicationRecords)
				{
					FReplicationInfo& ReplicationInfo = this->GetReplicationInfo(ObjectRecord.Record.Index);
					const uint32 ChangeMaskBitCount = ReplicationInfo.ChangeMaskBitCount;
					this->HandleDiscardedRecord(ObjectRecord.Record, ReplicationInfo, ObjectRecord.AttachmentRecord);
					if (ObjectRecord.Record.HasChangeMask)
					{
						FChangeMaskStorageOrPointer::Free(ObjectRecord.Record.ChangeMaskOrPtr, ChangeMaskBitCount, s_DefaultChangeMaskAllocator);
					}
				}
			});
		}
		return;

		case EReplicatedObjectState::DebugObject:
		{
			check(IsObjectIndexForOOBAttachment(InternalIndex));

			// Deal with it similar to if the entire state has been sent as we need to go through all records.
			Attachments.ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Discard, ENetObjectAttachmentType::DebugObject, ObjectIndexForOOBAttachment, AttachmentRecord);

			DebugObjectSendQueue.AckObjects([this](const FHugeObjectContext& HugeObjectContext)
			{
				const FReplicationInfo& HugeObjectReplicationInfo = this->GetReplicationInfo(HugeObjectContext.RootObjectInternalIndex);
				for (const FObjectRecord& ObjectRecord : HugeObjectContext.BatchRecord.ObjectReplicationRecords)
				{
					FReplicationInfo& ReplicationInfo = this->GetReplicationInfo(ObjectRecord.Record.Index);
					const uint32 ChangeMaskBitCount = ReplicationInfo.ChangeMaskBitCount;
					this->HandleDiscardedRecord(ObjectRecord.Record, ReplicationInfo, ObjectRecord.AttachmentRecord);
					if (ObjectRecord.Record.HasChangeMask)
					{
						FChangeMaskStorageOrPointer::Free(ObjectRecord.Record.ChangeMaskOrPtr, ChangeMaskBitCount, s_DefaultChangeMaskAllocator);
					}
				}
			});
		}
		return;

	}
}

template<>
void FReplicationWriter::HandleDroppedRecord<FReplicationWriter::EReplicatedObjectState::WaitOnCreateConfirmation>(FReplicationWriter::EReplicatedObjectState CurrentState, const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
{
	const uint32 InternalIndex = RecordInfo.Index;

	if (CurrentState < EReplicatedObjectState::Created)
	{
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectData(InternalIndex);

		// We can resend creation info even if we are marked for destroy/endrepliation as long as we have cached creation info.
		const bool bCanSendCreationInfo = ObjectData.bHasCachedCreationInfo || (!ObjectsPendingDestroy.GetBit(InternalIndex) && !ObjectData.bPendingEndReplication);
		if (bCanSendCreationInfo)
		{
			// Mark object as having dirty changes
			MarkObjectDirty(InternalIndex, "DroppedWaitOnCreate");

			// Resend creation data
			SetState(InternalIndex, EReplicatedObjectState::PendingCreate);

			// Must also restore changemask
			FNetBitArrayView ChangeMask(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount);
			FNetBitArrayView LostChangeMask = FChangeMaskUtil::MakeChangeMask(RecordInfo.ChangeMaskOrPtr, Info.ChangeMaskBitCount);
			ChangeMask.Combine(LostChangeMask, FNetBitArrayView::OrOp);

			// Mark changemask dirty
			Info.HasDirtyChangeMask = 1U;

			// Indicate that we have dirty subobjects
			Info.HasDirtySubObjects = 1U;

			// Mark attachments as dirty
			Info.HasAttachments |= RecordInfo.HasAttachments;

			if (Info.IsSubObject)
			{
				// Mark owner dirty as well as subobjects only are scheduled together with owner
				uint32 SubObjectOwnerInternalIndex = ObjectData.SubObjectRootIndex;

				FReplicationInfo& SubObjectOwnerReplicationInfo = GetReplicationInfo(SubObjectOwnerInternalIndex);
				if (ensure(SubObjectOwnerReplicationInfo.GetState() < EReplicatedObjectState::PendingDestroy))
				{
					// Mark owner as dirty
					MarkObjectDirty(SubObjectOwnerInternalIndex, "DroppedWaitOnCreate2");

					// Indicate that we have dirty subobjects
					SubObjectOwnerReplicationInfo.HasDirtySubObjects = 1U;

					// Give slight priority bump to owner
					SchedulingPriorities[SubObjectOwnerInternalIndex] += FReplicationWriter::LostStatePriorityBump;
				}
			}
		}
		else
		{
			SetState(InternalIndex, EReplicatedObjectState::PendingCreate);
			StopReplication(InternalIndex);
		}
	}
	else if (CurrentState == EReplicatedObjectState::SubObjectPendingDestroy || CurrentState == EReplicatedObjectState::PendingDestroy)
	{
		// If Object has been destroyed while we where waiting for creation ack we can just stop replication
		SetState(InternalIndex, EReplicatedObjectState::WaitOnDestroyConfirmation);
		SetState(InternalIndex, EReplicatedObjectState::Destroyed);
		StopReplication(InternalIndex);
	}
}

template<>
void FReplicationWriter::HandleDroppedRecord<FReplicationWriter::EReplicatedObjectState::Created>(FReplicationWriter::EReplicatedObjectState CurrentState, const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
{
	const uint32 InternalIndex = RecordInfo.Index;

	// An object in PendingDestroy/WaitOnDestroyConfirmation can end up being replicated again via CancelPendingDestroy.
	if (CurrentState < EReplicatedObjectState::Destroyed)
	{
		// Mask in any lost changes
		bool bNeedToResendAttachments = RecordInfo.HasAttachments;
		bool bNeedToResendState = false;

		FNetBitArrayView LostChangeMask = FChangeMaskUtil::MakeChangeMask(RecordInfo.ChangeMaskOrPtr, static_cast<uint32>(RecordInfo.HasChangeMask ? Info.ChangeMaskBitCount : 1U));
		if (RecordInfo.HasChangeMask)
		{
			// Iterate over all data in flight for this object and mask away any already re-transmitted changes
			// N.B. We don't check if this object is in huge object mode and check to see if any of these changes were part of that payload.
			const FReplicationRecord::FRecordInfo* CurrentRecordInfo = ReplicationRecord.GetInfoForIndex(RecordInfo.NextIndex);
			while (CurrentRecordInfo)
			{
				if (CurrentRecordInfo->HasChangeMask)
				{
					const FNetBitArrayView CurrentRecordInfoChangeMask(FChangeMaskUtil::MakeChangeMask(CurrentRecordInfo->ChangeMaskOrPtr, Info.ChangeMaskBitCount));
					LostChangeMask.Combine(CurrentRecordInfoChangeMask, FNetBitArrayView::AndNotOp);
				}

				CurrentRecordInfo = ReplicationRecord.GetInfoForIndex(CurrentRecordInfo->NextIndex);
			};

			bNeedToResendState = LostChangeMask.IsAnyBitSet();
		}

		// if we lost changes that are not already retransmitted we update the changemask
		if (bNeedToResendState | bNeedToResendAttachments | Info.TearOff)
		{
			if (bNeedToResendState)
			{
				FNetBitArrayView ChangeMask(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount);
				ChangeMask.Combine(LostChangeMask, FNetBitArrayView::OrOp);
			}

			if (CurrentState < EReplicatedObjectState::PendingDestroy)
			{
				// Mark object as having dirty changes
				MarkObjectDirty(InternalIndex, "DroppedCreated");

				// Mark changemask as dirty
				Info.HasDirtyChangeMask |= bNeedToResendState;

				// Mark attachments as dirty
				Info.HasAttachments |= bNeedToResendAttachments;

				// Give slight priority bump
				SchedulingPriorities[InternalIndex] += FReplicationWriter::LostStatePriorityBump;

				if (Info.IsSubObject)
				{
					// Mark owner dirty as well as subobjects only are scheduled together with owner
					const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectData(InternalIndex);
					uint32 SubObjectOwnerInternalIndex = ObjectData.SubObjectRootIndex;

					FReplicationInfo& SubObjectOwnerReplicationInfo = GetReplicationInfo(SubObjectOwnerInternalIndex);

					if (ensure(SubObjectOwnerReplicationInfo.GetState() < EReplicatedObjectState::PendingDestroy))
					{
						// Mark owner as dirty
						MarkObjectDirty(SubObjectOwnerInternalIndex, "DroppedCreated2");

						// Indicate that we have dirty subobjects
						SubObjectOwnerReplicationInfo.HasDirtySubObjects = 1U;

						// Give slight priority bump to owner
						SchedulingPriorities[SubObjectOwnerInternalIndex] += FReplicationWriter::LostStatePriorityBump;
					}
				}
			}
		}
	}
}

template<>
void FReplicationWriter::HandleDroppedRecord<FReplicationWriter::EReplicatedObjectState::WaitOnDestroyConfirmation>(FReplicationWriter::EReplicatedObjectState CurrentState, const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
{
	const uint32 InternalIndex = RecordInfo.Index;

	ensureMsgf(CurrentState == EReplicatedObjectState::WaitOnDestroyConfirmation || CurrentState == EReplicatedObjectState::CancelPendingDestroy, TEXT("Expected object ( InternalIndex: %u ) not to be in state %s"), InternalIndex, LexToString(CurrentState));

	// If we want to cancel the destroy and lost the destroy packet we can resume normal replication.
	if (CurrentState == EReplicatedObjectState::CancelPendingDestroy)
	{
		checkfSlow(!RecordInfo.WroteTearOff, TEXT("Torn off objects can't cancel destroy. ( InternalIndex: %u ) %s"), InternalIndex, ToCStr(NetRefHandleManager->GetReplicatedObjectData(InternalIndex).RefHandle.ToString()));

		if (RecordInfo.WroteDestroySubObject && Info.SubObjectPendingDestroy)
		{
			// 2024-10-23. Look into enabling this ensure at a proper time. 
			//ensureMsgf(false, TEXT("Subobjects destroyed individually should not be canceled. Object: %s"), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)));
			// If the subobject owner still is replicated and valid
			FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);

			FNetBitArrayView ChangeMask(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount);
			Info.HasDirtyChangeMask |= ChangeMask.IsAnyBitSet();
			Info.SubObjectPendingDestroy = 0U;
			ObjectsPendingDestroy.ClearBit(InternalIndex);
			SetState(InternalIndex, EReplicatedObjectState::Created);

			ObjectsWithDirtyChanges.SetBitValue(InternalIndex, Info.HasDirtyChangeMask);

			// If owner is not pending destroy we mark it as dirty as appropriate.
			FReplicationInfo& OwnerInfo = ReplicatedObjects[ObjectData.SubObjectRootIndex];
			if (OwnerInfo.GetState() < EReplicatedObjectState::PendingDestroy)
			{
				OwnerInfo.HasDirtySubObjects |= Info.HasDirtyChangeMask;
				ensureMsgf(!bValidateObjectsWithDirtyChanges || OwnerInfo.GetState() != EReplicatedObjectState::Invalid, TEXT("Object (InternalIndex: % u) with Invalid state potentially marked dirty."), ObjectData.SubObjectRootIndex);
				ObjectsWithDirtyChanges.SetBitValue(ObjectData.SubObjectRootIndex, ObjectsWithDirtyChanges.GetBit(ObjectData.SubObjectRootIndex) || OwnerInfo.HasDirtySubObjects);
			}
		}
		else
		{
			// Check whether there are any dirty changes and mark object as dirty as appropriate.
			FNetBitArrayView ChangeMask(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount);
			Info.HasDirtyChangeMask |= ChangeMask.IsAnyBitSet();

			ObjectsWithDirtyChanges.SetBitValue(InternalIndex, Info.HasDirtyChangeMask);

			ObjectsPendingDestroy.ClearBit(InternalIndex);

			SetState(InternalIndex, EReplicatedObjectState::Created);
		}	
	}
	else
	{
		// We dropped a packet with tear-off data, that is a destroy with state data so we need to resend that state
		if (RecordInfo.WroteTearOff)
		{
			ensureMsgf(Info.TearOff, TEXT("Expected object ( InternalIndex: %u ) to have TearOff set. Current state %s."), InternalIndex, LexToString(CurrentState));

			SetState(InternalIndex, EReplicatedObjectState::PendingTearOff);

			if (RecordInfo.HasChangeMask)
			{
				// Must also restore changemask
				FNetBitArrayView ChangeMask(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount);
				FNetBitArrayView LostChangeMask = FChangeMaskUtil::MakeChangeMask(RecordInfo.ChangeMaskOrPtr, Info.ChangeMaskBitCount);
				ChangeMask.Combine(LostChangeMask, FNetBitArrayView::OrOp);

				// Mark changemask dirty
				Info.HasDirtyChangeMask = 1U;
			}

			// Mark attachments as dirty
			Info.HasAttachments |= RecordInfo.HasAttachments;

			// Mark object as having dirty changes
			MarkObjectDirty(InternalIndex, "DroppedWaitOnDestroy");

			// Mark parent as dirty
			uint32 ParentInternalIndex = InternalIndex;
			if (Info.IsSubObject)
			{
				const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);
				if (ensure(ObjectData.SubObjectRootIndex != InvalidInternalNetRefIndex))
				{
					ParentInternalIndex = ObjectData.SubObjectRootIndex;
					MarkObjectDirty(ParentInternalIndex, "DroppedWaitOnDestroy2");

					FReplicationInfo& OwnerInfo = ReplicatedObjects[ObjectData.SubObjectRootIndex];
					if (ensure(OwnerInfo.GetState() < EReplicatedObjectState::PendingDestroy))
					{
						// Indicate that we have dirty subobjects
						OwnerInfo.HasDirtySubObjects = 1U;
					}
				}
			}

			// Bump prio
			SchedulingPriorities[ParentInternalIndex] += FReplicationWriter::TearOffPriority;
		}
		else if (RecordInfo.WroteDestroySubObject && Info.SubObjectPendingDestroy)
		{
			// If the subobject owner still is replicated and valid
			FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);

			// If owner is not pending destroy we mark it as dirty so that we can replicate subobject destruction properly
			// We might get away with not doing this if owner or subobject does not have any unconfirmed changes in flight.
			FReplicationInfo& OwnerInfo = ReplicatedObjects[ObjectData.SubObjectRootIndex];
			if (OwnerInfo.GetState() < EReplicatedObjectState::PendingDestroy)
			{
				MarkObjectDirty(ObjectData.SubObjectRootIndex, "DroppedWaitOnDestroy2");
				OwnerInfo.HasDirtySubObjects = 1U;

				SetState(InternalIndex, EReplicatedObjectState::SubObjectPendingDestroy);
				ObjectsWithDirtyChanges.SetBit(InternalIndex);
				ObjectsPendingDestroy.SetBit(InternalIndex);
			}
		}
		else
		{
			// Mark for resend of Destroy
			ObjectsPendingDestroy.SetBit(InternalIndex);
			ObjectsWithDirtyChanges.ClearBit(InternalIndex);
			Info.HasDirtyChangeMask = 0U;

			SetState(InternalIndex, EReplicatedObjectState::PendingDestroy);
		}
	}
}

void FReplicationWriter::HandleDroppedRecord(const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
{
	EReplicatedObjectState LostObjectState = (EReplicatedObjectState)RecordInfo.ReplicatedObjectState;
	EReplicatedObjectState CurrentState = Info.GetState();
	const uint32 InternalIndex = RecordInfo.Index;

	check(CurrentState != EReplicatedObjectState::Invalid);

	UE_LOG_REPWRITER(VeryVerbose, TEXT("Handle dropped data for ( InternalIndex: %u ) %s, LostState %s, CurrentState is %s"), InternalIndex, ToCStr(NetRefHandleManager->GetReplicatedObjectData(InternalIndex).RefHandle.ToString()), LexToString(LostObjectState), LexToString(CurrentState));

	// If we loose a baseline we must notify the BaselineManager and invalidate our pendingbaselineindex
	if (RecordInfo.NewBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
	{
		check(RecordInfo.NewBaselineIndex == Info.PendingBaselineIndex);
		UE_LOG_REPWRITER(VeryVerbose, TEXT("Lost baseline %u for ( InternalIndex: %u )"), RecordInfo.NewBaselineIndex, InternalIndex);
		
		BaselineManager->LostBaseline(Parameters.ConnectionId, InternalIndex, RecordInfo.NewBaselineIndex);
		Info.PendingBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
	}

	switch (LostObjectState)
	{
		// We dropped creation state, restore state to PendingCreate and bump priority to make sure we sent it again
		case EReplicatedObjectState::WaitOnCreateConfirmation:
		{
			HandleDroppedRecord<EReplicatedObjectState::WaitOnCreateConfirmation>(CurrentState, RecordInfo, Info, AttachmentRecord);
		}
		break;

		// Object is created, update lost state data unless object are currently being flushed/teared-off or destroyed
		case EReplicatedObjectState::Created:
		case EReplicatedObjectState::WaitOnFlush:
		{
			HandleDroppedRecord<EReplicatedObjectState::Created>(CurrentState, RecordInfo, Info, AttachmentRecord);
		}
		break;

		case EReplicatedObjectState::WaitOnDestroyConfirmation:
		{
			HandleDroppedRecord<EReplicatedObjectState::WaitOnDestroyConfirmation>(CurrentState, RecordInfo, Info, AttachmentRecord);
		}
		break;

		case EReplicatedObjectState::CancelPendingDestroy:
		{
			checkf(false, TEXT("%s"), TEXT("CancelPendingDestroy is not a state that should be replicated."));
		}
		break;

		case EReplicatedObjectState::AttachmentToObjectNotInScope:
		{
			check(IsObjectIndexForOOBAttachment(InternalIndex));
			Attachments.ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Lost, ENetObjectAttachmentType::OutOfBand, ObjectIndexForOOBAttachment, AttachmentRecord);
		}
		return;

		case EReplicatedObjectState::HugeObject:
		{
			check(IsObjectIndexForOOBAttachment(InternalIndex));
			Attachments.ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Lost, ENetObjectAttachmentType::HugeObject, ObjectIndexForOOBAttachment, AttachmentRecord);
		}
		return;

		case EReplicatedObjectState::DebugObject:
		{
			check(IsObjectIndexForOOBAttachment(InternalIndex));
			Attachments.ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Lost, ENetObjectAttachmentType::DebugObject, ObjectIndexForOOBAttachment, AttachmentRecord);
		}
		return;

		
		default:
		break;
	};

	if (RecordInfo.HasAttachments)
	{
		Attachments.ProcessPacketDeliveryStatus(EPacketDeliveryStatus::Lost, ENetObjectAttachmentType::Normal, InternalIndex, AttachmentRecord);
	}
}

void FReplicationWriter::ProcessDeliveryNotification(EPacketDeliveryStatus PacketDeliveryStatus)
{
#if UE_NET_VALIDATE_REPLICATION_RECORD
	check(s_ValidateReplicationRecord(&ReplicationRecord, NetRefHandleManager->GetMaxActiveObjectCount() + 1U, true));
#endif

	const uint32 RecordCount = ReplicationRecord.PopRecord();

	if (RecordCount > 0)
	{
		TReplicationRecordHelper Helper(ReplicatedObjects, ReplicatedObjectsRecordInfoLists, &ReplicationRecord);

		if (PacketDeliveryStatus == EPacketDeliveryStatus::Delivered)
		{
			Helper.Process(RecordCount,
				[this](const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
				{ 
					HandleDeliveredRecord(RecordInfo, Info, AttachmentRecord);
				}
			);
		}
		else if (PacketDeliveryStatus == EPacketDeliveryStatus::Lost)
		{
			Helper.Process(RecordCount,
				[this](const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
				{
					HandleDroppedRecord(RecordInfo, Info, AttachmentRecord);
				}
			);
		}
		else if (PacketDeliveryStatus == EPacketDeliveryStatus::Discard)
		{
			Helper.Process(RecordCount,
				[this](const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
				{
					HandleDiscardedRecord(RecordInfo, Info, AttachmentRecord);
				}
			);
		}
		else
		{
			checkf(false, TEXT("Unknown packet delivery status %u"), unsigned(PacketDeliveryStatus));
		}
	}
}

void FReplicationWriter::CreateObjectRecord(const FNetBitArrayView* ChangeMask, const FReplicationInfo& Info, const FBatchObjectInfo& ObjectInfo, FReplicationWriter::FObjectRecord& OutRecord)
{
	OutRecord.AttachmentRecord = ObjectInfo.AttachmentRecord.ReliableReplicationRecord;

	FReplicationRecord::FRecordInfo& RecordInfo = OutRecord.Record;

	RecordInfo.Index = ObjectInfo.InternalIndex;

	RecordInfo.NextIndex = FReplicationRecord::InvalidReplicationRecordIndex;

	if (ObjectInfo.AttachmentType == ENetObjectAttachmentType::HugeObject)
	{
		RecordInfo.ReplicatedObjectState = uint8(EReplicatedObjectState::HugeObject);
	}
	else if (ObjectInfo.AttachmentType == ENetObjectAttachmentType::DebugObject)
	{
		RecordInfo.ReplicatedObjectState = uint8(EReplicatedObjectState::DebugObject);
	}
	else
	{
		RecordInfo.ReplicatedObjectState = (uint8)Info.GetState();
	}

	RecordInfo.HasChangeMask = ChangeMask ? 1U : 0U;
	RecordInfo.HasAttachments = (OutRecord.AttachmentRecord.IsValid() ? 1U : 0U);
	RecordInfo.WroteTearOff = ObjectInfo.bSentTearOff;
	RecordInfo.WroteDestroySubObject = Info.SubObjectPendingDestroy;
	RecordInfo.HasSubObjectRecord = 0U;
	// If we wrote a new baseline we need to store it in the record
	RecordInfo.NewBaselineIndex = ObjectInfo.bSentState ? ObjectInfo.NewBaselineIndex : FDeltaCompressionBaselineManager::InvalidBaselineIndex;

	if (ChangeMask)
	{
		// $IRIS: TODO: Implement other type of changemask allocator that utilizes the the fifo nature of the record
		// https://jira.it.epicgames.com/browse/UE-127372
		// Allocate and copy changemask
		FChangeMaskStorageOrPointer::Alloc(RecordInfo.ChangeMaskOrPtr, ChangeMask->GetNumBits(), s_DefaultChangeMaskAllocator);
		FChangeMaskUtil::CopyChangeMask(RecordInfo.ChangeMaskOrPtr, *ChangeMask);
	}
	else
	{
		// Clear change mask
		RecordInfo.ChangeMaskOrPtr = FChangeMaskStorageOrPointer();
	}
}

void FReplicationWriter::CommitObjectRecord(uint32 InternalObjectIndex, const FObjectRecord& ObjectRecord)
{
	// Push and link replication record to data already in-flight
	ReplicationRecord.PushInfoAndAddToList(ReplicatedObjectsRecordInfoLists[InternalObjectIndex], ObjectRecord.Record, ObjectRecord.AttachmentRecord.ToUint64());
}

void FReplicationWriter::CommitObjectDestroyRecord(uint32 InternalObjectIndex, const FObjectRecord& ObjectRecord, const FReplicationRecord::FSubObjectRecord& SubObjectRecord)
{
	ReplicationRecord.PushInfoAndAddToList(ReplicatedObjectsRecordInfoLists[InternalObjectIndex], ObjectRecord.Record, SubObjectRecord);
}

void FReplicationWriter::CommitBatchRecord(const FBatchRecord& BatchRecord)
{
	for (const FObjectRecord& ObjectRecord : BatchRecord.ObjectReplicationRecords)
	{
		 CommitObjectRecord(ObjectRecord.Record.Index, ObjectRecord);
	}
}

void FReplicationWriter::UpdateStreamDebugFeatures()
{
	StreamDebugFeatures = EReplicationDataStreamDebugFeatures::None;
#if UE_NET_REPLICATIONDATASTREAM_DEBUG
	// Enable these features for object replication. This isn't the best test.
	if (Parameters.ReplicationSystem && Parameters.ReplicationSystem->IsServer())
	{
		StreamDebugFeatures |= (bDebugBatchSizePerObjectEnabled ? EReplicationDataStreamDebugFeatures::BatchSizePerObject : EReplicationDataStreamDebugFeatures::None);
		StreamDebugFeatures |= (bDebugSentinelsEnabled ? EReplicationDataStreamDebugFeatures::Sentinels : EReplicationDataStreamDebugFeatures::None);
	}
#endif
}

void FReplicationWriter::WriteStreamDebugFeatures(FNetSerializationContext& Context)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	WriteReplicationDataStreamDebugFeatures(Writer, StreamDebugFeatures);
}

uint32 FReplicationWriter::WriteObjectsPendingDestroy(FNetSerializationContext& Context)
{
	return WriteRootObjectsPendingDestroy(Context);
}

uint32 FReplicationWriter::WriteRootObjectsPendingDestroy(FNetSerializationContext& Context)
{
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	UE_NET_TRACE_SCOPE(RootObjectsPendingDestroy, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	uint32 WrittenCount = 0;

	// Write how many destroyed objects we have
	const uint32 HeaderPos = Writer.GetPosBits();
	constexpr uint32 DestroyObjectBitCount = 16U;
	constexpr uint32 MaxDestroyObjectCount = (1U << DestroyObjectBitCount) - 1U;

	Writer.WriteBits(WrittenCount, DestroyObjectBitCount);

	// Can't write if bitstream is overflown. Shouldn't write if we're an OOB packet or in replication record starvation mode.
	if (Writer.IsOverflown() || WriteContext.bIsOOBPacket || WriteContext.bIsInReplicationRecordStarvation)
	{
		return 0U;
	}

	// Check whether we're still allowed to write object destroys.
	uint32 EffectiveMaxDestroyObjectCount = MaxDestroyObjectCount;
	const int32 MaxDestroyObjectsPerFrame = GReplicationWriterMaxDestroyObjectsPerFrame;
	if (MaxDestroyObjectsPerFrame > 0)
	{
		if (WriteContext.WrittenDestroyObjectCount > static_cast<uint32>(MaxDestroyObjectsPerFrame))
		{
			return 0U;
		}

		EffectiveMaxDestroyObjectCount = static_cast<uint32>(MaxDestroyObjectsPerFrame) - WriteContext.WrittenDestroyObjectCount;
	}

	auto AreSubObjectsReadyToBeDestroyed = [this](uint32 InternalIndex, const FReplicationConditionals::FSubObjectsToReplicateArray& FilteredSubObjects, TArrayView<const FInternalNetRefIndex>& SubObjects)
	{
		for (uint32 SubObjectIndex : SubObjects)
		{
			FReplicationInfo& SubObjectInfo = GetReplicationInfo(SubObjectIndex);
			const EReplicatedObjectState SubObjectState = SubObjectInfo.GetState();

			// A subobject may have stopped replicating immediately when going out of scope if it was never replicated to begin with.
			if (SubObjectState == EReplicatedObjectState::Invalid)
			{
				continue;
			}

			// Filtered out subobjects still in PendingCreate can be stopped.
			if (SubObjectState == EReplicatedObjectState::PendingCreate && !FilteredSubObjects.Contains(SubObjectIndex))
			{
				StopReplication(SubObjectIndex);
				continue;
			}

			if (SubObjectState == EReplicatedObjectState::WaitOnFlush)
			{
				if (GetFlushStatus(SubObjectIndex, SubObjectInfo, SubObjectInfo.FlushFlags) == EFlushFlags::FlushFlags_None)
				{
					ensureMsgf(!SubObjectInfo.TearOff, TEXT("Torn off subobjects should not be destroyed via the pending destroy. ( InternalIndex: %u )"), SubObjectIndex);
					SetPendingDestroyOrSubObjectPendingDestroyState(SubObjectIndex, SubObjectInfo);
				}
			}

			if (!(SubObjectState == EReplicatedObjectState::PendingDestroy || SubObjectState == EReplicatedObjectState::SubObjectPendingDestroy))
			{
				return false;
			}

			if (!SubObjectInfo.IsCreationConfirmed)
			{
				return false;
			}
		}

		return true;
	};

	bool bWroteAllDestroyedObjects = true;
	for (uint32 InternalIndex = 0U; ((InternalIndex = ObjectsPendingDestroy.FindFirstOne(InternalIndex + 1U)) != FNetBitArray::InvalidIndex) && (WrittenCount < EffectiveMaxDestroyObjectCount); )
	{
		FReplicationInfo& Info = GetReplicationInfo(InternalIndex);
		const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);

		if (ObjectData.IsSubObject())
		{
			// Have the root object query status of subobjects
			continue;
		}

		// Don't send destroy until object creation has been acked.
		if (!Info.IsCreationConfirmed)
		{
			continue;
		}

		// Already waiting on destroy confirmation or need to replicate via the regular path
		if (Info.GetState() == EReplicatedObjectState::WaitOnDestroyConfirmation || Info.GetState() == EReplicatedObjectState::PendingTearOff)
		{
			continue;
		}

		// Lookup once.
		TArrayView<const FInternalNetRefIndex> AllSubObjects = NetRefHandleManager->GetSubObjects(InternalIndex);

		if (Info.GetState() == EReplicatedObjectState::WaitOnFlush)
		{
			if (GetFlushStatus(InternalIndex, Info, Info.FlushFlags) != EFlushFlags::FlushFlags_None)
			{
				continue;
			}

			// Tear off can happen while an object is out of scope. We should go through the state replication tear off path.
			if (Info.TearOff)
			{
				// Check that all subobjects are ready to be torn off too.
				const bool bAllSubObjectsAreReadyToBeTornOff = [this, InternalIndex, &AllSubObjects]()
				{
					for (const FInternalNetRefIndex SubObjectIndex : AllSubObjects)
					{
						FReplicationInfo& SubObjectInfo = ReplicatedObjects[SubObjectIndex];
						if (SubObjectInfo.GetState() == EReplicatedObjectState::WaitOnFlush)
						{
							if (GetFlushStatus(SubObjectIndex, SubObjectInfo, SubObjectInfo.FlushFlags) == EFlushFlags::FlushFlags_None)
							{
								SetState(SubObjectIndex, EReplicatedObjectState::PendingTearOff);
								ObjectsWithDirtyChanges.SetBit(SubObjectIndex);
							}
							else
							{
								return false;
							}
						}
					}

					return true;
				}();

				if (bAllSubObjectsAreReadyToBeTornOff)
				{
					SetState(InternalIndex, EReplicatedObjectState::PendingTearOff);
					ObjectsWithDirtyChanges.SetBit(InternalIndex);
					Info.HasDirtySubObjects = 1;
				}
				// Either we are waiting for flush or we are PendingTearOff.
				continue;
			}
			else
			{
				// Object and subobjects are now flushed and can be destroyed
				SetPendingDestroyOrSubObjectPendingDestroyState(InternalIndex, Info);
			}
		}


		FReplicationConditionals::FSubObjectsToReplicateArray SubObjectsToReplicate;
		ReplicationConditionals->GetSubObjectsToReplicate(Parameters.ConnectionId, InternalIndex, SubObjectsToReplicate);

		if (!AreSubObjectsReadyToBeDestroyed(InternalIndex, SubObjectsToReplicate, AllSubObjects))
		{
			bWroteAllDestroyedObjects = false;
			continue;
		}
		
		// Unexpected. Get more info.
		if (Info.GetState() != EReplicatedObjectState::PendingDestroy)
		{
			ensureMsgf(Info.GetState() == EReplicatedObjectState::PendingDestroy, TEXT("Skipping writing destroy for object %s which is in unexpected state %s. IsSubObject: %" UINT64_FMT " IsDestructionInfo: %" UINT64_FMT), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), LexToString(Info.GetState()), Info.IsSubObject, Info.IsDestructionInfo);
			continue;
		}

		// We do not support destroying an object that is currently being sent as a huge object.
		if (IsObjectPartOfActiveHugeObject(InternalIndex))
		{
			UE_LOGF(LogIris, Verbose, "Skipping writing destroy for object %ls which is part of active huge object.", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex));
			bWroteAllDestroyedObjects = false;
			continue;
		}

		UE_NET_TRACE_OBJECT_SCOPE(ObjectData.RefHandle,  Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

		FNetBitStreamRollbackScope RollbackScope(Writer);

		// Write handle with the needed bitCount
		WriteNetRefHandleId(Context, ObjectData.RefHandle);

		EInternalDetachReason Reason = ObjectData.InternalDetachReason;

		{
			// Look if the object is stopping replication for an explicit reason
			if (NetRefHandleManager->IsDestroyedStartupObject(InternalIndex))
			{
				Reason = EInternalDetachReason::StaticDestroyed;
			}
			else if (ObjectData.bTearOff)
			{
				// The object data reason should have been set already
				ensureMsgf(Reason == EInternalDetachReason::TornOff, TEXT("TearOff/IDR mismatch: got %u. Object: %s Info.TearOff: %d"), static_cast<uint32>(Reason), ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)), Info.TearOff);
			}
			// If the object is still replicating, then it's detaching because it's no longer relevant to this client
			else if (!ObjectData.bIsAsyncStopping && !ObjectData.bPendingEndReplication)
			{
				Reason = EInternalDetachReason::NotRelevant;
			}

			Writer.WriteBits((uint32)Reason, GetDetachReasonBitsNeeded());
		}

		if (Writer.IsOverflown())
		{
			break;
		}

		// Must update state before pushing record
		SetState(InternalIndex, EReplicatedObjectState::WaitOnDestroyConfirmation);
		Info.HasDirtyChangeMask = 0U;

		// update transmission record
		FBatchObjectInfo ObjectInfo = {};
		FObjectRecord ObjectRecord;
		FReplicationRecord::FSubObjectRecord SubObjectRecord;

		ObjectInfo.InternalIndex = InternalIndex;
		CreateObjectRecord(nullptr, Info, ObjectInfo, ObjectRecord);

		// Fill in subobject record
		for (const uint32 SubObjectIndex : AllSubObjects)
		{
			FReplicationInfo& SubObjectInfo = GetReplicationInfo(SubObjectIndex);
			if (SubObjectInfo.GetState() == EReplicatedObjectState::Invalid)
			{
				continue;
			}

			SetState(SubObjectIndex, EReplicatedObjectState::WaitOnDestroyConfirmation);
			SubObjectInfo.HasDirtyChangeMask = 0U;
			SubObjectInfo.SubObjectPendingDestroy = 0U;

			FReplicationRecord::FSubObjectRecord::FSubObjectInfo& SubObjectRecordInfo = SubObjectRecord.SubObjectInfos.AddDefaulted_GetRef();
			SubObjectRecordInfo.Index = SubObjectIndex;
			SubObjectRecordInfo.ReplicatedObjectState = static_cast<uint32>(SubObjectInfo.GetState());
		}

		ObjectRecord.Record.HasSubObjectRecord = !SubObjectRecord.SubObjectInfos.IsEmpty();
		CommitObjectDestroyRecord(InternalIndex, ObjectRecord, SubObjectRecord);

		++WrittenCount;
	}

	// Write Header
	{
		FNetBitStreamWriteScope WriteScope(Writer, HeaderPos);
		Writer.WriteBits(WrittenCount, DestroyObjectBitCount);
	}

	bWroteAllDestroyedObjects = bWroteAllDestroyedObjects && !Writer.IsOverflown() && (WrittenCount < MaxDestroyObjectCount);
	WriteContext.bHasDestroyedObjectsToSend = !bWroteAllDestroyedObjects;
	WriteContext.WrittenDestroyObjectCount += WrittenCount;
	return WrittenCount;
}

FReplicationWriter::FCannotSendInfo* FReplicationWriter::ShouldWarnIfCannotSend(const FReplicationInfo& Info, FInternalNetRefIndex InternalIndex) const
{
#if UE_NET_ENABLE_REPLICATIONWRITER_CANNOT_SEND_WARNING
	if (GReplicationWriterCannotSendWarningInterval > 0)
	{
		if (Info.HasCannotSendInfo)
		{
			FCannotSendInfo* CannotSendInfo = &CannotSendInfos.FindChecked(InternalIndex);
			if (--CannotSendInfo->SuppressWarningCounter == 0U)
			{
				return CannotSendInfo;
			}
		}
		else
		{
			// Store info
			FCannotSendInfo& CannotSendInfo = CannotSendInfos.Add(InternalIndex);
			CannotSendInfo.SuppressWarningCounter = GReplicationWriterCannotSendWarningInterval;
			CannotSendInfo.StartCycles = FPlatformTime::Cycles64();
			Info.HasCannotSendInfo = 1U;
		}
	}
#endif
	return static_cast<FCannotSendInfo*>(nullptr);
};

// local macro to save some typing
#if UE_NET_ENABLE_REPLICATIONWRITER_CANNOT_SEND_WARNING
#	define UE_NET_REPLICATIONWRITER_WARN_IF_CANNOT_SEND(Format, ...) \
		if (FCannotSendInfo* CannotSendInfo = ShouldWarnIfCannotSend(Info, InternalIndex)) \
		{ \
			UE_LOG(LogIris, Warning, TEXT("Conn: %u Object %s Blocked from sending for %f seconds: ") Format, Parameters.ConnectionId, *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - CannotSendInfo->StartCycles), ##__VA_ARGS__); \
		}
#else
#	define UE_NET_REPLICATIONWRITER_WARN_IF_CANNOT_SEND(...)
#endif

bool FReplicationWriter::CanSendObject(uint32 InternalIndex) const
{
	const FReplicationInfo& Info = GetReplicationInfo(InternalIndex);
	const EReplicatedObjectState State = Info.GetState();

	if (State == EReplicatedObjectState::PendingCreate)
	{
		// We cannot replicate the same netrefhandle using different internal indices.
		const FNetRefHandleManager::FReplicatedObjectData& Data = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);		
		for (const FInternalNetRefIndex& PendingDestroyIndex : NetRefHandleManager->GetInternalIndicesReplicatingNetRefHandle(Data.RefHandle))
		{
			// If we find the current InternalIndex we know that we can start replicating object identified by it.
			if (PendingDestroyIndex == InternalIndex)
			{
				break;
			}

			const FReplicationInfo& PendingDestroyInfo = GetReplicationInfo(PendingDestroyIndex);
			const EReplicatedObjectState PendingDestroyState = PendingDestroyInfo.GetState();
				
			if (PendingDestroyState != EReplicatedObjectState::Invalid)
			{
				UE_NET_REPLICATIONWRITER_WARN_IF_CANNOT_SEND(TEXT("Waiting on previous instance %s to finish replication."), *NetRefHandleManager->PrintObjectFromIndex(PendingDestroyIndex));
				return false;
			}				
		}
	}

	// Currently we do wait for CreateConfirmation before sending more data
	// We might want to change this and allow "bombing" creation info until we get confirmation to minimize latency
	// We also prevent objects from being transmitted if they are waiting on destroy/tear-off confirmation or cancelling destroy.
	if (State == EReplicatedObjectState::WaitOnCreateConfirmation || State == EReplicatedObjectState::WaitOnDestroyConfirmation || State == EReplicatedObjectState::CancelPendingDestroy)
	{
		UE_NET_REPLICATIONWRITER_WARN_IF_CANNOT_SEND(TEXT("Due to State (%s)"), LexToString(State));
		return false;
	}

	// Don't send more recent state that could arrive before the huge state. We only need to check the parent.
	if (IsActiveHugeObject(InternalIndex))
	{
		if (!Info.IsSubObject)
		{
			UE_NET_REPLICATIONWRITER_WARN_IF_CANNOT_SEND(TEXT("IsActiveHugeObject"));
			return false;
		}
	}

	// Disabled for now, this can be used if we want to enforce creation dependencies on the server side
#if UE_NET_ENFORCE_CREATION_DEPENDENCIES_ON_SERVER
	// Deal with initial dependencies
	if ((State == EReplicatedObjectState::PendingCreate && NetRefHandleManager->GetObjectsWithCreationDependencies().GetBit(InternalIndex)))
	{
		for (const FInternalNetRefIndex& InitDependencyIndex : NetRefHandleManager->GetCreationDependencies(InternalIndex))
		{
			// Verify status of initial dependency to see if we can send this object
			const FReplicationInfo& DependencyReplicationInfo = GetReplicationInfo(InitDependencyIndex);
			if (!(DependencyReplicationInfo.IsCreationConfirmed || (WriteContext.ObjectsWrittenThisPacket.GetBit(InitDependencyIndex) && !IsActiveHugeObject(InitDependencyIndex))))
			{
				UE_LOG_REPWRITER(VeryVerbose, TEXT("ReplicationWriter: Cannot send internal index (%u) due to waiting on initial dependency internal index (%d)"), InternalIndex, InitDependencyIndex);
				UE_NET_REPLICATIONWRITER_WARN_IF_CANNOT_SEND(TEXT("Waiting on init dependency %s "), *NetRefHandleManager->PrintObjectFromIndex(InitDependencyIndex));

				return false;
			}
		}
	}
#endif

	if (Info.HasDirtySubObjects)
	{
		for (FInternalNetRefIndex SubObjectInternalIndex : NetRefHandleManager->GetSubObjects(InternalIndex))
		{
			if (!CanSendObject(SubObjectInternalIndex))
			{
				UE_NET_REPLICATIONWRITER_WARN_IF_CANNOT_SEND(TEXT("SubObject %s cannot be sent"), *NetRefHandleManager->PrintObjectFromIndex(SubObjectInternalIndex));
				return false;
			}
		}
	}

	// Currently we enforce a strict dependency on the state of initial dependent objects unless they are already serialized in the same packet
	if (NetRefHandleManager->GetObjectsWithDependentObjectsInternalIndices().GetBit(InternalIndex))
	{
		for (const FDependentObjectInfo DependentObjectInfo : NetRefHandleManager->GetDependentObjectInfos(InternalIndex))
		{
			const FInternalNetRefIndex DependentInternalIndex = DependentObjectInfo.NetRefIndex;

			// If the dependent object already has been written in this packet and is not part of a huge object we do not need to do any further checks.
			// Note: To avoid waiting for ack of huge dependent object we could remove the special scheduling of dependent actors and instead handle this when we write the batch
			if (WriteContext.ObjectsWrittenThisPacket.GetBit(DependentInternalIndex) && !IsActiveHugeObject(DependentInternalIndex))
			{
				continue;
			}

			const FReplicationInfo& DependentReplicationInfo = GetReplicationInfo(DependentInternalIndex);
			if (IsInitialState(DependentReplicationInfo.GetState()))
			{
				// if we cannot send the initial dependent object we must wait until we can.
				if (!CanSendObject(DependentInternalIndex))
				{
					UE_LOGF(LogIris, Verbose, "ReplicationWriter: Cannot send: %ls due to waiting on init dependency of: %ls", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), *NetRefHandleManager->PrintObjectFromIndex(DependentInternalIndex));
					UE_NET_REPLICATIONWRITER_WARN_IF_CANNOT_SEND(TEXT("Waiting on init dependency %s "), *NetRefHandleManager->PrintObjectFromIndex(DependentInternalIndex));

					return false;
				}

				// if the dependent object are scheduled before parent and did not fit in this packet, we cannot write the parent either and have to wait until creation is confirmed
				if ((DependentObjectInfo.SchedulingHint == EDependentObjectSchedulingHint::ScheduleBeforeParent) && ObjectsWithDirtyChanges.GetBit(DependentInternalIndex))
				{
					UE_LOGF(LogIris, Verbose, "ReplicationWriter: Cannot send: %ls due to waiting on ScheduleBefore dependency of: %ls", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), *NetRefHandleManager->PrintObjectFromIndex(DependentInternalIndex));
					UE_NET_REPLICATIONWRITER_WARN_IF_CANNOT_SEND(TEXT("Waiting on ScheduleBefore dependency %s "), *NetRefHandleManager->PrintObjectFromIndex(DependentInternalIndex));

					return false;
				}
			}
		}
	}

	return true;
}

#undef UE_NET_REPLICATIONWRITER_WARN_IF_CANNOT_SEND

void FReplicationWriter::SerializeObjectStateDelta(FNetSerializationContext& Context, uint32 InternalIndex, const FReplicationInfo& Info, const FNetRefHandleManager::FReplicatedObjectData& ObjectData, const uint8* ReplicatedObjectStateBuffer, FDeltaCompressionBaseline& CurrentBaseline, uint32 CreatedBaselineIndex)
{
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	// Write baseline info
	Writer.WriteBits(Info.LastAckedBaselineIndex, FDeltaCompressionBaselineManager::BaselineIndexBitCount);
	if (Info.LastAckedBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
	{
		// Verify assumptions made about new baselineindices
		check(CurrentBaseline.IsValid());
		check(CreatedBaselineIndex == FDeltaCompressionBaselineManager::InvalidBaselineIndex || CreatedBaselineIndex == (Info.LastAckedBaselineIndex + 1) % FDeltaCompressionBaselineManager::MaxBaselineCount);

		// Do we want to store a new baseline?
		Writer.WriteBool(CreatedBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex);

		UE_NET_TRACE_SCOPE(DeltaCompressed, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

		FReplicationProtocolOperations::SerializeWithMaskDelta(Context, Info.GetChangeMaskStoragePointer(), ReplicatedObjectStateBuffer, CurrentBaseline.StateBuffer, ObjectData.Protocol);
	}
	else
	{
		// if we do not have a valid LastAckedBaselineIndex we need to write the full CreatedBaselineIndex
		Writer.WriteBits(CreatedBaselineIndex, FDeltaCompressionBaselineManager::BaselineIndexBitCount);

		// $IRIS: $TODO: Consider Delta compressing against default state
		// Write non delta compressed state
		FReplicationProtocolOperations::SerializeWithMask(Context, Info.GetChangeMaskStoragePointer(), ReplicatedObjectStateBuffer, ObjectData.Protocol);	
	}
}

FReplicationWriter::EWriteObjectStatus FReplicationWriter::WriteObjectAndSubObjects(FNetSerializationContext& Context, uint32 InternalIndex, FWriteObjectTraits WriteObjectTraits, FBatchInfo& OutBatchInfo)
{
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();
	
	FReplicationInfo& Info = GetReplicationInfo(InternalIndex);
	const EReplicatedObjectState State = Info.GetState();

	// As an object might still have subobjects pending destroy in the list of subobjects
	if (State == EReplicatedObjectState::Invalid || !ensureMsgf(State > EReplicatedObjectState::Invalid && State < EReplicatedObjectState::PendingDestroy, TEXT("Unsupported state %s. Object: %s IsSubObject: %d HadDirtyBit: %d ( InternalIndex: %u )"),
		  ToCStr(LexToString(State)),
		  ToCStr(NetRefHandleManager->PrintObjectFromIndex(InternalIndex)),
		  Info.IsSubObject,
		  ObjectsWithDirtyChanges.GetBit(InternalIndex),
		  InternalIndex))
	{
		return EWriteObjectStatus::InvalidState;
	}

	// If this object or anything else included in the batch did not write any data we will rollback any data written for the object
	FNetBitStreamRollbackScope ObjectRollbackScope(Writer);

	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);
	const FNetRefHandle NetRefHandle = ObjectData.RefHandle;

	IRIS_PROFILER_PROTOCOL_NAME(ObjectData.Protocol?ObjectData.Protocol->DebugName->Name:TEXT("NoProtocol"));

#if UE_NET_TRACE_ENABLED
	FNetRefHandle NetRefHandleForTraceScope = NetRefHandle;
	if (InternalIndex == ObjectIndexForOOBAttachment)
	{
		const ENetObjectAttachmentType OOBWriteAttachmentType = (ENetObjectAttachmentType)WriteObjectTraits.OOBWriteAttachmentType;
		if (OOBWriteAttachmentType == ENetObjectAttachmentType::HugeObject || OOBWriteAttachmentType == ENetObjectAttachmentType::DebugObject)
		{
			const FHugeObjectSendQueue& HugeObjectSendQueueForDebug = OOBWriteAttachmentType == ENetObjectAttachmentType::HugeObject ? HugeObjectSendQueue : DebugObjectSendQueue;
			const FInternalNetRefIndex HugeObjectInternalIndex = HugeObjectSendQueueForDebug.GetRootObjectInternalIndexForTrace();
			if (HugeObjectInternalIndex != InvalidInternalNetRefIndex)
			{
				NetRefHandleForTraceScope = NetRefHandleManager->GetReplicatedObjectDataNoCheck(HugeObjectInternalIndex).RefHandle;
			}
		}
	}
	UE_NET_TRACE_NAMED_OBJECT_SCOPE(ObjectTraceScope, NetRefHandleForTraceScope, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
#endif

	// We only need to write batch header for the root object
	const bool bWriteBatchHeader = !Info.IsSubObject;
	uint32 InitialStateHeaderPos = 0U;
	const uint32 NumBitsUsedForBatchSize = WriteObjectTraits.bIsWritingHugeObjectBatch == 0U ? Parameters.NumBitsUsedForBatchSize : Parameters.NumBitsUsedForHugeObjectBatchSize;

	// This is the beginning of what we treat as a batch on the receiving end
	if (bWriteBatchHeader)
	{
		// Write bit indicating that we are not a destruction info.
		constexpr bool bIsDestructionInfo = false;
		Writer.WriteBool(bIsDestructionInfo);

		WriteSentinel(&Writer, TEXT("RootObjectBatchInfo"));

		// A batch starts with (RefHandleId | BatchSize | bHasBatchObjectData | bHasExports | bHasCreationDependencyHandles | ?CreationParentHandleList)
		// We write the header up front, and then we seek back and update relevant info is the object + subobjects is successfully serialized along with necessary exports

		// We send the Index of the handle to the remote end
		// $IRIS: $TODO: consider sending the internal index instead to save bits and only send handle when we create the object, https://jira.it.epicgames.com/browse/UE-127373
		WriteNetRefHandleId(Context, NetRefHandle);

		InitialStateHeaderPos = Writer.GetPosBits();
		{
			UE_NET_TRACE_SCOPE(BatchSize, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
			Writer.WriteBits(0U, NumBitsUsedForBatchSize);
		}

		// Did we write serialize any data related to batch owner
		constexpr bool bHasBatchOwnerData = false;
		Writer.WriteBool(bHasBatchOwnerData);

		// If the batch has exports, they are at the end of the batch
		// We handle this on the reading side to avoid rewriting the entire object to insert exports up front.	
		constexpr bool bHasExports = false;
		Writer.WriteBool(bHasExports);

		// Write the object creation dependencies
		//TODO: Only send the CreationDependency data when it's an InitState creation data payload

		TConstArrayView<const FInternalNetRefIndex> CreationDependencies = NetRefHandleManager->GetCreationDependencies(InternalIndex);

		if (CreationDependencies.IsEmpty())
		{
			constexpr bool bHasCreationDependencyHandles = false;
			Writer.WriteBool(bHasCreationDependencyHandles);
		}
		else
		{
			for (FInternalNetRefIndex ParentInternalIndex : CreationDependencies)
			{
				constexpr bool bHasCreationDependencyHandles = true;
				Writer.WriteBool(bHasCreationDependencyHandles);

				const FNetRefHandle ParentHandle = NetRefHandleManager->GetNetRefHandleFromInternalIndex(ParentInternalIndex);
				ensureMsgf(ParentHandle.IsValid(), TEXT("Invalid parent found in CreationDependency of child: %s. Stored parent was:  (InternalIndex: %u)"), *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), ParentInternalIndex);

				//UE_LOGF(LogIris, Verbose, "Writing CreationDependency to child: %ls | Parent: %ls", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), *NetRefHandleManager->PrintObjectFromIndex(ParentInternalIndex));
				
				WriteNetRefHandle(Context, ParentHandle);
			}

			// Flag that this was the last handle
			constexpr bool bHasCreationDependencyHandles = false;
			Writer.WriteBool(bHasCreationDependencyHandles);
		}

		if (Writer.IsOverflown())
		{
			return EWriteObjectStatus::BitStreamOverflow;
		}
	}
	
	// Create a temporary batch entry. We don't want to push it to the batch info unless we're successful.
	FBatchObjectInfo BatchEntry = {};

	uint32 CreatedBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
	FDeltaCompressionBaseline CurrentBaseline;

	// We need to release created baseline if we fail to commit anything to batchrecord
	ON_SCOPE_EXIT
	{
		if (CreatedBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
		{
			UE_IRIS_PARALLEL_EXPR(FScopedBaselineAccess BaselineRW(BaselineManager, true));
			UE_LOG_REPWRITER(VeryVerbose, TEXT("Destroy cancelled baseline %u for ( InternalIndex: %u )"), CreatedBaselineIndex, InternalIndex);
			BaselineManager->DestroyBaseline(Parameters.ConnectionId, InternalIndex, CreatedBaselineIndex);
		}
	};

	// Only write data for the object if we have data to write
	uint8* ReplicatedObjectStateBuffer = NetRefHandleManager->GetReplicatedObjectStateBufferNoCheck(InternalIndex);

	const bool bIsInitialState = IsInitialState(State);
	if (InternalIndex != ObjectIndexForOOBAttachment && !ObjectData.Protocol)
	{
		ensureMsgf(ObjectData.Protocol, TEXT("Failed to replicate ( InternalIndex: %u ) %s, Protocol: nullptr, InstanceProtocol pointer: %p, HasCachedCreationInfo: %u"), InternalIndex, *NetRefHandle.ToString(), ObjectData.InstanceProtocol, ObjectData.bHasCachedCreationInfo);
		return EWriteObjectStatus::NoInstanceProtocol;
	}

	// Objects affected by conditionals might need to modify the changemask
	const bool bNeedToFilterChangeMask = (bIsInitialState || Info.HasDirtyChangeMask || Info.HasDirtyConditionals) && Info.HasChangemaskFilter;
	if (bNeedToFilterChangeMask)
	{
		ApplyFilterToChangeMask(OutBatchInfo.ParentInternalIndex, InternalIndex, Info, ObjectData.Protocol, ReplicatedObjectStateBuffer, bIsInitialState);
#if UE_NET_IRIS_CSV_STATS
		if (!bIsInitialState && Info.HasDirtyChangeMask)
		{
			WriteContext.Stats.AddNumberOfReplicatedObjectStatesMaskedOut(1U);
		}
#endif
	}

	// Even if root is not affected by dirty conditionals one of our subobjects might be.
	if (!Info.IsSubObject && Info.HasDirtyConditionals)
	{
		for (uint32 SubObjectIndex : NetRefHandleManager->GetSubObjects(InternalIndex))
		{
			FReplicationInfo& SubObjectInfo = GetReplicationInfo(SubObjectIndex);

			// Need to be a bit careful what we explicitly dirty here, as we might have subobjects waiting for creation confirmation.
			if ((SubObjectInfo.GetState() != EReplicatedObjectState::Invalid && SubObjectInfo.GetState() < EReplicatedObjectState::PendingDestroy) && CanSendObject(SubObjectIndex))
			{
				// Better safe than sorry, we do not want to dirty something going out of scope.
				if (ObjectsInScope.GetBit(SubObjectIndex))
				{
					MarkObjectDirty(SubObjectIndex, "UpdateGlobalConditional");
				}
				SubObjectInfo.HasDirtyConditionals = 1;
			}

			// Always process subobjects when we have updated conditionals to ensure that nested conditionals are processed.
			Info.HasDirtySubObjects = 1;
		}
	}
	Info.HasDirtyConditionals = 0;

	// For replicated subobject and tear off of created SubObjects that are filtered out by SubObject conditionals we want to replicate the destroy/tearoff but skip the data.
	// Note: IsFilteredOutSubObjectTearOff will only be set to ensure that a SubObject which has been replicated before being filtered out is property released.
	const bool bAllowStateAndAttachmentSending = !(Info.SubObjectPendingDestroy || Info.IsFilteredOutSubObjectTearOff);
	const bool bIsObjectIndexForAttachment = IsObjectIndexForOOBAttachment(InternalIndex);
	const bool bHasState = (bIsInitialState || Info.HasDirtyChangeMask) && !!WriteObjectTraits.bWriteState && bAllowStateAndAttachmentSending;
	const bool bHasAttachments = (Info.HasAttachments || bIsObjectIndexForAttachment) && bAllowStateAndAttachmentSending;
	const bool bWriteAttachments = bHasAttachments && !!(WriteObjectTraits.bWriteAttachments) && bAllowStateAndAttachmentSending;
	BatchEntry.bHasUnsentAttachments = bHasAttachments;

	// Check if we must defer tearoff until after flush
	const bool bSentTearOff = Info.TearOff && (GetFlushStatus(InternalIndex, Info, uint32(Info.FlushFlags | EFlushFlags::FlushFlags_FlushTornOffSubObjects)) == EFlushFlags::FlushFlags_None);

	Context.SetIsInitState(bIsInitialState);

	if (bHasState | bWriteAttachments | bSentTearOff | Info.SubObjectPendingDestroy)
	{
#if UE_NET_REPLICATIONDATASTREAM_DEBUG
		uint32 BatchSizePos = Writer.GetPosBits();
		const uint32 BatchSizeBits = EnumHasAnyFlags(StreamDebugFeatures, EReplicationDataStreamDebugFeatures::BatchSizePerObject) ? Parameters.NumBitsUsedForHugeObjectBatchSize : 0U;
		if (BatchSizeBits)
		{
			UE_NET_TRACE_SCOPE(BatchSize, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
			Writer.WriteBits(0U, BatchSizeBits);
		}
#endif

		// Only need to write the handle if this is a subobject
		if (Info.IsSubObject)
		{
			// We send the Index of the handle to the remote end
			// $IRIS: $TODO: consider sending the internal index instead to save bits and only send handle when we create the object, https://jira.it.epicgames.com/browse/UE-127373
			UE_NET_TRACE_SCOPE(SubObjectHandle, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
			WriteNetRefHandleId(Context, NetRefHandle);
		}

		// Store position of destroy header bits
		const uint32 ReplicatedDestroyHeaderBitPos = Writer.GetPosBits();

		// We only need to write this for actual replicated objects
		const bool bWriteReplicatedDestroyHeader = !bIsObjectIndexForAttachment;
		if (bWriteReplicatedDestroyHeader)
		{
			// Write destroy header bits, we always want to write the same number of bits to be able to update the header afterwards when we know what data made it into the packet
			Writer.WriteBits((uint32)EReplicatedDestroyHeaderFlags::None, GetDestroyHeaderFlagsBitCount());
		}

		if (Writer.WriteBool(bHasState))
		{
			WriteSentinel(&Writer, TEXT("HasState"));

			BatchEntry.bSentState = 1;

			// If the last transmitted baseline is acknowledged we can request a new baseline to be stored for the current state, we cannot compress against it until it has been acknowledged
			if (Info.IsDeltaCompressionEnabled)
			{
				UE_IRIS_PARALLEL_EXPR(FScopedBaselineAccess BaselineRW(BaselineManager, true));
				// Lookup current baseline that we should compress against
				if (Info.LastAckedBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
				{
					CurrentBaseline = BaselineManager->GetBaseline(Parameters.ConnectionId, InternalIndex, Info.LastAckedBaselineIndex);

					// If we cannot find the baseline it has become invalidated, if that is the case we must invalidate all tracking and request a new baseline to be created
					if (!CurrentBaseline.IsValid())
					{
						InvalidateBaseline(InternalIndex, Info);
					}
				}

				if (Info.PendingBaselineIndex == FDeltaCompressionBaselineManager::InvalidBaselineIndex)
				{
					// For new objects we start with baselineindex 0
					const uint32 NextBaselineIndex = bIsInitialState ? 0U : (Info.LastAckedBaselineIndex + 1U) % FDeltaCompressionBaselineManager::MaxBaselineCount;
					FDeltaCompressionBaseline NewBaseline = BaselineManager->CreateBaseline(Parameters.ConnectionId, InternalIndex, NextBaselineIndex);				
					if (NewBaseline.IsValid())
					{
						CreatedBaselineIndex = NextBaselineIndex;

						// $IRIS: $TODO: Currently due to how repnotifies are implemented we might have to write an extra changemask when sending a new baseline to avoid extra calls to repnotifies
						// Modify changemask to include any data we have in flight to ensure baseline integrity on receiving end
						if (PatchupObjectChangeMaskWithInflightChanges(InternalIndex, Info))
						{
							// Mask off changemasks that may have been disabled due to conditionals.
							ApplyFilterToChangeMask(OutBatchInfo.ParentInternalIndex, InternalIndex, Info, ObjectData.Protocol, ReplicatedObjectStateBuffer, bIsInitialState);
						}

						UE_LOG_REPWRITER(VeryVerbose, TEXT("Created new baseline %u for ( InternalIndex: %u )"), CreatedBaselineIndex, InternalIndex);
					}
				}
			}

			// $TODO: Consider rewriting the FReplicationProtocolOperations::SerializeWithMask() methods to accept the changemask passed in the Context rather setting it up again later
			FNetBitArrayView ChangeMask = MakeNetBitArrayView(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount);
			Context.SetChangeMask(&ChangeMask);

			// Collect potential exports and append them to the list of pending exports to be exported with the batch
			CollectAndAppendExports(Context, ReplicatedObjectStateBuffer, ObjectData.Protocol);
			
			// Role downgrade
			{
				FInternalNetSerializationContext* InternalContext = Context.GetInternalContext();
				InternalContext->bDowngradeAutonomousProxyRole = (Context.GetLocalConnectionId() != ReplicationFiltering->GetOwningConnection(InternalIndex));
				InternalContext->bAlwaysSwapRolesOnReplication = InternalContext->ReplicationSystem->AlwaysSwapRolesOnReplication();
			}

			if (Writer.WriteBool(bIsInitialState))
			{
				// Creation Info
				{
					UE_NET_TRACE_SCOPE(CreationInfo, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

					// Warn if we cannot replicate this object
					if (!ObjectData.Protocol || (!ObjectData.InstanceProtocol && !(ObjectData.bHasCachedCreationInfo == 1U)))
					{
						UE_LOG_REPWRITER_WARNING(TEXT("Failed to replicate ( InternalIndex: %u ) %s, ProtocolName: %s, InstanceProtocol pointer: %p, HasCachedCreationInfo: %u"), InternalIndex, *NetRefHandle.ToString(), (ObjectData.Protocol ? ToCStr(ObjectData.Protocol->DebugName) : TEXT("nullptr")), ObjectData.InstanceProtocol, ObjectData.bHasCachedCreationInfo);
						ensureMsgf(ObjectData.Protocol, TEXT("Failed to replicate ( InternalIndex: %u ) %s, Protocol: nullptr, InstanceProtocol pointer: %p, HasCachedCreationInfo: %u"), InternalIndex, *NetRefHandle.ToString(), ObjectData.InstanceProtocol, ObjectData.bHasCachedCreationInfo);
						return EWriteObjectStatus::NoInstanceProtocol;
					}

					if (Writer.WriteBool(Info.IsDeltaCompressionEnabled))
					{
						// As we might fail to create a baseline for initial state we need to include it here.
						Writer.WriteBits(CreatedBaselineIndex, FDeltaCompressionBaselineManager::BaselineIndexBitCount);
					}

					const bool bIsDestructionInfo = (Info.IsDestructionInfo == 1U);					
					FReplicationBridgeSerializationContext BridgeContext(Context, Parameters.ConnectionId, bIsDestructionInfo);

					bool bWriteSuccess = false;
					if (Info.IsDestructionInfo)
					{
						bWriteSuccess = WriteNetRefHandleDestructionInfo(Context, NetRefHandle);
					}
					else
					{
						bWriteSuccess = FBridgeSerialization::WriteNetRefHandleCreationInfo(ReplicationBridge, BridgeContext, NetRefHandle, InternalIndex);
					}

					// We need to send creation info, so if we fail we skip this object for now
					if (!bWriteSuccess)
					{
						if (!Context.HasErrorOrOverflow())
						{
							UE_LOG_REPWRITER_WARNING(TEXT("Failed to replicate ( InternalIndex: %u ) %s, ProtocolName: %s, InstanceProtocol pointer: %p, HasCachedCreationInfo: %u"), InternalIndex, *NetRefHandle.ToString(), (ObjectData.Protocol ? ToCStr(ObjectData.Protocol->DebugName) : TEXT("nullptr")), ObjectData.InstanceProtocol, ObjectData.bHasCachedCreationInfo);
							ensureMsgf(ObjectData.Protocol, TEXT("Failed to replicate ( InternalIndex: %u ) %s, Protocol: nullptr, InstanceProtocol pointer: %p, HasCachedCreationInfo: %u"), InternalIndex, *NetRefHandle.ToString(), ObjectData.InstanceProtocol, ObjectData.bHasCachedCreationInfo);

							// Unforced error, treat it as we have no instance and cannot create this object but we can continue with other objects
							return EWriteObjectStatus::NoInstanceProtocol;
						}
						else
						{
							return Context.HasError() ? EWriteObjectStatus::Error : EWriteObjectStatus::BitStreamOverflow;
						}
					}
				}
				// Serialize initial state data for this object using delta compression against default state
				FReplicationProtocolOperations::SerializeInitialStateWithMask(Context, Info.GetChangeMaskStoragePointer(), ReplicatedObjectStateBuffer, ObjectData.Protocol);

				UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_AND_COUNT_FOR_OBJECT(Context.GetNetStatsContext(), Writer.GetPosBits() - ObjectRollbackScope.GetStartPos(), WriteCreationInfo, InternalIndex);
			}
			else
			{
				if (Info.IsDeltaCompressionEnabled)
				{
					UE_IRIS_PARALLEL_EXPR(FScopedBaselineAccess BaselineReadOnly(BaselineManager, false));
					SerializeObjectStateDelta(Context, InternalIndex, Info, ObjectData, ReplicatedObjectStateBuffer, CurrentBaseline, CreatedBaselineIndex);
				}
				else
				{
					// Serialize state data for this object
					FReplicationProtocolOperations::SerializeWithMask(Context, Info.GetChangeMaskStoragePointer(), ReplicatedObjectStateBuffer, ObjectData.Protocol);
				}
			}

			WriteSentinel(&Writer, TEXT("HasStateEnd"));
		}

		{
			const uint32 HasAttachmentsWritePos = Writer.GetPosBits();
			Writer.WriteBool(bWriteAttachments);
			if (Writer.IsOverflown())
			{
				return EWriteObjectStatus::BitStreamOverflow;
			}

			if (bWriteAttachments)
			{
				FNetBitStreamWriter AttachmentWriter = Writer.CreateSubstream();
				FNetSerializationContext AttachmentContext = Context.MakeSubContext(&AttachmentWriter);
				BatchEntry.AttachmentType = ENetObjectAttachmentType::Normal;
				if (bIsObjectIndexForAttachment)
				{
					check((ENetObjectAttachmentType)WriteObjectTraits.OOBWriteAttachmentType != ENetObjectAttachmentType::Normal);
					BatchEntry.AttachmentType = (ENetObjectAttachmentType)WriteObjectTraits.OOBWriteAttachmentType;
					AttachmentWriter.WriteBits(WriteObjectTraits.OOBWriteAttachmentType, NetObjectAttachmentTypeBitCount);
				}

				const EAttachmentWriteStatus AttachmentWriteStatus = Attachments.Serialize(AttachmentContext, BatchEntry.AttachmentType, InternalIndex, NetRefHandle, BatchEntry.AttachmentRecord, BatchEntry.bHasUnsentAttachments);
				if (BatchEntry.AttachmentType == ENetObjectAttachmentType::HugeObject)
				{
					if (AttachmentWriteStatus == EAttachmentWriteStatus::ReliableWindowFull)
					{
						HugeObjectSendQueue.Stats.StartStallTime = FPlatformTime::Cycles64();
					}
					else
					{
						// Clear stall time now that we were theoretically able to send something.
						HugeObjectSendQueue.Stats.StartStallTime = 0;
					}
				}
				else if (BatchEntry.AttachmentType == ENetObjectAttachmentType::DebugObject)
				{
					if (AttachmentWriteStatus == EAttachmentWriteStatus::ReliableWindowFull)
					{
						DebugObjectSendQueue.Stats.StartStallTime = FPlatformTime::Cycles64();
					}
					else
					{
						// Clear stall time now that we were theoretically able to send something.
						DebugObjectSendQueue.Stats.StartStallTime = 0;
					}			
				}

				// If we didn't manage to fit any attachments then clear the HasAttachments bool in the packet
				if (AttachmentWriter.GetPosBits() == 0 || AttachmentWriter.IsOverflown())
				{
					BatchEntry.bSentAttachments = 0;

					const uint32 BitsThatWasAvailableForAttachements = Writer.GetBitsLeft();

					Writer.DiscardSubstream(AttachmentWriter);
					{
						FNetBitStreamWriteScope HasAttachmentsWriteScope(Writer, HasAttachmentsWritePos);
						Writer.WriteBool(false);
					}

					Info.HasAttachments = BatchEntry.bHasUnsentAttachments;
					
					// If we should have had enough space to write a attachment, the attachment + exports must be huge and we need to fall back on using the huge object path
					const uint32 SplitThreshold = PartialNetObjectAttachmentHandler->GetConfig()->GetBitCountSplitThreshold() * 2;
					const bool bFallbackToHugeObjectPath = !(BatchEntry.AttachmentType == ENetObjectAttachmentType::HugeObject || BatchEntry.AttachmentType == ENetObjectAttachmentType::DebugObject) && BatchEntry.bHasUnsentAttachments && (BitsThatWasAvailableForAttachements >= SplitThreshold);
					if (bFallbackToHugeObjectPath)
					{
						UE_LOGF(LogIris, Verbose, "Failed to write large attachment for object %ls ( InternalIndex: %u ), forcing fallback on hugeobject for attachments", *NetRefHandle.ToString(), InternalIndex);
						Writer.DoOverflow();
					}
					else if (!(BatchEntry.bSentState || bSentTearOff || Info.SubObjectPendingDestroy || Info.HasDirtySubObjects || bWriteBatchHeader))
					{
						// If we didn't send state and didn't send any attachments let's rollback
						ObjectRollbackScope.Rollback();
					}
				}
				else
				{
					BatchEntry.bSentAttachments = 1;

					Writer.CommitSubstream(AttachmentWriter);

					// Update the HasAttachments info based on this object batch failing. If the batch is a success we update again.
					Info.HasAttachments = Attachments.HasUnsentAttachments(BatchEntry.AttachmentType, InternalIndex);
				}
			}
		}

		if (Writer.IsOverflown())
		{
			UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_FOR_OBJECT_AS_WASTE(Context.GetNetStatsContext(), Writer.GetPosBits() - ObjectRollbackScope.GetStartPos(), Write, InternalIndex);
			return EWriteObjectStatus::BitStreamOverflow;
		}

#if UE_NET_REPLICATIONDATASTREAM_DEBUG
		// Write batch size if we didn't roll back the whole thing already.
		if (BatchSizeBits && (Writer.GetPosBits() > BatchSizePos))
		{
			const uint32 BatchSize = Writer.GetPosBits()  - BatchSizePos - BatchSizeBits;
			FNetBitStreamWriteScope WriteScope(Writer, BatchSizePos);
			Writer.WriteBits(BatchSize, BatchSizeBits);
		}
#endif

		if (bWriteReplicatedDestroyHeader)
		{
			// Rewrite destroy header if necessary	
			if (bSentTearOff || Info.SubObjectPendingDestroy)
			{
				EReplicatedDestroyHeaderFlags ReplicatedDestroyHeaderFlags = EReplicatedDestroyHeaderFlags::None;

				if (bSentTearOff)
				{
					ReplicatedDestroyHeaderFlags |= EReplicatedDestroyHeaderFlags::TearOff;
				}

				// Write SubObject destroy
				if (Info.SubObjectPendingDestroy)
				{
					ReplicatedDestroyHeaderFlags |= EReplicatedDestroyHeaderFlags::IsSubObject;
					
					const bool bShouldDestroyInstance = ObjectData.RefHandle.IsDynamic() || NetRefHandleManager->IsDestroyedStartupObject(InternalIndex);					
					if (bShouldDestroyInstance)
					{
						ReplicatedDestroyHeaderFlags |= EReplicatedDestroyHeaderFlags::DestroySubObject;
					}
				}

				FNetBitStreamWriteScope WriteScope(Writer, ReplicatedDestroyHeaderBitPos);
				Writer.WriteBits((uint32)ReplicatedDestroyHeaderFlags, GetDestroyHeaderFlagsBitCount());
			}
			else if (bWriteBatchHeader && !(BatchEntry.bSentState || BatchEntry.bSentAttachments))
			{
				// No need for the destroy header as we did not write any data at all for the batch.
				Writer.Seek(ReplicatedDestroyHeaderBitPos);
			}
		}
	}

	UE_NET_TRACE_EXIT_NAMED_SCOPE(ObjectTraceScope);

	// Success so far. Fill in batch entry. Keep index to update info later as the array can resize.
	const int ParentBatchEntryIndex = OutBatchInfo.ObjectInfos.Num();
	{
		FBatchObjectInfo& FinalBatchEntry = OutBatchInfo.ObjectInfos.Emplace_GetRef();
		FinalBatchEntry = MoveTemp(BatchEntry);

		FinalBatchEntry.bIsInitialState = bIsInitialState;
		FinalBatchEntry.InternalIndex = InternalIndex;
		FinalBatchEntry.bHasDirtySubObjects = false;
		FinalBatchEntry.bSentTearOff = bSentTearOff;
		FinalBatchEntry.bSentDestroySubObject = Info.SubObjectPendingDestroy;
		FinalBatchEntry.NewBaselineIndex = CreatedBaselineIndex;
		if (InternalIndex != ObjectIndexForOOBAttachment)
		{
			// Mark this object as written this tick to avoid sending it multiple times
			WriteContext.ObjectsWrittenThisPacket.SetBit(InternalIndex);
		}
	}

	// Reset CreatedBaselineIndex to avoid it being released on scope exit
	CreatedBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;

	// Write dirty sub objects
	const uint32 SubObjectStartPos = Writer.GetPosBits();
	uint32 SubObjectsWrittenBits = 0U;
	if (Info.HasDirtySubObjects && !Info.IsSubObject)
	{
		bool bHasDirtySubObjects = false;
		
		FReplicationConditionals::FSubObjectsToReplicateArray SubObjectsToReplicate;
		ReplicationConditionals->GetSubObjectsToReplicate(Parameters.ConnectionId, InternalIndex, SubObjectsToReplicate);

		// We need to do some additional logic for filtered out subobjects, pending destroy or tearoff
		for (uint32 SubObjectIndex : NetRefHandleManager->GetSubObjects(InternalIndex))
		{
			FReplicationInfo& SubObjectInfo = GetReplicationInfo(SubObjectIndex);
			if (SubObjectInfo.GetState() == EReplicatedObjectState::SubObjectPendingDestroy || (SubObjectInfo.IsCreationConfirmed && SubObjectInfo.TearOff))
			{
				if (!SubObjectsToReplicate.Contains(SubObjectIndex))
				{
					SubObjectsToReplicate.Add(SubObjectIndex);
					if (SubObjectInfo.TearOff)
					{
						SubObjectInfo.IsFilteredOutSubObjectTearOff = 1U;
					}
				}
			}
			else if (SubObjectInfo.GetState() == EReplicatedObjectState::PendingCreate)
			{
				if (!SubObjectsToReplicate.Contains(SubObjectIndex))
				{
					SubObjectInfo.IsFilteredOutSubObjectPendingCreate = 1U;
				}
			}
		}
		
		for (FInternalNetRefIndex SubObjectInternalIndex : SubObjectsToReplicate)
		{
			if (!ObjectsWithDirtyChanges.GetBit(SubObjectInternalIndex))
			{
				continue;
			}

			const int BatchObjectInfoCount = OutBatchInfo.ObjectInfos.Num();
			EWriteObjectStatus SubObjectWriteStatus = WriteObjectAndSubObjects(Context, SubObjectInternalIndex, WriteObjectTraits, OutBatchInfo);
			if (!IsWriteObjectSuccess(SubObjectWriteStatus))
			{
				// SubObject will rollback on fail (and report its own waste) but we as we will rollback successfully written subobjects it is better to at least report it with the owner.
				UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_FOR_OBJECT_AS_WASTE(Context.GetNetStatsContext(), Writer.GetPosBits() - ObjectRollbackScope.GetStartPos(), Write, InternalIndex);
				return SubObjectWriteStatus;
			}

			// There are success statuses where no object info is added. In such case we shouldn't read from it.
			if (OutBatchInfo.ObjectInfos.Num() > BatchObjectInfoCount)
			{
				const FBatchObjectInfo& SubObjectEntry = OutBatchInfo.ObjectInfos.Last();
				bHasDirtySubObjects |= SubObjectEntry.bHasDirtySubObjects || SubObjectEntry.bHasUnsentAttachments;
			}
		}

		SubObjectsWrittenBits = Writer.GetPosBits() - SubObjectStartPos;

		// Update parent batch info
		{
			FBatchObjectInfo& ParentBatchEntry = OutBatchInfo.ObjectInfos[ParentBatchEntryIndex];
			ParentBatchEntry.bHasDirtySubObjects |= bHasDirtySubObjects;
		}
	}

	// ObjectBatch ends here
	// We include the size of the data written so we can skip it if needed.
	if (OutBatchInfo.ParentInternalIndex == InternalIndex)
	{
		FBatchObjectInfo& ParentBatchEntry = OutBatchInfo.ObjectInfos[ParentBatchEntryIndex];
		
		const bool bWroteData = (ParentBatchEntry.bSentState || ParentBatchEntry.bSentAttachments || bSentTearOff || Info.SubObjectPendingDestroy);
		if (bWroteData || (SubObjectsWrittenBits != 0U))
		{
			const uint32 MaxBatchSize = NumBitsUsedForBatchSize == 32U ? ~0U : ((1U << NumBitsUsedForBatchSize) - 1U);
			const uint32 WrittenBitsInBatch = (Writer.GetPosBits() - InitialStateHeaderPos) - NumBitsUsedForBatchSize;

			// Validate size written (excluding exports)
			if (WrittenBitsInBatch >= MaxBatchSize)
			{
				UE_LOGF(LogIris, Error, "FReplicationWriter::WriteObjectAndSubObjects batch too large Conn: %u, WrittenBitsInBatch: %u >= MaxBatchSize:%u when writing object %ls ( InternalIndex: %u )", Parameters.ConnectionId, WrittenBitsInBatch, MaxBatchSize, *NetRefHandle.ToString(), InternalIndex);
				ensureMsgf(WrittenBitsInBatch >= MaxBatchSize, TEXT("FReplicationWriter::WriteObjectAndSubObjects batch too large WrittenBitsInBatch: %u >= MaxBatchSize:%u when writing object %s ( InternalIndex: %u )"), WrittenBitsInBatch, MaxBatchSize, *NetRefHandle.ToString(), InternalIndex);
				Context.SetError(NetError_ObjectStateTooLarge);
				Writer.DoOverflow();

				return EWriteObjectStatus::BitStreamOverflow;
			}

			const FObjectReferenceCache::EWriteExportsResult WriteExportResult = ObjectReferenceCache->WritePendingExports(Context, InternalIndex);

			if (WriteExportResult == FObjectReferenceCache::EWriteExportsResult::BitStreamOverflow)
			{
				// If we fail to write exports, we fail the entire object
				UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_FOR_OBJECT_AS_WASTE(Context.GetNetStatsContext(), Writer.GetPosBits() - ObjectRollbackScope.GetStartPos(), Write, InternalIndex);
				return EWriteObjectStatus::BitStreamOverflow;	
			}

			const bool bWroteExports = WriteExportResult == FObjectReferenceCache::EWriteExportsResult::WroteExports;

			// Update header
			if (ensure(bWriteBatchHeader))
			{
				FNetBitStreamWriteScope SizeScope(Writer, InitialStateHeaderPos);
				Writer.WriteBits(WrittenBitsInBatch, NumBitsUsedForBatchSize);
				Writer.WriteBool(bWroteData);
				Writer.WriteBool(bWroteExports);
			}

			ParentBatchEntry.bSentBatchData = 1U;

			UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_FOR_OBJECT(Context.GetNetStatsContext(), (Writer.GetPosBits() - ObjectRollbackScope.GetStartPos()) - SubObjectsWrittenBits, Write, InternalIndex);
		}
		// If we did not write any data we rollback any written headers and report a success
		else
		{
			// if we or our subobjects did not write any data, rollback and forget about everything
			ObjectRollbackScope.Rollback();
		}	
	}

	return EWriteObjectStatus::Success;
}

FReplicationWriter::EWriteObjectStatus FReplicationWriter::WriteObjectInBatch(FNetSerializationContext& Context, uint32 InternalIndex, FWriteObjectTraits WriteObjectTraits, FBatchInfo& OutBatchInfo)
{
	{
		UE_NET_IRIS_STATS_TIMER(Timer, Context.GetNetStatsContext());
		UE_NET_TRACE_WRITE_OBJECT_SCOPE(NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex).RefHandle, Timer);

		// Reset pending exports
		FNetExportContext* ExportContext = Context.GetExportContext();
		if (ExportContext)
		{
			ExportContext->ClearPendingExports();
		}

		// Write parent object and subobjects
		const EWriteObjectStatus WriteObjectStatus = WriteObjectAndSubObjects(Context, InternalIndex, WriteObjectTraits, OutBatchInfo);
		if (!IsWriteObjectSuccess(WriteObjectStatus))
		{
			UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT_AS_WASTE(Timer, Write, InternalIndex);
			return WriteObjectStatus;
		}

		UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT(Timer, Write, InternalIndex);
	}

	// Include dependent objects as separate batch, (for hugeobjects they will be included as they are written to a separate bitstream)
	{
		const uint32 OldBatchInfoParentInternalIndex = OutBatchInfo.ParentInternalIndex;
		for (const FDependentObjectInfo DependentObjectInfo : NetRefHandleManager->GetDependentObjectInfos(InternalIndex))
		{
			const FInternalNetRefIndex DependentInternalIndex = DependentObjectInfo.NetRefIndex;
			const bool bIsDependentInitialState = IsInitialState(GetReplicationInfo(DependentInternalIndex).GetState());
			if (bIsDependentInitialState && !WriteContext.ObjectsWrittenThisPacket.GetBit(DependentInternalIndex))
			{
				OutBatchInfo.ParentInternalIndex = DependentInternalIndex;
				EWriteObjectStatus DependentObjectWriteStatus = WriteObjectInBatch(Context, DependentInternalIndex, WriteObjectTraits, OutBatchInfo);
				if (!IsWriteObjectSuccess(DependentObjectWriteStatus))
				{
					// Restore ParentInternalIndex
					OutBatchInfo.ParentInternalIndex = OldBatchInfoParentInternalIndex;
					return DependentObjectWriteStatus;
				}
			}
		}

		// Restore ParentInternalIndex
		OutBatchInfo.ParentInternalIndex = OldBatchInfoParentInternalIndex;
	}

	return EWriteObjectStatus::Success;
}

FReplicationWriter::EWriteStatus FReplicationWriter::PrepareAndSendHugeObjectPayload(FNetSerializationContext& Context, FInternalNetRefIndex InternalIndex, ENetObjectAttachmentType AttachmentType)
{
	IRIS_PROFILER_SCOPE(FReplicationWriter_PrepareAndSendHugeObjectPayload);

	const bool bIsPreparingDebugObject = AttachmentType == ENetObjectAttachmentType::DebugObject;
	FHugeObjectSendQueue& InHugeObjectSendQueue = bIsPreparingDebugObject ?  DebugObjectSendQueue : HugeObjectSendQueue;

	// Sanity check
	if (InHugeObjectSendQueue.IsFull() || (!bIsPreparingDebugObject && InHugeObjectSendQueue.IsObjectInQueue(InternalIndex, false)))
	{
		ensureMsgf(false, TEXT("PrepareAndSendHugeObjectPayload (Huge|Debug)ObjectSendQueue should not be full or already transmitting: %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalIndex));
		return EWriteStatus::Skipped;
	}

	typedef uint32 HugeObjectStorageType;
	const uint32 BitsPerStorageWord = sizeof(HugeObjectStorageType) * 8;

	TArray<HugeObjectStorageType> HugeObjectPayload;
	HugeObjectPayload.AddUninitialized(static_cast<int32>(PartialNetObjectAttachmentHandler->GetConfig()->GetTotalMaxPayloadBitCount() + (BitsPerStorageWord - 1U))/BitsPerStorageWord);

	// Setup a special context for the huge object serialization.
	FNetBitStreamWriter HugeObjectWriter;
	const uint32 MaxHugeObjectPayLoadBytes = HugeObjectPayload.Num() * sizeof(HugeObjectStorageType);
	HugeObjectWriter.InitBytes(HugeObjectPayload.GetData(), MaxHugeObjectPayLoadBytes);
	FNetSerializationContext HugeObjectSerializationContext = Context.MakeSubContext(&HugeObjectWriter);

#if UE_NET_TRACE_ENABLED
	if (!InHugeObjectSendQueue.TraceCollector)
	{
		InHugeObjectSendQueue.TraceCollector = UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace);
	}
	else
	{
		InHugeObjectSendQueue.TraceCollector->Reset();
	}
	HugeObjectSerializationContext.SetTraceCollector(InHugeObjectSendQueue.TraceCollector);
#endif

	// Huge object header needed for the receiving side to be able to process this correctly.
	FNetObjectBlob::FHeader HugeObjectHeader = {};
	const uint32 HeaderPos = HugeObjectWriter.GetPosBits();
	FNetObjectBlob::SerializeHeader(HugeObjectSerializationContext, HugeObjectHeader);

	// As the huge object will most likely be processed out of order we need to write the stream debug features again
	WriteStreamDebugFeatures(HugeObjectSerializationContext);

	const uint32 PastHeaderPos = HugeObjectWriter.GetPosBits();

	FHugeObjectContext HugeObjectContext;

	FBatchInfo BatchInfo;
	BatchInfo.Type = EBatchInfoType::Internal;
	BatchInfo.ParentInternalIndex = InternalIndex;
	FWriteObjectTraits WriteObjectTraits({.bWriteState = 1U, .OOBWriteAttachmentType = (uint32)ENetObjectAttachmentType::OutOfBand, .bIsWritingHugeObjectBatch = 1U});

	// Get the creation going as quickly as possible for hugeobjects by not including attachments. For DebugObjects we include everything.
	const bool bIncludeAttachements = bIsPreparingDebugObject || !Context.IsInitState();
	if (bIncludeAttachements)
	{
		WriteObjectTraits.bWriteAttachments = 1U;
	}
	
	// Push new ExportContext for the hugeobject-batch as we cannot share exports with an OOB object
	{
		HugeObjectContext.BatchExports.Reset();
		FNetExports::FExportScope ExportScope = NetExports->MakeExportScope(HugeObjectSerializationContext, HugeObjectContext.BatchExports);

		UE_NET_TRACE_SCOPE(HugeObjectState, *HugeObjectSerializationContext.GetBitStreamWriter(), HugeObjectSerializationContext.GetTraceCollector(), ENetTraceVerbosity::Trace);

		// We can encounter other errors than bitstream overflow now that we've got a really large buffer to write to.
		const EWriteObjectStatus WriteHugeObjectStatus = WriteObjectInBatch(HugeObjectSerializationContext, InternalIndex, WriteObjectTraits, BatchInfo);

		// If we cannot fit the object in the largest supported buffer then we will never fit the object.
		if (WriteHugeObjectStatus == EWriteObjectStatus::BitStreamOverflow)
		{
			// Cleanup data from batch
			HandleObjectBatchFailure(WriteHugeObjectStatus, BatchInfo, WriteBitStreamInfo);

			const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);
			UE_LOGF(LogIris, Error, "Unable to fit %ls (protocol: %ls) in maximum combined payload of %u bytes. Connection %u will be disconnected.", 
				*NetRefHandleManager->PrintObjectFromIndex(InternalIndex), ToCStr(ObjectData.Protocol ? ObjectData.Protocol->DebugName : nullptr), 
				MaxHugeObjectPayLoadBytes, Context.GetLocalConnectionId());
			ensure(false);

			Context.SetError(NetError_ObjectStateTooLarge);
			return EWriteStatus::Abort;
		}

		const bool bWroteData = HugeObjectWriter.GetPosBits() > PastHeaderPos;
		// If we encounter some other error or did end up not writing any data due to full attachment queue etc. we skip the huge object and try to write other data instead.
		if (!IsWriteObjectSuccess(WriteHugeObjectStatus) || !bWroteData)
		{
			// Cleanup data from batch
			HandleObjectBatchFailure(WriteHugeObjectStatus, BatchInfo, WriteBitStreamInfo);

			UE_LOGF(LogIris, Verbose, "Problem writing huge object %ls. WriteObjectStatus: %u. Trying smaller object.", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), unsigned(WriteHugeObjectStatus));
			return EWriteStatus::Skipped;
		}
	}

	if (InHugeObjectSendQueue.IsEmpty())
	{
		InHugeObjectSendQueue.Stats.StartSendingTime = FPlatformTime::Cycles64();
	}

	HugeObjectContext.RootObjectInternalIndex = InternalIndex;

	// Store batch record for later processing once the whole state is acked.
	HandleObjectBatchSuccess(BatchInfo, HugeObjectContext.BatchRecord);
	// We want to track the number of Batches
	HugeObjectHeader.ObjectCount = HugeObjectContext.BatchRecord.BatchCount;

	// Write huge object header
	{
		FNetBitStreamWriteScope WriteScope(HugeObjectWriter, HeaderPos);
		FNetObjectBlob::SerializeHeader(HugeObjectSerializationContext, HugeObjectHeader);
	}

	HugeObjectWriter.CommitWrites();

	// Create a NetObjectBlob from the temporary buffer and split it into multiple smaller pieces.
	const uint32 PayLoadBitCount = HugeObjectWriter.GetPosBits();
	const uint32 StorageWordsWritten = (PayLoadBitCount + (BitsPerStorageWord - 1))/BitsPerStorageWord;

	check(StorageWordsWritten <= (uint32)HugeObjectPayload.Num());

	TArrayView<HugeObjectStorageType> PayloadView(HugeObjectPayload.GetData(), StorageWordsWritten);
	TRefCountPtr<FNetObjectBlob> NetObjectBlob = NetObjectBlobHandler->CreateNetObjectBlob(PayloadView, PayLoadBitCount);
	TArray<TRefCountPtr<FNetBlob>> PartialNetBlobs;
	const bool bSplitSuccess = PartialNetObjectAttachmentHandler->SplitRawDataNetBlob(TRefCountPtr<FRawDataNetBlob>(NetObjectBlob.GetReference()), PartialNetBlobs, InHugeObjectSendQueue.DebugName);
	if (!bSplitSuccess)
	{
		UE_LOGF(LogIris, Error, "Unable to split huge object %ls payload. Connection %u will be disconnected.", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), Context.GetLocalConnectionId());
		Context.SetError(NetError_ObjectStateTooLarge);
		return EWriteStatus::Abort;
	}

	const bool bEnqueueSuccess = Attachments.Enqueue(AttachmentType, ObjectIndexForOOBAttachment, MakeArrayView(PartialNetBlobs.GetData(), PartialNetBlobs.Num()));
	if (!bEnqueueSuccess)
	{
		UE_LOGF(LogIris, Error, "Unable to enqueue huge object attachments: %ls. Connection %u will be disconnected.", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), Context.GetLocalConnectionId());
		Context.SetError(GNetError_InternalError);
		return EWriteStatus::Abort;
	}

	// Add huge object to queue
	HugeObjectContext.Blobs = MoveTemp(PartialNetBlobs);
	const bool bHugeObjectWasEnqueued = InHugeObjectSendQueue.EnqueueHugeObject(HugeObjectContext);
	check(bHugeObjectWasEnqueued);
	if (!bHugeObjectWasEnqueued)
	{
		UE_LOGF(LogIris, Error, "Unable to enqueue huge object: %ls. Connection %u will be disconnected.", *NetRefHandleManager->PrintObjectFromIndex(InternalIndex), Context.GetLocalConnectionId());
		Context.SetError(GNetError_InternalError);
		return EWriteStatus::Abort;
	}

	// Write huge object attachment(s)
	if (bIsPreparingDebugObject)
	{
		// DebugObject attachments are sent separately to be able to not count them against bandwidth limits.
		return EWriteStatus::Skipped;
	}
	else
	{
		FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();
		UE_NET_TRACE_SCOPE(Batch, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		FNetBitStreamRollbackScope RollbackScope(Writer);
		
		FBatchInfo HugeObjectBatchInfo;
		HugeObjectBatchInfo.Type = EBatchInfoType::HugeObject;
		HugeObjectBatchInfo.ParentInternalIndex = InvalidInternalNetRefIndex;

		const FWriteObjectTraits WriteHugeObjectInfo({.bWriteAttachments = 1U, .OOBWriteAttachmentType = (uint32)ENetObjectAttachmentType::HugeObject});
		const EWriteObjectStatus HugeObjectStatus = WriteObjectInBatch(Context, ObjectIndexForOOBAttachment, WriteHugeObjectInfo, HugeObjectBatchInfo);
		const uint32 BitsWrittenByHugeObjectAttachments = Writer.GetPosBits() - RollbackScope.GetStartPos();
		if (!IsWriteObjectSuccess(HugeObjectStatus) || (BitsWrittenByHugeObjectAttachments == 0U))
		{
			// Need to call this in order to cleanup data associated with batch
			HandleObjectBatchFailure(HugeObjectStatus, HugeObjectBatchInfo, WriteBitStreamInfo);

			// It's unexpected, but not a critical error, if no part of the payload could be sent.
			// We do expect a smaller object to be sent though so that's why 0 is returned.
			ensureMsgf(BitsWrittenByHugeObjectAttachments == 0U || HugeObjectStatus == EWriteObjectStatus::BitStreamOverflow, TEXT("Expected split payload to not be able to generate other errors than overflow. Got %u"), unsigned(HugeObjectStatus));

			// Mark the context so that we can try to send the huge object in the next packet if we are allowed
			WriteContext.bHasHugeObjectToSend = 1;

			// Try to fit a smaller object.
			return EWriteStatus::Skipped;
		}

		FBatchRecord BatchRecord;
		HandleObjectBatchSuccess(HugeObjectBatchInfo, BatchRecord);
		CommitBatchRecord(BatchRecord);

		// If all chunks did not make it into the packet (expected) mark the context so that we can try to send the huge object in the next packet if we are allowed
		const bool bHasHugeObjectToSend = Attachments.HasUnsentAttachments(ENetObjectAttachmentType::HugeObject, ObjectIndexForOOBAttachment);
		WriteContext.bHasHugeObjectToSend = bHasHugeObjectToSend;
		if (!bHasHugeObjectToSend)
		{
			InHugeObjectSendQueue.Stats.EndSendingTime = FPlatformTime::Cycles64();
		}

		return EWriteStatus::Written;
	}
}

FReplicationWriter::FWriteBatchResult FReplicationWriter::WriteObjectBatch(FNetSerializationContext& Context, FInternalNetRefIndex InternalIndex, FWriteObjectTraits WriteObjectTraits)
{
	IRIS_PROFILER_SCOPE(FReplicationWriter_WriteObjectBatch);

	FReplicationInfo& ReplicationInfo = GetReplicationInfo(InternalIndex);

	// If this is a destruction info we treat it differently and just write the information required to destruct the object
	if (ReplicationInfo.IsDestructionInfo)
	{
		const EWriteStatus Status = WriteDestructionInfo(Context, InternalIndex);
		return FWriteBatchResult
		{
			.Status = Status,
			.NumWritten = (Status == EWriteStatus::Written) ? 1U : 0U,
		};
	}

	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	// We always send debug objects using the a special variant of the hugeobjectpath using a separate queue for written data.
	if (ReplicationInfo.IsDebugObject)
	{
		const EWriteStatus SendDebugObjectStatus = PrepareAndSendHugeObjectPayload(Context, InternalIndex, ENetObjectAttachmentType::DebugObject);

		// If the huge object wrote data it will be tracked as a single batch
		if (SendDebugObjectStatus == EWriteStatus::Written)
		{
			++WriteContext.WrittenBatchCount;
		}
				
		return FWriteBatchResult
		{
			.Status = SendDebugObjectStatus,
			.NumWritten = (SendDebugObjectStatus == EWriteStatus::Written) ? 1U : 0U,
		};
	}

	// Batch successful writes and commit them as a atomic batch.
	// It is a fail if we fail to write any subobject with dirty state.
	// It is also not ok to skip over creation header- if we do then the entire batch needs to be delayed.

	// Write object and subobjects. Try #1 - send state and attachments.
	{
		UE_NET_TRACE_SCOPE(Batch, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
		FNetBitStreamRollbackScope RollbackScope(Writer);
		FNetExportRollbackScope ExportRollbackScope(Context);

		WriteBitStreamInfo.BatchStartPos = Writer.GetPosBits();
		FBatchInfo BatchInfo;
		if (InternalIndex == ObjectIndexForOOBAttachment)
		{
			if ((ENetObjectAttachmentType)WriteObjectTraits.OOBWriteAttachmentType == ENetObjectAttachmentType::HugeObject)
			{
				BatchInfo.Type = EBatchInfoType::HugeObject;	
			}
			else if ((ENetObjectAttachmentType)WriteObjectTraits.OOBWriteAttachmentType == ENetObjectAttachmentType::DebugObject)
			{
				BatchInfo.Type = EBatchInfoType::DebugObject;	
			}
			else
			{
				BatchInfo.Type = EBatchInfoType::OOBAttachment;
			}
		}
		else
		{
			BatchInfo.Type = EBatchInfoType::Object;
		}
		BatchInfo.ParentInternalIndex = InternalIndex;

		// Write an object and its subobjects. If object has dependent objects pending creation we currently write them as well as an individual batch.
		const EWriteObjectStatus WriteObjectStatus = WriteObjectInBatch(Context, InternalIndex, WriteObjectTraits, BatchInfo);

		if (IsWriteObjectSuccess(WriteObjectStatus))
		{
			FBatchRecord BatchRecord;
			const uint32 WrittenObjectCount = HandleObjectBatchSuccess(BatchInfo, BatchRecord);

			// As a single batch also might include dependent objects which are treated as separate batches on the receiving end we need to account for this when tracking the written batch count.
			WriteContext.WrittenBatchCount += BatchRecord.BatchCount;

			CommitBatchRecord(BatchRecord);

			return FWriteBatchResult
			{
				.Status = EWriteStatus::Written,
				.NumWritten = WrittenObjectCount,
			};
		}

		const EWriteObjectRetryMode WriteRetryMode = HandleObjectBatchFailure(WriteObjectStatus, BatchInfo, WriteBitStreamInfo);

		// Regardless of the reason for fail we should rollback anything written.
		RollbackScope.Rollback();

		// Rollback exported references that was exported as part of the batch we just rolled back
		ExportRollbackScope.Rollback();

		switch (WriteRetryMode)
		{
			case EWriteObjectRetryMode::Abort:
			{
				return FWriteBatchResult{ .Status = EWriteStatus::Abort };
			}

			case EWriteObjectRetryMode::TrySmallObject:
			{
				++WriteContext.FailedToWriteSmallObjectCount;
				return FWriteBatchResult{ .Status = EWriteStatus::SkippedRetryNextPacket };
			}

			case EWriteObjectRetryMode::SplitHugeObject:
			{
				break;
			}

			default:
			{
				check(false);
			}
		}
	}

	// Try #2 - Object will be serialized to a temporary buffer of maximum supported size and split into multiple chunks.
	{
		const EWriteStatus SendHugeObjectStatus = PrepareAndSendHugeObjectPayload(Context, InternalIndex, ENetObjectAttachmentType::HugeObject);

		// If the huge object wrote data it will be tracked as a single batch
		if (SendHugeObjectStatus == EWriteStatus::Written)
		{
			++WriteContext.WrittenBatchCount;
		}
				
		return FWriteBatchResult
		{
			.Status = SendHugeObjectStatus,
			.NumWritten = (SendHugeObjectStatus == EWriteStatus::Written) ? 1U : 0U,
		};
	}
}

FReplicationWriter::EWriteStatus FReplicationWriter::WriteDestructionInfo(FNetSerializationContext& Context, uint32 InternalIndex)
{
	IRIS_PROFILER_SCOPE(FReplicationWriter_WriteDestructionInfo);

	const int32 MaxDestroyObjectsPerFrame = GReplicationWriterMaxDestroyObjectsPerFrame;
	if ((MaxDestroyObjectsPerFrame > 0) && (WriteContext.WrittenDestroyObjectCount >= static_cast<uint32>(MaxDestroyObjectsPerFrame)))
	{
		return EWriteStatus::Skipped;
	}

	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();
	UE_NET_TRACE_SCOPE(Batch, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	// Rollback for entire batch
	FNetBitStreamRollbackScope Rollback(Writer);
	FNetExportRollbackScope ExportRollbackScope(Context);

	// Only write data for the object if we have data to write
	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);

	// Special case for static objects that should be destroyed on the client but we have not replicated
	Writer.WriteBool(true);

	constexpr bool bIsDestructionInfo = true;
	FReplicationBridgeSerializationContext BridgeContext(Context, Parameters.ConnectionId, bIsDestructionInfo);

	// Push ForceInlineExportScope to inline exports instead of writing exports later.
	FForceInlineExportScope ForceInlineExportScope(Context.GetInternalContext());
	if (!WriteNetRefHandleDestructionInfo(Context, ObjectData.RefHandle))
	{
		// Trigger Rollback
		Writer.DoOverflow();

		return EWriteStatus::Skipped;
	}

	WriteSentinel(&Writer, TEXT("DestructionInfo"));

	// Push record
	if (!Writer.IsOverflown())
	{
		// We did write the initial state, change the state to WaitOnCreateConfirmation
		SetState(InternalIndex, EReplicatedObjectState::WaitOnCreateConfirmation);

		FReplicationInfo& Info = GetReplicationInfo(InternalIndex);

		FBatchObjectInfo ObjectInfo = {};
		ObjectInfo.InternalIndex = InternalIndex;
		FObjectRecord ObjectRecord;
		CreateObjectRecord(nullptr, Info, ObjectInfo, ObjectRecord);
		CommitObjectRecord(InternalIndex, ObjectRecord);

		Info.HasDirtyChangeMask = 0U;
		Info.HasDirtySubObjects = 0U;
		Info.HasAttachments = 0U;

		ObjectsWithDirtyChanges.ClearBit(InternalIndex);

#if UE_NET_ENABLE_REPLICATIONWRITER_CANNOT_SEND_WARNING
		if (Info.HasCannotSendInfo)
		{
			CannotSendInfos.Remove(InternalIndex);
			Info.HasCannotSendInfo = 0U;
		}
#endif

		// Reset scheduling priority
		SchedulingPriorities[InternalIndex] = 0.0f;

		WriteContext.Stats.AddNumberOfReplicatedDestructionInfos(1U);

		// We count this as an object batch
		++WriteContext.WrittenBatchCount;

		// We also count it as an object destroy.
		++WriteContext.WrittenDestroyObjectCount;
	}

	return Writer.IsOverflown() ? EWriteStatus::Abort : EWriteStatus::Written;
}

bool FReplicationWriter::WriteNetRefHandleDestructionInfo(FNetSerializationContext& Context, FNetRefHandle Handle)
{
	UE_NET_TRACE_SCOPE(DestructionInfo, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Trace);

	const FStaticDestructionInfo* Info = FBridgeSerialization::GetStaticDestructionInfo(ReplicationBridge, Handle);
	if (Info)
	{
		UE_LOGF(LogIris, VeryVerbose, "WriteNetRefHandleDestructionInfo on %ls | %ls | NetFactoryId: %u", *Handle.ToString(), *Info->StaticRef.ToString(), Info->NetFactoryId);
		// Write destruction info
		WriteFullNetObjectReference(Context, Info->StaticRef);
		Context.GetBitStreamWriter()->WriteBits(Info->NetFactoryId, FNetObjectFactoryRegistry::GetMaxBits());
	}
	else
	{
		UE_CALL_ONCE([&]()
		{
			UE_LOGF(LogIris, Error, "Unable to write destruction info object %ls in state %ls representing destroyed object %ls", *NetRefHandleManager->PrintObjectFromNetRefHandle(Handle), *PrintObjectInfo(NetRefHandleManager->GetInternalIndex(Handle)), *NetRefHandleManager->PrintObjectFromIndex(NetRefHandleManager->GetOriginalDestroyedStartupObjectIndex(NetRefHandleManager->GetInternalIndex(Handle))));
		}
		);
		ensureMsgf(false, TEXT("Unable to write destruction info object %s in state %s representing destroyed object %s"), *NetRefHandleManager->PrintObjectFromNetRefHandle(Handle), *PrintObjectInfo(NetRefHandleManager->GetInternalIndex(Handle)), *NetRefHandleManager->PrintObjectFromIndex(NetRefHandleManager->GetOriginalDestroyedStartupObjectIndex(NetRefHandleManager->GetInternalIndex(Handle))));
		Context.SetError(TEXT("DestructionInfoNotFound"), false);
		return false;
	}

	return !Context.HasErrorOrOverflow();
}

uint32 FReplicationWriter::WriteOOBAttachments(FNetSerializationContext& Context)
{
	uint32 WrittenObjectCount = 0U;

	if (WriteContext.WriteMode == EDataStreamWriteMode::PostTickDispatch)
	{
		if (WriteContext.bHasOOBAttachmentsToSend && CanSendObject(ObjectIndexForOOBAttachment) && (ReplicationRecord.GetUnusedInfoCount() > 0U))
		{
			IRIS_PROFILER_SCOPE(FReplicationWriter_WriteOOBAttachments);
			const FWriteBatchResult Result = WriteObjectBatch(Context, ObjectIndexForOOBAttachment, FWriteObjectTraits({.bWriteAttachments = 1U, .OOBWriteAttachmentType = (uint32)ENetObjectAttachmentType::OutOfBand}));
			if (Result.Status == EWriteStatus::Abort)
			{
				return WrittenObjectCount;
			}

			WriteContext.bHasOOBAttachmentsToSend = Attachments.HasUnsentAttachments(ENetObjectAttachmentType::OutOfBand, ObjectIndexForOOBAttachment);
			WrittenObjectCount += Result.NumWritten;
		}
	}
	else if (WriteContext.WriteMode == EDataStreamWriteMode::DebugData)
	{
		if (WriteContext.bHasDebugObjectToSend && (ReplicationRecord.GetUnusedInfoCount() > 0U))
		{
			IRIS_PROFILER_SCOPE(FReplicationWriter_WriteDebugObjectAttachments);
			const FWriteBatchResult Result = WriteObjectBatch(Context, ObjectIndexForOOBAttachment, FWriteObjectTraits({.bWriteAttachments = 1U, .OOBWriteAttachmentType = (uint32)ENetObjectAttachmentType::DebugObject}));
			if (Result.Status == EWriteStatus::Abort)
			{
				return WrittenObjectCount;
			}

			const bool bHasDebugObjectToSend = Attachments.HasUnsentAttachments(ENetObjectAttachmentType::DebugObject, ObjectIndexForOOBAttachment);
			WriteContext.bHasDebugObjectToSend = bHasDebugObjectToSend;
			if (!bHasDebugObjectToSend)
			{
				DebugObjectSendQueue.Stats.EndSendingTime = FPlatformTime::Cycles64();
			}

			WrittenObjectCount += Result.NumWritten;
		}
	}
	else
	{
		if (WriteContext.bHasHugeObjectToSend && (ReplicationRecord.GetUnusedInfoCount() > 0U))
		{
			IRIS_PROFILER_SCOPE(FReplicationWriter_WriteHugeObjectAttachments);
			const FWriteBatchResult Result = WriteObjectBatch(Context, ObjectIndexForOOBAttachment, FWriteObjectTraits({.bWriteAttachments = 1U, .OOBWriteAttachmentType = (uint32)ENetObjectAttachmentType::HugeObject}));
			if (Result.Status == EWriteStatus::Abort)
			{
				return WrittenObjectCount;
			}

			const bool bHasHugeObjectToSend = Attachments.HasUnsentAttachments(ENetObjectAttachmentType::HugeObject, ObjectIndexForOOBAttachment);
			WriteContext.bHasHugeObjectToSend = bHasHugeObjectToSend;
			if (!bHasHugeObjectToSend)
			{
				HugeObjectSendQueue.Stats.EndSendingTime = FPlatformTime::Cycles64();
			}

			WrittenObjectCount += Result.NumWritten;
		}

		if (WriteContext.bHasOOBAttachmentsToSend && CanSendObject(ObjectIndexForOOBAttachment) && (ReplicationRecord.GetUnusedInfoCount() > 0U))
		{
			IRIS_PROFILER_SCOPE(FReplicationWriter_WriteOOBAttachments);
			const FWriteBatchResult Result = WriteObjectBatch(Context, ObjectIndexForOOBAttachment, FWriteObjectTraits({.bWriteAttachments = 1U, .OOBWriteAttachmentType = (uint32)ENetObjectAttachmentType::OutOfBand}));
			if (Result.Status == EWriteStatus::Abort)
			{
				return WrittenObjectCount;
			}

			WriteContext.bHasOOBAttachmentsToSend = Attachments.HasUnsentAttachments(ENetObjectAttachmentType::OutOfBand, ObjectIndexForOOBAttachment);
			WrittenObjectCount += Result.NumWritten;
		}
	}

	return WrittenObjectCount;
}

uint32 FReplicationWriter::WriteObjects(FNetSerializationContext& Context)
{
	uint32 WrittenObjectCount = 0;
	
	const uint32 ObjectCount = WriteContext.ScheduledObjectCount;
	FScheduleObjectInfo* ObjectList = WriteContext.ScheduledObjectInfos.GetData();

	uint32 ObjectListIt = WriteContext.CurrentIndex;
	uint32 SortedCount = WriteContext.SortedObjectCount;

	// Write only if there are objects to send and we're not in replication record starvation mode.
	bool bContinue = WriteContext.bHasUpdatedObjectsToSend && !WriteContext.bIsInReplicationRecordStarvation;

	auto SendObjectFunction = [this, &Context, &WrittenObjectCount](FInternalNetRefIndex InternalIndex, bool bAppendCreationDependencies) -> EWriteStatus
	{
		if (this->WriteContext.ObjectsWrittenThisPacket.GetBit(InternalIndex) || WriteContext.ObjectsSkippedInThisPacket.Contains(InternalIndex))
		{
			// From a logic point of view we consider this object written if it is either successfully written to packet or if it is skipped and we will retry if we have more packets
			return EWriteStatus::Written;
		}
	
		// Deal with initial dependencies
		// If we have creationDependencies reschedule current object to after CreationDependencies to avoid unnecessary re-orderding waiting on the receiving end. 
		if (bAppendCreationDependencies)
		{
			const FReplicationInfo& Info = GetReplicationInfo(InternalIndex);
			const EReplicatedObjectState State = Info.GetState();

			if (State == EReplicatedObjectState::PendingCreate && NetRefHandleManager->GetObjectsWithCreationDependencies().GetBit(InternalIndex))
			{
				TArray<FExtraScheduleInfo> ScheduledInitialDependencies;
				for (FInternalNetRefIndex InitDependencyIndex : NetRefHandleManager->GetCreationDependencies(InternalIndex))
				{
					// Verify status of initial dependency to see if we can send this object
					const FReplicationInfo& DependencyReplicationInfo = GetReplicationInfo(InitDependencyIndex);
					if (!(DependencyReplicationInfo.IsCreationConfirmed || (WriteContext.ObjectsWrittenThisPacket.GetBit(InitDependencyIndex) && !IsActiveHugeObject(InitDependencyIndex))))
					{
						UE_LOG_REPWRITER(VeryVerbose, TEXT("ReplicationWriter:Scheduling initial dependency of internal index (%u) dependency internal index (%d)"), InternalIndex, InitDependencyIndex);
						ScheduledInitialDependencies.Add(FExtraScheduleInfo(InitDependencyIndex));
					}
				}

				if (!ScheduledInitialDependencies.IsEmpty())
				{
					// Schedule object after creation dependencies, this time without allowing it to schedule creation dependencies
					const bool bAllowScheduleCreationDependencies = false;
					ScheduledInitialDependencies.Add(FExtraScheduleInfo(InternalIndex, bAllowScheduleCreationDependencies));

					// We add them as one operation as additionally scheduled objects is used as a stack
					WriteContext.PushAdditionalScheduledObjects(ScheduledInitialDependencies);
						
					return EWriteStatus::RescheduledDueToCreationDependency;
				}
			}
		}

		if (!this->CanSendObject(InternalIndex))
		{
			// Even if the object cannot be sent, consider it written.
			return EWriteStatus::Written;
		}

		const FWriteBatchResult Result = this->WriteObjectBatch(Context, InternalIndex, FWriteObjectTraits({.bWriteState = 1U, .bWriteAttachments = 1U}));

		if (Result.Status == EWriteStatus::Written)
		{
			WrittenObjectCount += Result.NumWritten;
		}

		return Result.Status;
	};
	
	while (bContinue && (ObjectListIt < ObjectCount || !WriteContext.AdditionalObjectSchedulingStack.IsEmpty()))
	{
		for (;;)
		{
			// Process objects that has been explicitly scheduled due to dependencies etc.
			while (bContinue && WriteContext.AdditionalObjectSchedulingStack.Num() > 0)
			{
				UE_NET_TRACE_SCOPE(DependentObjectData, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

				const FExtraScheduleInfo ExtraScheduleInfo = WriteContext.AdditionalObjectSchedulingStack.Pop();

				checkSlow(ExtraScheduleInfo.InternalIndex != ObjectIndexForOOBAttachment);
				ensureMsgf(GetReplicationInfo(ExtraScheduleInfo.InternalIndex).GetState() != EReplicatedObjectState::Invalid, TEXT("DependentObject with InternalIndex %u is not in scope"), ExtraScheduleInfo.InternalIndex);

				//UE_LOG_REPWRITER(VeryVerbose, TEXT("Writing AdditionalObjectSchedulingStack: %s"), *NetRefHandleManager->PrintObjectFromIndex(ExtraScheduleInfo.InternalIndex));

				const EWriteStatus Status = SendObjectFunction(ExtraScheduleInfo.InternalIndex, ExtraScheduleInfo.bAppendCreationDependencies);
				if (Status == EWriteStatus::Abort)
				{
					UE_LOG_REPWRITER(VeryVerbose, TEXT("Aborted writing of additionally scheduled object: %s. Added back to the stack."), *NetRefHandleManager->PrintObjectFromIndex(ExtraScheduleInfo.InternalIndex));

					// If we fail, we put the object back on the pending send stack and try again in the next packet of the batch
					// The reason for pop before using the index is that the SendObjectFunction will push new dependent objects on the stack
					WriteContext.AdditionalObjectSchedulingStack.Push(ExtraScheduleInfo);
					bContinue = false;
					break;
				}

				// If we overflown and could not fit the object in the packet
				if (Status == EWriteStatus::Skipped)
				{
					//UE_LOG_REPWRITER(VeryVerbose, TEXT("Skipping over additional scheduled object: %s"), *NetRefHandleManager->PrintObjectFromIndex(ExtraScheduleInfo.InternalIndex));
				}
				else if (Status == EWriteStatus::SkippedRetryNextPacket)
				{
					//UE_LOG_REPWRITER(VeryVerbose, TEXT("Skipping over additional scheduled object: %s will retry next packet"), *NetRefHandleManager->PrintObjectFromIndex(ExtraScheduleInfo.InternalIndex));
					WriteContext.ObjectsSkippedInThisPacket.AddUnique(ExtraScheduleInfo.InternalIndex);
				}
			}

			// Partial sort next batch
			if (ObjectListIt >= SortedCount)
			{
				SortedCount += SortScheduledObjects(ObjectList, ObjectCount, ObjectListIt);
			}

			// Pick next object from original scheduling list.
			if (bContinue && ObjectListIt < SortedCount)
			{
				const FInternalNetRefIndex InternalIndex = ObjectList[ObjectListIt].Index;

				checkSlow(InternalIndex != ObjectIndexForOOBAttachment);

				// If the object has creation dependencies we will schedule unsent creation dependencies before we try to send the object again.
				const bool bAppendCreationDependencies = UE::Net::IsSchedulingCreationDependenciesFirst();
				const EWriteStatus Status = SendObjectFunction(InternalIndex, bAppendCreationDependencies);
				
				if (Status == EWriteStatus::Abort)
				{
					//UE_LOG_REPWRITER(VeryVerbose, TEXT("Aborted writing of regular object: %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalIndex));
					bContinue = false;
					break;
				}

				if (Status == EWriteStatus::Skipped)
				{
					//UE_LOG_REPWRITER(VeryVerbose, TEXT("Skipped writing of regular object: %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalIndex));
				}
				else if (Status == EWriteStatus::SkippedRetryNextPacket)
				{
					//UE_LOG_REPWRITER(VeryVerbose, TEXT("Skipped writing of regular object will retry next packet if allowed: %s"), *NetRefHandleManager->PrintObjectFromIndex(InternalIndex));
					WriteContext.ObjectsSkippedInThisPacket.AddUnique(InternalIndex);
				}

				++ObjectListIt;
			}
			else
			{
				break;
			}

		}
	}

	// If we skipped objects, we move them to the extra scheduling stack so that they can be written in following packets if bandwidth allows.
	if (WriteContext.ObjectsSkippedInThisPacket.Num())
	{
		// We add them as one operation as additionally scheduled objects is used as a stack
		TArray<FExtraScheduleInfo, TInlineAllocator<32>> RescheduledSkippedObjects;
		for (FInternalNetRefIndex InternalIndex : WriteContext.ObjectsSkippedInThisPacket)
		{
			RescheduledSkippedObjects.Add(FExtraScheduleInfo(InternalIndex));
		}
		
		WriteContext.PushAdditionalScheduledObjects(RescheduledSkippedObjects);

		// Reset so that we can try again in next packet sent this frame if allowed
		WriteContext.ObjectsSkippedInThisPacket.Reset();
	}

	// if we have more data to write, store state so that we can continue if we are allowed to write more data
	WriteContext.bHasUpdatedObjectsToSend = (ObjectListIt != ObjectCount) || (WriteContext.AdditionalObjectSchedulingStack.Num() > 0U);
	WriteContext.CurrentIndex = ObjectListIt;
	WriteContext.SortedObjectCount = SortedCount;

	// Reset objects written this packet
	WriteContext.ObjectsWrittenThisPacket.ClearAllBits();

	return WrittenObjectCount;
}

uint32 FReplicationWriter::HandleObjectBatchSuccess(const FBatchInfo& BatchInfo, FReplicationWriter::FBatchRecord& OutRecord)
{
	uint32 WrittenObjectCount = 0;
	uint32 WrittenBatchCount = 0;

	const bool bTrackObjectStats = BatchInfo.Type != EBatchInfoType::Internal;
	uint32 ObjectCount = 0;
	uint32 AttachmentCount = 0;
	uint32 DeltaCompressedObjectCount = 0;

	OutRecord.ObjectReplicationRecords.Reserve(BatchInfo.ObjectInfos.Num());
	for (const FBatchObjectInfo& BatchObjectInfo : BatchInfo.ObjectInfos)
	{
		//UE_LOG_REPWRITER(VeryVerbose, TEXT("FReplicationWriter::Wrote Object with %s InitialState: %u"), *NetRefHandleManager->NetHandle.ToString(), bIsInitialState ? 1u : 0u);
		FReplicationInfo& Info = GetReplicationInfo(BatchObjectInfo.InternalIndex);

		// We did write the initial state, change the state to WaitOnCreateConfirmation
		if (BatchObjectInfo.bIsInitialState)
		{
			Info.IsFilteredOutSubObjectPendingCreate = 0U;
			SetState(BatchObjectInfo.InternalIndex, EReplicatedObjectState::WaitOnCreateConfirmation);
		}
		else if (Info.TearOff)
		{
			Info.IsFilteredOutSubObjectTearOff = 0U;
			if (BatchObjectInfo.bSentTearOff)
			{
				SetState(BatchObjectInfo.InternalIndex, EReplicatedObjectState::PendingTearOff);
				SetState(BatchObjectInfo.InternalIndex, EReplicatedObjectState::WaitOnDestroyConfirmation);
			}
			else
			{
				UE_LOG_REPWRITER(Verbose, TEXT("ReplicationWriter::HandleObjectBatchSuccess for ( InternalIndex: %u ) Waiting for flush before tearoff"), BatchObjectInfo.InternalIndex);
				SetState(BatchObjectInfo.InternalIndex, EReplicatedObjectState::WaitOnFlush);
			}
		}
		else if (BatchObjectInfo.bSentDestroySubObject)
		{			
			SetState(BatchObjectInfo.InternalIndex, EReplicatedObjectState::WaitOnDestroyConfirmation);
		}

		// We're now committing to what we wrote so inform the attachments writer.
		if (BatchObjectInfo.AttachmentRecord.IsValid())
		{
			Attachments.CommitReplicationRecord(BatchObjectInfo.AttachmentType, BatchObjectInfo.InternalIndex, BatchObjectInfo.AttachmentRecord);
		}

		AttachmentCount += BatchObjectInfo.bSentAttachments;

		// Update transmission record.
		if (BatchObjectInfo.bSentState)
		{
			FNetBitArrayView ChangeMask(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount);
			FObjectRecord& ObjectRecord = OutRecord.ObjectReplicationRecords.AddDefaulted_GetRef();
			CreateObjectRecord(&ChangeMask, Info, BatchObjectInfo, ObjectRecord);

			// The object no longer has any dirty state, but may still have attachments that didn't fit
			ChangeMask.ClearAllBits();

			++ObjectCount;
			if (Info.LastAckedBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
			{
				++DeltaCompressedObjectCount;
			}
		}
		else if (BatchObjectInfo.AttachmentRecord.IsValid() || BatchObjectInfo.bSentTearOff || BatchObjectInfo.bSentDestroySubObject)
		{
			FObjectRecord& ObjectRecord = OutRecord.ObjectReplicationRecords.AddDefaulted_GetRef();
			CreateObjectRecord(nullptr, Info, BatchObjectInfo, ObjectRecord);
		}

		// Schedule rest of dependent objects for replication, note there is no guarantee that they will replicate in same packet
		TArray<FExtraScheduleInfo> DependentObjects;
		for (const FDependentObjectInfo DependentObjectInfo : NetRefHandleManager->GetDependentObjectInfos(BatchObjectInfo.InternalIndex))
		{
			const FInternalNetRefIndex DependentInternalIndex = DependentObjectInfo.NetRefIndex;
			if (ObjectsWithDirtyChanges.GetBit(DependentInternalIndex) && !WriteContext.ObjectsWrittenThisPacket.GetBit(DependentInternalIndex))
			{
				DependentObjects.Add(FExtraScheduleInfo(DependentInternalIndex));
				// Bumping the scheduling priority here will make sure that they will be picked up by scheduling the next tick/frame even if not all are allowed to replicate this frame
				SchedulingPriorities[DependentInternalIndex] = FMath::Max(SchedulingPriorities[BatchObjectInfo.InternalIndex], SchedulingPriorities[DependentInternalIndex]);
			}
		}
		if (DependentObjects.Num())
		{
			WriteContext.PushAdditionalScheduledObjects(DependentObjects);
		}

		if (BatchObjectInfo.bSentState | BatchObjectInfo.bSentAttachments | BatchObjectInfo.bSentTearOff | BatchObjectInfo.bSentDestroySubObject)
		{
			++WrittenObjectCount;
		}

		WrittenBatchCount += BatchObjectInfo.bSentBatchData;

		Info.HasDirtyChangeMask = 0U;
		Info.HasDirtySubObjects = BatchObjectInfo.bHasDirtySubObjects;
		Info.HasAttachments = BatchObjectInfo.bHasUnsentAttachments;

		// Indicate that we are now waiting for a new baseline to be acknowledged
		if (BatchObjectInfo.bSentState && BatchObjectInfo.NewBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
		{
			Info.PendingBaselineIndex = BatchObjectInfo.NewBaselineIndex;
		}
		
		const bool bObjectIsStillDirty = BatchObjectInfo.bHasUnsentAttachments || BatchObjectInfo.bHasDirtySubObjects;
		ObjectsWithDirtyChanges.SetBitValue(BatchObjectInfo.InternalIndex, bObjectIsStillDirty);

		// Reset scheduling priority if everything was replicated
		if (!bObjectIsStillDirty)
		{
			SchedulingPriorities[BatchObjectInfo.InternalIndex] = 0.0f;
		}

#if UE_NET_ENABLE_REPLICATIONWRITER_CANNOT_SEND_WARNING
		if (Info.HasCannotSendInfo)
		{
			const FCannotSendInfo CannotSendInfo = CannotSendInfos.FindAndRemoveChecked(BatchObjectInfo.InternalIndex);
			Info.HasCannotSendInfo = 0U;

			if (UE_LOG_ACTIVE(LogIris, Verbose))
			{
				UE_LOG_REPWRITER(Verbose, TEXT("Blocked Object %s was sent after waiting for %f s"), *NetRefHandleManager->PrintObjectFromIndex(BatchObjectInfo.InternalIndex), FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - CannotSendInfo.StartCycles));
			}
		}
#endif
	}

#if UE_NET_IRIS_CSV_STATS
	if (bTrackObjectStats)
	{
		FNetSendStats& NetStats = WriteContext.Stats;

		// We count RootObjects if anything is sent in an object batch, even if it's just subobjects or attachments. This is to mimic UReplicationGraph::ReplicateSingleActor stats.
		if (BatchInfo.Type == EBatchInfoType::Object)
		{
			if (ObjectCount || AttachmentCount)
			{
				NetStats.AddNumberOfReplicatedRootObjects(1U);
			}
		}
		NetStats.AddNumberOfReplicatedObjects(ObjectCount);
		NetStats.AddNumberOfDeltaCompressedReplicatedObjects(DeltaCompressedObjectCount);
	}
#endif

	OutRecord.BatchCount = WrittenBatchCount;

	return WrittenObjectCount;
}

FReplicationWriter::EWriteObjectRetryMode FReplicationWriter::HandleObjectBatchFailure(FReplicationWriter::EWriteObjectStatus WriteObjectStatus, const FBatchInfo& BatchInfo, const FReplicationWriter::FBitStreamInfo& BatchBitStreamInfo)
{
	IRIS_PROFILER_SCOPE(FReplicationWriter_HandleObjectBatchFailure);
	
	// Cleanup data stored in BatchInfo
	for (const FBatchObjectInfo& BatchObjectInfo : BatchInfo.ObjectInfos)
	{
		// If we did not end up using the baseline we need to release it
		if (BatchObjectInfo.bSentState && BatchObjectInfo.NewBaselineIndex != FDeltaCompressionBaselineManager::InvalidBaselineIndex)
		{
			UE_IRIS_PARALLEL_EXPR(FScopedBaselineAccess BaselineRW(BaselineManager, true));
			BaselineManager->LostBaseline(Parameters.ConnectionId, BatchObjectInfo.InternalIndex, BatchObjectInfo.NewBaselineIndex);
		}
		
		if (BatchObjectInfo.InternalIndex != ObjectIndexForOOBAttachment)
		{
			// If we failed to write the batch and we wrote data for an object we need to mark it as not written, if we want to try again
			WriteContext.ObjectsWrittenThisPacket.ClearBit(BatchObjectInfo.InternalIndex);
		}
	}

	if (WriteObjectStatus == EWriteObjectStatus::NoInstanceProtocol || WriteObjectStatus == EWriteObjectStatus::InvalidOwner)
	{
		return EWriteObjectRetryMode::TrySmallObject;
	}

	// If there is not enough space left to fit any object, stop serializing more objects immediately
	const uint32 BitsLeft = BatchBitStreamInfo.ReplicationCapacity - (BatchBitStreamInfo.BatchStartPos - BatchBitStreamInfo.ReplicationStartPos);
	if (BitsLeft < Parameters.SmallObjectBitThreshold)
	{
		return EWriteObjectRetryMode::Abort;
	}

	// If there are more bits left than the split threshold we treat it as a huge object and proceed with splitting.
	// We expect at least one part of the payload to be sendable if there are more bits left than the split threshold.
	if (CanQueueHugeObject() && PartialNetObjectAttachmentHandler != nullptr)
	{
		const uint32 SplitThreshold = PartialNetObjectAttachmentHandler->GetConfig()->GetBitCountSplitThreshold();
		if (BitsLeft > SplitThreshold)
		{
			//UE_LOG_REPWRITER(VeryVerbose, TEXT("FReplicationWriter::HandleObjectBatchFailure Failed to write object with ParentInternalIndex: %u EWriteObjectRetryMode::SplitHugeObject"), BatchInfo.ParentInternalIndex);
			return EWriteObjectRetryMode::SplitHugeObject;
		}
	}
	else
	{
		IRIS_PROFILER_SCOPE(FReplicationWriter_BlockedByHugeOBjectAlreadyBeingSent);
	}

	// If we have more packets available to send, don't try to fit smaller objects in the leftover buffer space
	if (WriteContext.MaxPacketsToSend == 0U || WriteContext.NumWrittenPacketsInThisBatch < WriteContext.MaxPacketsToSend-1)
	{
		return EWriteObjectRetryMode::Abort;
	}

	// For the last packet of this tick, try to fit other small objects until we skip too many (default: 10)
	if (WriteContext.FailedToWriteSmallObjectCount >= Parameters.MaxFailedSmallObjectCount)
	{
		return EWriteObjectRetryMode::Abort;
	}

	// Default- try some more, hopefully smaller state, objects.
	///UE_LOG_REPWRITER(VeryVerbose,TEXT("FReplicationWriter::HandleObjectBatchFailure Failed to write object with ParentInternalIndex: %u EWriteObjectRetryMode::TrySmallObject"), BatchInfo.ParentInternalIndex);
	return EWriteObjectRetryMode::TrySmallObject;
}

UDataStream::EWriteResult FReplicationWriter::BeginWrite(const UDataStream::FBeginWriteParameters& Params)
{
	IRIS_PROFILER_SCOPE(FReplicationWriter_PrepareWrite);

	// For now we do not support partial writes
	check(!WriteContext.bIsValid);

	if (!bReplicationEnabled)
	{
		return UDataStream::EWriteResult::NoData;
	}

	// If we've run out of replication records we cannot send anything
	if (ReplicationRecord.GetUnusedInfoCount() == 0)
	{
		return UDataStream::EWriteResult::NoData;
	}

	// Initialize context which can be used over multiple calls to WriteData
	WriteContext.bHasUpdatedObjectsToSend  = 0U;
	WriteContext.bHasDestroyedObjectsToSend = 0U;
	WriteContext.bHasHugeObjectToSend = 0U;
	WriteContext.bHasOOBAttachmentsToSend = 0U;
	WriteContext.bHasDebugObjectToSend = 0U;
	WriteContext.ScheduledObjectCount = 0u;

	WriteContext.WriteMode = Params.WriteMode;

	// Setup for writing PostTickDispatch data, currently this is only writing unreliable OOBAttachments.
	if (WriteContext.WriteMode == EDataStreamWriteMode::PostTickDispatch)
	{
		const bool bHasUnsentOOBAttachments = Attachments.HasUnsentUnreliableAttachments(ENetObjectAttachmentType::OutOfBand, ObjectIndexForOOBAttachment);
		if (!bHasUnsentOOBAttachments)
		{
			return UDataStream::EWriteResult::NoData;
		}
		WriteContext.bHasOOBAttachmentsToSend = bHasUnsentOOBAttachments;
	}
	else if (WriteContext.WriteMode == EDataStreamWriteMode::DebugData)
	{
		const bool bHasUnsentDebugObject = Attachments.HasUnsentAttachments(ENetObjectAttachmentType::DebugObject, ObjectIndexForOOBAttachment);
		if (!bHasUnsentDebugObject)
		{
			return UDataStream::EWriteResult::NoData;
		}
		WriteContext.bHasDebugObjectToSend = bHasUnsentDebugObject;
	}
	else
	{
		// See if we have any work to do
		const bool bHasUpdatedObjectsToSend = ObjectsWithDirtyChanges.IsAnyBitSet();
		const bool bHasDestroyedObjectsToSend = ObjectsPendingDestroy.IsAnyBitSet();
		const bool bHasUnsentOOBAttachments = Attachments.HasUnsentAttachments(ENetObjectAttachmentType::OutOfBand, ObjectIndexForOOBAttachment);
		const bool bHasUnsentHugeObject = Attachments.HasUnsentAttachments(ENetObjectAttachmentType::HugeObject, ObjectIndexForOOBAttachment);

		// Nothing to send
		if (!(bHasUpdatedObjectsToSend | bHasDestroyedObjectsToSend | bHasUnsentOOBAttachments | bHasUnsentHugeObject))
		{
			return UDataStream::EWriteResult::NoData;
		}

		// Initialize context which can be used over multiple calls to WriteData
		WriteContext.bHasUpdatedObjectsToSend  = bHasUpdatedObjectsToSend | bHasUnsentOOBAttachments | bHasUnsentHugeObject;
		WriteContext.bHasDestroyedObjectsToSend = bHasDestroyedObjectsToSend;
		WriteContext.bHasHugeObjectToSend = bHasUnsentHugeObject;
		WriteContext.bHasOOBAttachmentsToSend = bHasUnsentOOBAttachments;

		WriteContext.ScheduledObjectCount = ScheduleObjects(WriteContext.ScheduledObjectInfos);
	}

	// Reset AdditionalObjectSchedulingStack
	WriteContext.AdditionalObjectSchedulingStack.Reset();

	WriteContext.CurrentIndex = 0U;
	WriteContext.FailedToWriteSmallObjectCount = 0U;
	WriteContext.WrittenDestroyObjectCount = 0U;
	WriteContext.SortedObjectCount = 0U;
	WriteContext.NumWrittenPacketsInThisBatch = 0U;
	WriteContext.MaxPacketsToSend = Params.MaxPackets;

	// Updated properly in Write
	WriteContext.bIsInReplicationRecordStarvation = 0;
	WriteContext.bIsOOBPacket = 0;

	// Clear net stats. Used for CVS and Network Insights stats.
	WriteContext.Stats.Reset();
	WriteContext.Stats.SetNumberOfRootObjectsScheduledForReplication(WriteContext.ScheduledObjectCount);

	WriteContext.bIsValid = true;

	return UDataStream::EWriteResult::HasMoreData;
}

void FReplicationWriter::EndWrite()
{
	IRIS_PROFILER_SCOPE(FReplicationWriter_FinishWrite);

	if (WriteContext.bIsValid)
	{
#if UE_NET_IRIS_CSV_STATS
		// Update stats
		{
			FNetSendStats& Stats = WriteContext.Stats;
			if (!HugeObjectSendQueue.IsEmpty())
			{
				Stats.SetNumberOfActiveHugeObjects(HugeObjectSendQueue.NumRootObjectsInTransit());

				if (HugeObjectSendQueue.Stats.EndSendingTime != 0)
				{
					Stats.AddHugeObjectWaitingTime(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - HugeObjectSendQueue.Stats.EndSendingTime));
				}
				if (HugeObjectSendQueue.Stats.StartStallTime != 0)
				{
					Stats.AddHugeObjectStallTime(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - HugeObjectSendQueue.Stats.StartStallTime));
				}
			}

			FNetSendStats& TotalStats = Parameters.ReplicationSystem->GetReplicationSystemInternal()->GetSendStats();
			TotalStats.Accumulate(Stats);
		}
#endif

		const uint32 NumPendingAdditionallySchedule1dObjects = WriteContext.AdditionalObjectSchedulingStack.Num();
		const uint32 NumPendingObjectsToWrite = WriteContext.ScheduledObjectCount - WriteContext.CurrentIndex;

		//Update TickReplicationStats (lock must be acquired to be threadsafe)
		UE_IRIS_PARALLEL_EXPR(Parameters.ReplicationSystem->GetReplicationSystemInternal()->AcquireTickReplicationStatsWriteLock());
		FReplicationStats& ReplicationStats = Parameters.ReplicationSystem->GetReplicationSystemInternal()->GetTickReplicationStats();
		ReplicationStats.PendingObjectCount += NumPendingObjectsToWrite;
		ReplicationStats.PendingDependentObjectCount += NumPendingAdditionallySchedule1dObjects;
		ReplicationStats.HugeObjectSendQueue += HugeObjectSendQueue.NumRootObjectsInTransit();
		ReplicationStats.MaxPendingObjectCount = FMath::Max(ReplicationStats.MaxPendingObjectCount, NumPendingObjectsToWrite);
		ReplicationStats.MaxPendingDependentObjectCount = FMath::Max(ReplicationStats.MaxPendingDependentObjectCount, NumPendingAdditionallySchedule1dObjects);
		ReplicationStats.MaxHugeObjectSendQueue = FMath::Max(ReplicationStats.MaxHugeObjectSendQueue, HugeObjectSendQueue.NumRootObjectsInTransit());
		++ReplicationStats.SampleCount;
		UE_IRIS_PARALLEL_EXPR(Parameters.ReplicationSystem->GetReplicationSystemInternal()->ReleaseTickReplicationStatsWriteLock());

 #if UE_NET_ENABLE_REPLICATIONWRITER_LOG
		// See if we failed to write any objects
		if (NumPendingAdditionallySchedule1dObjects)
		{
			UE_LOG_REPWRITER(VeryVerbose, TEXT("FReplicationWriter::EndWrite() Has %u more scheduled objects to write"), NumPendingAdditionallySchedule1dObjects);
		}
		
		if (NumPendingObjectsToWrite)
		{
			UE_LOG_REPWRITER(VeryVerbose, TEXT("FReplicationWriter::EndWrite() Has %u more objects with dirty data"), NumPendingObjectsToWrite);
		}
#endif

		// Intentional use of Empty to avoid wasting memory due to rare frames with huge amounts of scheduled objects.
		WriteContext.ScheduledObjectInfos.Empty();
		WriteContext.bIsValid = false;
	}
}

void FReplicationWriter::Update(const UDataStream::FUpdateParameters& Params)
{
	if (Params.UpdateType == UDataStream::EUpdateType::PostTickFlush)
	{
		bool bHasUnsentReliable = false;
		if (GMaxUnsentOrderedUnreliableAttachmentAtEndOfTick == 0)
		{
			Attachments.DropUnreliableAttachments(ENetObjectAttachmentType::OutOfBand, ObjectIndexForOOBAttachment, bHasUnsentReliable);
		}
		else if (GMaxUnsentOrderedUnreliableAttachmentAtEndOfTick > 0)
		{
			const SIZE_T UnreliableCount = Attachments.GetUnreliableAttachmentCount(ENetObjectAttachmentType::OutOfBand, ObjectIndexForOOBAttachment);
			if (UnreliableCount >= GMaxUnsentOrderedUnreliableAttachmentAtEndOfTick)
			{
				Attachments.DropUnreliableAttachments(ENetObjectAttachmentType::OutOfBand, ObjectIndexForOOBAttachment, bHasUnsentReliable);
				UE_LOGF(LogIris, Warning, "FReplicationWriter::Discarded %zu unsent ordered unreliable attachments", UnreliableCount);
			}
		}
	}
}

bool FReplicationWriter::HasDataToSend(const FWriteContext& Context) const
{
	return WriteContext.bIsValid & (Context.bHasDestroyedObjectsToSend | Context.bHasUpdatedObjectsToSend | Context.bHasHugeObjectToSend | Context.bHasOOBAttachmentsToSend | Context.bHasDebugObjectToSend);
}

UDataStream::EWriteResult FReplicationWriter::Write(FNetSerializationContext& Context)
{
	IRIS_PROFILER_SCOPE(FReplicationWriter_Write);

	if (!HasDataToSend(WriteContext))
	{
		return UDataStream::EWriteResult::NoData;
	}

	// We have somethings in the WriteContext that must be reset each packet
	WriteContext.FailedToWriteSmallObjectCount = 0U;
	WriteContext.WrittenBatchCount = 0U;
	// Check whether we are running low on replication records or not and need to throttle object replication.
	WriteContext.bIsInReplicationRecordStarvation = ReplicationRecord.GetUnusedInfoCount() < static_cast<uint32>(GReplicationWriterReplicationRecordStarvationThreshold);
	
	// Packets going over the bandwidth limit are considered an OOB packet which should avoid writing things like objects pending destroy.  PostTickDispath packets are also flagged OOB to minimize extra serialization data.
	WriteContext.bIsOOBPacket = (WriteContext.WriteMode == EDataStreamWriteMode::PostTickDispatch) || (WriteContext.WriteMode == EDataStreamWriteMode::DebugData) || (WriteContext.MaxPacketsToSend != 0 && WriteContext.NumWrittenPacketsInThisBatch >= WriteContext.MaxPacketsToSend);

	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	// Setup internal context
	FInternalNetSerializationContext InternalContext(Parameters.ReplicationSystem);
	Context.SetLocalConnectionId(Parameters.ConnectionId);
	Context.SetInternalContext(&InternalContext);

	// Give some info for the case when we consider splitting a huge object.
	WriteBitStreamInfo.ReplicationStartPos = Writer.GetPosBits();
	WriteBitStreamInfo.ReplicationCapacity = Writer.GetBitsLeft();

	UpdateStreamDebugFeatures();

	UE_NET_TRACE_SCOPE(ReplicationData, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::Trace);
	
	FNetBitStreamRollbackScope Rollback(Writer);

	WriteStreamDebugFeatures(Context);

	const uint32 HeaderPos = Writer.GetPosBits();

	uint32 WrittenObjectCount = 0;
	const uint32 OldReplicationInfoCount = ReplicationRecord.GetInfoCount();

	// Written batch count
	Writer.WriteBits(0U, 16);

	// Write timestamps etc? Or do we do this in header.
	// WriteReplicationFrameData();

	const uint32 WrittenObjectsPendingDestroyCount = WriteObjectsPendingDestroy(Context);
	WrittenObjectCount += WrittenObjectsPendingDestroyCount;

	// Only reason for overflow here is if we did not fit header
	if (Writer.IsOverflown())
	{
		return UDataStream::EWriteResult::NoData;
	}

	WrittenObjectCount += WriteOOBAttachments(Context);

	WrittenObjectCount += WriteObjects(Context);

	UDataStream::EWriteResult WriteResult = UDataStream::EWriteResult::Ok;

	// If we have more data to write, request more updates
	// $IRIS $TODO: When we have better control over bandwidth usage, introduce setting to only allow over-commit if we have a huge object or split rpc to send
	// https://jira.it.epicgames.com/browse/UE-127371
	if (HasDataToSend(WriteContext))
	{
		if (WriteContext.bHasHugeObjectToSend || WriteContext.bHasDebugObjectToSend || (WriteContext.bHasOOBAttachmentsToSend && CanSendObject(ObjectIndexForOOBAttachment)))
		{
			WriteResult = UDataStream::EWriteResult::HasMoreData;
		}
		else if (!WriteContext.bIsInReplicationRecordStarvation && (WriteContext.MaxPacketsToSend == 0 || (WriteContext.NumWrittenPacketsInThisBatch < WriteContext.MaxPacketsToSend-1)))
		{
			WriteResult = UDataStream::EWriteResult::HasMoreData;
		}
		else
		{
			WriteResult = UDataStream::EWriteResult::Ok;
		}
	}
	
	if (!Writer.IsOverflown() && WrittenObjectCount > 0)
	{
		{
			// Seek back to HeaderPos and update the header
			FNetBitStreamWriteScope WriteScope(Writer, HeaderPos);
			const uint32 TotalWrittenBatchCount = WriteContext.WrittenBatchCount + WrittenObjectsPendingDestroyCount;
			Writer.WriteBits(TotalWrittenBatchCount, 16);
		}

		//UE_LOG_REPWRITER(VeryVerbose,TEXT("FReplicationWriter::Write() Wrote %u Objects for ConnectionId:%u, ReplicationSystemId: %u."), WrittenObjectCount, Parameters.ConnectionId, Parameters.ReplicationSystem->GetId());	
	
		// Push record
		const uint16 ReplicationInfoCount = static_cast<uint16>(ReplicationRecord.GetInfoCount() - OldReplicationInfoCount);
		ReplicationRecord.PushRecord(ReplicationInfoCount);

#if UE_NET_VALIDATE_REPLICATION_RECORD
		check(s_ValidateReplicationRecord(&ReplicationRecord, NetRefHandleManager->GetMaxActiveObjectCount() + 1U, false));
#endif

#if UE_NET_TRACE_ENABLED
		if (FNetTraceCollector* Collector = HugeObjectSendQueue.TraceCollector)
		{
			FNetTrace::FoldTraceCollector(Context.GetTraceCollector(), Collector, GetBitStreamPositionForNetTrace(Writer));
			Collector->Reset();
		}
		if (WriteContext.WriteMode == EDataStreamWriteMode::DebugData)
		{
			if (FNetTraceCollector* Collector = DebugObjectSendQueue.TraceCollector)
			{
				FNetTrace::FoldTraceCollector(Context.GetTraceCollector(), Collector, GetBitStreamPositionForNetTrace(Writer));
				Collector->Reset();
			}
		}
#endif

		++WriteContext.NumWrittenPacketsInThisBatch;
	}
	else 
	{
		// Trigger rollback as we did not write any data
		Writer.DoOverflow();
		WriteResult = UDataStream::EWriteResult::NoData;

		UE_CLOGF(WrittenObjectCount > 0, LogIris, Verbose, "Packet overflow when writing");
	}

	// Report packet stats
	UE_NET_TRACE_PACKET_STATSCOUNTER(Parameters.ReplicationSystem->GetId(), Parameters.ConnectionId, ReplicationWriter.WrittenObjectCount, WrittenObjectCount, ENetTraceVerbosity::Trace);
	UE_NET_TRACE_PACKET_STATSCOUNTER(Parameters.ReplicationSystem->GetId(), Parameters.ConnectionId, ReplicationWriter.WrittenBatchCount, WriteContext.WrittenBatchCount, ENetTraceVerbosity::Trace);
	UE_NET_TRACE_PACKET_STATSCOUNTER(Parameters.ReplicationSystem->GetId(), Parameters.ConnectionId, ReplicationWriter.FailedToWriteSmallObjectCount, WriteContext.FailedToWriteSmallObjectCount, ENetTraceVerbosity::Trace);
	UE_NET_TRACE_PACKET_STATSCOUNTER(Parameters.ReplicationSystem->GetId(), Parameters.ConnectionId, ReplicationWriter.RemainingObjectsPendingWriteCount, WriteContext.ScheduledObjectCount - WriteContext.CurrentIndex, ENetTraceVerbosity::Trace);
	UE_NET_TRACE_PACKET_STATSCOUNTER(Parameters.ReplicationSystem->GetId(), Parameters.ConnectionId, ReplicationWriter.ScheduledObjectCount, WriteContext.ScheduledObjectCount, ENetTraceVerbosity::Trace);

	return WriteResult;
}

void FReplicationWriter::SetupReplicationInfoForAttachmentsToObjectsNotInScope()
{
	FReplicationInfo& Info = GetReplicationInfo(ObjectIndexForOOBAttachment);
	Info = FReplicationInfo();
	Info.State = static_cast<uint32>(EReplicatedObjectState::AttachmentToObjectNotInScope);
	ReplicationRecord.ResetList(ReplicatedObjectsRecordInfoLists[ObjectIndexForOOBAttachment]);
}

void FReplicationWriter::ApplyFilterToChangeMask(uint32 ParentInternalIndex, uint32 InternalIndex, FReplicationInfo& Info, const FReplicationProtocol* Protocol, const uint8* InternalStateBuffer, bool bIsInitialState)
{
	const uint32* ConditionalChangeMaskPointer = (EnumHasAnyFlags(Protocol->ProtocolTraits, EReplicationProtocolTraits::HasConditionalChangeMask) ? reinterpret_cast<const uint32*>(InternalStateBuffer + Protocol->GetConditionalChangeMaskOffset()) : static_cast<const uint32*>(nullptr));
	const bool bChangeMaskWasModified = ReplicationConditionals->ApplyConditionalsToChangeMask(Parameters.ConnectionId, bIsInitialState, ParentInternalIndex, InternalIndex, Info.GetChangeMaskStoragePointer(), ConditionalChangeMaskPointer, Protocol);
	if (bChangeMaskWasModified)
	{
		Info.HasDirtyChangeMask = MakeNetBitArrayView(Info.GetChangeMaskStoragePointer(), Info.ChangeMaskBitCount).IsAnyBitSet() ? 1 : 0;
	}
}

void FReplicationWriter::InvalidateBaseline(uint32 InternalIndex, FReplicationInfo& Info)
{
	FReplicationRecord::FRecordInfoList& RecordInfoList = ReplicatedObjectsRecordInfoLists[InternalIndex];

	// Iterate over all data in flight for this object and mark any new baselines as invalid to avoid acking or nacking an invalidated baseline
	FReplicationRecord::FRecordInfo* CurrentRecordInfo = ReplicationRecord.GetInfoForIndex(RecordInfoList.FirstRecordIndex);
	while (CurrentRecordInfo)
	{
		CurrentRecordInfo->NewBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
		CurrentRecordInfo = ReplicationRecord.GetInfoForIndex(CurrentRecordInfo->NextIndex);
	};

	Info.LastAckedBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
	Info.PendingBaselineIndex = FDeltaCompressionBaselineManager::InvalidBaselineIndex;
}

bool FReplicationWriter::HasInFlightStateChanges(const FReplicationRecord::FRecordInfo* RecordInfo) const
{
	while (RecordInfo)
	{
		if (RecordInfo->HasChangeMask)
		{
			return true;
		}
		RecordInfo = ReplicationRecord.GetInfoForIndex(RecordInfo->NextIndex);
	};

	return false;
}

bool FReplicationWriter::HasInFlightStateChanges(uint32 InternalIndex, const FReplicationInfo& Info) const
{
	const FReplicationRecord::FRecordInfoList& RecordInfoList = ReplicatedObjectsRecordInfoLists[InternalIndex];
	const FReplicationRecord::FRecordInfo* CurrentRecordInfo = ReplicationRecord.GetInfoForIndex(RecordInfoList.FirstRecordIndex);

	return HasInFlightStateChanges(CurrentRecordInfo);
}

bool FReplicationWriter::PatchupObjectChangeMaskWithInflightChanges(uint32 InternalIndex, FReplicationInfo& Info)
{
	bool bInFlightChangesAdded = false;

	const FReplicationRecord::FRecordInfoList& RecordInfoList = ReplicatedObjectsRecordInfoLists[InternalIndex];

	FNetBitArrayView ChangeMask = FChangeMaskUtil::MakeChangeMask(Info.ChangeMaskOrPtr, Info.ChangeMaskBitCount);

	// Iterate over all data in flight for this object and include any changes in flight to ensure atomicity when received
	// N.B. We don't check if this object is in huge object mode and check to see if any of these changes were part of that payload.
	const FReplicationRecord::FRecordInfo* CurrentRecordInfo = ReplicationRecord.GetInfoForIndex(RecordInfoList.FirstRecordIndex);
	while (CurrentRecordInfo)
	{
		if (CurrentRecordInfo->HasChangeMask)
		{
			bInFlightChangesAdded = true;
			const FNetBitArrayView CurrentRecordInfoChangeMask(FChangeMaskUtil::MakeChangeMask(CurrentRecordInfo->ChangeMaskOrPtr, Info.ChangeMaskBitCount));
			ChangeMask.Combine(CurrentRecordInfoChangeMask, FNetBitArrayView::OrOp);
		}

		CurrentRecordInfo = ReplicationRecord.GetInfoForIndex(CurrentRecordInfo->NextIndex);
	};

	return bInFlightChangesAdded;
}

void FReplicationWriter::SetNetExports(FNetExports& InNetExports)
{
	NetExports = &InNetExports;
}

bool FReplicationWriter::IsActiveHugeObject(uint32 InternalIndex) const
{
	constexpr bool bIncludeSubObjects = false;
	return HugeObjectSendQueue.IsObjectInQueue(InternalIndex, bIncludeSubObjects);
}

bool FReplicationWriter::IsObjectPartOfActiveHugeObject(uint32 InternalIndex) const
{
	constexpr bool bFullSearch = true;
	return HugeObjectSendQueue.IsObjectInQueue(InternalIndex, bFullSearch);
}

bool FReplicationWriter::CanQueueHugeObject() const
{
	if (HugeObjectSendQueue.IsFull())
	{
		return false;
	}

	// Check whether the reliable queue is full in which case there's no point in queueing additional huge objects.
	if (!Attachments.CanSendMoreReliableAttachments(ENetObjectAttachmentType::HugeObject, ObjectIndexForOOBAttachment))
	{
		return false;
	}

	return true;
}

bool FReplicationWriter::CanQueueDebugObject() const
{
	if (DebugObjectSendQueue.IsFull())
	{
		return false;
	}

	// Check whether the reliable queue is full in which case there's no point in queueing additional huge objects.
	if (!Attachments.CanSendMoreReliableAttachments(ENetObjectAttachmentType::DebugObject, ObjectIndexForOOBAttachment))
	{
		return false;
	}

	return true;
}


void FReplicationWriter::FreeHugeObjectSendQueues()
{
	auto FreeContextFunc = [this](const FHugeObjectContext& HugeObjectContext)
	{
		for (const FObjectRecord& ObjectRecord : HugeObjectContext.BatchRecord.ObjectReplicationRecords)
		{
			FReplicationInfo& ReplicationInfo = this->GetReplicationInfo(ObjectRecord.Record.Index);
			const uint32 ChangeMaskBitCount = ReplicationInfo.ChangeMaskBitCount;
			if (ObjectRecord.Record.HasChangeMask)
			{
				FChangeMaskStorageOrPointer::Free(ObjectRecord.Record.ChangeMaskOrPtr, ChangeMaskBitCount, s_DefaultChangeMaskAllocator);
			}
		}
	};
	HugeObjectSendQueue.FreeContexts(FreeContextFunc);
	DebugObjectSendQueue.FreeContexts(FreeContextFunc);
}

void FReplicationWriter::CollectAndAppendExports(FNetSerializationContext& Context, uint8* RESTRICT InternalBuffer, const FReplicationProtocol* Protocol) const
{
	FNetExportContext* ExportContext = Context.GetExportContext();
	if (!ExportContext)
	{
		return;
	}

	FNetReferenceCollector Collector(ENetReferenceCollectorTraits::OnlyCollectReferencesThatCanBeExported);
	FReplicationProtocolOperationsInternal::CollectReferences(Context, Collector, InternalBuffer, Protocol);

	for (const FNetReferenceCollector::FReferenceInfo& Info : MakeArrayView(Collector.GetCollectedReferences()))
	{
		ObjectReferenceCache->AddPendingExport(*ExportContext, Info.Reference);
	}
}

bool FReplicationWriter::IsWriteObjectSuccess(EWriteObjectStatus Status) const
{
	return (Status == EWriteObjectStatus::Success) | (Status == EWriteObjectStatus::InvalidState);
}

void FReplicationWriter::DiscardAllRecords()
{
	TReplicationRecordHelper Helper(ReplicatedObjects, ReplicatedObjectsRecordInfoLists, &ReplicationRecord);

	const uint32 RecordCount = ReplicationRecord.GetRecordCount();
	for (uint32 RecordIt = 0, RecordEndIt = RecordCount; RecordIt != RecordEndIt; ++RecordIt)
	{
		if (const uint32 RecordInfoCount = ReplicationRecord.PopRecord())
		{
			Helper.Process(RecordInfoCount,
				[this](const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
				{
					HandleDiscardedRecord(RecordInfo, Info, AttachmentRecord);
				}
			);
		}
	}
}

// Do minimal work to free references and resources. We assume connection removal handling will be dealt with by respective subsystems,
// such as the DeltaCompressionBaselinemanager releasing baselines. 
void FReplicationWriter::StopAllReplication()
{
	if (ReplicatedObjects.Num() == 0)
	{
		return;
	}

	// Don't process special index.
	ReplicatedObjects[ObjectIndexForOOBAttachment].State = (uint8)EReplicatedObjectState::Invalid;

	// We cannot tell for sure which objects need processing so we check them all.
	int32 ReplicatedObjectsCount = ReplicatedObjects.Num();
	for (int32 InternalIndex = 0; InternalIndex < ReplicatedObjectsCount; InternalIndex++)
	{
		const FReplicationInfo& Info = GetReplicationInfo(InternalIndex);

		if (Info.GetState() == EReplicatedObjectState::Invalid)
		{
			continue;
		}

		// Free allocated ChangeMask (if it is allocated)
		FChangeMaskStorageOrPointer::Free(Info.ChangeMaskOrPtr, Info.ChangeMaskBitCount, s_DefaultChangeMaskAllocator);

		// Release object reference
		NetRefHandleManager->ReleaseNetObjectRef(InternalIndex);
	}
}

void FReplicationWriter::MarkObjectDirty(FInternalNetRefIndex InternalIndex, const char* Caller)
{
	if (bValidateObjectsWithDirtyChanges)
	{
		const FReplicationInfo& ObjectInfo = ReplicatedObjects[InternalIndex];
		if (!ensureMsgf(ObjectInfo.GetState() != EReplicatedObjectState::Invalid && ObjectInfo.GetState() < EReplicatedObjectState::PendingDestroy, TEXT("Object ( InternalIndex: %u ) with Invalid state marked dirty. Caller: %hs"), InternalIndex, Caller))
		{
			return;
		}
	}

	ObjectsWithDirtyChanges.SetBit(InternalIndex);
}

void FReplicationWriter::WriteSentinel(FNetBitStreamWriter* Writer, const TCHAR* DebugName)
{
#if UE_NET_REPLICATIONDATASTREAM_DEBUG
	if (EnumHasAnyFlags(StreamDebugFeatures, EReplicationDataStreamDebugFeatures::Sentinels))
	{
		WriteSentinelBits(Writer, 8U);
	}
#endif
}

FString FReplicationWriter::PrintObjectInfo(FInternalNetRefIndex ObjectIndex) const
{
	const FReplicationInfo& ObjectInfo = ReplicatedObjects[ObjectIndex];

	TStringBuilder<512> InfoBuilder;

	InfoBuilder.Appendf(TEXT("Status info (InternalIndex: %u) : 0x%llx (%s) | %s | SchedulingPriority: %f | LastAckedBaselineIndex: %u | Flags"), 
		ObjectIndex,
		ObjectInfo.Value, 
		(ObjectInfo.IsSubObject ? TEXT("SubObject") : (ObjectInfo.IsDestructionInfo ? TEXT("DestructionInfo") : TEXT("RootObject"))),
		LexToString(ObjectInfo.GetState()), 
		SchedulingPriorities[ObjectIndex],
		ObjectInfo.LastAckedBaselineIndex);

#define FREPLICATIONWRITER_APPENDFLAG(FlagName) \
	if (ObjectInfo.FlagName) \
	{ \
		InfoBuilder << TEXT(" | " #FlagName); \
	}
	// Relevant flags
	FREPLICATIONWRITER_APPENDFLAG(HasDirtyChangeMask);
	FREPLICATIONWRITER_APPENDFLAG(HasDirtySubObjects);
	FREPLICATIONWRITER_APPENDFLAG(HasAttachments);
	FREPLICATIONWRITER_APPENDFLAG(IsCreationConfirmed);
	FREPLICATIONWRITER_APPENDFLAG(TearOff);
	FREPLICATIONWRITER_APPENDFLAG(SubObjectPendingDestroy);
	FREPLICATIONWRITER_APPENDFLAG(IsDeltaCompressionEnabled);
	FREPLICATIONWRITER_APPENDFLAG(HasDirtyConditionals);

#undef FREPLICATIONWRITER_APPENDFLAG

	if (ObjectsWithDirtyChanges.GetBit(ObjectIndex))
	{
		InfoBuilder << TEXT(" | IsInDirtyChanges");
	}
	if (ObjectsPendingDestroy.GetBit(ObjectIndex))
	{
		InfoBuilder << TEXT(" | IsInPendingDestroy");
	}
	if (ObjectsInScope.GetBit(ObjectIndex))
	{
		InfoBuilder << TEXT(" | IsInScope");
	}
	if (IsActiveHugeObject(ObjectIndex))
	{
		InfoBuilder << TEXT(" | IsActiveHugeObject");
	}

	return InfoBuilder.ToString();
}

FReplicationWriter::FHugeObjectContext::FHugeObjectContext() = default;

FReplicationWriter::FHugeObjectContext::~FHugeObjectContext() = default;

// HugeObjectSendQueue implementation
FReplicationWriter::FHugeObjectSendQueue::FHugeObjectSendQueue(const TCHAR* QueueName, bool bInAllowMultipleBatchesPerRootObjectInTransit)
: DebugName(CreatePersistentNetDebugName(QueueName))
, bAllowMultipleBatchesPerRootObjectInTransit(bInAllowMultipleBatchesPerRootObjectInTransit)
{
#if UE_NET_TRACE_ENABLED
	DebugName->DebugNameId = FNetTrace::TraceName(DebugName->Name);
#endif
}

FReplicationWriter::FHugeObjectSendQueue::~FHugeObjectSendQueue()
{
	UE_NET_TRACE_DESTROY_COLLECTOR(TraceCollector);
	TraceCollector = nullptr;
}

// TODO: If reliable queue is full should we keep on filling up?
bool FReplicationWriter::FHugeObjectSendQueue::IsFull() const
{
	const int32 QueueSize = FPlatformMath::Max<int32>(GReplicationWriterMaxHugeObjectsInTransit, 1);
	return RootObjectsInTransit.Num() > QueueSize;
}

bool FReplicationWriter::FHugeObjectSendQueue::IsEmpty() const
{
	return RootObjectsInTransit.IsEmpty();
}

uint32 FReplicationWriter::FHugeObjectSendQueue::NumRootObjectsInTransit() const
{
	return static_cast<uint32>(RootObjectsInTransit.Num());
}

bool FReplicationWriter::FHugeObjectSendQueue::EnqueueHugeObject(const FHugeObjectContext& Context)
{
	if (IsFull())
	{
		return false;
	}

	int32* RootObjectInTransitEntry = RootObjectsInTransit.Find(Context.RootObjectInternalIndex);
	if (RootObjectInTransitEntry)
	{
		if (!bAllowMultipleBatchesPerRootObjectInTransit)
		{
			ensureMsgf(false, TEXT("An object that is already in the huge object queue should not try replicating again ( InternalIndex: %u )"), Context.RootObjectInternalIndex);
			return false;
		}
		else
		{
			// Bump the number of contexts in flight for this rootobject.
			++*RootObjectInTransitEntry;
		}		
	}
	else
	{
		RootObjectsInTransit.Add(Context.RootObjectInternalIndex, 1);
	}

	// Note: Lists don't have methods to perform moving of an element.
	SendContexts.AddTail(Context);
	return true;
}

// Returns true if the object is a huge object root object or part of any huge object's payload. The latter is an expensive operation.
bool FReplicationWriter::FHugeObjectSendQueue::IsObjectInQueue(FInternalNetRefIndex ObjectIndex, bool bFullSearch) const
{
	if (IsEmpty())
	{
		return false;
	}

	if (RootObjectsInTransit.Contains(ObjectIndex))
	{
		return true;
	}

	if (!bFullSearch)
	{
		return false;
	}

	for (const FHugeObjectContext& Context : SendContexts)
	{
		const FBatchRecord& BatchRecord = Context.BatchRecord;
		for (const FObjectRecord& ObjectRecord : MakeArrayView(BatchRecord.ObjectReplicationRecords.GetData(), BatchRecord.ObjectReplicationRecords.Num()))
		{
			if (ObjectIndex == ObjectRecord.Record.Index)
			{
				return true;
			}
		}
	}

	return false;
}

 FInternalNetRefIndex FReplicationWriter::FHugeObjectSendQueue::GetRootObjectInternalIndexForTrace() const
 {
	 const TDoubleLinkedList<FHugeObjectContext>::TDoubleLinkedListNode* TailNode = SendContexts.GetTail();
	 if (TailNode)
	 {
		 return TailNode->GetValue().RootObjectInternalIndex;
	 }

	return InvalidInternalNetRefIndex;
 }

void FReplicationWriter::FHugeObjectSendQueue::AckObjects(TFunctionRef<void (const FHugeObjectContext& Context)> AckHugeObject)
{
	for (TDoubleLinkedList<FHugeObjectContext>::TDoubleLinkedListNode* Node = SendContexts.GetHead(), *NextNode = nullptr; Node != nullptr; Node = NextNode)
	{
		NextNode = Node->GetNextNode();

		FHugeObjectContext& Context = Node->GetValue();

		// Iterate over the blobs backwards to break out of the loop as quickly as possible.
		bool bObjectIsAcked = true;
		for (TRefCountPtr<FNetBlob>& Blob : ReverseIterate(Context.Blobs))
		{
			const uint32 RefCount = Blob.GetRefCount();		
			if (RefCount > 1)
			{
				bObjectIsAcked = false;
				break;
			}
			else if (RefCount == 1)
			{
				// We no longer need to keep this blob around as we're the only thing referencing it.
				Blob.SafeRelease();
			}
		}

		if (!bObjectIsAcked)
		{
			// As clients deliver hugeobjects parts in order we cannot ack later objects until previous ones have been fully acked.
			break;
		}

		AckHugeObject(Context);

		// Remove from fast lookup set.
		int32* RootObjectsInTransitEntry = RootObjectsInTransit.Find(Context.RootObjectInternalIndex);
		if (RootObjectsInTransitEntry)
		{
			// If this was the last entry we remove it
			if (--(*RootObjectsInTransitEntry) == 0)
			{
				RootObjectsInTransit.Remove(Context.RootObjectInternalIndex);
			}
		}

		// Remove from queue.
		SendContexts.RemoveNode(Node);

		ensure((SendContexts.IsEmpty() == RootObjectsInTransit.IsEmpty()));
	}

	if (RootObjectsInTransit.IsEmpty())
	{
		Stats = FStats();
	}
}

void FReplicationWriter::FHugeObjectSendQueue::FreeContexts(TFunctionRef<void (const FHugeObjectContext& Context)> FreeHugeObject)
{
	for (const FHugeObjectContext& Context : SendContexts)
	{
		FreeHugeObject(Context);
	}

	SendContexts.Empty();
	RootObjectsInTransit.Empty();
}

} // end namespace UE::Net::Private
