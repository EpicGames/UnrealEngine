// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This class will never be included from public headers

#include "Containers/Array.h"
#include "Containers/List.h"
#include "Containers/Map.h"
#include "Containers/Set.h"

#include "Iris/Core/BitTwiddling.h"

#include "Iris/DataStream/DataStream.h"

#include "Iris/ReplicationSystem/AttachmentReplication.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "Iris/ReplicationSystem/NetExports.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/ReplicationSystem/ReplicationDataStreamDebug.h"
#include "Iris/ReplicationSystem/ReplicationRecord.h"
#include "Iris/ReplicationSystem/ReplicationTypes.h"

#include "Iris/Stats/NetStats.h"

#include "Misc/EnumClassFlags.h"

#include "Net/Core/NetBitArray.h"

// Forward declaration
class UReplicationSystem;
class UNetObjectBlobHandler;
class UPartialNetObjectAttachmentHandler;
class UObjectReplicationBridge;

namespace UE::Net
{
	class FNetBitStreamReader;
	class FNetBitStreamWriter;
	class FNetObjectAttachment;
	class FNetSerializationContext;
	struct FReplicationProtocol;
	
	namespace Private
	{
		struct FChangeMaskCache;
		class FNetRefHandleManager;
		class FReliableNetBlobQueue;
		class FReplicationConditionals;
		class FReplicationFiltering;
		class FReplicationSystemInternal;
		class FDeltaCompressionBaselineManager;
		class FDeltaCompressionBaseline;
	}
}

namespace UE::Net::Private
{

class FReplicationWriter
{
public:
	// Scheduling constants
	static constexpr float CreatePriority = 1.f;
	static constexpr float TearOffPriority = 1.f;
	static constexpr float LostStatePriorityBump = 1.f;
	static constexpr float SchedulingThresholdPriority = 1.f;

	static constexpr uint32 PreAllocatedScheduledObjectCount = 2048U;

	// When scheduling there is no point in scheduling more objects than we can fit in a packet
	static const uint32 PartialSortObjectCount = 128u;

public:

	// State
	enum class EReplicatedObjectState : uint8
	{
		Invalid = 0,

		// Special states
		AttachmentToObjectNotInScope,	// Special state for object index used for sending attachments to remote owned objects
		HugeObject,						// Special state for object index used for sending parts of huge object payloads
		DebugObject,					// Special state for object index used for sending parts of debug object payloads

		// Normal states
		PendingCreate,					// Not yet created, no data in flight
		WaitOnCreateConfirmation,		// Waiting for confirmation from remote, we can send state data, but if we do we must also include creation info until the object is Created
		Created,						// Confirmed by remote, this is the normal replicating state
		WaitOnFlush,					// Pending flush, we are waiting for all in-flight state data to be acknowledged
		PendingTearOff,					// TearOff should be sent
		SubObjectPendingDestroy,		// SubObject destroy should be sent
		CancelPendingDestroy,			// Destroy was sent but object wants to start replicating again
		PendingDestroy,					// Object is set to be destroyed
		WaitOnDestroyConfirmation,		// Destroy has been sent, waiting for response from client
		Destroyed,						// Confirmed as Destroyed,
		PermanentlyDestroyed,			// DestructionInfo has been confirmed as received

		Count
	};

	static_assert(UE::Net::GetBitsNeeded(EReplicatedObjectState::Count) <= 5, "EReplicatedObjectState must fit in 5 bits. See FReplicationInfo::State and FReplicationRecord::FRecordInfo::ReplicatedObjectState members.");

	static const TCHAR* LexToString(const EReplicatedObjectState State);

	enum EFlushFlags : uint32
	{
		FlushFlags_None				= 0U,
		FlushFlags_FlushState		= 1U << 0U,											// Make sure that all current state data is acknowledged before we stop replicating the object
		FlushFlags_FlushReliable	= FlushFlags_FlushState << 1U,						// Make sure that all enqueued Reliable RPCs are delivered before we stop replicating the object
		FlushFlags_FlushTornOffSubObjects	= FlushFlags_FlushReliable << 1U,					// Make sure that we flush TearOff and replicated destroy properly
		FlushFlags_All				= FlushFlags_FlushState | FlushFlags_FlushReliable | FlushFlags_FlushTornOffSubObjects,
		FlushFlags_Default			= FlushFlags_FlushReliable,
	};

	// Bookkeeping info required for a replicated object
	// Keep as small as possible since there is one per replicated object per connection
	// Changemask can and will most likely be replaced by a smaller index into a pool to reduce overhead
	struct FReplicationInfo
	{
		inline FReplicationInfo();

		FChangeMaskStorageOrPointer ChangeMaskOrPtr;			// Changemask storage or pointer to storage	
		union 
		{
			uint64 Value;
			struct 
			{
				uint64 ChangeMaskBitCount : 16;							// This is cached to avoid having to look it up in the protocol
				uint64 State : 5;										// Current state
				uint64 HasDirtySubObjects : 1;							// Set if this object has dirty subobjects
				uint64 IsSubObject : 1;									// Set if this object is a subobject
				uint64 HasDirtyChangeMask : 1;							// Set if the ChangeMask might be dirty, if not set the changemask should be zero!
				uint64 HasAttachments : 1;								// Set if there are attachments, such as RPCs, waiting to be sent
				uint64 HasChangemaskFilter : 1;							// Do we need to filter our changemask or not
				uint64 IsDestructionInfo : 1;							// If this is a destruction info
				uint64 IsCreationConfirmed : 1;							// We know that this object has been created on the receiving end
				uint64 TearOff : 1;										// This object should be torn off
				uint64 SubObjectPendingDestroy : 1;						// This object is a subobject that should be destroyed when we replicate owner
				uint64 IsDeltaCompressionEnabled : 1;					// Set to 1 if deltacompression is enabled for this object
				uint64 LastAckedBaselineIndex : 2;						// Last acknowledged baseline index which we can use for deltacompresion
				uint64 PendingBaselineIndex : 2;						// Baseline index pending acknowledgment from client
				uint64 FlushFlags : 3;									// Flags indicating what we are waiting for when flushing
				uint64 HasDirtyConditionals : 1;						// If this flag is set, we must update conditionals.
				mutable uint64 HasCannotSendInfo : 1;					// If this flag is set, the object has been prevented from being sent at least once.
				uint64 IsDebugObject : 1;								// If this flag is set, the object is a debugobject and is always sent using DebugObject attachments.
				uint64 IsFilteredOutSubObjectPendingCreate : 1;			// If this flag is set, the object is a subobject that was filtered out while sending pending create which requires some extra attention
				uint64 IsFilteredOutSubObjectTearOff : 1;				// If this flag is set, the object is a subobject confirmed as created that was filtered out while marked as tearoff which requires some extra attention
			};
		};

		static const uint32 LocalChangeMaskMaxBitCount = 64u;

		EReplicatedObjectState GetState() const { return (EReplicatedObjectState)State; }

		ChangeMaskStorageType* GetChangeMaskStoragePointer() { return ChangeMaskOrPtr.GetPointer(ChangeMaskBitCount); }
		const ChangeMaskStorageType* GetChangeMaskStoragePointer() const { return ChangeMaskOrPtr.GetPointer(ChangeMaskBitCount); }
	};

	static_assert(sizeof(FReplicationInfo) == 16, "Expected sizeof FReplicationInfo to be 16 bytes");

	struct FCannotSendInfo
	{
		uint64 StartCycles = 0;
		uint32 SuppressWarningCounter = 0U;
	};

public:
	FReplicationWriter();
	~FReplicationWriter();

	// Init
	void Init(const FReplicationParameters& InParameters);

	void Deinit();

	// Update new or existing/destroyed 
	void UpdateScope(const FNetBitArrayView& ScopedObjects);

	// Force update DirtyChangeMasks and mark objects for flush and/or tearoff depending on flags
	void ForceUpdateDirtyChangeMasks(const FChangeMaskCache& CachedChangeMasks, EFlushFlags ExtraFlushFlags, bool bMarkForTearOff) { InternalUpdateDirtyChangeMasks(CachedChangeMasks, ExtraFlushFlags, bMarkForTearOff); }

	// Called to propagate changes to global lifetime conditionals
	void UpdateDirtyGlobalLifetimeConditionals(TArrayView<FInternalNetRefIndex> ObjectsWithDirtyConditionals);

	// Propagate dirty changemasks
	void UpdateDirtyChangeMasks(const FChangeMaskCache& CachedChangeMasks) { InternalUpdateDirtyChangeMasks(CachedChangeMasks, EFlushFlags::FlushFlags_None, false); }

	// Returns objects that are in need of a priority update. It could be dirty, new or objects in need of resending.
	const FNetBitArray& GetObjectsRequiringPriorityUpdate() const;

	// UpdatedPriorities contains priorities for all objects. Objects in need of a priority update should use the newly calculated priorities.
	void UpdatePriorities(const float* UpdatedPriorities);

	UDataStream::EWriteResult BeginWrite(const UDataStream::FBeginWriteParameters& Params);

	// WriteData to Packet, returns true for now if data was written
	UDataStream::EWriteResult Write(FNetSerializationContext& Context);

	void EndWrite();

	// Deal with processing of lost and delivered data.
	void ProcessDeliveryNotification(EPacketDeliveryStatus PacketDeliveryStatus);

	void SetReplicationEnabled(bool bInReplicationEnabled);
	bool IsReplicationEnabled() const;

	/** Is the object in a state where we can send data about it. It will be false when the object is about to be destroyed or we are waiting for destroy confirmation. */
	bool IsInReplicatingState(FInternalNetRefIndex ObjectInternalIndex) const;

	void SetNetExports(FNetExports& InNetExports);

	// Attachments
	// Queue NetObjectAttachments, returns whether the attachments was enqueued or not.
	bool QueueNetObjectAttachments(FInternalNetRefIndex OwnerInternalIndex, FInternalNetRefIndex SubObjectInternalIndex, TArrayView<const TRefCountPtr<FNetBlob>> Attachments, ENetObjectAttachmentSendPolicyFlags SendFlags);

	bool AreAllReliableAttachmentsSentAndAcked() const;

	bool HasDebugDataToSend() const;

	void Update(const UDataStream::FUpdateParameters& Params);

	[[nodiscard]] FString PrintObjectInfo(FInternalNetRefIndex ObjectIndex) const;

private:
	// Various types

	enum : uint32
	{
		ObjectIndexForOOBAttachment = 0U,
	};

	// Propagate dirty changemasks
	void InternalUpdateDirtyChangeMasks(const FChangeMaskCache& CachedChangeMasks, EFlushFlags ExtraFlushFlags, bool bTearOff);

	struct FScheduleObjectInfo
	{
		uint32 Index;
		float SortKey;
	};

	using FScheduledObjectInfosArray = TArray<FScheduleObjectInfo, TInlineAllocator<PreAllocatedScheduledObjectCount>>;

	struct FExtraScheduleInfo
	{
		FExtraScheduleInfo(FInternalNetRefIndex InInternalIndex, bool bInAllowAppendCreationDependencies = true)
		: InternalIndex(InInternalIndex) 
		, bAppendCreationDependencies(bInAllowAppendCreationDependencies)
		{
		}

		FInternalNetRefIndex InternalIndex = 0U;
		bool bAppendCreationDependencies = false;
	};

	// We persist some information during write so that we can support writing multiple packets with data without re-doing scheduling work.
	struct FWriteContext
	{
		FWriteContext() : bIsValid(0) {}

		// Objects we have written in this packet to avoid sending same object multiple times
		FNetBitArray ObjectsWrittenThisPacket;

		// Push a list of objects to the AdditionalObjectSchedulingStack. It will be added in reverse to maintain order.
		void PushAdditionalScheduledObjects(TArrayView<FExtraScheduleInfo> ObjectsToSchedule)
		{ 
			for (int32 It = ObjectsToSchedule.Num() - 1; It >= 0; It--)
			{
				AdditionalObjectSchedulingStack.Push(ObjectsToSchedule[It]);
			}
		}

		// Additional objects that we have scheduled to be written, it could be dependencies, dependent objects.
		TArray<FExtraScheduleInfo, TInlineAllocator<32>> AdditionalObjectSchedulingStack;

		// Objects skipped this packet since they did not fit, that we want to retry next packet
		TArray<FInternalNetRefIndex, TInlineAllocator<32>> ObjectsSkippedInThisPacket;

		// Scheduled objects
		FScheduledObjectInfosArray ScheduledObjectInfos;
		uint32 ScheduledObjectCount;

		// For performance sake we do partial sorting so we need to track how many object we have sorted.
		uint32 SortedObjectCount;
		// The index into the scheduled objects array to attempt to replicate next.
		uint32 CurrentIndex;

		// Written batch count, which excludes objects pending destroy and subobjects
		uint32 WrittenBatchCount;

		// How many objects that were attempted to be replicated but which ultimately didn't fit in the packet.
		uint32 FailedToWriteSmallObjectCount;

		// How many root object destroys and destruction infos have been replicated.
		uint32 WrittenDestroyObjectCount;

		// How many packets have we written to?
		uint32 NumWrittenPacketsInThisBatch = 0U;

		// How many packets are we allowed to send in this batch. Based off the netspeed limit. When 0 = unlimited packets
		uint32 MaxPacketsToSend = 1U;

		EDataStreamWriteMode WriteMode = EDataStreamWriteMode::Full;	
	
		// Whether we're running low on ReplicationRecords or not. If starving only OOB and huge objects are sent, if at all possible.
		uint32 bIsInReplicationRecordStarvation : 1;
		// Whether there are destroyed objects to send or not
		uint32 bHasDestroyedObjectsToSend : 1;
		// Whether there are dirty objects to send or not
		uint32 bHasUpdatedObjectsToSend : 1;
		// Whether there is at least one huge object to send or not
		uint32 bHasHugeObjectToSend : 1;
		// Whether there are OOB attachments to send or not
		uint32 bHasOOBAttachmentsToSend : 1;
		// Whether there is at least one debug object to send or not
		uint32 bHasDebugObjectToSend : 1;

		// Whether this packet is mainly intended for OOB and HugeObject data
		uint32 bIsOOBPacket : 1;
		// Whether this context is valid or not
		uint32 bIsValid : 1;

		FNetSendStats Stats;
	};

	struct FBatchObjectInfo
	{
		FNetRefHandle Handle;
		uint32 InternalIndex;
		FNetObjectAttachmentsWriter::FCommitRecord AttachmentRecord;
		ENetObjectAttachmentType AttachmentType;
		bool bHasUnsentAttachments;
		uint32 NewBaselineIndex : 2;
		uint32 bIsInitialState : 1;
		uint32 bSentState : 1;
		uint32 bSentAttachments : 1;
		uint32 bHasDirtySubObjects : 1;
		uint32 bSentTearOff : 1;
		uint32 bSentDestroySubObject : 1;
		uint32 bSentBatchData : 1;
	};

	enum class EBatchInfoType : uint32
	{
		Object,
		HugeObject,
		DebugObject,
		OOBAttachment,
		Internal,
		// Currently unused as destruction infos don't create a BatchInfo.
		DestructionInfo,
	};

	struct FBatchInfo
	{
		TArray<FBatchObjectInfo, TInlineAllocator<16>> ObjectInfos;
		uint32 ParentInternalIndex;
		EBatchInfoType Type;
	};

	struct FObjectRecord
	{
		FReplicationRecord::FRecordInfo Record;
		FNetObjectAttachmentsWriter::FReliableReplicationRecord AttachmentRecord;
	};

	struct FBatchRecord
	{
		TArray<FObjectRecord, TInlineAllocator<16>> ObjectReplicationRecords;
		uint32 BatchCount = 0U;
	};

	struct FBitStreamInfo
	{
		uint32 ReplicationStartPos;
		uint32 BatchStartPos;
		uint32 ReplicationCapacity;
	};

	enum class EHugeObjectSendStatus : uint32
	{
		Idle,
		Sending,
	};

	struct FHugeObjectContext
	{
		FHugeObjectContext();
		~FHugeObjectContext();

		FInternalNetRefIndex RootObjectInternalIndex = 0;
		FBatchRecord BatchRecord;
		FNetExportContext::FBatchExports BatchExports;

		// The entire payload. When refcount reaches one for all blobs the object has been fully acked.
		TArray<TRefCountPtr<FNetBlob>> Blobs;
	};

	class FHugeObjectSendQueue
	{
	public:
		FHugeObjectSendQueue(const TCHAR* QueueName, bool bInAllowMultipleBatchesPerRootObjectInTransit = false);
		~FHugeObjectSendQueue();

		// If the queue is full we can't start another send.
		bool IsFull() const;
		bool IsEmpty() const;
		uint32 NumRootObjectsInTransit() const;

		// Enqueue huge object info and return true if it can be sent.
		bool EnqueueHugeObject(const FHugeObjectContext& Context);

		// Returns true if the object is a huge object root object or part of any huge object's payload. The latter is an expensive operation.
		bool IsObjectInQueue(FInternalNetRefIndex ObjectIndex, bool bFullSearch) const;

		// Best effort implementation of getting a valid index for trace.
		FInternalNetRefIndex GetRootObjectInternalIndexForTrace() const;

		// Call AckHugeObject on all objects determined to have been fully processed.
		void AckObjects(TFunctionRef<void (const FHugeObjectContext& Context)> AckHugeObject);

		void FreeContexts(TFunctionRef<void (const FHugeObjectContext& Context)> FreeHugeObject);

	public:
		// Public members
		struct FStats
		{
			// Cycle counter for when the huge object context went from idle to sending.
			uint64 StartSendingTime = 0;
			// Cycle counter for when the last part of huge object was sent.
			uint64 EndSendingTime = 0;
			// Cycle counter for when it was detected that no more parts of the huge object could be sent until some of the first parts have been acked.
			uint64 StartStallTime = 0;
		};

		FStats Stats;

		FNetTraceCollector* TraceCollector = nullptr;
		const FNetDebugName* DebugName = nullptr;

	private:
		TMap<FInternalNetRefIndex, int32> RootObjectsInTransit;
		TDoubleLinkedList<FHugeObjectContext> SendContexts;
		bool bAllowMultipleBatchesPerRootObjectInTransit = false;
	};

	struct FWriteObjectTraits
	{
		uint32 bWriteState : 1U = 0U;
		uint32 bWriteAttachments : 1U = 0U;
		uint32 OOBWriteAttachmentType : 2U = 0U;
		uint32 bIsWritingHugeObjectBatch : 1U = 0U;
	};

	enum class EWriteObjectRetryMode : unsigned
	{
		// Stop trying to write more object this frame.
		Abort,
		// Continue with something smaller, it might succeed.
		TrySmallObject,
		// The object is probably huge. Enter special mode for huge objects.
		SplitHugeObject,
	};

	enum class EWriteObjectStatus : unsigned
	{
		Success,

		// The object is in an invalid state and won't be written. This is not considered a failure.
		InvalidState,

		// BitStream overflow.
		BitStreamOverflow,

		// A detached instance, which no longer has an instance protocol. This object cannot be replicated.
		NoInstanceProtocol,

		// A subobject with an invalid owner.
		InvalidOwner,

		// Some error occurred while serializing the object.
		Error,

	};

private:

	void SetNetObjectListsSize(FInternalNetRefIndex NewMaxInternalIndex);
	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

	uint32 GetDefaultFlushFlags() const;
	uint32 GetFlushStatus(uint32 InternalIndex, const FReplicationInfo& Info, uint32 FlushFlagsToTest = EFlushFlags::FlushFlags_Default) const;
	void SetPendingDestroyOrSubObjectPendingDestroyState(uint32 Index, FReplicationInfo& Info);

	bool IsObjectIndexForOOBAttachment(uint32 InternalIndex) const { return InternalIndex == ObjectIndexForOOBAttachment; }

	void GetInitialChangeMask(ChangeMaskStorageType* ChangeMaskStorage, const FReplicationProtocol* Protocol);

	// Start replication of new object with the specified InternalIndex
	void StartReplication(uint32 InternalIndex);

	// Stop replication object with the specified InternalIndex
	void StopReplication(uint32 InternalIndex);

	// Get ReplicationInfo for specified InternalIndex, prefer to use this method over direct access.
	FReplicationInfo& GetReplicationInfo(uint32 InternalIndex);
	const FReplicationInfo& GetReplicationInfo(uint32 InternalIndex) const;

	// Set the state of a ReplicatedObject, prefer this method to enable logging
	void SetState(uint32 InternalIndex, EReplicatedObjectState NewState);

	// Write index part of handle
	void WriteNetRefHandleId(FNetSerializationContext& Context, FNetRefHandle RefHandle);
		
	// Create new ObjectRecord
	// Note: be aware that it will allocate a copy of the ChangeMask that needs to be handled if the record is not Committed
	void CreateObjectRecord(const FNetBitArrayView* ChangeMask, const FReplicationInfo& Info, const FBatchObjectInfo& ObjectInfo, FObjectRecord& OutRecord);
	
	// Push and link info for written object to ReplicationRecord
	void CommitObjectRecord(uint32 InternalObjectIndex, const FObjectRecord& Record);
	// Push and link info for destroyed root object
	void CommitObjectDestroyRecord(uint32 InternalObjectIndex, const FObjectRecord& ObjectRecord, const FReplicationRecord::FSubObjectRecord& SubObjectRecord);

	void CommitBatchRecord(const FBatchRecord& BatchRecord);

	void ScheduleDependentObjects(uint32 Index, float ParentPriority, TArray<float>& LocalPriorities, FScheduledObjectInfosArray& ScheduledObjectIndices);

	uint32 ScheduleObjects(FScheduledObjectInfosArray& ScheduledObjectIndices);
	
	// Partial sort of OutScheduledObjectIndices, will sort at most PartialSortObjectCount objects
	uint32 SortScheduledObjects(FScheduleObjectInfo* ScheduledObjectIndices, uint32 ScheduledObjectCount, uint32 StartIndex);

	// Update the active set of stream debug features based on cvars, build configuration etc. Enabling certain debug features can help track down bitstream errors. 
	void UpdateStreamDebugFeatures();
	// Write stream debug features.
	void WriteStreamDebugFeatures(FNetSerializationContext& Context);
	// Write all objects pending destroy (or as many as we fit in the current packet). Will call one of WriteObjectsAndSubObjectsPendingDestroy and WriteRootObjectsPendingDestroy depending on cvar net.Iris.DestroyRootAndSubObjectsIndividually
	uint32 WriteObjectsPendingDestroy(FNetSerializationContext& Context);
	// Used when net.Iris.DestroyRootAndSubObjectsIndividually is true. Will send destroy for every root and subobject. Leagcy Iris behavior.
	uint32 WriteObjectsAndSubObjectsPendingDestroy(FNetSerializationContext& Context);
	// Used when net.Iris.DestroyRootAndSubObjectsIndividually is false. Will only send destroy for root objects and subobjects will be gathered on the client side and destroyed. This guarantees atomic object hierarchy destroys.
	uint32 WriteRootObjectsPendingDestroy(FNetSerializationContext& Context);

	// Write object and SubObjects
	EWriteObjectStatus WriteObjectAndSubObjects(FNetSerializationContext& Context, uint32 InternalIndex, FWriteObjectTraits WriteObjectTraits, FBatchInfo& OutBatchInfo);

	// Write objects recursive
	EWriteObjectStatus WriteObjectInBatch(FNetSerializationContext& Context, uint32 InternalIndex, FWriteObjectTraits WriteObjectTraits, FBatchInfo& OutBatchInfo);

	enum class EWriteStatus : int32
	{
		// Stream is full and we should stop serializing more objects
		Abort = -1,
		// Object was skipped and won't be retried again. Ex: failed dependency, waiting for creation confirmation or buffer won't fit the object and its the last packet to send.
		Skipped = 0,
		// Object was skipped due to not enough space but will be retried in next packet if we are allowed to write more data.
		SkippedRetryNextPacket = 1,
		// Object was successfully written
		Written = 2,
		// Object is rescheduled due to creation dependency
		RescheduledDueToCreationDependency = 3,
	};

	// Write destruction info for an object that should be destroyed
	// returns > 1 if Objects was written, 0 if the objects was skipped (failed dependency or waiting for creation confirmation) -1 if the stream is full and we should stop
	EWriteStatus WriteDestructionInfo(FNetSerializationContext& Context, uint32 InternalIndex);

	bool WriteNetRefHandleDestructionInfo(FNetSerializationContext& Context, FNetRefHandle Handle);

	struct FWriteBatchResult
	{
		EWriteStatus Status = EWriteStatus::Skipped;
		uint32 NumWritten = 0;
	};

	// Write Object and any subobject(s) to stream as an atomic batch
	// returns > 1 if Objects was written, 0 if the objects was skipped (failed dependency or waiting for creation confirmation) -1 if the stream is full and we should stop
	FWriteBatchResult WriteObjectBatch(FNetSerializationContext& Context, FInternalNetRefIndex InternalIndex, FWriteObjectTraits WriteObjectTraits);

	EWriteStatus PrepareAndSendHugeObjectPayload(FNetSerializationContext& Context, FInternalNetRefIndex InternalIndex, ENetObjectAttachmentType AttachmentType);

	// Write OOBAttachments
	uint32 WriteOOBAttachments(FNetSerializationContext& Context);

	// Write as many scheduled objects to stream as we can fit.
	uint32 WriteObjects(FNetSerializationContext& Context);

	// Updates ReplicationInfos, pushes ReplicationRecords etc after a successful call to WriteObjectInBatch() on a top level object
	uint32 HandleObjectBatchSuccess(const FBatchInfo& BatchInfo, FBatchRecord& OutRecord);

	// Determines the best course of action after a WriteObjectBatch() call failed.
	EWriteObjectRetryMode HandleObjectBatchFailure(EWriteObjectStatus WriteObjectStatus, const FBatchInfo& BatchInfo, const FBitStreamInfo& BatchBitStreamInfo);

	// Update logic for dropped RecordInfo
	void HandleDroppedRecord(const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord);
	template<EReplicatedObjectState LostState> void HandleDroppedRecord(EReplicatedObjectState CurrentState, const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord);

	// Update logic for delivered RecordInfo
	void HandleDeliveredRecord(const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord);

	// Update logic for discarded RecordInfo, for preventing memory leaks on disconnect and shutdown.
	void HandleDiscardedRecord(const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord);

	// Setup replication info to be able to send attachments to objects not in scope
	void SetupReplicationInfoForAttachmentsToObjectsNotInScope();

	void ApplyFilterToChangeMask(uint32 ParentInternalIndex, uint32 InternalIndex, FReplicationInfo& Info, const FReplicationProtocol* Protocol, const uint8* InternalStateBuffer, bool bIsInitialState);

	// Patchup changemask to include any in-flight changes. Returns true if in-flight changes were added.
	bool PatchupObjectChangeMaskWithInflightChanges(uint32 InternalIndex, FReplicationInfo& Info);

	// Invalidate all inflight baseline information
	void InvalidateBaseline(uint32 InternalIndex, FReplicationInfo& Info);

	// Returns true if the record chain starting from the provided RecordInfo contains any records with statedata
	// Note: Does not check if it is part of a huge object
	bool HasInFlightStateChanges(uint32 InternalIndex, const FReplicationInfo& Info) const;

	// Returns true if object has pending state changes in flight
	// Note: Does not check if it is part of a huge object
	bool HasInFlightStateChanges(const FReplicationRecord::FRecordInfo* RecordInfo) const;

	// Returns true object and subobjects can be created on remote
	bool CanSendObject(uint32 InternalIndex) const;

	inline bool IsInitialState(const EReplicatedObjectState State) const { return State == EReplicatedObjectState::PendingCreate || State == EReplicatedObjectState::WaitOnCreateConfirmation; }

	bool IsActiveHugeObject(uint32 InternalIndex) const;
	bool IsObjectPartOfActiveHugeObject(uint32 InternalIndex) const;

	bool CanQueueHugeObject() const;
	bool CanQueueDebugObject() const;

	void FreeHugeObjectSendQueues();

	bool HasDataToSend(const FWriteContext& Context) const;

	void CollectAndAppendExports(FNetSerializationContext& Context, uint8* RESTRICT InternalBuffer, const FReplicationProtocol* Protocol) const;

	bool IsWriteObjectSuccess(EWriteObjectStatus Status) const;

	void SerializeObjectStateDelta(FNetSerializationContext& Context, uint32 InternalIndex, const FReplicationInfo& Info, const FNetRefHandleManager::FReplicatedObjectData& ObjectData, const uint8* RESTRICT ReplicatedObjectStateBuffer, FDeltaCompressionBaseline& CurrentBaseline, uint32 CreatedBaselineIndex);

	void DiscardAllRecords();
	void StopAllReplication();

	void MarkObjectDirty(FInternalNetRefIndex InternalIndex, const char* Caller);

	// Writes a sentinel naking it easier for the receiver to detect bitstream errors.
	void WriteSentinel(FNetBitStreamWriter* Writer, const TCHAR* DebugName);

	// Returns a valid FCannotSendInfo if we should start tracking how long we are blocked from sending.
	FCannotSendInfo* ShouldWarnIfCannotSend(const FReplicationInfo& Info, FInternalNetRefIndex InternalIndex) const;

private:
	// Replication parameters
	FReplicationParameters Parameters;

	// Record of all in-flight data
	FReplicationRecord ReplicationRecord;

	// Tracking information for the state of all objects
	TArray<FReplicationInfo> ReplicatedObjects;

	// Tracking information linking all in-flight data per object
	TArray<FReplicationRecord::FRecordInfoList> ReplicatedObjectsRecordInfoLists;

	// Each replicated object has a scheduling priority that is bumped every time we have a chance to send and zeroed out every time the object is successfully sent
	TArray<float> SchedulingPriorities;

	// Track Objects Pending Destroy?
	FNetBitArray ObjectsPendingDestroy;

	// Objects in this bitArray with dirty change masks
	FNetBitArray ObjectsWithDirtyChanges;

	// Track Objects That is in scope for this connection
	FNetBitArray ObjectsInScope;
	
	// Handles logic for all attachments to objects.
	FNetObjectAttachmentsWriter Attachments;

	// Cached internal systems
	FReplicationSystemInternal* ReplicationSystemInternal = nullptr;
	FNetRefHandleManager* NetRefHandleManager = nullptr;
	UObjectReplicationBridge* ReplicationBridge = nullptr;
	FDeltaCompressionBaselineManager* BaselineManager = nullptr;
	FObjectReferenceCache* ObjectReferenceCache = nullptr;
	const FReplicationFiltering* ReplicationFiltering = nullptr;
	FReplicationConditionals* ReplicationConditionals = nullptr;
	const UPartialNetObjectAttachmentHandler* PartialNetObjectAttachmentHandler = nullptr;
	const UNetObjectBlobHandler* NetObjectBlobHandler = nullptr;
	FNetExports* NetExports = nullptr;
	FNetTypeStats* NetTypeStats = nullptr;

	FWriteContext WriteContext;
	FBitStreamInfo WriteBitStreamInfo;
	FHugeObjectSendQueue HugeObjectSendQueue;
	FHugeObjectSendQueue DebugObjectSendQueue;

	// Features that can aid in tracking down bitstream errors etc.
	EReplicationDataStreamDebugFeatures StreamDebugFeatures = EReplicationDataStreamDebugFeatures::None;

	// Is replication enabled?
	bool bReplicationEnabled = false;
	
	// Should we use high prio create?
	const bool bHighPrioCreate = false;

	mutable TMap<FInternalNetRefIndex, FCannotSendInfo> CannotSendInfos;
};

inline FReplicationWriter::FReplicationInfo::FReplicationInfo()
: Value(0U)
{
}

template<FReplicationWriter::EReplicatedObjectState LostState> void FReplicationWriter::HandleDroppedRecord(FReplicationWriter::EReplicatedObjectState CurrentState, const FReplicationRecord::FRecordInfo& RecordInfo, FReplicationWriter::FReplicationInfo& Info, const FNetObjectAttachmentsWriter::FReliableReplicationRecord& AttachmentRecord)
{
	//static_assert(false, "Expected specialization to exist.");
}

}
