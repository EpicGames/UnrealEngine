// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedStringNetToken.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/StringTokenStore.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"

namespace UE::Net::Private
{

void FSharedStringNetToken::AssignToken(FNetToken InToken, const FTokenSource InTokenSource)
{
	// In order to store a shared token we need to keep track of its resolve context.
	checkf(InTokenSource.ReplicationSystemId != InvalidReplicationSystemId, TEXT("FSharedStringNetToken::AssignToken requires a valid ReplicationSystemId."));

	Token = InToken;
	TokenSource = InTokenSource;
}

void FSharedStringNetToken::MarkStale()
{
	TokenSource = FTokenSource();
}

const TCHAR* FSharedStringNetToken::ResolveToken() const
{
	// Can only resolve this token if we know which TokenStore it came from
	if (!ensureMsgf(!IsStale(), TEXT("FSharedStringNetToken::ResolveToken: stored token is stale, it can't be resolved.")))
	{
		return nullptr;
	}

	const UReplicationSystem* ReplicationSystem = GetReplicationSystem(TokenSource.ReplicationSystemId);

	if (!ensureMsgf(ReplicationSystem, TEXT("FSharedStringNetToken::ResolveToken: source ReplicationSystemId is invalid.")))
	{
		return nullptr;
	}

	const FNetTokenStore* const NetTokenStore = ReplicationSystem->GetNetTokenStore();
	const FStringTokenStore* const StringTokenStore = NetTokenStore->GetDataStore<FStringTokenStore>();
	const FNetTokenStoreState* const RemoteNetTokenStoreState = TokenSource.ConnectionId != InvalidConnectionId ? NetTokenStore->GetRemoteNetTokenStoreState(TokenSource.ConnectionId) : nullptr;

	return StringTokenStore->ResolveToken(Token, RemoteNetTokenStoreState);
}

bool FSharedStringNetToken::IsEqualToToken(const FNetToken& OtherToken, const FNetTokenResolveContext& OtherResolveContext) const
{
	// Can only compare this token if we know which TokenStore it came from
	if (!ensureMsgf(!IsStale(), TEXT("FSharedStringNetToken::IsEqualToToken: stored token is stale, it can't be compared.")))
	{
		return false;
	}

	// An invalid token doesn't need a resolve context
	if (!OtherToken.IsValid() && !Token.IsValid())
	{
		return true;
	}

	if (!ensureMsgf(OtherResolveContext.NetTokenStore, TEXT("FSharedStringNetToken::IsEqualToToken: OtherResolveContext requires a valid NetTokenStore.")))
	{
		return false;
	}

	const UReplicationSystem* ReplicationSystem = GetReplicationSystem(TokenSource.ReplicationSystemId);

	if (!ensureMsgf(ReplicationSystem, TEXT("FSharedStringNetToken::IsEqualToToken: source ReplicationSystemId %u is invalid."), TokenSource.ReplicationSystemId))
	{
		return false;
	}

	// Can only compare token values directly if they're from the same token store
	const FNetTokenStore* const ThisNetTokenStore = ReplicationSystem->GetNetTokenStore();
	const FStringTokenStore* const ThisStringTokenStore = ThisNetTokenStore->GetDataStore<FStringTokenStore>();
	const FStringTokenStore* const OtherStringTokenStore = OtherResolveContext.NetTokenStore->GetDataStore<FStringTokenStore>();

	if (ThisStringTokenStore == OtherStringTokenStore)
	{
		return Token == OtherToken;
	}
	else
	{
		// Different token stores: resolve the tokens and compare the strings.
		const FNetTokenStoreState* const ThisRemoteNetTokenStoreState = TokenSource.ConnectionId != InvalidConnectionId ? ThisNetTokenStore->GetRemoteNetTokenStoreState(TokenSource.ConnectionId) : nullptr;
		const TCHAR* const ThisResolvedToken = ThisStringTokenStore->ResolveToken(Token, ThisRemoteNetTokenStoreState);
		const TCHAR* const OtherResolvedToken = OtherStringTokenStore->ResolveToken(OtherToken, OtherResolveContext.RemoteNetTokenStoreState);

		if (ThisResolvedToken && OtherResolvedToken)
		{
			return FCString::Strcmp(ThisResolvedToken, OtherResolvedToken) == 0;
		}
		else
		{
			return ThisResolvedToken == OtherResolvedToken;
		}
	}
}

void FSharedStringNetToken::WriteNetToken(FNetSerializationContext& Context) const
{
	if (!ensureMsgf(!IsStale(), TEXT("FSharedStringNetToken::WriteNetToken: stored token is stale, it can't be written.")))
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	const uint32 ContextRepSystemId = Context.GetInternalContext()->ReplicationSystem->GetId();

	if (!ensureMsgf(ContextRepSystemId == TokenSource.ReplicationSystemId, TEXT("FSharedStringNetToken::WriteNetToken: mismatched replication systems! This ID: %u, context ID: %u"), TokenSource.ReplicationSystemId, ContextRepSystemId))
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	const FStringTokenStore* const ContextStringTokenStore = Context.GetNetTokenStore()->GetDataStore<FStringTokenStore>();

	ContextStringTokenStore->WriteNetToken(Context, Token);
}

void FSharedStringNetToken::ConditionalWriteNetTokenData(FNetSerializationContext& Context, Private::FNetExportContext* ExportContext) const
{
	if (!ensureMsgf(!IsStale(), TEXT("FSharedStringNetToken::ConditionalWriteNetTokenData: stored token is stale, it can't be written.")))
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	const uint32 ContextRepSystemId = Context.GetInternalContext()->ReplicationSystem->GetId();

	if (!ensureMsgf(ContextRepSystemId == TokenSource.ReplicationSystemId, TEXT("FSharedStringNetToken::ConditionalWriteNetTokenData: mismatched replication systems! This ID: %u, context ID: %u"), TokenSource.ReplicationSystemId, ContextRepSystemId))
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	Context.GetNetTokenStore()->ConditionalWriteNetTokenData(Context, ExportContext, Token);
}


FString FSharedStringNetToken::ToString() const
{
	FString Result;
	if (IsStale())
	{
		Result = TEXT("Stale SharedStringNetToken");
	}
	else
	{
		Result = FString::Printf(TEXT("SharedStringNetToken (Auth:%u TypeId=%u Index=%u RepSystemId=%u ConnectionId=%u)"),
			Token.IsAssignedByAuthority(), Token.GetTypeId(), Token.GetIndex(), TokenSource.ReplicationSystemId, TokenSource.ConnectionId);
	}
	return Result;
}

}