// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Iris/IrisConstants.h"
#include "Net/Core/NetToken/NetToken.h"

namespace UE::Net
{
	class FNetSerializationContext;

	namespace Private
	{
		class FNetExportContext;
	}
}

namespace UE::Net::Private
{

/**
 * Representation of a string FNetToken that's safe to use in a context that's shared between
 * multiple replication systems, like a shared ObjectReferenceCache.
 * 
 * Stores an FNetToken, the replication system it was sourced from, and if it was assigned remotely,
 * also the connection ID it was sourced from. These IDs are used to look up the correct FNetTokenStore
 * and remote FNetTokenStoreState (if needed) to resolve or compare the token. The public API enforces that
 * the token will only be resolved/compared using the correct source token store & state.
 * 
 * When the ReplicationSystem that owns the source token store is destroyed, this shared token
 * must be marked as stale. This is handled in FReplicationSystemFactory::DestroyReplicationSystem.
 */
class FSharedStringNetToken
{
public:
	struct FTokenSource
	{
		/** The ReplicationSystem that assigned this token. Used to look up the corresponding FStringTokenStore. */
		uint32 ReplicationSystemId = InvalidReplicationSystemId;

		/** The connection within the replication system that sent the token, if it was remotely assigned. Used to look up the corresponding FNetTokenStoreState. */
		uint32 ConnectionId = InvalidConnectionId;
	};

	/** Assigns a token. Remote tokens need to specify a valid ConnectionId in the FTokenSource. */
	void AssignToken(FNetToken InToken, const FTokenSource InTokenSource);

	/**
	 * Shared tokens become meaningless without their original ResolveContext.
	 * This token must be invalidated when the ResolveContext is destroyed.
	 * Called via FReplicationSystemFactory::DestroyReplicationSystem and FObjectReferenceCache::ReplicationSystemDestroyed.
	 */
	void MarkStale();

	/** Returns true if the stored token is valid and its backing store is also valid/not stale. */
	bool IsValid() const
	{
		return Token.IsValid() && !IsStale();
	}

	/** Returns true if the stored token's backing store is stale/has been destroyed. */
	bool IsStale() const
	{
		return TokenSource.ReplicationSystemId == InvalidReplicationSystemId;
	}

	/** Resolves the token if it's not stale using its source token store and if needed, the remote token store state. Returns null if the token is stale. */
	const TCHAR* ResolveToken() const;

	/**
	 * Compare this token to another, using their respective backing token stores if necessary.
	 * If both tokens come from the same store, the tokens can be compared directly.
	 * If the tokens come from different stores, we have to resolve the strings and compare those.
	 * Stale shared tokens are not considered equal to any (non-shared) NetToken.
	 * If OtherToken is invalid, it's considered equal only if this token is also invalid and not stale.
	 */
	bool IsEqualToToken(const FNetToken& OtherToken, const FNetTokenResolveContext& OtherResolveContext) const;

	/** Return a string representation of this token. */
	FString ToString() const;

	/** Returns the replication system that assigned this token. */
	uint32 GetSourceReplicationSystemId() const
	{
		return TokenSource.ReplicationSystemId;
	}

	/** Returns the connection ID that assigned this token. It is only relevant together with the source replication system ID. */
	uint32 GetSourceConnectionId() const
	{
		return TokenSource.ConnectionId;
	}

	/**
	 * Forwards to WriteNetToken of the token store of the FNetSerializationContext. Validates that the context's store is the same as this token's store.
	 */
	void WriteNetToken(UE::Net::FNetSerializationContext& Context) const;

	/**
	 * Forwards to ConditionalWriteNetTokenData of the token store of the FNetSerializationContext. Validates that the context's store is the same as the token's store.
	 */
	void ConditionalWriteNetTokenData(FNetSerializationContext& Context, Private::FNetExportContext* ExportContext) const;

private:
	FNetToken Token;
	FTokenSource TokenSource;
};

}
