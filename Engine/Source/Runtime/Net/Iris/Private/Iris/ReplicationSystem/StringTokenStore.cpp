// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/StringTokenStore.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Hash/CityHash.h"

#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#	define UE_NET_ENABLE_STRING_TOKEN_LOG 0
#else
#	define UE_NET_ENABLE_STRING_TOKEN_LOG 1
#endif 

#if UE_NET_ENABLE_STRING_TOKEN_LOG
#	define UE_LOG_STRINGTOKEN(Format, ...)  UE_LOG(LogNetToken, Verbose, Format, ##__VA_ARGS__)
#else
#	define UE_LOG_STRINGTOKEN(...)
#endif

#define UE_LOG_STRINGTOKEN_WARNING(Format, ...)  UE_LOG(LogNetToken, Warning, Format, ##__VA_ARGS__)

namespace UE::Net
{

FNetToken FStringTokenStore::GetOrCreateToken(const TCHAR* Name, uint32 Length)
{
	FNetToken Result;
	const FNetTokenStoreKey Key = GetOrCreatePersistentString(Name, Length);
	if (Key.IsValid())
	{
		Result = GetNetTokenFromKey(Key);
		if (!Result.IsValid())		
		{
			Result = CreateAndStoreTokenForKey(Key);
			UE_LOG_STRINGTOKEN(TEXT("FStringTokenStore::GetOrCreateToken - Created %s for %s"), *Result.ToString(), Name);
		}
	}

	return Result;
}

FNetToken FStringTokenStore::GetOrCreateToken(const FString& String)
{
	return GetOrCreateToken(ToCStr(String), String.Len());
}

FNetTokenDataStore::FNetTokenStoreKey FStringTokenStore::GetOrCreatePersistentString(const TCHAR* Name, uint32 Length)
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(&TokenStore));

	// Hash name
	const uint32 NameSize = Length * sizeof(TCHAR);
	uint64 HashedName = CityHash64((const char*)Name, NameSize);
	
	if (const FNetTokenStoreKey* ExistingKey = HashToKey.Find(HashedName))
	{
		return *ExistingKey;
	}

	const FNetTokenStoreKey NewKey = GetNextNetTokenStoreKey();
	if (NewKey.IsValid())
	{
		// Allocate memory and copy persistent string
		TCHAR* PersistentString = (TCHAR*)Allocator.Alloc(NameSize + sizeof(TCHAR), alignof(TCHAR));
		FCString::Strncpy((TCHAR*)PersistentString, Name, Length + 1U);

		HashToKey.Add(HashedName, NewKey);
		StoredStrings.Add(PersistentString);
	
		return NewKey;
	}

	return FNetTokenStoreKey();
}

FStringTokenStore::FStringTokenStore(FNetTokenStore& InTokenStore)
: FNetTokenDataStore(InTokenStore)
, Allocator()
{
	// As we use an array for our storage we must match the size of the StoredTokens array.
	// We assume that Index 0 is invalid.
	StoredStrings.SetNum(StoredTokens.Num());
}

const TCHAR* FStringTokenStore::ResolveToken(FNetToken Token, const FNetTokenStoreState* NetTokenStoreState) const
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(&TokenStore));
	const FNetTokenStoreState* TokenStoreState = TokenStore.IsLocalToken(Token) ? TokenStore.GetLocalNetTokenStoreState() : NetTokenStoreState;
	if (Token.IsValid() && ensureMsgf(TokenStoreState, TEXT("FStringTokenStore::ResolveToken Needs valid TokenStoreState to resolve %s"), *Token.ToString()))
	{
		const FNetTokenStoreKey StoreKey = GetTokenKey(Token, *TokenStoreState);
		if (StoreKey.IsValid() && StoreKey.GetKeyIndex() < (uint32)StoredStrings.Num())
		{
			return StoredStrings[StoreKey.GetKeyIndex()];
		}
		else
		{
			UE_LOGF(LogNetToken, Error, "FStringTokenStore::ResolveToken failed to resolve %ls in NetTokenDataStore: %ls", *Token.ToString(), *GetTokenStoreName().ToString());
		}
	}

	return nullptr;
}

void FStringTokenStore::WriteTokenData(FNetSerializationContext& Context, FNetTokenStoreKey TokenStoreKey) const
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(&TokenStore));
	// $TODO: $IRIS: Do not calculate the length of the string to write the data.
	WriteString(Context.GetBitStreamWriter(), FStringView(StoredStrings[TokenStoreKey.GetKeyIndex()]));
}

void FStringTokenStore::WriteTokenData(FArchive& Ar, FNetTokenStoreKey TokenStoreKey, UPackageMap* Map /*= nullptr*/) const
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(&TokenStore));
	FString Temp(StoredStrings[TokenStoreKey.GetKeyIndex()]);
	Ar << Temp;
}

FNetTokenDataStore::FNetTokenStoreKey FStringTokenStore::ReadTokenData(FNetSerializationContext& Context, const FNetToken& NetToken)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// Read the token data and add it to the string store without assigning LocalToken
	FString Temp;
	ReadString(Reader, Temp);

	if (!Reader->IsOverflown())
	{
		return GetOrCreatePersistentString(*Temp, Temp.Len());;
	}
	else
	{
		return FNetTokenStoreKey();
	}
}

FNetTokenDataStore::FNetTokenStoreKey FStringTokenStore::ReadTokenData(FArchive& Ar, const FNetToken& NetToken, UPackageMap* Map /*= nullptr*/)
{
	// Read the token data and add it to the string store without assigning LocalToken
	FString Temp;
	Ar << Temp;

	if (!Ar.IsError())
	{
		return GetOrCreatePersistentString(*Temp, Temp.Len());
	}
	else
	{
		return FNetTokenStoreKey();
	}
}


}
