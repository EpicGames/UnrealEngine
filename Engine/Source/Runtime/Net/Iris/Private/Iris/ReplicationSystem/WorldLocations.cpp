// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationSystem/WorldLocations.h"
#include "Iris/Core/IrisMemoryTracker.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldLocations)

namespace UE::Net
{

void FWorldLocations::Init(const FWorldLocationsInitParams& InitParams)
{
	ValidInfoIndexes.Init(InitParams.MaxInternalNetRefIndex);
	ObjectsWithDirtyInfo.Init(InitParams.MaxInternalNetRefIndex);
	ObjectsRequiringFrequentWorldLocationUpdate.Init(InitParams.MaxInternalNetRefIndex);
	StorageIndexes.SetNum(InitParams.MaxInternalNetRefIndex);

	ReservedStorageSlot.Init(InitParams.PreallocatedStorageCount);	
	StoredObjectInfo = TNetChunkedArray<FObjectInfo, StorageElementsPerChunk>(InitParams.PreallocatedStorageCount, EInitMemory::Constructor);

	MinWorldPos = GetDefault<UWorldLocationsConfig>()->MinPos;
	MaxWorldPos = GetDefault<UWorldLocationsConfig>()->MaxPos;
	MaxNetCullDistance = GetDefault<UWorldLocationsConfig>()->MaxNetCullDistance;
	
	NetRefHandleManager = &InitParams.ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager();
}

void FWorldLocations::PostSendUpdate()
{
	// Clear the dirty info list to start fresh for the next frame
#if DO_ENSURE
	bLockdownDirtyList = false;
#endif

	ObjectsWithDirtyInfo.ClearAllBits();
}

void FWorldLocations::OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex)
{
	ValidInfoIndexes.SetNumBits(NewMaxInternalIndex);
	ObjectsWithDirtyInfo.SetNumBits(NewMaxInternalIndex);
	ObjectsRequiringFrequentWorldLocationUpdate.SetNumBits(NewMaxInternalIndex);
	StorageIndexes.SetNum(NewMaxInternalIndex);
}

void FWorldLocations::InitObjectInfoCache(FInternalNetRefIndex ObjectIndex)
{
	if (ValidInfoIndexes.IsBitSet(ObjectIndex))
	{
		// Only init on first assignment
		return;
	}

	ValidInfoIndexes.SetBit(ObjectIndex);

	// Find an available slot
	uint32 AvailableSlot = ReservedStorageSlot.FindFirstZero();

	// No more slots available, grow the storage space by a single chunk
	if (AvailableSlot == FNetBitArray::InvalidIndex)
	{
		LLM_SCOPE_BYTAG(Iris);
		AvailableSlot = ReservedStorageSlot.GetNumBits();
		StoredObjectInfo.Add(StorageElementsPerChunk);
		ReservedStorageSlot.SetNumBits(AvailableSlot + StorageElementsPerChunk);
	}

	ReservedStorageSlot.SetBit(AvailableSlot);
	StorageIndexes[ObjectIndex] = AvailableSlot;
}

void FWorldLocations::RemoveObjectInfoCache(FInternalNetRefIndex ObjectIndex)
{
	if (!ValidInfoIndexes.IsBitSet(ObjectIndex))
	{
		// Object did not register a location
		return;
	}

	ValidInfoIndexes.ClearBit(ObjectIndex);
	ObjectsWithDirtyInfo.ClearBit(ObjectIndex);
	ObjectsRequiringFrequentWorldLocationUpdate.ClearBit(ObjectIndex);

	const uint32 StorageIndex = StorageIndexes[ObjectIndex];
	StorageIndexes[ObjectIndex] = INDEX_NONE;

	// Default construct the info since it can be reused in the future.
	StoredObjectInfo[StorageIndex] = FObjectInfo();

	ReservedStorageSlot.ClearBit(StorageIndex);
}

void FWorldLocations::SetObjectInfo(FInternalNetRefIndex ObjectIndex, const FVector& Location, const float NetCullDistance)
{
	ensure(!bLockdownDirtyList);

	checkSlow(ValidInfoIndexes.IsBitSet(ObjectIndex));
	FObjectInfo& TargetObjectInfo = GetObjectInfo(ObjectIndex);

	const FVector ClampedLoc = ClampPositionToBoundary(Location);
	
	const bool bHasCullDistanceChanged = TargetObjectInfo.CullDistance != NetCullDistance;
	const bool bHasInfoChanged = ObjectsWithDirtyInfo.GetBit(ObjectIndex) || TargetObjectInfo.WorldLocation != ClampedLoc || bHasCullDistanceChanged;

	TargetObjectInfo.WorldLocation = ClampedLoc;

	// For now we just warn, this will be clamped by filter.
	if (bHasCullDistanceChanged && MaxNetCullDistance > 0.f && NetCullDistance > MaxNetCullDistance)
	{
		UE_LOGF(LogIrisNetCull, Verbose, "FWorldLocations::SetObjectInfo ReplicatedObject %ls cull distance %f is above the max %f. Consider making object always relevant instead", *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), NetCullDistance, MaxNetCullDistance);
		ensureMsgf(false, TEXT("FWorldLocations::SetObjectInfo ReplicatedObject %s cull distance %f is above the max %f. Consider making object always relevant instead"), *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), NetCullDistance, MaxNetCullDistance);
	}

	TargetObjectInfo.CullDistance = NetCullDistance;

	ObjectsWithDirtyInfo.OrBitValue(ObjectIndex, bHasInfoChanged);
}

bool FWorldLocations::SetCullDistanceOverride(FInternalNetRefIndex ObjectIndex, float CullDistance)
{
	ensure(!bLockdownDirtyList);

	if (ValidInfoIndexes.IsBitSet(ObjectIndex))
	{
		// TODO: Check for zero or clamp down on huge values ?
		if (GetObjectInfo(ObjectIndex).CullDistanceOverride != CullDistance)
		{
			// For now we just warn, this will be clamped by filter.
			if (MaxNetCullDistance > 0.f && CullDistance > MaxNetCullDistance)
			{
				UE_LOGF(LogIrisNetCull, Verbose, "FWorldLocations::SetCullDistanceOverride ReplicatedObject %ls cull distance %f is above the max %f. Consider making object always relevant instead", *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), CullDistance, MaxNetCullDistance);	
				ensureMsgf(false, TEXT("FWorldLocations::SetCullDistanceOverride ReplicatedObject %s cull distance %f is above the max %f. Consider making object always relevant instead"), *NetRefHandleManager->PrintObjectFromIndex(ObjectIndex), CullDistance, MaxNetCullDistance);	
			}

			GetObjectInfo(ObjectIndex).CullDistanceOverride = CullDistance;
			ObjectsWithDirtyInfo.SetBitValue(ObjectIndex, true);
		}
		return true;
	}

	return false;
}

bool FWorldLocations::ClearCullDistanceOverride(FInternalNetRefIndex ObjectIndex)
{
	ensure(!bLockdownDirtyList);

	if (ValidInfoIndexes.IsBitSet(ObjectIndex) && GetObjectInfo(ObjectIndex).CullDistanceOverride != FLT_MAX)
	{
		GetObjectInfo(ObjectIndex).CullDistanceOverride = FLT_MAX;
		ObjectsWithDirtyInfo.SetBitValue(ObjectIndex, true);
		return true;
	}

	return false;
}

void FWorldLocations::LockDirtyInfoList(bool bLock)
{
#if DO_ENSURE
	bLockdownDirtyList = bLock;
#endif
}

} // end namespace UE::Net
