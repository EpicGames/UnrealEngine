// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternalNetSerializers.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Logging/StructuredLog.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net
{

struct FArrayPropertyNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true;
	static constexpr bool bHasDynamicState = true;

	// Types
	struct FQuantizedType
	{
		// How many elements the current allocation can hold.
		uint16 ElementCapacityCount;
		// How many elements are valid
		uint16 ElementCount;
		void* ElementStorage;
	};

	typedef FScriptArray SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FArrayPropertyNetSerializerConfig ConfigType;

	//
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	static void FreeDynamicStateInternal(FNetSerializationContext&, QuantizedType& Array, const ConfigType* Config);
	static void GrowDynamicStateInternal(FNetSerializationContext&, QuantizedType& Array, const ConfigType* Config, uint16 NewElementCount);
	static void ShrinkDynamicStateInternal(FNetSerializationContext&, QuantizedType& Array, const ConfigType* Config, uint16 NewElementCount);
	static void AdjustArraySize(FNetSerializationContext&, QuantizedType& Array, const ConfigType* Config, uint16 NewElementCount);
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FArrayPropertyNetSerializer);

 uint32 GetNetArrayPropertyData(NetSerializerValuePointer QuantizedArray, NetSerializerValuePointer& OutArrayData)
{
	FArrayPropertyNetSerializer::QuantizedType& Array = *reinterpret_cast<FArrayPropertyNetSerializer::QuantizedType*>(QuantizedArray);

	if (Array.ElementCount > 0U && Array.ElementStorage)
	{
		OutArrayData = NetSerializerValuePointer(Array.ElementStorage);

		return Array.ElementCount;
	}

	return 0U;
}

void FArrayPropertyNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& Array = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	if (Array.ElementCount == 0)
	{
		Writer->WriteBits(1U, 1U);
		return;
	}

	Writer->WriteBits(0U, 1U);
	Writer->WriteBits(Array.ElementCount, Config->ElementCountBitCount);

	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

	FNetSerializeArgs ElementArgs;
	ElementArgs.Version = 0;
	ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
	ElementArgs.Source = NetSerializerValuePointer(Array.ElementStorage);

	const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
	const uint32 ElementSize = ElementStateDescriptor->InternalSize;

	// Check if we have additional changemask bits
	const FNetBitArrayView* ChangeMask = Args.ChangeMaskInfo.BitCount > 1U ? Context.GetChangeMask() : nullptr;
	if (!ChangeMask)
	{
		for (uint32 ElementIt = 0, ElementEndIt = Array.ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			UE_NET_TRACE_SCOPE(Element, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
			ElementSerializer->Serialize(Context, ElementArgs);
			ElementArgs.Source += ElementSize;
		}
	}
	else
	{
		// We currently use a simple modulo scheme for bits in the changemask, if we have more elements in the array then is covered by the changemask
		// several entries in the array will be considered dirty and be serialized
		// As the first bit is used by the owning property we need to offset by one and deduct one from the usable bits
		const uint32 ChangeMaskBitOffset = Args.ChangeMaskInfo.BitOffset + 1U;
		const uint32 ChangeMaskBitCount = Args.ChangeMaskInfo.BitCount - 1U;

		for (uint32 ElementIt = 0, ElementEndIt = Array.ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			if (ChangeMask->GetBit(ChangeMaskBitOffset + (ElementIt % ChangeMaskBitCount)))
			{
				UE_NET_TRACE_SCOPE(Element, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
				ElementSerializer->Serialize(Context, ElementArgs);
			}
			ElementArgs.Source += ElementSize;
		}
	}
}

void FArrayPropertyNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& Array = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FNetReferenceCollector& Collector = *reinterpret_cast<FNetReferenceCollector*>(Args.Collector);

	if (Array.ElementCount == 0)
	{
		return;
	}

	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

	FNetSerializeArgs ElementArgs;
	ElementArgs.Version = 0;
	ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
	ElementArgs.Source = NetSerializerValuePointer(Array.ElementStorage);

	const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
	const uint32 ElementSize = ElementStateDescriptor->InternalSize;

	// Check if we have additional changemask bits, if we have we only collect references for elements corresponding to dirty bits
	const FNetBitArrayView* ChangeMask = Args.ChangeMaskInfo.BitCount > 1U ? Context.GetChangeMask() : nullptr;
	if (!ChangeMask)
	{
		for (uint32 ElementIt = 0, ElementEndIt = Array.ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			Private::FReplicationStateOperationsInternal::CollectReferences(Context, Collector, Args.ChangeMaskInfo, reinterpret_cast<uint8*>(ElementArgs.Source), ElementStateDescriptor);
			ElementArgs.Source += ElementSize;
		}
	}
	else
	{
		// We currently use a simple modulo scheme for bits in the changemask, if we have more elements in the array then is covered by the changemask
		// several entries in the array will be considered dirty and be serialized
		// As the first bit is used by the owning property we need to offset by one and deduct one from the usable bits
		const uint32 ChangeMaskBitOffset = Args.ChangeMaskInfo.BitOffset + 1U;
		const uint32 ChangeMaskBitCount = Args.ChangeMaskInfo.BitCount - 1U;

		// This is the info that will be stored in the entries
		for (uint32 ElementIt = 0, ElementEndIt = Array.ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			const uint32 ElementBitOffset = ChangeMaskBitOffset + (ElementIt % ChangeMaskBitCount);
			if (ChangeMask->GetBit(ElementBitOffset))
			{
				FNetSerializerChangeMaskParam LocalChangeMaskInfo;
				LocalChangeMaskInfo.BitCount = 1U;
				LocalChangeMaskInfo.BitOffset = ElementBitOffset;

				// Collect references
				Private::FReplicationStateOperationsInternal::CollectReferences(Context, Collector, LocalChangeMaskInfo, reinterpret_cast<uint8*>(ElementArgs.Source), ElementStateDescriptor);
			}
			ElementArgs.Source += ElementSize;
		}
	}
}

/*
 * If array shrinks we may need to destruct elements (if element HasDynamicAllocation)
 * If array grows we may need to construct elements (if element HasDynamicAllocation)
 * 
 * If element has dynamic allocation we may not want to early out and just set element count to zero
 * as that may hold an unnecessarily large amount of memory.
 */

void FArrayPropertyNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	QuantizedType& Array = *reinterpret_cast<QuantizedType*>(Args.Target);
	const uint32 CurrentElementCount = Array.ElementCount;

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const uint32 bIsEmpty = Reader->ReadBits(1U);
	const uint32 NewElementCount = (bIsEmpty ? 0U : Reader->ReadBits(Config->ElementCountBitCount));

	if (NewElementCount > Config->MaxElementCount)
	{
		UE_LOGFMT(LogIris, Error, "FArrayPropertyNetSerializer::Deserialize Element count exceeds Config->MaxElementCount. {NewElementCount} > {MaxElementCount}. Config->ElementCountBitCount: {ElementCountBitCount}", NewElementCount, Config->MaxElementCount, Config->ElementCountBitCount);
		Context.SetError(GNetError_ArraySizeTooLarge);
		return;
	}

	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

	// Memory management
	AdjustArraySize(Context, Array, Config, static_cast<uint16>(NewElementCount));

	// Deserialize
	FNetDeserializeArgs ElementArgs;
	ElementArgs.Version = 0;
	ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
	ElementArgs.Target = NetSerializerValuePointer(Array.ElementStorage);

	const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
	const uint32 ElementSize = ElementStateDescriptor->InternalSize;

	// Check if we have additional changemask bits
	const FNetBitArrayView* ChangeMask = Args.ChangeMaskInfo.BitCount > 1U ? Context.GetChangeMask() : nullptr;
	if (!ChangeMask)
	{
		for (uint32 ElementIt = 0, ElementEndIt = Array.ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			UE_NET_TRACE_SCOPE(Element, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
			ElementSerializer->Deserialize(Context, ElementArgs);
			ElementArgs.Target += ElementSize;
		}
	}
	else
	{
		// We currently use a simple modulo scheme for bits in the changemask, if we have more elements in the array then is covered by the changemask
		// several entries in the array will be considered dirty and be serialized
		// As the first bit is used by the owning property we need to offset by one and deduct one from the usable bits
		const uint32 ChangeMaskBitOffset = Args.ChangeMaskInfo.BitOffset + 1U;
		const uint32 ChangeMaskBitCount = Args.ChangeMaskInfo.BitCount - 1U;

		for (uint32 ElementIt = 0, ElementEndIt = Array.ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			if (ChangeMask->GetBit(ChangeMaskBitOffset + (ElementIt % ChangeMaskBitCount)))
			{
				UE_NET_TRACE_SCOPE(Element, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
				ElementSerializer->Deserialize(Context, ElementArgs);
			}
			ElementArgs.Target += ElementSize;
		}
	}
}

/** The below implementation assumes we wouldn't get the call unless the array had changed in the first place. That is
  * why we don't have an expensive IsEqual check as we expect the arrays to never or rarely be equal. The actual delta
  * compression just relies on the per element delta compressions. While this is naive approach this serializer is only
  * used in backwards compatibility mode.
  * We might want to support something like directed delta based on longest common subsequence. It can be costly to
  * calculate the LCS since it needs to be done in the SerializeDelta call. So LCS should probably not be enabled
  * by default. In a perfect world the serialization can be shared across many connections.
  */
void FArrayPropertyNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& Array = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& PrevArray = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	const uint32 ElementCount = Array.ElementCount;
	const uint32 PrevElementCount = PrevArray.ElementCount;
	const bool bSameSizeArray = (ElementCount == PrevElementCount);
	Writer->WriteBits(bSameSizeArray, 1U);
	if (!bSameSizeArray)
	{
		Writer->WriteBits(Array.ElementCount, Config->ElementCountBitCount);
	}

	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];
	const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
	const uint32 ElementSize = ElementStateDescriptor->InternalSize;
	const FNetBitArrayView* ChangeMask = Args.ChangeMaskInfo.BitCount > 1U ? Context.GetChangeMask() : nullptr;

	// We currently use a simple modulo scheme for bits in the changemask, if we have more elements in the array then is covered by the changemask
	// several entries in the array will be considered dirty and be serialized
	// As the first bit is used by the owning property we need to offset by one and deduct one from the usable bits
	const uint32 ChangeMaskBitOffset = ChangeMask ? Args.ChangeMaskInfo.BitOffset + 1U : 0U;
	const uint32 ChangeMaskBitCount = ChangeMask ? Args.ChangeMaskInfo.BitCount - 1U : 0U;

	// Elements in the current array up to the previous size can use delta serialization.
	if (PrevElementCount)
	{
		FNetSerializeDeltaArgs ElementArgs;
		ElementArgs.Version = 0;
		ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
		ElementArgs.Source = NetSerializerValuePointer(Array.ElementStorage);
		ElementArgs.Prev = NetSerializerValuePointer(PrevArray.ElementStorage);

		if (!ChangeMaskBitCount)
		{
			for (uint32 ElementIt = 0, ElementEndIt = FMath::Min(ElementCount, PrevElementCount); ElementIt < ElementEndIt; ++ElementIt)
			{
				UE_NET_TRACE_SCOPE(Element, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
				ElementSerializer->SerializeDelta(Context, ElementArgs);
				ElementArgs.Source += ElementSize;
				ElementArgs.Prev += ElementSize;
			}
		}
		else
		{
			for (uint32 ElementIt = 0, ElementEndIt = FMath::Min(ElementCount, PrevElementCount); ElementIt < ElementEndIt; ++ElementIt)
			{
				if (ChangeMask->GetBit(ChangeMaskBitOffset + (ElementIt % ChangeMaskBitCount)))
				{
					UE_NET_TRACE_SCOPE(Element, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
					ElementSerializer->SerializeDelta(Context, ElementArgs);
				}
				ElementArgs.Source += ElementSize;
				ElementArgs.Prev += ElementSize;
			}
		}
	}

	// For elements that are beyond the previous size we rely on the element serializer having minimal bandwidth behavior in the standard serialization.
	if (ElementCount > PrevElementCount)
	{
		FNetSerializeArgs ElementArgs;
		ElementArgs.Version = 0;
		ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
		ElementArgs.Source = NetSerializerValuePointer(Array.ElementStorage) + PrevElementCount*ElementSize;

		if (!ChangeMaskBitCount)
		{
			for (uint32 ElementIt = PrevElementCount, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
			{
				UE_NET_TRACE_SCOPE(Element, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
				ElementSerializer->Serialize(Context, ElementArgs);
				ElementArgs.Source += ElementSize;
			}
		}
		else
		{
			for (uint32 ElementIt = PrevElementCount, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
			{
				if (ChangeMask->GetBit(ChangeMaskBitOffset + (ElementIt % ChangeMaskBitCount)))
				{
					UE_NET_TRACE_SCOPE(Element, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
					ElementSerializer->Serialize(Context, ElementArgs);
				}
				ElementArgs.Source += ElementSize;
			}
		}
	}
}

void FArrayPropertyNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	QuantizedType& Array = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& PrevArray = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const uint32 PrevElementCount = PrevArray.ElementCount;
	const bool bSameSizeArray = !!Reader->ReadBits(1U);
	const uint32 ElementCount = (bSameSizeArray ? PrevElementCount : Reader->ReadBits(Config->ElementCountBitCount));

	if (ElementCount > Config->MaxElementCount)
	{
		UE_LOGFMT(LogIris, Error, "FArrayPropertyNetSerializer::DeserializeDelta Element count exceeds Config->MaxElementCount. {ElementCount} > {MaxElementCount}. Config->ElementCountBitCount: {ElementCountBitCount} bSameSizeArray: {bSameSizeArray}", ElementCount, Config->MaxElementCount, Config->ElementCountBitCount, bSameSizeArray);
		Context.SetError(GNetError_ArraySizeTooLarge);
		return;
	}

	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];
	const FNetBitArrayView* ChangeMask = Args.ChangeMaskInfo.BitCount > 1U ? Context.GetChangeMask() : nullptr;

	// We currently use a simple modulo scheme for bits in the changemask, if we have more elements in the array then is covered by the changemask
	// several entries in the array will be considered dirty and be serialized
	// As the first bit is used by the owning property we need to offset by one and deduct one from the usable bits
	const uint32 ChangeMaskBitOffset = ChangeMask ? Args.ChangeMaskInfo.BitOffset + 1U : 0U;
	const uint32 ChangeMaskBitCount = ChangeMask ? Args.ChangeMaskInfo.BitCount - 1U : 0U;

	const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
	const uint32 ElementSize = ElementStateDescriptor->InternalSize;

	AdjustArraySize(Context, Array, Config, static_cast<uint16>(ElementCount));

	// Elements in the current array up to the previous size can use delta serialization.
	if (PrevElementCount)
	{
		FNetDeserializeDeltaArgs ElementArgs;
		ElementArgs.Version = 0;
		ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
		ElementArgs.Target = NetSerializerValuePointer(Array.ElementStorage);
		ElementArgs.Prev = NetSerializerValuePointer(PrevArray.ElementStorage);

		if (!ChangeMaskBitCount)
		{
			for (uint32 ElementIt = 0, ElementEndIt = FMath::Min(ElementCount, PrevElementCount); ElementIt < ElementEndIt; ++ElementIt)
			{
				UE_NET_TRACE_SCOPE(Element, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
				ElementSerializer->DeserializeDelta(Context, ElementArgs);
				ElementArgs.Target += ElementSize;
				ElementArgs.Prev += ElementSize;
			}
		}
		else
		{
			for (uint32 ElementIt = 0, ElementEndIt = FMath::Min(ElementCount, PrevElementCount); ElementIt < ElementEndIt; ++ElementIt)
			{
				if (ChangeMask->GetBit(ChangeMaskBitOffset + (ElementIt % ChangeMaskBitCount)))
				{
					UE_NET_TRACE_SCOPE(Element, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
					ElementSerializer->DeserializeDelta(Context, ElementArgs);
				}
				ElementArgs.Target += ElementSize;
				ElementArgs.Prev += ElementSize;
			}
		}
	}

	// For elements that are beyond the previous size we rely on the element serializer having minimal bandwidth behavior in the standard serialization.
	if (ElementCount > PrevElementCount)
	{
		FNetDeserializeArgs ElementArgs;
		ElementArgs.Version = 0;
		ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
		ElementArgs.Target = NetSerializerValuePointer(Array.ElementStorage) + PrevElementCount*ElementSize;

		if (!ChangeMaskBitCount)
		{
			for (uint32 ElementIt = PrevElementCount, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
			{
				UE_NET_TRACE_SCOPE(Element, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
				ElementSerializer->Deserialize(Context, ElementArgs);
				ElementArgs.Target += ElementSize;
			}
		}
		else
		{
			for (uint32 ElementIt = PrevElementCount, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
			{
				if (ChangeMask->GetBit(ChangeMaskBitOffset + (ElementIt % ChangeMaskBitCount)))
				{
					UE_NET_TRACE_SCOPE(Element, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
					ElementSerializer->Deserialize(Context, ElementArgs);
				}
				ElementArgs.Target += ElementSize;
			}
		}
	}
}

void FArrayPropertyNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);

	FScriptArrayHelper ScriptArrayHelper(Config->Property.Get(), reinterpret_cast<const void*>(Args.Source));
	const uint32 ElementCount = static_cast<uint32>(ScriptArrayHelper.Num());

	if (ElementCount > Config->MaxElementCount)
	{
		UE_LOGFMT(LogIris, Error, "FArrayPropertyNetSerializer::Quantize Element count exceeds Config->MaxElementCount. {ElementCount} > {MaxElementCount}.", ElementCount, Config->MaxElementCount);
		Context.SetError(GNetError_ArraySizeTooLarge);
		return;
	}

	QuantizedType& TargetArray = *reinterpret_cast<QuantizedType*>(Args.Target);
	AdjustArraySize(Context, TargetArray, Config, static_cast<uint16>(ElementCount));

	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

	FNetQuantizeArgs ElementArgs;
	ElementArgs.Version = 0;
	ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
	ElementArgs.Source = 0;
	ElementArgs.Target = NetSerializerValuePointer(TargetArray.ElementStorage);

	const FNetBitArrayView* ChangeMask = Args.ChangeMaskInfo.BitCount > 1U ? Context.GetChangeMask() : nullptr;

	const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
	const uint32 ElementSize = ElementStateDescriptor->InternalSize;

	if (!ChangeMask)
	{
		for (uint32 ElementIt = 0, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			ElementArgs.Source = NetSerializerValuePointer(static_cast<const void*>(ScriptArrayHelper.GetRawPtr(ElementIt)));
			ElementSerializer->Quantize(Context, ElementArgs);
			ElementArgs.Target += ElementSize;
		}
	}
	else
	{
		// We currently use a simple modulo scheme for bits in the changemask, if we have more elements in the array then is covered by the changemask
		// several entries in the array will be considered dirty and be serialized
		// As the first bit is used by the owning property we need to offset by one and deduct one from the usable bits
		const uint32 ChangeMaskBitOffset = ChangeMask ? Args.ChangeMaskInfo.BitOffset + 1U : 0U;
		const uint32 ChangeMaskBitCount = ChangeMask ? Args.ChangeMaskInfo.BitCount - 1U : 0U;

		for (uint32 ElementIt = 0, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			if (ChangeMask->GetBit(ChangeMaskBitOffset + (ElementIt % ChangeMaskBitCount)))
			{
				ElementArgs.Source = NetSerializerValuePointer(static_cast<const void*>(ScriptArrayHelper.GetRawPtr(ElementIt)));
				ElementSerializer->Quantize(Context, ElementArgs);
			}
			ElementArgs.Target += ElementSize;
		}
	}
}

void FArrayPropertyNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);

	FScriptArrayHelper ScriptArrayHelper(Config->Property.Get(), reinterpret_cast<void*>(Args.Target));
	const QuantizedType& SourceArray = *reinterpret_cast<QuantizedType*>(Args.Source);
	const uint32 ElementCount = SourceArray.ElementCount;
	ScriptArrayHelper.Resize(ElementCount);

	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

	FNetDequantizeArgs ElementArgs;
	ElementArgs.Version = 0;
	ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
	ElementArgs.Source = NetSerializerValuePointer(SourceArray.ElementStorage);
	ElementArgs.Target = 0;

	const FNetBitArrayView* ChangeMask = Args.ChangeMaskInfo.BitCount > 1U ? Context.GetChangeMask() : nullptr;

	const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
	const uint32 ElementSize = ElementStateDescriptor->InternalSize;

	// Currently we do not support partial dequantize using changemask for array properties due to complexities elsewhere (FastArrayReplicationFragment and PropertyReplicationState::ApplyState)
	if (!ChangeMask)
	{
		for (uint32 ElementIt = 0, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			ElementArgs.Target = NetSerializerValuePointer(static_cast<const void*>(ScriptArrayHelper.GetRawPtr(ElementIt)));
			ElementSerializer->Dequantize(Context, ElementArgs);
			ElementArgs.Source += ElementSize;
		}
	}
	else
	{
		// We currently use a simple modulo scheme for bits in the changemask, if we have more elements in the array then is covered by the changemask
		// several entries in the array will be considered dirty and be serialized
		// As the first bit is used by the owning property we need to offset by one and deduct one from the usable bits
		const uint32 ChangeMaskBitOffset = ChangeMask ? Args.ChangeMaskInfo.BitOffset + 1U : 0U;
		const uint32 ChangeMaskBitCount = ChangeMask ? Args.ChangeMaskInfo.BitCount - 1U : 0U;

		for (uint32 ElementIt = 0, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			if (ChangeMask->GetBit(ChangeMaskBitOffset + (ElementIt % ChangeMaskBitCount)))
			{
				ElementArgs.Target = NetSerializerValuePointer(static_cast<const void*>(ScriptArrayHelper.GetRawPtr(ElementIt)));
				ElementSerializer->Dequantize(Context, ElementArgs);
			}
			ElementArgs.Source += ElementSize;
		}
	}
}

bool FArrayPropertyNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);

		const QuantizedType& Array0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Array1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		const uint32 ElementCount0 = Array0.ElementCount;
		const uint32 ElementCount1 = Array1.ElementCount;
		if (ElementCount0 != ElementCount1)
		{
			return false;
		}

		const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
		const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

		FNetIsEqualArgs ElementArgs;
		ElementArgs.Version = 0;
		ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
		ElementArgs.Source0 = NetSerializerValuePointer(Array0.ElementStorage);
		ElementArgs.Source1 = NetSerializerValuePointer(Array1.ElementStorage);
		ElementArgs.bStateIsQuantized = true;
		const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
		const uint32 ElementSize = ElementStateDescriptor->InternalSize;
		for (uint32 ElementIt = 0, ElementEndIt = ElementCount0; ElementIt < ElementEndIt; ++ElementIt)
		{
			if (!ElementSerializer->IsEqual(Context, ElementArgs))
			{
				return false;
			}

			ElementArgs.Source0 += ElementSize;
			ElementArgs.Source1 += ElementSize;
		}
	}
	else
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);

		FScriptArrayHelper ScriptArrayHelper0(Config->Property.Get(), reinterpret_cast<const void*>(Args.Source0));
		FScriptArrayHelper ScriptArrayHelper1(Config->Property.Get(), reinterpret_cast<const void*>(Args.Source1));
		const uint32 ElementCount0 = static_cast<uint32>(ScriptArrayHelper0.Num());
		const uint32 ElementCount1 = static_cast<uint32>(ScriptArrayHelper1.Num());
		if (ElementCount0 != ElementCount1)
		{
			return false;
		}

		const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
		const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

		FNetIsEqualArgs ElementArgs;
		ElementArgs.Version = 0;
		ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
		ElementArgs.bStateIsQuantized = false;
		const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
		for (uint32 ElementIt = 0, ElementEndIt = ElementCount0; ElementIt < ElementEndIt; ++ElementIt)
		{
			ElementArgs.Source0 = NetSerializerValuePointer(static_cast<const void*>(ScriptArrayHelper0.GetRawPtr(ElementIt)));
			ElementArgs.Source1 = NetSerializerValuePointer(static_cast<const void*>(ScriptArrayHelper1.GetRawPtr(ElementIt)));
			if (!ElementSerializer->IsEqual(Context, ElementArgs))
			{
				return false;
			}
		}
	}

	return true;
}

bool FArrayPropertyNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);

	FScriptArrayHelper ScriptArrayHelper(Config->Property.Get(), reinterpret_cast<const void*>(Args.Source));
	const uint32 ElementCount = static_cast<uint32>(ScriptArrayHelper.Num());
	if (ElementCount > Config->MaxElementCount)
	{
		return false;
	}

	// We expect the inner element ReplicationStateDescriptor to contain exactly one element.
	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
	if (ElementStateDescriptor->MemberCount != 1U)
	{
		return false;
	}

	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

	FNetValidateArgs ElementArgs;
	ElementArgs.Version = 0;
	ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
	const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
	for (uint32 ElementIt = 0, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
	{
		ElementArgs.Source = NetSerializerValuePointer(static_cast<const void*>(ScriptArrayHelper.GetRawPtr(ElementIt)));
		if (!ElementSerializer->Validate(Context, ElementArgs))
		{
			return false;
		}
	}

	return true;
}

void FArrayPropertyNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	QuantizedType& TargetArray = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& SourceArray = *reinterpret_cast<const QuantizedType*>(Args.Source);

	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;

	const SIZE_T ElementSize = ElementStateDescriptor->InternalSize;
	const SIZE_T ElementAlignment = ElementStateDescriptor->InternalAlignment;

	// Clone storage
	void* ElementStorage = nullptr;
	if (SourceArray.ElementCount > 0)
	{
		ElementStorage = Context.GetInternalContext()->Alloc(ElementSize*SourceArray.ElementCount, ElementAlignment);
		FMemory::Memcpy(ElementStorage, SourceArray.ElementStorage, ElementSize*SourceArray.ElementCount);
	}
	TargetArray.ElementCapacityCount = SourceArray.ElementCount;
	TargetArray.ElementCount = SourceArray.ElementCount;
	TargetArray.ElementStorage = ElementStorage;

	// If no member has dynamic state then there's nothing more to do.
	if (!EnumHasAnyFlags(ElementStateDescriptor->Traits, EReplicationStateTraits::HasDynamicState) || SourceArray.ElementCount == 0)
	{
		return;
	}

	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];

	FNetCloneDynamicStateArgs ElementArgs;
	ElementArgs.Version = 0;
	ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
	ElementArgs.Source = NetSerializerValuePointer(SourceArray.ElementStorage);
	ElementArgs.Target = NetSerializerValuePointer(TargetArray.ElementStorage);

	const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
	for (uint32 ElementIt = 0, ElementEndIt = SourceArray.ElementCount; ElementIt < ElementEndIt; ++ElementIt)
	{
		ElementSerializer->CloneDynamicState(Context, ElementArgs);
		ElementArgs.Source += ElementSize;
		ElementArgs.Target += ElementSize;
	}
}

void FArrayPropertyNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	return FreeDynamicStateInternal(Context, *reinterpret_cast<QuantizedType*>(Args.Source), static_cast<const ConfigType*>(Args.NetSerializerConfig));
}

void FArrayPropertyNetSerializer::FreeDynamicStateInternal(FNetSerializationContext& Context, QuantizedType& Array, const ConfigType* Config)
{
	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;

	if (EnumHasAnyFlags(ElementStateDescriptor->Traits, EReplicationStateTraits::HasDynamicState))
	{
		const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];
		const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
		const uint32 ElementSize = ElementStateDescriptor->InternalSize;

		FNetFreeDynamicStateArgs ElementArgs;
		ElementArgs.Version = 0;
		ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
		ElementArgs.Source = NetSerializerValuePointer(Array.ElementStorage);

		for (uint32 ElementIt = 0, ElementEndIt = Array.ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			ElementSerializer->FreeDynamicState(Context, ElementArgs);
			ElementArgs.Source += ElementSize;
		}
	}

	Context.GetInternalContext()->Free((void*)(Array.ElementStorage));

	// Clear allocation info
	Array.ElementCapacityCount = 0;
	Array.ElementCount = 0;
	Array.ElementStorage = 0;
}

void FArrayPropertyNetSerializer::GrowDynamicStateInternal(FNetSerializationContext& Context, QuantizedType& Array, const ConfigType* Config, uint16 NewElementCount)
{
	checkSlow(NewElementCount > Array.ElementCapacityCount);

	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
	const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];
	const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
	const SIZE_T ElementSize = ElementStateDescriptor->InternalSize;
	const SIZE_T ElementAlignment = ElementStateDescriptor->InternalAlignment;

	void* NewElementStorage = Context.GetInternalContext()->Alloc(ElementSize*NewElementCount, ElementAlignment);
	FMemory::Memzero(NewElementStorage, ElementSize*NewElementCount);
	// We only need to copy the contents of the used elements, not the entire capacity.
	FMemory::Memcpy(NewElementStorage, Array.ElementStorage, ElementSize*Array.ElementCount);
	Context.GetInternalContext()->Free(Array.ElementStorage);

	Array.ElementCapacityCount = NewElementCount;
	Array.ElementCount = NewElementCount;
	Array.ElementStorage = NewElementStorage;
}

void FArrayPropertyNetSerializer::ShrinkDynamicStateInternal(FNetSerializationContext& Context, QuantizedType& Array, const ConfigType* Config, uint16 NewElementCount)
{
	const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;

	// Free memory allocated per element.
	if (EnumHasAnyFlags(ElementStateDescriptor->Traits, EReplicationStateTraits::HasDynamicState))
	{
		const FReplicationStateMemberSerializerDescriptor& ElementSerializerDescriptor = ElementStateDescriptor->MemberSerializerDescriptors[0];
		const FNetSerializer* ElementSerializer = ElementSerializerDescriptor.Serializer;
		const uint32 ElementSize = ElementStateDescriptor->InternalSize;

		FNetFreeDynamicStateArgs ElementArgs;
		ElementArgs.Version = 0;
		ElementArgs.NetSerializerConfig = ElementSerializerDescriptor.SerializerConfig;
		ElementArgs.Source = NetSerializerValuePointer(Array.ElementStorage) + NewElementCount*ElementSize;

		for (uint32 ElementIt = NewElementCount, ElementEndIt = Array.ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			ElementSerializer->FreeDynamicState(Context, ElementArgs);
			ElementArgs.Source += ElementSize;
		}
	}

	Array.ElementCount = NewElementCount;
}

void FArrayPropertyNetSerializer::AdjustArraySize(FNetSerializationContext& Context, QuantizedType& Array, const ConfigType* Config, uint16 NewElementCount)
{
	if (NewElementCount < Array.ElementCount)
	{
		// If the array is empty we free everything
		if (NewElementCount == 0)
		{
			FreeDynamicStateInternal(Context, Array, Config);
		}
		// Otherwise we shrink it, freeing allocations made by individual elements
		else
		{
			ShrinkDynamicStateInternal(Context, Array, Config, NewElementCount);
		}
	}
	else if (NewElementCount > Array.ElementCapacityCount)
	{
		GrowDynamicStateInternal(Context, Array, Config, NewElementCount);
	}
	// If element count is within the allocated capacity we just change the number of elements
	else
	{
		// Zero out data just to be sure, to always give new elements the same initial state.
		if (NewElementCount > Array.ElementCount)
		{
			const FReplicationStateDescriptor* ElementStateDescriptor = Config->StateDescriptor;
			const SIZE_T ElementSize = ElementStateDescriptor->InternalSize;
			void* ElementsToZeroOut = (void*)(NetSerializerValuePointer(Array.ElementStorage) + (Array.ElementCount * ElementSize));

			FMemory::Memzero(ElementsToZeroOut, ElementSize * (NewElementCount - Array.ElementCount));
		}
		Array.ElementCount = NewElementCount;
	}
}

}
