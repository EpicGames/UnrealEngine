// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/BitPacking.h"
#include "Iris/Core/BitTwiddling.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Traits/IntType.h"

namespace UE::Net::Private
{

inline uint32 FloatAsUint32(float Value)
{
	union FFloatAsUint
	{
		float Float;
		uint32 Uint;
	};

	FFloatAsUint FloatAsUint;
	FloatAsUint.Float = Value;
	return FloatAsUint.Uint;
}

inline float Uint32AsFloat(uint32 Value)
{
	union FFloatAsUint
	{
		float Float;
		uint32 Uint;
	};

	FFloatAsUint FloatAsUint;
	FloatAsUint.Uint = Value;
	return FloatAsUint.Float;
}

template<typename T>
static inline void SerializeUintDeltaImpl(FNetBitStreamWriter& Writer, const T Value, const T PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	static_assert(!TIsSigned<T>::Value, "This function assumes unsigned integral types");
	static_assert(sizeof(T) >= sizeof(int), "This function assumes types at least as large as ints");

	using SignedType = typename TSignedIntType<sizeof(T)>::Type;
	constexpr unsigned TypeBitCount = sizeof(T)*8U;

	const uint8 MaxDeltaBitCount = SmallBitCountTable[SmallBitCountTableEntryCount - 1U];
	const uint32 BitCountForTableIndex = GetBitsNeeded(SmallBitCountTableEntryCount);

	// The delta must be expressed in terms of maximum bit count (LargeBitCount). 
	const T UnsignedDelta = (Value - PrevValue) & (~T(0) >> (TypeBitCount - LargeBitCount));
	const T DeltaSignMask = T(1) << (LargeBitCount - 1U);
	const SignedType Delta = SignedType((UnsignedDelta ^ DeltaSignMask) - DeltaSignMask);
	const uint32 BitCountForDelta = GetBitsNeeded(Delta);
	if (BitCountForDelta <= MaxDeltaBitCount)
	{
		for (uint32 TableIndex = 0, TableEndIndex = SmallBitCountTableEntryCount; TableIndex != SmallBitCountTableEntryCount; ++TableIndex)
		{
			const uint32 SmallBitCount = SmallBitCountTable[TableIndex];
			if ((BitCountForDelta <= SmallBitCount) | (Delta == 0))
			{
				Writer.WriteBits(TableIndex + 1U, BitCountForTableIndex);
				if constexpr (TypeBitCount > 32U)
				{
					if (SmallBitCount > 32U)
					{
						Writer.WriteBits(static_cast<uint32>(static_cast<T>(Delta)), 32U);
						Writer.WriteBits(static_cast<uint32>(static_cast<T>(Delta) >> 32U), SmallBitCount - 32U);
						return;
					}
				}

				Writer.WriteBits(static_cast<uint32>(static_cast<T>(Delta)), SmallBitCount);
				return;
			}
		}
	}
	else
	{
		Writer.WriteBits(0U, BitCountForTableIndex);
		if constexpr (TypeBitCount > 32U)
		{
			if (LargeBitCount > 32U)
			{
				Writer.WriteBits(static_cast<uint32>(Value), 32U);
				Writer.WriteBits(static_cast<uint32>(Value >> 32U), LargeBitCount - 32U);
				return;
			}
		}

		Writer.WriteBits(static_cast<uint32>(Value), LargeBitCount);
	}
}

template<bool bMaskOutValue, bool bSignExtendValue, typename T>
static inline void DeserializeUintDeltaImpl(FNetBitStreamReader& Reader, T& OutValue, const T PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	static_assert(!TIsSigned<T>::Value, "This function assumes unsigned integral types");
	static_assert(sizeof(T) >= sizeof(int), "This function assumes types at least as large as ints");

	using SignedType = typename TSignedIntType<sizeof(T)>::Type;
	constexpr unsigned TypeBitCount = sizeof(T)*8U;

	// If the delta isn't small enough to be represented by one of the entries in the SmnallBitCountTable we use LargeBitCount to write down the full value instead.
	// The LargeBitCount value counts as an implicit member of the SmallBitCountTable. For that reason we want the SmallBitCountTableEntryCount to be a number of the
	// form 2^N - 1. If that is true we use all bits needed for the table index and don't need to clamp or handle out of bounds errors in here. As a bonus it will
	// also be as bandwidth efficient as possible.
	checkfSlow(FMath::IsPowerOfTwo(SmallBitCountTableEntryCount + 1) && SmallBitCountTableEntryCount > 0, TEXT("Table size should be a power of two minus one."));

	const uint32 BitCountForTableIndex = GetBitsNeeded(SmallBitCountTableEntryCount);
	const uint32 TableIndex = Reader.ReadBits(BitCountForTableIndex);
	if (TableIndex)
	{
		const uint32 BitCountForDelta = SmallBitCountTable[TableIndex - 1U];
		T Delta;
		if ((TypeBitCount > 32U) && (BitCountForDelta > 32U))
		{
			Delta = Reader.ReadBits(32U);
			Delta |= static_cast<T>(Reader.ReadBits(BitCountForDelta - 32U)) << (TypeBitCount/2U);
		}
		else
		{
			Delta = Reader.ReadBits(BitCountForDelta);
		}

		// Mask the shift amount to avoid undefined behavior when BitCountForDelta == 0.
		const T DeltaSignMask = T(1) << ((BitCountForDelta - 1U) & (TypeBitCount - 1U));
		Delta = (Delta ^ DeltaSignMask) - DeltaSignMask;
		T Value = PrevValue + Delta;
		if constexpr (bSignExtendValue)
		{
			// We treat the number as LargeBitCount bits wide and handle overflow/underflow first before sign extending.
			const T ValueMask = ~T(0) >> (TypeBitCount - LargeBitCount);
			const T ValueSignMask = T(1) << (LargeBitCount - 1U);
			Value = ((Value & ValueMask) ^ ValueSignMask) - ValueSignMask;
		}
		else if constexpr (bMaskOutValue)
		{
			const T ValueMask = ~T(0) >> (TypeBitCount - LargeBitCount);
			Value &= ValueMask;
		}
		OutValue = Value;
	}
	else
	{
		T Value;
		if ((TypeBitCount > 32U) && (LargeBitCount > 32U))
		{
			Value = Reader.ReadBits(32U);
			Value |= static_cast<T>(Reader.ReadBits(LargeBitCount - 32U)) << (TypeBitCount/2U);
		}
		else
		{
			Value = Reader.ReadBits(LargeBitCount);
		}

		if constexpr (bSignExtendValue)
		{
			const T ValueSignMask = T(1) << (LargeBitCount - 1U);
			Value = (Value ^ ValueSignMask) - ValueSignMask;
		}
		else if constexpr (bMaskOutValue)
		{
			const T ValueMask = ~T(0) >> (TypeBitCount - LargeBitCount);
			Value &= ValueMask;
		}
		OutValue = Value;
	}
}

}

// Implementation of the public API
namespace UE::Net
{

void SerializeIntDelta(FNetBitStreamWriter& Writer, const int32 Value, const int32 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	return Private::SerializeUintDeltaImpl(Writer, static_cast<uint32>(Value), static_cast<uint32>(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
}

void DeserializeIntDelta(FNetBitStreamReader& Reader, int32& OutValue, const int32 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	constexpr bool bMaskOutValue = false;
	constexpr bool bSignExtendValue = true;
	return Private::DeserializeUintDeltaImpl<bMaskOutValue, bSignExtendValue>(Reader, reinterpret_cast<uint32&>(OutValue), static_cast<uint32>(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
}

void SerializeUintDelta(FNetBitStreamWriter& Writer, const uint32 Value, const uint32 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	return Private::SerializeUintDeltaImpl(Writer, Value, PrevValue, SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
}

void DeserializeUintDelta(FNetBitStreamReader& Reader, uint32& OutValue, const uint32 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	constexpr bool bMaskOutValue = true;
	constexpr bool bSignExtendValue = false;
	return Private::DeserializeUintDeltaImpl<bMaskOutValue, bSignExtendValue>(Reader, OutValue, PrevValue, SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
}

void SerializeIntDelta(FNetBitStreamWriter& Writer, const int64 Value, const int64 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	return Private::SerializeUintDeltaImpl(Writer, static_cast<uint64>(Value), static_cast<uint64>(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
}

void DeserializeIntDelta(FNetBitStreamReader& Reader, int64& OutValue, const int64 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	constexpr bool bMaskOutValue = false;
	constexpr bool bSignExtendValue = true;
	return Private::DeserializeUintDeltaImpl<bMaskOutValue, bSignExtendValue>(Reader, reinterpret_cast<uint64&>(OutValue), static_cast<uint64>(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
}

void SerializeUintDelta(FNetBitStreamWriter& Writer, const uint64 Value, const uint64 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	return Private::SerializeUintDeltaImpl(Writer, Value, PrevValue, SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
}

void DeserializeUintDelta(FNetBitStreamReader& Reader, uint64& OutValue, const uint64 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	constexpr bool bMaskOutValue = true;
	constexpr bool bSignExtendValue = false;
	return Private::DeserializeUintDeltaImpl<bMaskOutValue, bSignExtendValue>(Reader, OutValue, PrevValue, SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
}

uint32 QuantizeSignedUnitFloat(float Value, uint32 BitCount)
{
	if (BitCount < 24U)
	{
		// One bit is used for the sign bit
		const float Scale = float((1 << (BitCount - 1)) - 1);
		const float ScaledValue = Value*Scale;
		// If value is -1.0f we want -((1 << (BitCount - 1)) - 1) to be sent, not -((1 << (BitCount - 1)) - 2).
		// This will ensure -1.0f roundtrips perfectly. So round negative values towards -infinity.
		const int32 IntegerValue = int32(ScaledValue + FMath::Sign(Value)*0.5f);
		return static_cast<uint32>(IntegerValue);
	}
	else
	{
		/*
		 * All values in range [1.0f, 2.0f) share the same exponent. By taking note of
		 * the sign and rebase the absolute value from [0.0f, 1.0f] to [1.0f, 2.0f] we
		 * can replicate the float as sign and significand. We also need a bit to
		 * be able to differentiate between 1.0f and 2.0f which both have all zeros as significand.
		 * We use that bit to special case an original value of 0.0f, since that is a common
		 * value. In that case we do not need to replicate the significand, meaning +/- 0.0f
		 * can be replicated with just two bits and all other values with 25 bits.
		 */
		const uint32 ValueAsUint = Private::FloatAsUint32(Value);
		const float RebasedValue = FMath::Abs(Value) + 1.0f;
		const uint32 RebasedValueAsUint = Private::FloatAsUint32(RebasedValue);
		const uint32 ValueIsNotZero = uint32(RebasedValueAsUint != 0x3F800000U);
		const uint32 ValueSignBit = ValueAsUint >> 31U;
		const uint32 QuantizedValue = (ValueSignBit << 24U) | (ValueIsNotZero << 23U) | (RebasedValueAsUint & ((1U << 23U) - 1U));
		return QuantizedValue;
	}
}

float DequantizeSignedUnitFloat(uint32 Value, uint32 BitCount)
{
	if (BitCount < 24U)
	{
		const float InvScale = 1.0f/float((1 << (BitCount - 1U)) - 1);
		const float FloatValue = float(int32(Value))*InvScale;
		return FloatValue;
	}
	else
	{
		const uint32 SignAndIsNotZero = Value >> 23U;
		uint32 RebasedFloatAsUint = Value & ((1U << 23U) - 1U);
		const uint32 RebasedFloatWasTwo = (SignAndIsNotZero & 1U) & (RebasedFloatAsUint == 0U);
		RebasedFloatAsUint += 0x3F800000U + (RebasedFloatWasTwo << 23U);

		float FloatValue = Private::Uint32AsFloat(RebasedFloatAsUint) - 1.0f;
		// Apply sign
		FloatValue = Private::Uint32AsFloat(Private::FloatAsUint32(FloatValue) | ((SignAndIsNotZero & 2U) << 30U));
		return FloatValue;
	}
}

}
