// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternalNetSerializers.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"

namespace UE::Net
{

struct FBitfieldNetSerializer
{
	static const uint32 Version = 0;
	static const bool bUseDefaultDelta = false;

	typedef uint8 SourceType;
	typedef uint8 QuantizedType;
	typedef FBitfieldNetSerializerConfig ConfigType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FBitfieldNetSerializer);

void FBitfieldNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(Value, 1U);
}

void FBitfieldNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const QuantizedType Value = static_cast<QuantizedType>(Reader->ReadBits(1U));

	QuantizedType* Target = reinterpret_cast<QuantizedType*>(Args.Target);
	*Target = Value;
}

void FBitfieldNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const SourceType Value = *reinterpret_cast<const SourceType*>(Args.Source);
	const QuantizedType Bit = !!(Value & Config->BitMask);

	*reinterpret_cast<QuantizedType*>(Args.Target) = Bit;
}

void FBitfieldNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const SourceType BitMask = Config->BitMask;
	const QuantizedType Bit = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const SourceType Value = (Bit ? BitMask : SourceType(0));

	const SourceType TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	const SourceType NewTargetValue = (TargetValue & ~BitMask) | Value;

	*reinterpret_cast<SourceType*>(Args.Target) = NewTargetValue;
}

bool FBitfieldNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	static_assert(std::is_same_v<SourceType, QuantizedType>, "FBitfieldNetSerializer::IsEqual needs to be re-implemented");

	const SourceType Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
	const SourceType Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);
	if (Args.bStateIsQuantized)
	{
		return Value0 == Value1;
	}
	else
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const SourceType MaskDiff = (Value0 ^ Value1) & Config->BitMask;
		return MaskDiff == SourceType(0);
	}
}

bool FBitfieldNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	return FMath::IsPowerOfTwo(Config->BitMask);
}

bool InitBitfieldNetSerializerConfigFromProperty(FBitfieldNetSerializerConfig& OutConfig, const FBoolProperty* Bitfield)
{
	// UBoolProperty states the field size can be up to 8 bytes, but it seems to not be true. We rely on it being exactly one byte.
	alignas(16) uint8 BitfieldStorage[8];
	BitfieldStorage[0] = 0;
	Bitfield->SetPropertyValue(BitfieldStorage, true);
	OutConfig.BitMask = BitfieldStorage[0];

	if (BitfieldStorage[0] == 0)
	{
		ensureMsgf(false, TEXT("Someone has changed how bitfield properties work under the hood. Unable to properly replicate bitfield %s::%s."), ToCStr(Bitfield->GetOwnerVariant().GetName()), ToCStr(Bitfield->GetName()));
		return false;
	}


	if (!FMath::IsPowerOfTwo(BitfieldStorage[0]))
	{
		ensureMsgf(false, TEXT("Someone has changed how bitfield properties work under the hood. Unable to properly replicate bitfield %s::%s."), ToCStr(Bitfield->GetOwnerVariant().GetName()), ToCStr(Bitfield->GetName()));
		return false;
	}

	return true;
}

}
