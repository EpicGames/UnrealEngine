// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"

namespace UE::Net
{

struct FBoolNetSerializer
{
	static const uint32 Version = 0;
	static constexpr bool bUseDefaultDelta = false; // No meaning to delta compress a bool

	// Use uint8 instead of bool to avoid issue with certain compilers assuming the value of a bool can only be 0 or 1.
	// Uninitialized bools are certainly not guaranteed to be 0 or 1.
	typedef uint8 SourceType; 
	typedef FBoolNetSerializerConfig ConfigType; 

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);
};
UE_NET_IMPLEMENT_SERIALIZER(FBoolNetSerializer);
static_assert(sizeof(FBoolNetSerializer::SourceType) == sizeof(bool), "bool has unexpected size");

const FBoolNetSerializer::ConfigType FBoolNetSerializer::DefaultConfig;

void FBoolNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const SourceType Value = *reinterpret_cast<const SourceType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits((Value != 0 ? 1U : 0U), 1U);
}

void FBoolNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const SourceType Value = static_cast<SourceType>(Reader->ReadBits(1U));

	SourceType* Target = reinterpret_cast<SourceType*>(Args.Target);
	*Target = Value;
}

bool FBoolNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	const SourceType Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
	const SourceType Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);

	const bool bValue0 = (Value0 != 0);
	const bool bValue1 = (Value1 != 0);

	return bValue0 == bValue1;
}

bool FBoolNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType Value = *reinterpret_cast<const SourceType*>(Args.Source);

	// By failing values other than 0 and 1 we assist in finding uninitialized bools.
	return (Value <= 1);
}

}
