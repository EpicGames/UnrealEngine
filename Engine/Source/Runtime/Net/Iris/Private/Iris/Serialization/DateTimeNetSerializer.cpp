// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/DateTimeNetSerializer.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Misc/DateTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DateTimeNetSerializer)

namespace UE::Net
{

struct FDateTimeNetSerializer
{
	static const uint32 Version = 0;

	typedef FDateTime SourceType;
	typedef int64 QuantizedType;
	
	typedef FDateTimeNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&); \
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&); \

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
};

const FDateTimeNetSerializer::ConfigType FDateTimeNetSerializer::DefaultConfig;

UE_NET_IMPLEMENT_SERIALIZER(FDateTimeNetSerializer);

void FDateTimeNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	WriteInt64(Writer, *reinterpret_cast<const QuantizedType*>(Args.Source));
}

void FDateTimeNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	*reinterpret_cast<QuantizedType*>(Args.Target) = ReadInt64(Reader);
}

void FDateTimeNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	TargetValue = SourceValue.GetTicks();
}

void FDateTimeNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target = SourceType(Source);
}

bool FDateTimeNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		return Value0 == Value1;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<SourceType*>(Args.Source1);
		return Value0 == Value1;
	}
}

}
