// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/EnumNetSerializers.h"
#include "Iris/Serialization/IntRangeNetSerializerUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnumNetSerializers)

namespace UE::Net::Private
{

template<typename InSourceType, typename EnumNetSerializerConfig>
struct FEnumNetSerializerBase : public FIntRangeNetSerializerBase<InSourceType, EnumNetSerializerConfig>
{
	static bool Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
	{
		const EnumNetSerializerConfig* Config = static_cast<const EnumNetSerializerConfig*>(Args.NetSerializerConfig);
		const uint32 BitCount = Config->BitCount;

		// Detect invalid bit count
		if (BitCount > sizeof(InSourceType)*8U)
		{
			return false;
		}

		const InSourceType Value = *reinterpret_cast<const InSourceType*>(Args.Source);

		// Detect values outside of the valid range. This check needs to be performed before the enum check due to the generated _MAX value.
		const InSourceType ClampedValue = FMath::Clamp(Value, Config->LowerBound, Config->UpperBound);
		if (Value != ClampedValue)
		{
			return false;
		}

		// This is slow as it will go through all values in the enum.
		if (const UEnum* Enum = Config->Enum)
		{
			return Enum->IsValidEnumValue(Value);
		}

		return true;
	}
};

}

namespace UE::Net
{

struct FEnumInt8NetSerializer : public Private::FEnumNetSerializerBase<int8, FEnumInt8NetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FEnumInt8NetSerializer);

struct FEnumInt16NetSerializer : public Private::FEnumNetSerializerBase<int16, FEnumInt16NetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FEnumInt16NetSerializer);

struct FEnumInt32NetSerializer : public Private::FEnumNetSerializerBase<int32, FEnumInt32NetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FEnumInt32NetSerializer);

struct FEnumInt64NetSerializer : public Private::FEnumNetSerializerBase<int64, FEnumInt64NetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FEnumInt64NetSerializer);

struct FEnumUint8NetSerializer : public Private::FEnumNetSerializerBase<uint8, FEnumUint8NetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FEnumUint8NetSerializer);

struct FEnumUint16NetSerializer : public Private::FEnumNetSerializerBase<uint16, FEnumUint16NetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FEnumUint16NetSerializer);

struct FEnumUint32NetSerializer : public Private::FEnumNetSerializerBase<uint32, FEnumUint32NetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FEnumUint32NetSerializer);

struct FEnumUint64NetSerializer : public Private::FEnumNetSerializerBase<uint64, FEnumUint64NetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FEnumUint64NetSerializer);

}
