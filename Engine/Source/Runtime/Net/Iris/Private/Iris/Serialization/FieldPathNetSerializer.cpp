// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternalNetSerializers.h"
#include "UObject/CoreNet.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"

namespace UE::Net
{

struct FFieldPathForNetSerializer : public FFieldPath
{
public:
	TWeakObjectPtr<UStruct> GetResolvedOwner() const { return ResolvedOwner; }
	void SetResolvedOwner(const TWeakObjectPtr<UStruct>& InResolvedOwner) { ResolvedOwner = InResolvedOwner; }

	const TArray<FName>& GetPath() const { return Path; }
	void SetPath(const TArray<FName>& InPath) { Path = InPath; }

	void ClearCachedField() { FFieldPath::ClearCachedField(); }
};

struct FFieldPathNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bHasCustomNetReference = true;

	// Types
	struct FQuantizedType
	{
		alignas(8) uint8 QuantizedStruct[48];
	};

	typedef FFieldPathForNetSerializer SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FFieldPathNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	//
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	private:
		virtual void OnPostFreezeNetSerializerRegistry() override;
	};

	static void FieldPathToFieldPathNetSerializerSerializationHelper(const SourceType& FieldPath, FFieldPathNetSerializerSerializationHelper& OutStruct);

	static FFieldPathNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	static FStructNetSerializerConfig StructNetSerializerConfig;
	static const FNetSerializer* StructNetSerializer;
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FFieldPathNetSerializer);

const FFieldPathNetSerializer::ConfigType FFieldPathNetSerializer::DefaultConfig;

FFieldPathNetSerializer::FNetSerializerRegistryDelegates FFieldPathNetSerializer::NetSerializerRegistryDelegates;
FStructNetSerializerConfig FFieldPathNetSerializer::StructNetSerializerConfig;
const FNetSerializer* FFieldPathNetSerializer::StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);

void FFieldPathNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	FNetSerializeArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->Serialize(Context, InternalArgs);
}

void FFieldPathNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetDeserializeArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->Deserialize(Context, InternalArgs);
}

void FFieldPathNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	FNetSerializeDeltaArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->SerializeDelta(Context, InternalArgs);
}

void FFieldPathNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetDeserializeDeltaArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->DeserializeDelta(Context, InternalArgs);
}

void FFieldPathNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);

	FFieldPathNetSerializerSerializationHelper IntermediateValue;
	FieldPathToFieldPathNetSerializerSerializationHelper(SourceValue, IntermediateValue);

	FNetQuantizeArgs InternalArgs = Args;
	InternalArgs.Source = NetSerializerValuePointer(&IntermediateValue);
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->Quantize(Context, InternalArgs);
}

void FFieldPathNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);

	FFieldPathNetSerializerSerializationHelper IntermediateValue;

	FNetDequantizeArgs InternalArgs = Args;
	InternalArgs.Target = NetSerializerValuePointer(&IntermediateValue);
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;

	StructNetSerializer->Dequantize(Context, InternalArgs);

	TargetValue.SetResolvedOwner(IntermediateValue.Owner);
	TargetValue.SetPath(IntermediateValue.PropertyPath);
	TargetValue.ClearCachedField();
}

bool FFieldPathNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		FNetIsEqualArgs InternalArgs = Args;
		InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
		return StructNetSerializer->IsEqual(Context, InternalArgs);
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		FFieldPathNetSerializerSerializationHelper IntermediateValue0;
		FFieldPathNetSerializerSerializationHelper IntermediateValue1;

		FieldPathToFieldPathNetSerializerSerializationHelper(SourceValue0, IntermediateValue0);
		FieldPathToFieldPathNetSerializerSerializationHelper(SourceValue1, IntermediateValue1);

		FNetIsEqualArgs InternalArgs = Args;
		InternalArgs.Source0 = NetSerializerValuePointer(&IntermediateValue0);
		InternalArgs.Source1 = NetSerializerValuePointer(&IntermediateValue1);
		InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
		return StructNetSerializer->IsEqual(Context, InternalArgs);
	}
}

bool FFieldPathNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);

	FFieldPathNetSerializerSerializationHelper IntermediateValue;
	FieldPathToFieldPathNetSerializerSerializationHelper(SourceValue, IntermediateValue);

	FNetValidateArgs InternalArgs = Args;
	InternalArgs.Source = NetSerializerValuePointer(&IntermediateValue);
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;

	return StructNetSerializer->Validate(Context, InternalArgs);
}

void FFieldPathNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	using namespace Private;

	const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfig.StateDescriptor;
	if (Descriptor && EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasObjectReference))
	{
		FNetCollectReferencesArgs InternalArgs = Args;
		InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
		StructNetSerializer->CollectNetReferences(Context, InternalArgs);
	}
}

void FFieldPathNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	FNetCloneDynamicStateArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->CloneDynamicState(Context, InternalArgs);
}

void FFieldPathNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	FNetFreeDynamicStateArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->FreeDynamicState(Context, InternalArgs);
}

void FFieldPathNetSerializer::FieldPathToFieldPathNetSerializerSerializationHelper(const SourceType& FieldPath, FFieldPathNetSerializerSerializationHelper& OutStruct)
{
	const TWeakObjectPtr<UStruct>& Owner = FieldPath.GetResolvedOwner();
	// Without a valid owner we don't know the full path to the property and will not be able to resolve it.
	if (Owner.IsValid())
	{
		OutStruct.Owner = Owner;
		OutStruct.PropertyPath = FieldPath.GetPath();
	}
}

void FFieldPathNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	UStruct* Struct = FFieldPathNetSerializerSerializationHelper::StaticStruct();
	StructNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(Struct);
	const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfig.StateDescriptor.GetReference();
	check(Descriptor != nullptr);

	// Validate our assumptions regarding quantized state size and alignment.
	static_assert(offsetof(FQuantizedType, QuantizedStruct) == 0U, "Expected buffer for struct to be first member of FQuantizedType.");
	if (sizeof(FQuantizedType::QuantizedStruct) < Descriptor->InternalSize || alignof(FQuantizedType) < Descriptor->InternalAlignment)
	{
		LowLevelFatalError(TEXT("FQuantizedType::QuantizedStruct has size %u and alignment %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType::QuantizedStruct)), uint32(alignof(FQuantizedType)), uint32(Descriptor->InternalSize), uint32(Descriptor->InternalAlignment)); 
	}
}

}
