// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/FloatNetSerializers.h"
#include "Iris/Serialization/BitPacking.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloatNetSerializers)

namespace UE::Net
{

struct FFloatNetSerializer
{
	static const uint32 Version = 0;

	/**
	 * We are interested in the bit representation of the float, not IEEE 754 behavior. This is particularly
	 * relevant for IsEqual where for example -0.0f == +0.0f if the values were treated as floats
	 * rather than the bit representation of the floats. By using uint32 as SourceType we can avoid
	 * implementing some functions.
	 */
	typedef uint32 SourceType;
	typedef FFloatNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig = ConfigType();

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

private:
	inline static const uint8 DeltaBitCountTable[] = 
	{
		// Same value optimization.
		0,
		// Allows for small changes (0.1) at small values (100) and larger changes at larger values.
		16,
		// Allows exponent to increment or decrement, i.e. doubling or halfing a value.
		25,
	};
	static constexpr uint32 DeltaBitCountTableEntryCount = sizeof(DeltaBitCountTable)/sizeof(DeltaBitCountTable[0]);
};
UE_NET_IMPLEMENT_SERIALIZER(FFloatNetSerializer);

struct FDoubleNetSerializer
{
	static const uint32 Version = 0;

	typedef uint64 SourceType;
	typedef FDoubleNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig = ConfigType();

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

private:
	inline static const uint8 DeltaBitCountTable[] = 
	{
		// Same value optimization.
		0,
		// Allows for small changes (0.1) at small values (1000+) and larger changes at larger values.
		42,
		// Allows exponent to increment or decrement, i.e. doubling or halfing a value.
		54,
	};
	static constexpr uint32 DeltaBitCountTableEntryCount = sizeof(DeltaBitCountTable)/sizeof(DeltaBitCountTable[0]);
};
UE_NET_IMPLEMENT_SERIALIZER(FDoubleNetSerializer);

// FFloatNetSerializer implementation
void FFloatNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const uint32 Value = *reinterpret_cast<const uint32*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	if (Writer->WriteBool(Value != 0))
	{
		Writer->WriteBits(Value, 32U);
	}
}

void FFloatNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const uint32 Value = Reader->ReadBool() ? Reader->ReadBits(32U) : 0U;

	uint32* Target = reinterpret_cast<uint32*>(Args.Target);
	*Target = Value;
}

void FFloatNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const uint32 Value = *reinterpret_cast<const uint32*>(Args.Source);
	const uint32 PrevValue = *reinterpret_cast<const uint32*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	SerializeUintDelta(*Writer, Value, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 32U);
}

void FFloatNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	uint32& Target = *reinterpret_cast<uint32*>(Args.Target);
	const uint32 PrevValue = *reinterpret_cast<const uint32*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	DeserializeUintDelta(*Reader, Target, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 32U);
}

// FDoubleNetSerializer implementation
void FDoubleNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const uint64 Value = *reinterpret_cast<const uint64*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	if (Writer->WriteBool(Value != 0))
	{
		Writer->WriteBits(static_cast<uint32>(Value), 32U);
		Writer->WriteBits(static_cast<uint32>(Value >> 32U), 32U);
	}
}

void FDoubleNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	uint64 Value = 0;
	
	if (Reader->ReadBool())
	{
		Value = Reader->ReadBits(32U);
		Value |= static_cast<uint64>(Reader->ReadBits(32U)) << 32U;
	}

	uint64* Target = reinterpret_cast<uint64*>(Args.Target);
	*Target = Value;
}

void FDoubleNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const uint64 Value = *reinterpret_cast<const uint64*>(Args.Source);
	const uint64 PrevValue = *reinterpret_cast<const uint64*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	SerializeUintDelta(*Writer, Value, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 64U);
}

void FDoubleNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	uint64& Target = *reinterpret_cast<uint64*>(Args.Target);
	const uint64 PrevValue = *reinterpret_cast<const uint64*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	DeserializeUintDelta(*Reader, Target, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 64U);
}

}
