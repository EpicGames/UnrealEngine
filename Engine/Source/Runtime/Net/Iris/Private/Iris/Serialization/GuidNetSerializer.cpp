// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/GuidNetSerializer.h"
#include "Misc/Guid.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GuidNetSerializer)

namespace UE::Net
{

struct FGuidNetSerializer
{
	static const uint32 Version = 0;

	typedef FGuid SourceType;
	typedef FGuidNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);
};
UE_NET_IMPLEMENT_SERIALIZER(FGuidNetSerializer);

const FGuidNetSerializer::ConfigType FGuidNetSerializer::DefaultConfig;

void FGuidNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const FGuid& Value = *reinterpret_cast<const FGuid*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	if (Writer->WriteBool(Value.IsValid()))
	{
		Writer->WriteBits(Value.A, 32U);
		Writer->WriteBits(Value.B, 32U);
		Writer->WriteBits(Value.C, 32U);
		Writer->WriteBits(Value.D, 32U);
	}
}

void FGuidNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// Don't rely on the Guid default constructor.
	FGuid Value(0, 0, 0, 0);
	if (Reader->ReadBool())
	{
		Value.A = Reader->ReadBits(32U);
		Value.B = Reader->ReadBits(32U);
		Value.C = Reader->ReadBits(32U);
		Value.D = Reader->ReadBits(32U);
	}

	FGuid& Target = *reinterpret_cast<FGuid*>(Args.Target);
	Target = Value;
}

}
