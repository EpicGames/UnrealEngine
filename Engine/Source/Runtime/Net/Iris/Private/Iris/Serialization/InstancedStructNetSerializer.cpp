// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/InstancedStructNetSerializer.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedStructNetSerializer)


#include "StructUtils/InstancedStruct.h"
#include "HAL/IConsoleManager.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/NetSerializerArrayStorage.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/QuantizedObjectReference.h"
#include "Templates/IsPODType.h"

FInstancedStructNetSerializerConfig::FInstancedStructNetSerializerConfig()
: FNetSerializerConfig()
{
	ConfigTraits = ENetSerializerConfigTraits::NeedDestruction;
}

FInstancedStructNetSerializerConfig::~FInstancedStructNetSerializerConfig() = default;

namespace UE::Net
{

static int32 MaxCachedInstancedStructDescriptorCount = 8;
static FAutoConsoleVariableRef CVarMaxCachedInstancedStructDescriptors(TEXT("InstancedStruct.MaxCachedReplicationStateDescriptors"), MaxCachedInstancedStructDescriptorCount, TEXT("How many ReplicationStateDescriptors the InstancedStructNetSerializer is allowed to cache for InstancedStructs without a type allow list. Warning: A value <= 0 means an unlimited amount of descriptors."));

const FName NetError_InstancedStructNetSerializer_InvalidStructType("Invalid struct type");

struct FFInstancedStructNetSerializerQuantizedData
{
	FNetSerializerAlignedStorage StructData;
	FQuantizedObjectReference StructType;

	// Not serialized. Fully qualified path. For ReplicationStateDescriptor lookup, validation etc. 
	FName StructName;
	// Not serialized. To optimize away some calls like dynamic memory management and object references.
	EReplicationStateTraits StructDescriptorTraits;
};

struct FInstancedStructPropertyNetSerializerInfo : public FNamedStructPropertyNetSerializerInfo
{
public:
	FInstancedStructPropertyNetSerializerInfo();

protected:
	virtual bool CanUseDefaultConfig(const FProperty* Property) const;
	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override;
};

}

template<> struct TIsPODType<UE::Net::FFInstancedStructNetSerializerQuantizedData> { enum { Value = true }; };

namespace UE::Net
{

struct FInstancedStructNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bIsForwardingSerializer = true;
	static constexpr bool bHasCustomNetReference = true;

	typedef FInstancedStruct SourceType;
	typedef FFInstancedStructNetSerializerQuantizedData QuantizedType;
	typedef FInstancedStructNetSerializerConfig ConfigType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

	static void Apply(FNetSerializationContext&, const FNetApplyArgs&);

private:
	// Frees dynamic memory allocated by the struct instance. Zeros the struct storage. Does not free the struct storage. After the call the Value is ready to be re-purposed for a different type of struct.
	static void FreeStructInstance(FNetSerializationContext&, FInstancedStructNetSerializerConfig*, QuantizedType& Value);
	// Frees dynamic memory allocated by the struct instance, frees the storage for the struct instance and reset the entire quantized state to default.
	static void Reset(FNetSerializationContext&, FInstancedStructNetSerializerConfig*, QuantizedType&);

	static void InternalFreeStructInstance(FNetSerializationContext&, FInstancedStructNetSerializerConfig*, QuantizedType&);

	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates()
		{
			UE_NET_UNREGISTER_NETSERIALIZER_INFO(FInstancedStructPropertyNetSerializerInfo);
		}

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override
		{
			UE_NET_REGISTER_NETSERIALIZER_INFO(FInstancedStructPropertyNetSerializerInfo);
		}

		UE_NET_IMPLEMENT_NETSERIALIZER_INFO(FInstancedStructPropertyNetSerializerInfo);
	};

	inline static const FNetSerializer* StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);
	inline static const FNetSerializer* ObjectNetSerializer = &UE_NET_GET_SERIALIZER(FObjectNetSerializer);
	inline static FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};

UE_NET_IMPLEMENT_SERIALIZER(FInstancedStructNetSerializer);

void FInstancedStructNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);

	FInstancedStructNetSerializerConfig* Config = const_cast<FInstancedStructNetSerializerConfig*>(static_cast<const FInstancedStructNetSerializerConfig*>(Args.NetSerializerConfig));
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	FStructNetSerializerConfig StructConfig;
	if (!Value.StructName.IsNone())
	{
		StructConfig.StateDescriptor = Config->DescriptorCache.FindOrAddDescriptor(Value.StructName);
		ensureMsgf(StructConfig.StateDescriptor.IsValid(), TEXT("Struct type is no longer resolvable: %s. Sending FInstancedStruct as uninitialized."), ToCStr(Value.StructName.ToString()));
	}

	if (Writer->WriteBool(StructConfig.StateDescriptor.IsValid()))
	{
		// Serialize struct type
		{
			FNetSerializeArgs SerializeArgs = Args;
			SerializeArgs.Source = NetSerializerValuePointer(&Value.StructType);
			SerializeArgs.NetSerializerConfig = ObjectNetSerializer->DefaultConfig;
			ObjectNetSerializer->Serialize(Context, SerializeArgs);
		}

		// Serialize struct data
		{
			FNetSerializeArgs SerializeArgs = Args;
			SerializeArgs.Source = NetSerializerValuePointer(Value.StructData.GetData());
			SerializeArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
			StructNetSerializer->Serialize(Context, SerializeArgs);
		}
	}
}

void FInstancedStructNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);

	FInstancedStructNetSerializerConfig* Config = const_cast<FInstancedStructNetSerializerConfig*>(static_cast<const FInstancedStructNetSerializerConfig*>(Args.NetSerializerConfig));

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	// Was the instanced struct valid on the sending side?
	if (Reader->ReadBool())
	{
		FQuantizedObjectReference StructType;
		const UScriptStruct* Struct = nullptr;

		// Deserialize struct type
		{
			FNetDeserializeArgs DeserializeArgs = Args;
			DeserializeArgs.Target = NetSerializerValuePointer(&StructType);
			DeserializeArgs.NetSerializerConfig = ObjectNetSerializer->DefaultConfig;
			ObjectNetSerializer->Deserialize(Context, DeserializeArgs);
		}

		// Dequantize to get the UScriptStruct.
		// $IRIS TODO : Allow receiving end to skip payloads which it's unable to parse due to missing struct.
		{
			UObject* Object = nullptr;

			FNetDequantizeArgs DequantizeArgs = {};
			DequantizeArgs.Source = NetSerializerValuePointer(&StructType);
			DequantizeArgs.Target = NetSerializerValuePointer(&Object);
			DequantizeArgs.NetSerializerConfig = ObjectNetSerializer->DefaultConfig;
			ObjectNetSerializer->Dequantize(Context, DequantizeArgs);

			if (Object != nullptr)
			{
				Struct = Cast<UScriptStruct>(Object);
				ensureMsgf(Struct != nullptr, TEXT("Unable to cast object %s to UScriptStruct"), ToCStr(Object->GetPathName()));
				if (Struct == nullptr)
				{
					Context.SetError(GNetError_InvalidValue);
					return;
				}
			}
		}

		// Deserialize struct data
		if (Struct != nullptr)
		{
			FStructNetSerializerConfig StructConfig;
			StructConfig.StateDescriptor = Config->DescriptorCache.FindOrAddDescriptor(Struct);
			if (!StructConfig.StateDescriptor.IsValid())
			{
				ensureMsgf(StructConfig.StateDescriptor.IsValid(), TEXT("Unable to create ReplicationStateDescriptor for struct %s."), ToCStr(Struct->GetPathName()));
				Context.SetError(GNetError_InvalidValue);
				return;
			}

			// If we changed types we need to free the struct, adjust the storage size and update struct info.
			if (StructType != Value.StructType)
			{
				FreeStructInstance(Context, Config, Value);
				Value.StructData.AdjustSize(Context, StructConfig.StateDescriptor->InternalSize, StructConfig.StateDescriptor->InternalAlignment);
				Value.StructType = StructType;

				Value.StructDescriptorTraits = StructConfig.StateDescriptor->Traits;
				const FString& PathNameString = Struct->GetPathName();
				Value.StructName = FName(PathNameString);
			}

			FNetDeserializeArgs DeserializeArgs = Args;
			DeserializeArgs.Target = NetSerializerValuePointer(Value.StructData.GetData());
			DeserializeArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
			StructNetSerializer->Deserialize(Context, DeserializeArgs);
		}
		else
		{
			UE_LOGF(LogIris, Error, "Unable to find struct using FQuantizedObjectReference %ls when deserializing instanced struct.", ToCStr(StructType.ToString()));
			ensure(Struct != nullptr);

			Context.SetError(GNetError_InvalidValue);
			return;
		}
	}
	else
	{
		Reset(Context, Config, Value);
	}
}

void FInstancedStructNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// If the type has changed proceed with regular serialization.
	if (Writer->WriteBool(Value.StructName != PrevValue.StructName))
	{
		Serialize(Context, Args);
		return;
	}
	
	// If the instance isn't set there's nothing to delta serialize
	if (Value.StructName.IsNone())
	{
		return;
	}

	FInstancedStructNetSerializerConfig* Config = const_cast<FInstancedStructNetSerializerConfig*>(static_cast<const FInstancedStructNetSerializerConfig*>(Args.NetSerializerConfig));

	FStructNetSerializerConfig StructConfig;
	StructConfig.StateDescriptor = Config->DescriptorCache.FindOrAddDescriptor(Value.StructName);
	ensureMsgf(StructConfig.StateDescriptor.IsValid(), TEXT("Struct type is no longer resolvable: %s. Sending FInstancedStruct as uninitialized."), ToCStr(Value.StructName.ToString()));

	if (Writer->WriteBool(StructConfig.StateDescriptor.IsValid()))
	{
		// Delta serialize struct data
		FNetSerializeDeltaArgs DeltaSerializeArgs = Args;
		DeltaSerializeArgs.Source = NetSerializerValuePointer(Value.StructData.GetData());
		DeltaSerializeArgs.Prev = NetSerializerValuePointer(PrevValue.StructData.GetData());
		DeltaSerializeArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
		StructNetSerializer->SerializeDelta(Context, DeltaSerializeArgs);
	}
}

void FInstancedStructNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	if (const bool bTypeDiffers = Reader->ReadBool())
	{
		Deserialize(Context, Args);
		return;
	}

	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FInstancedStructNetSerializerConfig* Config = const_cast<FInstancedStructNetSerializerConfig*>(static_cast<const FInstancedStructNetSerializerConfig*>(Args.NetSerializerConfig));

	// If the instance isn't set there's nothing to deserialize
	if (PrevValue.StructName.IsNone())
	{
		Reset(Context, Config, Value);
		return;
	}

	// The sender may not have succeeded in serializing the data due to struct type no longer existing.
	if (!Reader->ReadBool())
	{
		Reset(Context, Config, Value);
		return;
	}
	
	// Delta deserialize
	FStructNetSerializerConfig StructConfig;
	StructConfig.StateDescriptor = Config->DescriptorCache.FindOrAddDescriptor(PrevValue.StructName);
	if (!StructConfig.StateDescriptor.IsValid())
	{
		ensureMsgf(StructConfig.StateDescriptor.IsValid(), TEXT("Unable to create ReplicationStateDescriptor for struct %s."), ToCStr(PrevValue.StructName.ToString()));
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	// Fix up struct info and allocations if the target value is changing types.
	if (Value.StructName != PrevValue.StructName)
	{
		FreeStructInstance(Context, Config, Value);
		Value.StructData.AdjustSize(Context, StructConfig.StateDescriptor->InternalSize, StructConfig.StateDescriptor->InternalAlignment);
		Value.StructType = PrevValue.StructType;
		Value.StructDescriptorTraits = PrevValue.StructDescriptorTraits;
		Value.StructName = PrevValue.StructName;
	}

	// Delta deserialize struct data
	{
		FNetDeserializeDeltaArgs DeltaDeserializeArgs = Args;
		DeltaDeserializeArgs.Target = NetSerializerValuePointer(Value.StructData.GetData());
		DeltaDeserializeArgs.Prev = NetSerializerValuePointer(PrevValue.StructData.GetData());
		DeltaDeserializeArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
		StructNetSerializer->DeserializeDelta(Context, DeltaDeserializeArgs);
	}
}

void FInstancedStructNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FInstancedStructNetSerializerConfig* Config = const_cast<FInstancedStructNetSerializerConfig*>(static_cast<const FInstancedStructNetSerializerConfig*>(Args.NetSerializerConfig));

	if (Source.IsValid() && !Context.IsInitializingDefaultState())
	{
		const UScriptStruct* Struct = Source.GetScriptStruct();

		TRefCountPtr<const FReplicationStateDescriptor> DescriptorRef = Config->DescriptorCache.FindOrAddDescriptor(Struct);
		if (ensureMsgf(DescriptorRef.IsValid(), TEXT("Unable to create descriptor for struct %s. Unexpected."), ToCStr(Struct->GetFullName())))
		{
			const FString& PathNameString = Struct->GetPathName();
			const FName PathName(PathNameString);
			// If the struct type is different from the previous instance we need adjust the memory allocation.
			if (PathName != Target.StructName)
			{
				// We need to free the previous struct data prior to overwriting it. Doing this early.
				FreeStructInstance(Context, Config, Target);

				// Adjust struct storage size
				Target.StructData.AdjustSize(Context, DescriptorRef->InternalSize, DescriptorRef->InternalAlignment);

				// Adjust struct name and traits
				Target.StructName = PathName;
				Target.StructDescriptorTraits = DescriptorRef->Traits;

				// Quantize the struct type since the receiver need it to be able to serialize the data properly.
				{
					FNetQuantizeArgs QuantizeArgs = Args;
					QuantizeArgs.Source = NetSerializerValuePointer(&Struct);
					QuantizeArgs.Target = NetSerializerValuePointer(&Target.StructType);
					QuantizeArgs.NetSerializerConfig = ObjectNetSerializer->DefaultConfig;
					ObjectNetSerializer->Quantize(Context, QuantizeArgs);
				}
			}

			// Quantize the struct instance into the target memory.
			if (Target.StructData.Num() > 0)
			{
				FStructNetSerializerConfig StructConfig;
				StructConfig.StateDescriptor = DescriptorRef;

				FNetQuantizeArgs QuantizeArgs = Args;
				QuantizeArgs.Source = NetSerializerValuePointer(Source.GetMemory());
				QuantizeArgs.Target = NetSerializerValuePointer(Target.StructData.GetData());
				QuantizeArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
				StructNetSerializer->Quantize(Context, QuantizeArgs);
			}

			return;
		}
	}

	// Path taken for uninitialized FInstancedStruct or if an error was detected.
	Reset(Context, Config, Target);
}

void FInstancedStructNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	FInstancedStructNetSerializerConfig* Config = const_cast<FInstancedStructNetSerializerConfig*>(static_cast<const FInstancedStructNetSerializerConfig*>(Args.NetSerializerConfig));

	if (!Source.StructType.IsValid())
	{
		Target.Reset();
		return;
	}

	const UScriptStruct* Struct = nullptr;
	{
		UObject* Object = nullptr;

		FNetDequantizeArgs DequantizeArgs = {};
		DequantizeArgs.Source = NetSerializerValuePointer(&Source.StructType);
		DequantizeArgs.Target = NetSerializerValuePointer(&Object);
		DequantizeArgs.NetSerializerConfig = ObjectNetSerializer->DefaultConfig;
		ObjectNetSerializer->Dequantize(Context, DequantizeArgs);

		if (Object != nullptr)
		{
			Struct = Cast<UScriptStruct>(Object);
			ensureMsgf(Struct != nullptr, TEXT("Unable to cast object %s to UScriptStruct"), ToCStr(Object->GetPathName()));
		}
	}
	
	if (ensureMsgf(Struct != nullptr, TEXT("Unable to find struct using FQuantizedObjectReference %s"), ToCStr(Source.StructType.ToString())))
	{
		// Re-initialize if the type changes.
		if (Struct != Target.GetScriptStruct())
		{
			Target.InitializeAs(Struct);
		}

		FStructNetSerializerConfig StructConfig;
		StructConfig.StateDescriptor = Config->DescriptorCache.FindOrAddDescriptor(Struct);

		if (ensureMsgf(StructConfig.StateDescriptor.IsValid(), TEXT("Unable to create ReplicationStateDescriptor for struct %s."), ToCStr(Source.StructName.ToString())))
		{
			FNetDequantizeArgs DequantizeArgs = Args;
			DequantizeArgs.NetSerializerConfig = &StructConfig;
			DequantizeArgs.Source = NetSerializerValuePointer(Source.StructData.GetData());
			DequantizeArgs.Target = NetSerializerValuePointer(Target.GetMutableMemory());
			StructNetSerializer->Dequantize(Context, DequantizeArgs);
		}
	}
}

bool FInstancedStructNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (Value0.StructData.Num() != Value1.StructData.Num())
		{
			return false;
		}

		if (Value0.StructType != Value1.StructType)
		{
			return false;
		}

		if (FMemory::Memcmp(Value0.StructData.GetData(), Value1.StructData.GetData(), Value0.StructData.Num()) != 0)
		{
			return false;
		}

		return true;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		return Value0 == Value1;
	}
}

bool FInstancedStructNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Value = *reinterpret_cast<const SourceType*>(Args.Source);

	return true;
}

void FInstancedStructNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	FInstancedStructNetSerializerConfig* Config = const_cast<FInstancedStructNetSerializerConfig*>(static_cast<const FInstancedStructNetSerializerConfig*>(Args.NetSerializerConfig));

	Target.StructData.Clone(Context, Source.StructData);

	if (EnumHasAnyFlags(Source.StructDescriptorTraits, EReplicationStateTraits::HasDynamicState))
	{
		FStructNetSerializerConfig StructConfig;
		StructConfig.StateDescriptor = Config->DescriptorCache.FindOrAddDescriptor(Source.StructName);

		if (ensureMsgf(StructConfig.StateDescriptor.IsValid(), TEXT("Unable to create ReplicationStateDescriptor for struct %s."), ToCStr(Source.StructName.ToString())))
		{
			FNetCloneDynamicStateArgs CloneArgs = Args;
			CloneArgs.NetSerializerConfig = &StructConfig;
			CloneArgs.Source = NetSerializerValuePointer(Source.StructData.GetData());
			CloneArgs.Target = NetSerializerValuePointer(Target.StructData.GetData());
			StructNetSerializer->CloneDynamicState(Context, CloneArgs);
		}
	}
}

void FInstancedStructNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	FInstancedStructNetSerializerConfig* Config = const_cast<FInstancedStructNetSerializerConfig*>(static_cast<const FInstancedStructNetSerializerConfig*>(Args.NetSerializerConfig));
	
	InternalFreeStructInstance(Context, Config, Value);
	Value.StructData.Free(Context);
}

void FInstancedStructNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	FInstancedStructNetSerializerConfig* Config = const_cast<FInstancedStructNetSerializerConfig*>(static_cast<const FInstancedStructNetSerializerConfig*>(Args.NetSerializerConfig));

	if (Value.StructType.IsValid())
	{
		FNetReferenceCollector& Collector = *reinterpret_cast<FNetReferenceCollector*>(Args.Collector);

		// What's the proper reference type?
		const FNetReferenceInfo ReferenceInfo(FNetReferenceInfo::EResolveType::ResolveOnClient);
		Collector.Add(ReferenceInfo, Value.StructType, Args.ChangeMaskInfo);
	}

	if (EnumHasAnyFlags(Value.StructDescriptorTraits, EReplicationStateTraits::HasObjectReference))
	{
		FStructNetSerializerConfig StructConfig;
		StructConfig.StateDescriptor = Config->DescriptorCache.FindOrAddDescriptor(Value.StructName);

		if (ensureMsgf(StructConfig.StateDescriptor.IsValid(), TEXT("Unable to create ReplicationStateDescriptor for struct %s."), ToCStr(Value.StructName.ToString())))
		{
			FNetCollectReferencesArgs CollectReferencesArgs = Args;
			CollectReferencesArgs.NetSerializerConfig = &StructConfig;
			CollectReferencesArgs.Source = NetSerializerValuePointer(Value.StructData.GetData());
			StructNetSerializer->CollectNetReferences(Context, CollectReferencesArgs);
		}
	}
}

void FInstancedStructNetSerializer::Apply(FNetSerializationContext& Context, const FNetApplyArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);
	FInstancedStructNetSerializerConfig* Config = const_cast<FInstancedStructNetSerializerConfig*>(static_cast<const FInstancedStructNetSerializerConfig*>(Args.NetSerializerConfig));

	// If source and target has the same type make sure to not clobber not replicated properties.
	const UScriptStruct* ScriptStruct = Source.GetScriptStruct();
	if (ScriptStruct != Target.GetScriptStruct())
	{
		Target.InitializeAs(ScriptStruct);
	}

	if (ScriptStruct)
	{
		const TRefCountPtr<const FReplicationStateDescriptor> Descriptor = Config->DescriptorCache.FindOrAddDescriptor(ScriptStruct);
		FReplicationStateOperations::ApplyStruct(Context, Target.GetMutableMemory(), Source.GetMemory(), Descriptor);
	}
}

void FInstancedStructNetSerializer::FreeStructInstance(FNetSerializationContext& Context, FInstancedStructNetSerializerConfig* Config, QuantizedType& Value)
{
	InternalFreeStructInstance(Context, Config, Value);
	if (Value.StructData.Num() > 0)
	{
		FMemory::Memzero(Value.StructData.GetData(), Value.StructData.Num());
	}
}

void FInstancedStructNetSerializer::Reset(FNetSerializationContext& Context, FInstancedStructNetSerializerConfig* Config, QuantizedType& Value)
{
	InternalFreeStructInstance(Context, Config, Value);
	Value.StructData.Free(Context);

	FMemory::Memzero(&Value, sizeof(QuantizedType));
}

void FInstancedStructNetSerializer::InternalFreeStructInstance(FNetSerializationContext& Context, FInstancedStructNetSerializerConfig* Config, QuantizedType& Value)
{
	if (Value.StructData.Num() > 0 && EnumHasAnyFlags(Value.StructDescriptorTraits, EReplicationStateTraits::HasDynamicState))
	{
		FStructNetSerializerConfig StructConfig;
		StructConfig.StateDescriptor = Config->DescriptorCache.FindOrAddDescriptor(Value.StructName);

		FNetFreeDynamicStateArgs FreeArgs;
		FreeArgs.NetSerializerConfig = &StructConfig;
		FreeArgs.Source = NetSerializerValuePointer(Value.StructData.GetData());

		StructNetSerializer->FreeDynamicState(Context, FreeArgs);
	}
}

}

namespace UE::Net
{

void InitInstancedStructNetSerializerConfig(FInstancedStructNetSerializerConfig* Config, const FProperty* Property)
{
	// We want to be explicit about which structs are supported in the config. That requires UE-180981. For now let's allow any UScriptStruct.
	Config->SupportedTypes.Reset();

	{
		FString DebugName;
		DebugName.Reserve(256);

		FFieldVariant Owner = Property->GetOwnerVariant();
		if (const UObject* Object = Owner.ToUObject())
		{
			DebugName.Append(Object->GetName()).AppendChar(TEXT('.'));
		}
		else if (const FField* Field = Owner.ToField())
		{
			DebugName.Append(Field->GetName()).AppendChar(TEXT('.'));
		}
		DebugName.Append(Property->GetName());

		Config->DescriptorCache.SetDebugName(DebugName);
	}

	// Add supported type info to the cache.
	Config->DescriptorCache.AddSupportedTypes(TConstArrayView<TSoftObjectPtr<UScriptStruct>>(Config->SupportedTypes));

	const bool bIsAllowingArbitraryStruct = true;
	if (bIsAllowingArbitraryStruct)
	{
		Config->DescriptorCache.SetMaxCachedDescriptorCount(MaxCachedInstancedStructDescriptorCount);
	}
}

FInstancedStructPropertyNetSerializerInfo::FInstancedStructPropertyNetSerializerInfo()
: FNamedStructPropertyNetSerializerInfo(FName("InstancedStruct"), UE_NET_GET_SERIALIZER(FInstancedStructNetSerializer))
{
}

bool FInstancedStructPropertyNetSerializerInfo::CanUseDefaultConfig(const FProperty* Property) const
{
	// Creating property specific configs so that we can validate and allow only property specific types. This allows us property specific tracking of used types too.
	return false;
}

FNetSerializerConfig* FInstancedStructPropertyNetSerializerInfo::BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const
{
	FInstancedStructNetSerializerConfig* Config = new (NetSerializerConfigBuffer) FInstancedStructNetSerializerConfig();
	InitInstancedStructNetSerializerConfig(Config, Property);
	return Config;
}

}

namespace UE::Net::Private
{

FInstancedStructDescriptorCache::FInstancedStructDescriptorCache()
{
	// Intentionally left empty for debugging purposes.
}

FInstancedStructDescriptorCache::~FInstancedStructDescriptorCache()
{
	// Intentionally left empty for debugging purposes.
}

// Name for debugging purposes
void FInstancedStructDescriptorCache::SetDebugName(const FString& InDebugName)
{
	DebugName = InDebugName;
}

void FInstancedStructDescriptorCache::SetMaxCachedDescriptorCount(int32 MaxCount)
{
	if (MaxCount <= 0)
	{
		DescriptorLruCache.Empty(0);
		MaxCachedDescriptorCount = 0;
	}
	else
	{
		// Clear DescriptorMap which is only used for unlimited MaxCount
		UE_CLOGF(!DescriptorMap.IsEmpty(), LogIris, Warning, "Clearing DescriptorMap from FIstancedStructDescriptorCache %ls", ToCStr(DebugName));
		DescriptorMap.Empty();
		DescriptorLruCache.Empty(MaxCount);
		MaxCachedDescriptorCount = MaxCount;
	}
}

void FInstancedStructDescriptorCache::AddSupportedTypes(const TConstArrayView<TSoftObjectPtr<UScriptStruct>>& InSupportedTypes)
{
	for (const TSoftObjectPtr<UScriptStruct>& Type : InSupportedTypes)
	{
		SupportedTypes.Add(Type);
	}
}


bool FInstancedStructDescriptorCache::IsSupportedType(const UScriptStruct* Struct) const
{
	if (ensure(Struct != nullptr))
	{
		if (SupportedTypes.IsEmpty())
		{
			return true;
		}

		for (const TSoftObjectPtr<UScriptStruct>& SupportedType : SupportedTypes)
		{
			if (const UScriptStruct* SupportedStruct = SupportedType.Get())
			{
				if (Struct->IsChildOf(SupportedStruct))
				{
					return true;
				}
			}
		}
	}

	return false;
}

TRefCountPtr<const FReplicationStateDescriptor> FInstancedStructDescriptorCache::FindDescriptor(FName StructPath)
{
	UE::TScopeLock Lock(Mutex);
	if (MaxCachedDescriptorCount > 0)
	{
		return DescriptorLruCache.FindAndTouchRef(StructPath);
	}
	else
	{
		return DescriptorMap.FindRef(StructPath);
	}
}

TRefCountPtr<const FReplicationStateDescriptor> FInstancedStructDescriptorCache::FindDescriptor(const UScriptStruct* Struct)
{
	if (!Struct)
	{
		return TRefCountPtr<const FReplicationStateDescriptor>();
	}

	const FString& PathNameString = Struct->GetPathName();
	const FName PathName(PathNameString);
	return FindDescriptor(PathName);
}

TRefCountPtr<const FReplicationStateDescriptor> FInstancedStructDescriptorCache::FindOrAddDescriptor(FName StructPath)
{
	TRefCountPtr<const FReplicationStateDescriptor> Descriptor;

	Descriptor = FindDescriptor(StructPath);
	if (Descriptor.IsValid())
	{
		return Descriptor;
	}

	const UObject* Object = StaticLoadObject(UScriptStruct::StaticClass(), nullptr, ToCStr(StructPath.ToString()), nullptr, LOAD_None);
	if (const UScriptStruct* Struct = Cast<UScriptStruct>(Object))
	{
		Descriptor = CreateAndCacheDescriptor(Struct, StructPath);
	}
	else
	{
		// Cast fail?
		ensureMsgf(Object == nullptr, TEXT("Unable to cast object %s to UScriptStruct"), ToCStr(Object->GetPathName()));
	}

	return Descriptor;
}

TRefCountPtr<const FReplicationStateDescriptor> FInstancedStructDescriptorCache::FindOrAddDescriptor(const UScriptStruct* Struct)
{
	TRefCountPtr<const FReplicationStateDescriptor> Descriptor;
	if (ensure(Struct != nullptr))
	{
		const FString& PathNameString = Struct->GetPathName();
		const FName PathName(PathNameString);

		Descriptor = FindDescriptor(PathName);
		if (Descriptor.IsValid())
		{
			return Descriptor;
		}

		if (!IsSupportedType(Struct))
		{
			return Descriptor;
		}

		// Create descriptor and add it to the cache.
		Descriptor = CreateAndCacheDescriptor(Struct, PathName);
	}

	return Descriptor;
}

TRefCountPtr<const FReplicationStateDescriptor> FInstancedStructDescriptorCache::CreateAndCacheDescriptor(const UScriptStruct* Struct, FName StructPath)
{
	TRefCountPtr<const FReplicationStateDescriptor> Descriptor;

	FReplicationStateDescriptorBuilder::FParameters Params;
	Descriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(Struct, Params);

	{
		UE::TScopeLock Lock(Mutex);
		if (MaxCachedDescriptorCount > 0)
		{
			DescriptorLruCache.Add(StructPath, Descriptor);
		}
		else
		{
			DescriptorMap.Add(StructPath, Descriptor);
		}
	}

	return Descriptor;
}

}


