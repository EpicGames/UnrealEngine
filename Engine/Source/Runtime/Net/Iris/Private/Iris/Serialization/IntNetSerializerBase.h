// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Iris/Serialization/BitPacking.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Core/BitTwiddling.h"
#include <type_traits>

namespace UE::Net::Private
{

// FIntNetSerializerBase is expected to work for both signed and unsigned integers.
template<typename InSourceType, typename InConfigType>
struct FIntNetSerializerBase
{
	static const uint32 Version = 0;

	using SourceType = InSourceType;
	// For convenience we'll use an unsigned type as the quantized type
	using QuantizedType = std::make_unsigned_t<SourceType>;
	typedef InConfigType ConfigType;

	static void Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
	{
		const QuantizedType Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

		FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const uint32 BitCount = Config->BitCount;

		// Zero value optimization for larger bit counts
		if (BitCount >= ZeroValueOptimizationBitCount)
		{
			if (Writer->WriteBool(Value == QuantizedType(0)))
			{
				return;
			}
		}

		if constexpr (sizeof(SourceType) == 8)
		{
			Writer->WriteBits(static_cast<uint32>(Value), FMath::Min(BitCount, 32U));
			if (BitCount > 32U)
			{
				Writer->WriteBits(static_cast<uint32>(Value >> 32U), BitCount - 32U); // sizeof(SourceType)*4 == 32 for 64-bit types. This is to prevent compiler warning for shorter types.
			}
		}
		else
		{
			Writer->WriteBits(Value, BitCount);
		}
	}

	static void Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
	{
		QuantizedType* Target = reinterpret_cast<QuantizedType*>(Args.Target);

		FNetBitStreamReader* Reader = Context.GetBitStreamReader();

		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const uint32 BitCount = Config->BitCount;

		// Zero value optimization for larger bit counts
		if (BitCount >= ZeroValueOptimizationBitCount)
		{
			if (Reader->ReadBool())
			{
				*Target = SourceType(0);
				return;
			}
		}

		QuantizedType Value;
		if constexpr (sizeof(SourceType) == 8)
		{
			Value = Reader->ReadBits(FMath::Min(BitCount, 32U));
			if (BitCount > 32U)
			{
				Value |= static_cast<QuantizedType>(Reader->ReadBits(BitCount - 32U)) << 32U;
			}
		}
		else
		{
			Value = static_cast<QuantizedType>(Reader->ReadBits(BitCount));
		}

		// Sign-extend the value if needed
		if constexpr (std::is_signed_v<SourceType>)
		{
			const QuantizedType SignMask = static_cast<QuantizedType>(QuantizedType(1) << (BitCount - 1U));
			Value = (Value ^ SignMask) - SignMask;
		}

		*Target = Value;
	}

	static void SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
	{
		// For this case we require the values to be properly signed as the helper function requires it.
		const SourceType Value = *reinterpret_cast<const SourceType*>(Args.Source);
		const SourceType PrevValue = *reinterpret_cast<const SourceType*>(Args.Prev);

		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const uint8 MaxBitCount = Config->BitCount;

		if (MaxBitCount == 0)
		{
			return;
		}

		const uint32 IndexToBitCountEntries = GetDeltaBitCountTableIndex(MaxBitCount);
		const uint8* BitCountTable = DeltaBitCountTable[IndexToBitCountEntries];
		const uint32 BitCountTableEntryCount = DeltaBitCountTableEntryCount[IndexToBitCountEntries];

		FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
		if constexpr (std::is_signed_v<SourceType>)
		{
			SerializeIntDelta(*Writer, Value, PrevValue, BitCountTable, BitCountTableEntryCount, MaxBitCount);
		}
		else
		{
			SerializeUintDelta(*Writer, Value, PrevValue, BitCountTable, BitCountTableEntryCount, MaxBitCount);
		}
	}

	static void DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
	{
		SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);
		const SourceType PrevValue = *reinterpret_cast<const SourceType*>(Args.Prev);

		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const uint8 MaxBitCount = Config->BitCount;

		if (MaxBitCount == 0)
		{
			Target = SourceType(0);
			return;
		}

		const uint32 IndexToBitCountEntries = GetDeltaBitCountTableIndex(MaxBitCount);
		const uint8* BitCountTable = DeltaBitCountTable[IndexToBitCountEntries];
		const uint32 BitCountTableEntryCount = DeltaBitCountTableEntryCount[IndexToBitCountEntries];

		FNetBitStreamReader* Reader = Context.GetBitStreamReader();
		if constexpr (std::is_signed_v<SourceType>)
		{
			DeserializeIntDelta(*Reader, Target, PrevValue, BitCountTable, BitCountTableEntryCount, MaxBitCount);
		}
		else
		{
			DeserializeUintDelta(*Reader, Target, PrevValue, BitCountTable, BitCountTableEntryCount, MaxBitCount);
		}
	}

	static void Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
	{
		const SourceType Source = *reinterpret_cast<const SourceType*>(Args.Source);

		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const uint32 BitCount = Config->BitCount;

		QuantizedType Value;
		if constexpr (std::is_signed_v<SourceType>)
		{
			const QuantizedType SignMask = static_cast<QuantizedType>(QuantizedType(1) << (BitCount - 1U));
			const QuantizedType ValueMask = SignMask | (SignMask - QuantizedType(1));

			Value = static_cast<QuantizedType>(Source) & ValueMask;
			// Sign-extend
			Value = (Value ^ SignMask) - SignMask;
		}
		else
		{
			constexpr QuantizedType MaxValueForType = ~QuantizedType(0);
			const QuantizedType ValueMask = MaxValueForType >> (sizeof(QuantizedType)*8U - BitCount);
			Value = Source & ValueMask;
		}

		QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
		Target = Value;
	}

	static void Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
	{
		const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
		SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

		Target = Source;
	}

	static bool IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
	{
		if (Args.bStateIsQuantized)
		{
			const QuantizedType Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
			const QuantizedType Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

			return Value0 == Value1;
		}
		else
		{
			const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
			const uint32 BitCount = Config->BitCount;

			constexpr QuantizedType MaxValueForType = ~QuantizedType(0);
			const QuantizedType ValueMask = MaxValueForType >> (sizeof(SourceType)*8U - BitCount);

			// Only the lower BitCount bits of the values are considered when quantizing.
			const QuantizedType MaskedValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0) & ValueMask;
			const QuantizedType MaskedValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1) & ValueMask;

			return MaskedValue0 == MaskedValue1;
		}
	}

	static bool Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const uint32 BitCount = Config->BitCount;

		// Detect invalid bit count
		if (BitCount < 1 || BitCount > sizeof(SourceType)*8U)
		{
			return false;
		}

		const SourceType Value = *reinterpret_cast<const SourceType*>(Args.Source);
		const bool bIsValidValue = GetBitsNeeded(Value) <= BitCount;
		return bIsValidValue;
	}

private:
	static uint32 GetDeltaBitCountTableIndex(uint8 BitCount)
	{
		// Use a small lookup table.
		// It's indexed via (BitCount-1)/8 which is in the range [0,7].
		// In max BitCount: 65+ 64 56 48 40 32 24 16  8
		//           Index:   0  3  3  3  3  2  2  1  0
		constexpr uint64 TableIndexLUT = 0b1111111110100100U;
		constexpr uint32 MaxTableIndex = 3U;
		const uint32 LUTShiftAmount = (uint8(BitCount - 1) >> 3U) << 1U;
		const uint32 TableIndex = (TableIndexLUT >> LUTShiftAmount) & MaxTableIndex;
		return TableIndex;
	}

	/* These are delta compression bit count tables for bitcounts <= 8, 16, 32 and 64 respectively.
	 * What they have in common is that they all allow for same value optimization,
	 * i.e. the delta to be serialized is zero. The index into the table must always be written
	 * so for bit counts > 8 the minimum number of bits written is 2.
	 * The bit counts are not scientifically nor empirically proven. They're mainly aiming to keep small changes cheap.
	 */
	inline static const uint8 DeltaBitCountTable[4][3] = 
	{
		{0, 0 /* unused */, 0 /* unused */},
		{0, 4, 10},
		{0, 4, 14},
		{0, 14, 32},
	};

	inline static const uint32 DeltaBitCountTableEntryCount[4] = {1, 3, 3, 3};

	static constexpr uint32 ZeroValueOptimizationBitCount = 16U;
};

}
