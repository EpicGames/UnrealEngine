// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/IntNetSerializers.h"
#include "Iris/Serialization/IntNetSerializerBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IntNetSerializers)

namespace UE::Net
{

struct FInt8NetSerializer : public Private::FIntNetSerializerBase<int8, FInt8NetSerializerConfig>
{
	inline static const FInt8NetSerializerConfig DefaultConfig = FInt8NetSerializerConfig(uint8(8));
};
UE_NET_IMPLEMENT_SERIALIZER(FInt8NetSerializer);

struct FInt16NetSerializer : public Private::FIntNetSerializerBase<int16, FInt16NetSerializerConfig>
{
	inline static const FInt16NetSerializerConfig DefaultConfig = FInt16NetSerializerConfig(uint8(16));
};
UE_NET_IMPLEMENT_SERIALIZER(FInt16NetSerializer);

struct FInt32NetSerializer : public Private::FIntNetSerializerBase<int32, FInt32NetSerializerConfig>
{
	inline static const FInt32NetSerializerConfig DefaultConfig = FInt32NetSerializerConfig(uint8(32));
};
UE_NET_IMPLEMENT_SERIALIZER(FInt32NetSerializer);

struct FInt64NetSerializer : public Private::FIntNetSerializerBase<int64, FInt64NetSerializerConfig>
{
	inline static const FInt64NetSerializerConfig DefaultConfig = FInt64NetSerializerConfig(uint8(64));
};
UE_NET_IMPLEMENT_SERIALIZER(FInt64NetSerializer);

}
