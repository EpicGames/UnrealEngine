// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/IntNetSerializerBase.h"

namespace UE::Net::Private
{

// Similar to IntNetSerializerBase when it comes to serialization.
template<typename InSourceType, typename IntRangeNetSerializerConfig>
struct FIntRangeNetSerializerBase
{
	static const uint32 Version = 0;

	typedef InSourceType SourceType;
	typedef typename TUnsignedIntType<sizeof(InSourceType)>::Type QuantizedType;
	typedef IntRangeNetSerializerConfig ConfigType;

	static void Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
	{
		return FIntNetSerializerBase<QuantizedType, ConfigType>::Serialize(Context, Args);
	}

	static void Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
	{
		return FIntNetSerializerBase<QuantizedType, ConfigType>::Deserialize(Context, Args);
	}

	static void SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
	{
		return FIntNetSerializerBase<QuantizedType, ConfigType>::SerializeDelta(Context, Args);
	}

	static void DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
	{
		return FIntNetSerializerBase<QuantizedType, ConfigType>::DeserializeDelta(Context, Args);
	}

	static void Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const SourceType Value = *reinterpret_cast<const SourceType*>(Args.Source);
		const SourceType ClampedValue = FMath::Clamp(Value, Config->LowerBound, Config->UpperBound);
		const QuantizedType RebasedValue = static_cast<QuantizedType>(static_cast<QuantizedType>(ClampedValue) - static_cast<QuantizedType>(Config->LowerBound));

		*reinterpret_cast<QuantizedType*>(Args.Target) = RebasedValue;
	}

	static void Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const QuantizedType RebasedValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
		const SourceType Value = static_cast<SourceType>(static_cast<QuantizedType>(RebasedValue + static_cast<QuantizedType>(Config->LowerBound)));
		const SourceType ClampedValue = FMath::Clamp(Value, Config->LowerBound, Config->UpperBound);
		if (ClampedValue != Value)
		{
			Context.SetError(GNetError_InvalidValue);
			return; // Do not store any value in target!
		}
	
		*reinterpret_cast<SourceType*>(Args.Target) = ClampedValue;
	}

	static bool IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		if (Args.bStateIsQuantized)
		{
			const QuantizedType Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
			const QuantizedType Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

			return Value0 == Value1;
		}
		else
		{
			const SourceType Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
			const SourceType ClampedValue0 = FMath::Clamp(Value0, Config->LowerBound, Config->UpperBound);

			const SourceType Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);
			const SourceType ClampedValue1 = FMath::Clamp(Value1, Config->LowerBound, Config->UpperBound);

			return ClampedValue0 == ClampedValue1;
		}
	}

	static bool Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);

		// Detect invalid bit count
		const uint32 BitCount = Config->BitCount;
		if (BitCount > sizeof(SourceType)*8U)
		{
			return false;
		}

		// Detect values outside of the valid range
		const SourceType Value = *reinterpret_cast<const SourceType*>(Args.Source);
		const SourceType ClampedValue = FMath::Clamp(Value, Config->LowerBound, Config->UpperBound);

		return Value == ClampedValue;
	}
};

}
