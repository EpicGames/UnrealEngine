// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/IntRangeNetSerializers.h"
#include "Iris/Serialization/IntRangeNetSerializerUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IntRangeNetSerializers)

namespace UE::Net
{

struct FInt8RangeNetSerializer : public Private::FIntRangeNetSerializerBase<int8, FInt8RangeNetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FInt8RangeNetSerializer);

struct FInt16RangeNetSerializer : public Private::FIntRangeNetSerializerBase<int16, FInt16RangeNetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FInt16RangeNetSerializer);

struct FInt32RangeNetSerializer : public Private::FIntRangeNetSerializerBase<int32, FInt32RangeNetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FInt32RangeNetSerializer);

struct FInt64RangeNetSerializer : public Private::FIntRangeNetSerializerBase<int64, FInt64RangeNetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FInt64RangeNetSerializer);

}
