// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/InternalEnumNetSerializers.h"
#include "Iris/Core/BitTwiddling.h"
#include <limits>
#include <type_traits>

namespace UE::Net::Private
{

template<typename SourceType, typename ConfigType>
class TInitEnumNetSerializerConfig
{
public:
	static bool Init(ConfigType& OutConfig, const UEnum* Enum)
	{
		// NumEnums() include the potentially auto-generated _MAX enum value.
		const int32 EnumValueCount = Enum->NumEnums();

		using LargeIntegerType = std::conditional_t<std::is_signed_v<SourceType>, int64, uint64>;
		constexpr LargeIntegerType MaxSupportedValue = static_cast<LargeIntegerType>(std::numeric_limits<SourceType>::max());


		// Find smallest and largest values.
		if (EnumValueCount <= 0)
		{
			// At the time of this writing this case should not happen due to errors when a UENUM does not contain values.
			ensureAlways(EnumValueCount > 0);
			OutConfig.LowerBound = 0;
			OutConfig.UpperBound = 0;
			OutConfig.BitCount = static_cast<uint8>(GetBitsNeededForRange(OutConfig.LowerBound, OutConfig.UpperBound));
			OutConfig.Enum = Enum;
		}
		else
		{
			// Cannot use UEnum methods here due to issues with the generated _MAX value as well as uint64 values outside of the positive int64 range.
			LargeIntegerType SmallestValue = std::numeric_limits<LargeIntegerType>::max();
			LargeIntegerType LargestValue = std::numeric_limits<LargeIntegerType>::min();
			for (int32 EnumIt = 0, EnumEndIt = EnumValueCount; EnumIt != EnumEndIt; ++EnumIt)
			{
				const LargeIntegerType Value = static_cast<LargeIntegerType>(Enum->GetValueByIndex(EnumIt));
				SmallestValue = FMath::Min(SmallestValue, Value);
				// Prevent out of bounds auto-generated _MAX values from sneaking in.
				LargestValue = FMath::Max(LargestValue, FMath::Min(Value, MaxSupportedValue));
			}

			OutConfig.LowerBound = static_cast<SourceType>(SmallestValue);
			OutConfig.UpperBound = static_cast<SourceType>(LargestValue);
			OutConfig.BitCount = static_cast<uint8>(GetBitsNeededForRange(OutConfig.LowerBound, OutConfig.UpperBound));
			OutConfig.Enum = Enum;
		}

		return true;
	}
};

bool InitEnumNetSerializerConfig(FEnumInt8NetSerializerConfig& OutConfig, const UEnum* Enum)
{
	return TInitEnumNetSerializerConfig<int8, decltype(OutConfig)>::Init(OutConfig, Enum);
}

bool InitEnumNetSerializerConfig(FEnumInt16NetSerializerConfig& OutConfig, const UEnum* Enum)
{
	return TInitEnumNetSerializerConfig<int16, decltype(OutConfig)>::Init(OutConfig, Enum);
}

bool InitEnumNetSerializerConfig(FEnumInt32NetSerializerConfig& OutConfig, const UEnum* Enum)
{
	return TInitEnumNetSerializerConfig<int32, decltype(OutConfig)>::Init(OutConfig, Enum);
}

bool InitEnumNetSerializerConfig(FEnumInt64NetSerializerConfig& OutConfig, const UEnum* Enum)
{
	return TInitEnumNetSerializerConfig<int64, decltype(OutConfig)>::Init(OutConfig, Enum);
}

bool InitEnumNetSerializerConfig(FEnumUint8NetSerializerConfig& OutConfig, const UEnum* Enum)
{
	return TInitEnumNetSerializerConfig<uint8, decltype(OutConfig)>::Init(OutConfig, Enum);
}

bool InitEnumNetSerializerConfig(FEnumUint16NetSerializerConfig& OutConfig, const UEnum* Enum)
{
	return TInitEnumNetSerializerConfig<uint16, decltype(OutConfig)>::Init(OutConfig, Enum);
}

bool InitEnumNetSerializerConfig(FEnumUint32NetSerializerConfig& OutConfig, const UEnum* Enum)
{
	return TInitEnumNetSerializerConfig<uint32, decltype(OutConfig)>::Init(OutConfig, Enum);
}

bool InitEnumNetSerializerConfig(FEnumUint64NetSerializerConfig& OutConfig, const UEnum* Enum)
{
	return TInitEnumNetSerializerConfig<uint64, decltype(OutConfig)>::Init(OutConfig, Enum);
}

}
