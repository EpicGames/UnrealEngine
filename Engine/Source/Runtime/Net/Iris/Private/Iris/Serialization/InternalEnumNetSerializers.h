// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/EnumNetSerializers.h"

namespace UE::Net::Private
{

IRISCORE_API bool InitEnumNetSerializerConfig(FEnumInt8NetSerializerConfig& OutConfig, const UEnum* Enum);
IRISCORE_API bool InitEnumNetSerializerConfig(FEnumInt16NetSerializerConfig& OutConfig, const UEnum* Enum);
IRISCORE_API bool InitEnumNetSerializerConfig(FEnumInt32NetSerializerConfig& OutConfig, const UEnum* Enum);
IRISCORE_API bool InitEnumNetSerializerConfig(FEnumInt64NetSerializerConfig& OutConfig, const UEnum* Enum);

IRISCORE_API bool InitEnumNetSerializerConfig(FEnumUint8NetSerializerConfig& OutConfig, const UEnum* Enum);
IRISCORE_API bool InitEnumNetSerializerConfig(FEnumUint16NetSerializerConfig& OutConfig, const UEnum* Enum);
IRISCORE_API bool InitEnumNetSerializerConfig(FEnumUint32NetSerializerConfig& OutConfig, const UEnum* Enum);
IRISCORE_API bool InitEnumNetSerializerConfig(FEnumUint64NetSerializerConfig& OutConfig, const UEnum* Enum);

}
