// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "HAL/UnrealMemory.h"

namespace UE::Net::Private
{

FInternalNetSerializationContext::FInternalNetSerializationContext(UReplicationSystem* InReplicationSystem)
: ReplicationSystem(InReplicationSystem)
, ObjectReferenceCache(&InReplicationSystem->GetReplicationSystemInternal()->GetObjectReferenceCache())
, PackageMap(InReplicationSystem->GetReplicationSystemInternal()->GetIrisObjectReferencePackageMap())
, bDowngradeAutonomousProxyRole(0)
, bInlineObjectReferenceExports(0)
, bSerializeObjectReferencesAsRemoteIds(InReplicationSystem->IsUsingRemoteObjectReferences() ? 1 : 0)
, bAlwaysSwapRolesOnReplication(InReplicationSystem->AlwaysSwapRolesOnReplication() ? 1 : 0)
, bAllowExecuteRepNotify(ReplicationSystem->AllowExecuteRepNotify() ? 1 : 0)
{
}

void* FInternalNetSerializationContext::Alloc(SIZE_T Size, SIZE_T Alignment)
{
	return FMemory::Malloc(Size, static_cast<uint32>(Alignment));
}

void FInternalNetSerializationContext::Free(void* Ptr)
{
	return FMemory::Free(Ptr);
}

void* FInternalNetSerializationContext::Realloc(void* PrevAddress, SIZE_T NewSize, uint32 Alignment)
{
	return FMemory::Realloc(PrevAddress, NewSize, Alignment);
}

void FInternalNetSerializationContext::Init(const FInitParameters& InitParams)
{
	*this = FInternalNetSerializationContext(InitParams.ReplicationSystem);
	PackageMap = InitParams.PackageMap;
	ResolveContext = InitParams.ObjectResolveContext;
	ResolveContext.ReplicationSystem = InitParams.ReplicationSystem;
}

}
