// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Iris/IrisConstants.h"
#include "Iris/ReplicationSystem/ObjectReferenceCacheFwd.h"

class FRPCDoSDetection;
class UReplicationSystem;
class UObject;
class UIrisObjectReferencePackageMap;

namespace UE::Net::Private
{
	class FObjectReferenceCache;
}

namespace UE::Net::Private
{

class FInternalNetSerializationContext final
{
public:
	struct FInitParameters
	{
		UReplicationSystem* ReplicationSystem = nullptr;
		FNetObjectResolveContext ObjectResolveContext;
		UIrisObjectReferencePackageMap* PackageMap = nullptr;			
	};

	FInternalNetSerializationContext();
	explicit FInternalNetSerializationContext(UReplicationSystem* InReplicationSystem);

	void Init(const FInitParameters& Params);

	// We allow memory allocations for dynamic states
	void* Alloc(SIZE_T Size, SIZE_T Alignment);
	void Free(void* Ptr);
	void* Realloc(void* PrevAddress, SIZE_T NewSize, uint32 Alignment);

	UReplicationSystem* ReplicationSystem;
	FObjectReferenceCache* ObjectReferenceCache;
	FNetObjectResolveContext ResolveContext;

	// We have a special implementation of UPackageMap to capture references when using LastResortNetSerializer
	UIrisObjectReferencePackageMap* PackageMap = nullptr;

	// $IRIS TODO Roles really shouldn't be replicated as properties. In dire need of a proper authority system.
	// This is ONLY to be used by role serialization.
	uint32 bDowngradeAutonomousProxyRole : 1;

	// Allow References to be inlined in serialized state
	uint32 bInlineObjectReferenceExports : 1;

	// If true, object references will be quantized & serialized as FRemoteObjectIds instead of FNetObjectReferences.
	// Requires UE_WITH_REMOTE_OBJECT_HANDLE.
	uint32 bSerializeObjectReferencesAsRemoteIds : 1;

	// If true, swap local and remote roles when replicating.
	uint32 bAlwaysSwapRolesOnReplication : 1;

	// If true, allow calling rep notifies
	uint32 bAllowExecuteRepNotify : 1;

	// If true, always quantize a static object reference using the object's path
	uint32 bForceObjectPathSerialization : 1 = 0;

	// RPC DoS Detection instance for the owning connection (set per-connection, used during RPC dispatch)
	FRPCDoSDetection* RPCDoS = nullptr;
};

inline FInternalNetSerializationContext::FInternalNetSerializationContext()
: ReplicationSystem(nullptr)
, ObjectReferenceCache(nullptr)
, PackageMap(nullptr)
, bDowngradeAutonomousProxyRole(0)
, bInlineObjectReferenceExports(0)
, bSerializeObjectReferencesAsRemoteIds(0)
, bAlwaysSwapRolesOnReplication(0)
, bAllowExecuteRepNotify(1)
{}

// Scope used when we actually want to export references.
class FForceInlineExportScope
{
public:
	explicit FForceInlineExportScope(FInternalNetSerializationContext* InContext);
	~FForceInlineExportScope();

private:
	FInternalNetSerializationContext* InternalContext;
	uint32 bOldInlineObjectReferences = 0U;
};

inline FForceInlineExportScope::FForceInlineExportScope(FInternalNetSerializationContext* InContext)
: InternalContext(InContext)
{
	if (InternalContext)
	{
		bOldInlineObjectReferences = InternalContext->bInlineObjectReferenceExports;
		InternalContext->bInlineObjectReferenceExports = 1U;
	}
}

inline FForceInlineExportScope::~FForceInlineExportScope()
{
	if (InternalContext)
	{
		// Restore state
		InternalContext->bInlineObjectReferenceExports = bOldInlineObjectReferences;
	}
}

}
