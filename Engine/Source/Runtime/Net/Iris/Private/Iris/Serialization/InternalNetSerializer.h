// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/InternalNetSerializerBuilder.h"

namespace UE::Net::Private
{

template<typename NetSerializerImpl>
class TInternalNetSerializer final
{
public:
	static constexpr FNetSerializer ConstructNetSerializer(const TCHAR* Name)
	{
		// Start off with the basics
		FNetSerializer Serializer = TNetSerializer<NetSerializerImpl>::ConstructNetSerializer(Name);

		// Add additional internal stuff
		TInternalNetSerializerBuilder<NetSerializerImpl> Builder;
		Builder.Validate();

		Serializer.Traits |= Builder.GetTraits();

		return Serializer;
	}
};

}

#define UE_NET_DECLARE_SERIALIZER_INTERNAL UE_NET_DECLARE_SERIALIZER

#define UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(SerializerName) const UE::Net::FNetSerializer SerializerName ## NetSerializerInfo::Serializer = UE::Net::Private::TInternalNetSerializer<SerializerName>::ConstructNetSerializer(TEXT(#SerializerName)); \
	uint32 SerializerName ## NetSerializerInfo::GetQuantizedTypeSize() { return UE::Net::TNetSerializerBuilder<SerializerName>::GetQuantizedTypeSize(); }; \
	uint32 SerializerName ## NetSerializerInfo::GetQuantizedTypeAlignment() { return UE::Net::TNetSerializerBuilder<SerializerName>::GetQuantizedTypeAlignment(); }; \
	const FNetSerializerConfig* SerializerName ## NetSerializerInfo::GetDefaultConfig() { return UE::Net::TNetSerializerBuilder<SerializerName>::GetDefaultConfig(); };
