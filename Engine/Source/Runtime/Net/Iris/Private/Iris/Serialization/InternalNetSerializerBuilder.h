// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"

namespace UE::Net::Private
{

// This implementation only deals with specifics that only internal NetSerializer implementations are allowed to use.
template<typename NetSerializerImpl>
class TInternalNetSerializerBuilder final
{
private:
	enum class ETrueType : unsigned
	{
		Value = 1
	};

	enum class EFalseType : unsigned
	{
		Value = 0
	};

	struct FTraits
	{
	};

	template<typename U, U> struct FSignatureCheck;
	template<typename> struct FTypeCheck;

	enum ETraits : unsigned
	{
	};

public:
	static constexpr ENetSerializerTraits GetTraits() { return ENetSerializerTraits::None; }

	static void Validate()
	{
	}
};

}
