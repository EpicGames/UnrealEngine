// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/InternalNetSerializerDelegates.h"

namespace UE::Net::Private
{

void FInternalNetSerializerDelegates::BroadcastPreFreezeNetSerializerRegistry()
{
	FSimpleMulticastDelegate& PreFreezeNetSerializerRegistryDelegate = GetPreFreezeNetSerializerRegistryDelegate();
	PreFreezeNetSerializerRegistryDelegate.Broadcast();
	PreFreezeNetSerializerRegistryDelegate.Clear();
}

void FInternalNetSerializerDelegates::BroadcastPostFreezeNetSerializerRegistry()
{
	FSimpleMulticastDelegate& PostFreezeNetSerializerRegistryDelegate = GetPostFreezeNetSerializerRegistryDelegate();
	PostFreezeNetSerializerRegistryDelegate.Broadcast();
	PostFreezeNetSerializerRegistryDelegate.Clear();
}

void FInternalNetSerializerDelegates::BroadcastLoadedModulesUpdated()
{
	FSimpleMulticastDelegate& LoadedModulesUpdatedDelegate = GetLoadedModulesUpdatedDelegate();
	LoadedModulesUpdatedDelegate.Broadcast();
}

FSimpleMulticastDelegate& FInternalNetSerializerDelegates::GetPreFreezeNetSerializerRegistryDelegate()
{
	static FSimpleMulticastDelegate PreFreezeNetSerializerRegistryDelegate;
	return PreFreezeNetSerializerRegistryDelegate;
}

FSimpleMulticastDelegate& FInternalNetSerializerDelegates::GetPostFreezeNetSerializerRegistryDelegate()
{
	static FSimpleMulticastDelegate PostFreezeNetSerializerRegistryDelegate;
	return PostFreezeNetSerializerRegistryDelegate;
}

FSimpleMulticastDelegate& FInternalNetSerializerDelegates::GetLoadedModulesUpdatedDelegate()
{
	static FSimpleMulticastDelegate LoadedModulesUpdatedDelegate;
	return LoadedModulesUpdatedDelegate;
}

}
