// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

namespace UE::Net::Private
{

class FInternalNetSerializerDelegates
{
public:
	static void BroadcastPreFreezeNetSerializerRegistry();
	static void BroadcastPostFreezeNetSerializerRegistry();
	static void BroadcastLoadedModulesUpdated();

	static FSimpleMulticastDelegate& GetPreFreezeNetSerializerRegistryDelegate();
	static FSimpleMulticastDelegate& GetPostFreezeNetSerializerRegistryDelegate();
	static FSimpleMulticastDelegate& GetLoadedModulesUpdatedDelegate();
};

}
