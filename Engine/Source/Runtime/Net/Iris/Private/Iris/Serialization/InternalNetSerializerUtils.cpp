// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/InternalNetSerializerUtils.h"

namespace UE::Net
{

bool IsObjectReferenceNetSerializer(const FNetSerializer* NetSerializer)
{
	bool bIsObjectReferenceNetSerializer = false;
	bIsObjectReferenceNetSerializer = bIsObjectReferenceNetSerializer || (NetSerializer == &UE_NET_GET_SERIALIZER(FObjectNetSerializer));
	bIsObjectReferenceNetSerializer = bIsObjectReferenceNetSerializer || (NetSerializer == &UE_NET_GET_SERIALIZER(FObjectPtrNetSerializer));
	bIsObjectReferenceNetSerializer = bIsObjectReferenceNetSerializer || (NetSerializer == &UE_NET_GET_SERIALIZER(FWeakObjectNetSerializer));
	bIsObjectReferenceNetSerializer = bIsObjectReferenceNetSerializer || (NetSerializer == &UE_NET_GET_SERIALIZER(FScriptInterfaceNetSerializer));
	return bIsObjectReferenceNetSerializer;
}

}

