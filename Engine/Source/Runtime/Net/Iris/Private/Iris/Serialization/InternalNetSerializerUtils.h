// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/NetSerializers.h"

namespace UE::Net
{

bool IsStructNetSerializer(const FNetSerializer* Serializer);
bool IsArrayPropertyNetSerializer(const FNetSerializer* Serializer);
IRISCORE_API bool IsObjectReferenceNetSerializer(const FNetSerializer* Serializer);

}

namespace UE::Net
{

inline bool IsStructNetSerializer(const FNetSerializer* Serializer)
{
	return Serializer == &UE_NET_GET_SERIALIZER(FStructNetSerializer);
}

inline bool IsArrayPropertyNetSerializer(const FNetSerializer* Serializer)
{
	return Serializer == &UE_NET_GET_SERIALIZER(FArrayPropertyNetSerializer);
}

inline bool IsOptionalPropertyNetSerializer(const FNetSerializer* Serializer)
{
	return Serializer == &UE_NET_GET_SERIALIZER(FOptionalPropertyNetSerializer);
}


}
