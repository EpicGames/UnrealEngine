// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InternalNetSerializers)

FArrayPropertyNetSerializerConfig::FArrayPropertyNetSerializerConfig()
: FNetSerializerConfig()
{
	ConfigTraits = ENetSerializerConfigTraits::NeedDestruction;
}

FArrayPropertyNetSerializerConfig::~FArrayPropertyNetSerializerConfig() = default;

FOptionalPropertyNetSerializerConfig::FOptionalPropertyNetSerializerConfig()
: FNetSerializerConfig()
{
	// We have a generated descriptor, indicate that it must be destroyed
	ConfigTraits = ENetSerializerConfigTraits::NeedDestruction;
}

FOptionalPropertyNetSerializerConfig::~FOptionalPropertyNetSerializerConfig() = default;

