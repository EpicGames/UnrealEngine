// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/InternalNetSerializer.h"
#include "Templates/RefCounting.h"
#include "InternalNetSerializers.generated.h"

namespace UE::Net
{
	struct FReplicationStateDescriptor;
}

// Bitfield property
USTRUCT()
struct FBitfieldNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint8 BitMask = 0;
};

// Array property
USTRUCT()
struct FArrayPropertyNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	FArrayPropertyNetSerializerConfig();
	~FArrayPropertyNetSerializerConfig();

	FArrayPropertyNetSerializerConfig(const FArrayPropertyNetSerializerConfig&) = delete;
	FArrayPropertyNetSerializerConfig(FArrayPropertyNetSerializerConfig&&) = delete;
	
	FArrayPropertyNetSerializerConfig& operator=(const FArrayPropertyNetSerializerConfig&) = delete;
	FArrayPropertyNetSerializerConfig& operator=(FArrayPropertyNetSerializerConfig&&) = delete;

	UPROPERTY()
	uint16 MaxElementCount = 0;

	UPROPERTY()
	uint16 ElementCountBitCount = 0;

	UPROPERTY()
	TFieldPath<FArrayProperty> Property;

	TRefCountPtr<const UE::Net::FReplicationStateDescriptor> StateDescriptor;
};

template<>
struct TStructOpsTypeTraits<FArrayPropertyNetSerializerConfig> : public TStructOpsTypeTraitsBase2<FArrayPropertyNetSerializerConfig>
{
	enum
	{
		WithCopy = false
	};
};

/**
 * Any property that doesn't have any other option will end up using this.
 * As the name suggests it's a last resort.
 * - Cannot support delta compression in a meaningful way.
 * - Must allocate memory to store quantized state.
 */
USTRUCT()
struct FLastResortPropertyNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	static constexpr uint32 DefaultMaxQuantizedSizeBits = 65535;

	UPROPERTY()
	TFieldPath<FProperty> Property;

	UPROPERTY()
	bool bExcludeFromDefaultStateHash = false;

	UPROPERTY()
	uint32 MaxQuantizedSizeBits = DefaultMaxQuantizedSizeBits;
};

USTRUCT()
struct FOptionalPropertyNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

	IRISCORE_API FOptionalPropertyNetSerializerConfig();
	IRISCORE_API ~FOptionalPropertyNetSerializerConfig();

	FOptionalPropertyNetSerializerConfig(const FOptionalPropertyNetSerializerConfig&) = delete;
	FOptionalPropertyNetSerializerConfig(FOptionalPropertyNetSerializerConfig&&) = delete;

	FOptionalPropertyNetSerializerConfig& operator=(const FOptionalPropertyNetSerializerConfig&) = delete;
	FOptionalPropertyNetSerializerConfig& operator=(FOptionalPropertyNetSerializerConfig&&) = delete;

	UPROPERTY()
	TFieldPath<FOptionalProperty> Property;

	TRefCountPtr<const UE::Net::FReplicationStateDescriptor> StateDescriptor;
};

template<>
struct TStructOpsTypeTraits<FOptionalPropertyNetSerializerConfig> : public TStructOpsTypeTraitsBase2<FOptionalPropertyNetSerializerConfig>
{
	enum
	{
		WithCopy = false
	};
};


// ENetRole. With role swapping at deserialization
USTRUCT()
struct FNetRoleNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 RelativeInternalOffsetToOtherRole = 0;

	UPROPERTY()
	int32 RelativeExternalOffsetToOtherRole = 0;

	UPROPERTY()
	uint8 LowerBound = 0;
	UPROPERTY()
	uint8 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;

	UPROPERTY()
	uint8 AutonomousProxyValue = 0;
	UPROPERTY()
	uint8 SimulatedProxyValue = 0;

	const UEnum* Enum = nullptr;
};

USTRUCT()
struct FFieldPathNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TFieldPath<FProperty> Property;
};

USTRUCT()
struct FFieldPathNetSerializerSerializationHelper
{
	GENERATED_BODY()

	UPROPERTY();
	TWeakObjectPtr<UStruct> Owner;
	UPROPERTY();
	TArray<FName> PropertyPath;
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER_INTERNAL(FBitfieldNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER_INTERNAL(FLastResortPropertyNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER_INTERNAL(FArrayPropertyNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER_INTERNAL(FNetRoleNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER_INTERNAL(FFieldPathNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER_INTERNAL(FOptionalPropertyNetSerializer, IRISCORE_API);

IRISCORE_API bool InitBitfieldNetSerializerConfigFromProperty(FBitfieldNetSerializerConfig& OutConfig, const FBoolProperty* Bitfield);

IRISCORE_API bool InitLastResortPropertyNetSerializerConfigFromProperty(FLastResortPropertyNetSerializerConfig& OutConfig, const FProperty* Property);

uint32 GetNetArrayPropertyData(NetSerializerValuePointer QuantizedArray, NetSerializerValuePointer& OutArrayBuffer);

IRISCORE_API bool PartialInitNetRoleSerializerConfig(FNetRoleNetSerializerConfig& OutConfig, const UEnum* Enum);

}
