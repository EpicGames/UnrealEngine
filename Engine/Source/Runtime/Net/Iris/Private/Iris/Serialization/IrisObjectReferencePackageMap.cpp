// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/IrisObjectReferencePackageMap.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IrisObjectReferencePackageMap)

namespace UE
{
	namespace Net
	{
		bool bEnableIrisPackageMapNameExports = true;
		static FAutoConsoleVariableRef CVarEnableIrisPackageMapNameExports(TEXT("net.iris.EnableIrisPackageMapNameExports"), bEnableIrisPackageMapNameExports, TEXT("If enabled, iris captures and exports fnames when calling into old serialziation code instead of serializing a strings."));
	}
}

bool UIrisObjectReferencePackageMap::SerializeObject(FArchive& Ar, UClass* InClass, UObject*& Obj, FNetworkGUID* OutNetGUID)
{
	UE::Net::FIrisPackageMapExports* PackageMapExports = Ar.IsSaving() ? PackageMapExportsForWriting : PackageMapExportsForReading;

	if (!ensure(PackageMapExports))
	{
		return false;
	}

	UE::Net::FIrisPackageMapExports::FObjectReferenceArray* References = &PackageMapExports->References;

	if (Ar.IsSaving())
	{
		int32 Index;
		if (!References->Find(Obj, Index))
		{
			Index = References->Add(Obj);
		}
		uint32 IndexToWrite = static_cast<uint32>(Index);
		Ar.SerializeIntPacked(IndexToWrite);
	}
	else
	{
		uint32 ReadIndex;
		Ar.SerializeIntPacked(ReadIndex);
		if (References->IsValidIndex(ReadIndex))
		{
			Obj = (*References)[ReadIndex];
		}
		else
		{
			ensureMsgf(false, TEXT("UIrisObjectReferencePackageMap::SerializeObject, failed to read object reference index %u is out of bounds. Current ObjectReference num: %u"), ReadIndex, References->Num());
			return false;
		}
	}

	return true;
}

bool UIrisObjectReferencePackageMap::SerializeName(FArchive& Ar, FName& InName)
{
	using namespace UE::Net;

	UE::Net::FIrisPackageMapExports* PackageMapExports = Ar.IsSaving() ? PackageMapExportsForWriting : PackageMapExportsForReading;

	if (!bEnableIrisPackageMapNameExports)
	{
		return Super::SerializeName(Ar, InName);
	}

	if (!ensure(PackageMapExports))
	{
		return false;
	}

	FIrisPackageMapExports::FNameArray* Names = &PackageMapExports->Names;
	if (Ar.IsSaving())
	{
		int32 Index;
		if (!Names->Find(InName, Index))
		{
			Index = Names->Add(InName);
		}
		uint32 IndexToWrite = static_cast<uint32>(Index);
		Ar.SerializeIntPacked(IndexToWrite);
	}
	else
	{
		uint32 ReadIndex;
		Ar.SerializeIntPacked(ReadIndex);
		if (Names->IsValidIndex(ReadIndex))
		{
			InName = (*Names)[ReadIndex];
		}
		else
		{
			ensureMsgf(false, TEXT("UIrisObjectReferencePackageMap::SerializeName, failed to read name index %u is out of bounds. Current Name num: %u"), ReadIndex, Names->Num());
			return false;
		}
	}

	return true;
}

void UIrisObjectReferencePackageMap::InitForRead(const UE::Net::FIrisPackageMapExports* InPackageMapExports, const UE::Net::FNetTokenResolveContext& InNetTokenResolveContext)
{ 
	PackageMapExportsForReading = const_cast<UE::Net::FIrisPackageMapExports*>(InPackageMapExports);
	NetTokenResolveContext = InNetTokenResolveContext;
}

void UIrisObjectReferencePackageMap::InitForWrite(UE::Net::FIrisPackageMapExports* InPackageMapExports)
{
	PackageMapExportsForWriting = InPackageMapExports;
	if (ensureMsgf(PackageMapExportsForWriting, TEXT("UIrisObjectReferencePackageMap requires valid PackageMapExports to capture exports.")))
	{
		PackageMapExportsForWriting->Reset();
	}
}

namespace UE::Net
{

FIrisObjectReferencePackageMapReadScope::FIrisObjectReferencePackageMapReadScope(UIrisObjectReferencePackageMap* InPackageMap, const UE::Net::FIrisPackageMapExports* PackageMapExports, const UE::Net::FNetTokenResolveContext* NetTokenResolveContext)
: PackageMap(InPackageMap)
{
	if (PackageMap)
	{
		PackageMap->InitForRead(PackageMapExports, *NetTokenResolveContext);
	}
}

FIrisObjectReferencePackageMapReadScope::~FIrisObjectReferencePackageMapReadScope()
{
	if (PackageMap)
	{
		PackageMap->PackageMapExportsForReading = nullptr;
	}
}

FIrisObjectReferencePackageMapWriteScope::FIrisObjectReferencePackageMapWriteScope(UIrisObjectReferencePackageMap* InPackageMap, UE::Net::FIrisPackageMapExports* PackageMapExports)
: PackageMap(InPackageMap)
{
	if (!PackageMap)
	{
		return;
	}

	if (PackageMapExports && ensureMsgf(PackageMap->PackageMapExportsForReading != PackageMapExports, TEXT("FIrisObjectReferencePackageMapWriteScope cannot read and write from the same FIrisPackageMapExports as we are reading from.")))
	{
		PackageMapExports->Reset();
		PackageMap->PackageMapExportsForWriting = PackageMapExports;
	}
	else
	{
		PackageMap->PackageMapExportsForWriting = nullptr;
	}
}

FIrisObjectReferencePackageMapWriteScope::~FIrisObjectReferencePackageMapWriteScope()
{
	if (PackageMap)
	{
		PackageMap->PackageMapExportsForWriting = nullptr;
	}
}

} // UE::Net


