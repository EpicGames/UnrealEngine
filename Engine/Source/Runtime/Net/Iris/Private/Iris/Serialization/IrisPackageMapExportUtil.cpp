// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/IrisPackageMapExportUtil.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/NetSerializerArrayStorage.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Net/Core/NetToken/NetToken.h"
#include "Net/Core/Trace/NetTrace.h"
#include "NetExportContext.h"

namespace UE::Net
{

const FNetSerializer* FIrisPackageMapExportsUtil::ObjectNetSerializer = &UE_NET_GET_SERIALIZER(FObjectNetSerializer);
const FNetSerializer* FIrisPackageMapExportsUtil::NameNetSerializer = &UE_NET_GET_SERIALIZER(FNameAsNetTokenNetSerializer);
//const FNetSerializer* FIrisPackageMapExportsUtil::NameNetSerializer = &UE_NET_GET_SERIALIZER(FNameNetSerializer);

void FIrisPackageMapExportsUtil::Serialize(FNetSerializationContext& Context, const QuantizedType& Value)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// If we have any references, export them!
	{
		UE_NET_TRACE_SCOPE(ObjectReferences, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
		const uint32 NumReferences = Value.ObjectReferenceStorage.Num();
		if (Writer->WriteBool(NumReferences != 0))
		{
			UE::Net::WritePackedUint32(Writer, NumReferences);		
			FObjectNetSerializerConfig ObjectSerializerConfig;
			for (const FObjectNetSerializerQuantizedReferenceStorage& Ref : MakeArrayView(Value.ObjectReferenceStorage.GetData(), NumReferences))
			{
				FNetSerializeArgs ObjectArgs;
				ObjectArgs.NetSerializerConfig = &ObjectSerializerConfig;
				ObjectArgs.Source = NetSerializerValuePointer(&Ref);

				ObjectNetSerializer->Serialize(Context, ObjectArgs);
			}
		}
	}

	// If we have any names, export them!
	{
		UE_NET_TRACE_SCOPE(Names, *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
		const uint32 NumNames = Value.NameStorage.Num();
		if (Writer->WriteBool(NumNames != 0))
		{
			UE::Net::WritePackedUint32(Writer, NumNames);
			FNetSerializerConfig NameSerializerConfig;
			for (const QuantizedType::FQuantizedName& QuantizedName : MakeArrayView(Value.NameStorage.GetData(), NumNames))
			{
				FNetSerializeArgs NameArgs;
				NameArgs.NetSerializerConfig = &NameSerializerConfig;
				NameArgs.Source = NetSerializerValuePointer(&QuantizedName);

				NameNetSerializer->Serialize(Context, NameArgs);
			}
		}
	}

	// We now serialize NetTokens directly in the data but we still need to append exports.
	if (UE::Net::Private::FNetExportContext* ExportContext = Context.GetExportContext())
	{
		for (const FNetToken& NetToken : MakeArrayView(Value.NetTokenStorage.GetData(), Value.NetTokenStorage.Num()))
		{
			ExportContext->AddPendingExport(NetToken);
		}
	}
}

void FIrisPackageMapExportsUtil::Deserialize(FNetSerializationContext& Context, QuantizedType& Value)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// Read any object references
	{
		UE_NET_TRACE_SCOPE(ObjectReferences, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
		const bool bHasObjectReferences = Reader->ReadBool();
		if (bHasObjectReferences)
		{
			const uint32 NumReferences = UE::Net::ReadPackedUint32(Reader);

			if (NumReferences > MaxExports)
			{
				UE_LOGF(LogIris, Error, "FIrisPackageMapExportsUtil::Received too many object reference exports %u > max:%u", NumReferences, MaxExports);
				Context.SetError(GNetError_ArraySizeTooLarge);
				return;
			}

			Value.ObjectReferenceStorage.AdjustSize(Context, NumReferences);

			FObjectNetSerializerConfig ObjectSerializerConfig;
			for (FObjectNetSerializerQuantizedReferenceStorage& Ref : MakeArrayView(Value.ObjectReferenceStorage.GetData(), Value.ObjectReferenceStorage.Num()))
			{
				FNetDeserializeArgs ObjectArgs;
				ObjectArgs.NetSerializerConfig = &ObjectSerializerConfig;
				ObjectArgs.Target = NetSerializerValuePointer(&Ref);

				ObjectNetSerializer->Deserialize(Context, ObjectArgs);
			}
		}
		else
		{
			Value.ObjectReferenceStorage.Free(Context);
		}
	}

	// Read any exported names
	{
		UE_NET_TRACE_SCOPE(Names, *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
		const bool bHasNames = Reader->ReadBool();
		if (bHasNames)
		{
			const uint32 NumNames = UE::Net::ReadPackedUint32(Reader);

			if (NumNames > MaxExports)
			{
				UE_LOGF(LogIris, Error, "FIrisPackageMapExportsUtil::Received too many name exports %u > max:%u", NumNames, MaxExports);
				Context.SetError(GNetError_ArraySizeTooLarge);
				return;
			}

			Value.NameStorage.AdjustSize(Context, NumNames);

			FNetSerializerConfig NameSerializerConfig;
			for (QuantizedType::FQuantizedName& QuantizedName : MakeArrayView(Value.NameStorage.GetData(), Value.NameStorage.Num()))
			{
				FNetDeserializeArgs NameArgs;
				NameArgs.NetSerializerConfig = &NameSerializerConfig;
				NameArgs.Target = NetSerializerValuePointer(&QuantizedName);

				NameNetSerializer->Deserialize(Context, NameArgs);
			}
		}
		else
		{
			Value.NameStorage.Free(Context);
		}
	}

	Value.NetTokenStorage.Free(Context);
}

void FIrisPackageMapExportsUtil::Quantize(FNetSerializationContext& Context, const UE::Net::FIrisPackageMapExports& PackageMapExports, TArrayView<const UE::Net::FNetToken> NetTokensPendingExport, QuantizedType& Value)
{
	// Quantize captured references
	{
		const FIrisPackageMapExports::FObjectReferenceArray& ObjectReferences = PackageMapExports.References;
		const uint32 NumObjectReferences = ObjectReferences.Num();
		Value.ObjectReferenceStorage.AdjustSize(Context, NumObjectReferences);
		if (NumObjectReferences > 0)
		{
			FObjectNetSerializerConfig ObjectNetSerializerConfig;
			const TObjectPtr<UObject>* SourceReferences = ObjectReferences.GetData();
			FObjectNetSerializerQuantizedReferenceStorage* TargetReferences = Value.ObjectReferenceStorage.GetData();
			for (uint32 ReferenceIndex = 0; ReferenceIndex < NumObjectReferences; ++ReferenceIndex)
			{
				FNetQuantizeArgs ObjectArgs;
				ObjectArgs.NetSerializerConfig = &ObjectNetSerializerConfig;
				ObjectArgs.Source = NetSerializerValuePointer(SourceReferences + ReferenceIndex);
				ObjectArgs.Target = NetSerializerValuePointer(TargetReferences + ReferenceIndex);

				ObjectNetSerializer->Quantize(Context, ObjectArgs);
			}
		}
	}

	// Quantize captured names
	{
		const FIrisPackageMapExports::FNameArray& Names = PackageMapExports.Names;
		const uint32 NumNames = Names.Num();
		Value.NameStorage.AdjustSize(Context, NumNames);
		if (NumNames > 0)
		{
			FNetSerializerConfig NameNetSerializerConfig;
			const FName* SourceNames = Names.GetData();
			QuantizedType::FQuantizedName* TargetNames = Value.NameStorage.GetData();
			for (uint32 ReferenceIndex = 0; ReferenceIndex < NumNames; ++ReferenceIndex)
			{
				FNetQuantizeArgs NameArgs;
				NameArgs.NetSerializerConfig = &NameNetSerializerConfig;
				NameArgs.Source = NetSerializerValuePointer(SourceNames + ReferenceIndex);
				NameArgs.Target = NetSerializerValuePointer(TargetNames + ReferenceIndex);

				NameNetSerializer->Quantize(Context, NameArgs);
			}
		}
	}

	// Just store captured NetTokenExports they will be added as pending exports during serialization.
	{
		const uint32 NumNetTokens = NetTokensPendingExport.Num();
		Value.NetTokenStorage.AdjustSize(Context, NumNetTokens);
		FNetToken* TargetTokens = Value.NetTokenStorage.GetData();
		for (uint32 Index = 0; Index < NumNetTokens; ++Index)
		{
			TargetTokens[Index] = NetTokensPendingExport[Index];
		}
	}
}

void FIrisPackageMapExportsUtil::FreeDynamicState(QuantizedType& Value)
{
	FNetSerializationContext Context;
	Private::FInternalNetSerializationContext InternalContext;
	Context.SetInternalContext(&InternalContext);

	// Clear all info
	Value.ObjectReferenceStorage.Free(Context);
	Value.NameStorage.Free(Context);
	Value.NetTokenStorage.Free(Context);
}

void FIrisPackageMapExportsUtil::Dequantize(FNetSerializationContext& Context, const QuantizedType& Source, UE::Net::FIrisPackageMapExports& PackageMapExports)
{
	// References
	{
		UE::Net::FIrisPackageMapExports::FObjectReferenceArray& ObjectReferences = PackageMapExports.References;
		const uint32 NumObjectReferences = Source.ObjectReferenceStorage.Num();
		if (NumObjectReferences > 0U)
		{		
			ObjectReferences.SetNumUninitialized(NumObjectReferences);

			FObjectNetSerializerConfig ObjectNetSerializerConfig;
			const FObjectNetSerializerQuantizedReferenceStorage* SourceReferences = Source.ObjectReferenceStorage.GetData();
			TObjectPtr<UObject>* TargetReferences = ObjectReferences.GetData();
			for (uint32 ReferenceIndex = 0; ReferenceIndex < NumObjectReferences; ++ReferenceIndex)
			{
				FNetDequantizeArgs ObjectArgs;
				ObjectArgs.NetSerializerConfig = &ObjectNetSerializerConfig;
				ObjectArgs.Source = NetSerializerValuePointer(SourceReferences + ReferenceIndex);
				ObjectArgs.Target = NetSerializerValuePointer(TargetReferences + ReferenceIndex);

				ObjectNetSerializer->Dequantize(Context, ObjectArgs);
			}
		}
	}

	// Names
	{
		UE::Net::FIrisPackageMapExports::FNameArray& Names = PackageMapExports.Names;

		const uint32 NumNames = Source.NameStorage.Num();
		if (NumNames > 0U)
		{		
			Names.SetNumUninitialized(NumNames);

			FNetSerializerConfig NameNetSerializerConfig;
			const QuantizedType::FQuantizedName* SourceNames = Source.NameStorage.GetData();
			FName* TargetNames = Names.GetData();
			for (uint32 ReferenceIndex = 0; ReferenceIndex < NumNames; ++ReferenceIndex)
			{
				FNetDequantizeArgs NameArgs;
				NameArgs.NetSerializerConfig = &NameNetSerializerConfig;
				NameArgs.Source = NetSerializerValuePointer(SourceNames + ReferenceIndex);
				NameArgs.Target = NetSerializerValuePointer(TargetNames + ReferenceIndex);

				NameNetSerializer->Dequantize(Context, NameArgs);
			}
		}
	}
}

bool FIrisPackageMapExportsUtil::IsEqual(FNetSerializationContext& Context, const QuantizedType& Value0, const QuantizedType& Value1)
{
	if ((Value0.ObjectReferenceStorage.Num() != Value1.ObjectReferenceStorage.Num()) || (Value0.NameStorage.Num() != Value1.NameStorage.Num()) || (Value0.NetTokenStorage.Num() != Value1.NetTokenStorage.Num()))
	{
		return false;
	}

	if (Value0.ObjectReferenceStorage.Num() > 0 && FMemory::Memcmp(Value0.ObjectReferenceStorage.GetData(), Value1.ObjectReferenceStorage.GetData(), sizeof(FNetObjectReference) * Value0.ObjectReferenceStorage.Num()) != 0)
	{
		return false;
	}

	if (Value0.NameStorage.Num() > 0 && FMemory::Memcmp(Value0.NameStorage.GetData(), Value1.NameStorage.GetData(), sizeof(FName) * Value0.NameStorage.Num()) != 0)
	{
		return false;
	}

	return true;
}

void FIrisPackageMapExportsUtil::CloneDynamicState(FNetSerializationContext& Context, QuantizedType& Target, const QuantizedType& Source)
{
	Target.ObjectReferenceStorage.Clone(Context, Source.ObjectReferenceStorage);
	Target.NameStorage.Clone(Context, Source.NameStorage);
	Target.NetTokenStorage.Clone(Context, Source.NetTokenStorage);
}

void FIrisPackageMapExportsUtil::FreeDynamicState(FNetSerializationContext& Context, QuantizedType& Value)
{
	// Clear all info
	Value.ObjectReferenceStorage.Free(Context);
	Value.NameStorage.Free(Context);
	Value.NetTokenStorage.Free(Context);
}

void FIrisPackageMapExportsUtil::CollectNetReferences(FNetSerializationContext& Context, const QuantizedType& Value, const FNetSerializerChangeMaskParam& ChangeMaskInfo, FNetReferenceCollector& Collector)
{
	const FNetReferenceInfo ReferenceInfo(FNetReferenceInfo::EResolveType::ResolveOnClient);	
	for (const FObjectNetSerializerQuantizedReferenceStorage& Ref : MakeArrayView(Value.ObjectReferenceStorage.GetData(), Value.ObjectReferenceStorage.Num()))
	{
		const FQuantizedObjectReference& InternalReference = *reinterpret_cast<const FQuantizedObjectReference*>(&Ref.Storage);

		Collector.Add(ReferenceInfo, InternalReference, ChangeMaskInfo);
	}
}

bool FIrisPackageMapExportsUtil::Validate(FNetSerializationContext& Context, const QuantizedType& SourceValue)
{
	if ((SourceValue.ObjectReferenceStorage.Num() > MaxExports) || (SourceValue.NameStorage.Num() > MaxExports))
	{
		return false;
	}
	return true;
}

}
