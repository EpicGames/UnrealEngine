// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternalNetSerializers.h"
#include "UObject/CoreNet.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/NetSerializerArrayStorage.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"
#include "Iris/Serialization/IrisPackageMapExportUtil.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Net/Core/NetToken/NetTokenExportContext.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net
{

struct FFLastResortPropertyNetSerializerQuantizedType
{
	FIrisPackageMapExportsQuantizedType QuantizedExports;

	// How many bytes the current allocation can hold.
	uint32 ByteCapacity = 0;
	// How many bits are valid
	uint32 BitCount = 0;
	void* Storage = nullptr;
};

}

template <> struct TIsPODType<UE::Net::FFLastResortPropertyNetSerializerQuantizedType> { enum { Value = true }; };

namespace UE::Net
{

struct FLastResortPropertyNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bHasCustomNetReference = true;

	// Types

	// SourceType is unknown
	typedef void SourceType;
	typedef FFLastResortPropertyNetSerializerQuantizedType QuantizedType;
	typedef FLastResortPropertyNetSerializerConfig ConfigType;

	//
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	static constexpr uint32 AllocationAlignment = 4U;

	static void FreeDynamicStateInternal(FNetSerializationContext&, QuantizedType& Value);
	static void GrowDynamicStateInternal(FNetSerializationContext&, QuantizedType& Value, uint32 NewBitCount);
	static void ShrinkDynamicStateInternal(FNetSerializationContext&, QuantizedType& Value, uint32 NewBitCount);
	static void AdjustStorageSize(FNetSerializationContext&, QuantizedType& Value, uint32 NewBitCount);
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FLastResortPropertyNetSerializer);

void FLastResortPropertyNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	
	if (Context.IsInitializingDefaultState())
	{
		// If the config indicates that we should not be included in the default state hash write nothing
		if (Config->bExcludeFromDefaultStateHash)
		{
			return;
		}

		// For now we ignore this in default state hash if it has exported NetTokens as they will differ
		if (Value.QuantizedExports.NetTokenStorage.Num() > 0U)
		{
			return;
		}
	}

	UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Config->Property.Get()->GetName(), *Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

	// If we have any captured exports, serialize them.
	FIrisPackageMapExportsUtil::Serialize(Context, Value.QuantizedExports);

	// Write data.
	WritePackedUint32(Writer, Value.BitCount);
	if (Value.BitCount > 0)
	{
		Writer->WriteBitStream(static_cast<uint32*>(Value.Storage), 0U, Value.BitCount);
	}
}

void FLastResortPropertyNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	// For consistency, we should never get here. For now we ignore this in default state hash due to complications with asymmetrically serialized state.
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);
	const uint32 CurrentBitCount = Value.BitCount;

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Config->Property.Get()->GetName(), *Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

	// Read exports for packagemap.
	FIrisPackageMapExportsUtil::Deserialize(Context, Value.QuantizedExports);

	// Read the data
	const uint32 NewBitCount = ReadPackedUint32(Reader);
	if (!ensureMsgf(NewBitCount <= Config->MaxQuantizedSizeBits,
		TEXT("FLastResortPropertyNetSerializer::Deserialize data size of %u bits exceeds maximum of %u bits for property %s."),
		NewBitCount, Config->MaxQuantizedSizeBits, *Config->Property.Get()->GetName()))
	{
		Context.SetError(GNetError_ArraySizeTooLarge);
		return;
	}

	AdjustStorageSize(Context, Value, NewBitCount);

	Reader->ReadBitStream(static_cast<uint32*>(Value.Storage), NewBitCount);
}

void FLastResortPropertyNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FProperty* Property = Config->Property.Get();
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);

	// Setup UIrisObjectReferencePackageMap to capture exports
	Private::FInternalNetSerializationContext* InternalContext = Context.GetInternalContext();
	UIrisObjectReferencePackageMap* PackageMap = InternalContext ? InternalContext->PackageMap : nullptr;

	// Since this struct uses custom serialization path we need to explicitly capture exports in order to forward them to iris
	UE::Net::FIrisPackageMapExports PackageMapExports;
	UE::Net::FNetTokenExportContext::FNetTokenExports NetTokensPendingExport;

	if (PackageMap)
	{
		PackageMap->InitForWrite(&PackageMapExports);
	}

	// Use the Property serialization and store as binary blob.
	FNetBitWriter Archive(PackageMap, 8192);	
	FNetTokenExportScope NetTokenExportScope(Archive, Context.GetNetTokenStore(), NetTokensPendingExport);
	Property->NetSerializeItem(Archive, PackageMap, reinterpret_cast<void*>(Args.Source));

	const uint64 BitCount = Archive.GetNumBits();

	if (!ensureMsgf(BitCount <= Config->MaxQuantizedSizeBits,
		TEXT("FLastResortPropertyNetSerializer::Quantize: data size of %llu bits exceeds maximum of %u bits for property %s."),
		BitCount, Config->MaxQuantizedSizeBits, *Config->Property.Get()->GetName()))
	{
		Context.SetError(GNetError_ArraySizeTooLarge);
		return;
	}

	if (!ensureMsgf(!Archive.IsError(), TEXT("FLastResortPropertyNetSerializer::Quantize: NetBitWriter archive error in NetSerializeItem for property %s."), *Config->Property.Get()->GetName()))
	{
		Context.SetError(GNetError_ArraySizeTooLarge);
		return;
	}

	// Quantize captured exports
	FIrisPackageMapExportsUtil::Quantize(Context, PackageMapExports, NetTokensPendingExport, Value.QuantizedExports);

	// Deal with serialized data
	AdjustStorageSize(Context, Value, static_cast<uint32>(BitCount));
	if (BitCount > 0)
	{
		FMemory::Memcpy(Value.Storage, Archive.GetData(), (BitCount + 7U)/8U);
	}
}

void FLastResortPropertyNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FProperty* Property = Config->Property.Get();

	QuantizedType& Source = *reinterpret_cast<QuantizedType*>(Args.Source);

	// Dequantize and inject exports
	Private::FInternalNetSerializationContext* InternalContext = Context.GetInternalContext();
	UIrisObjectReferencePackageMap* PackageMap = InternalContext ? InternalContext->PackageMap : nullptr;
	UE::Net::FIrisPackageMapExports PackageMapExports;

	FIrisPackageMapExportsUtil::Dequantize(Context, Source.QuantizedExports, PackageMapExports);	

	// Setup resolve context for call into NetSerialize
	UE::Net::FNetTokenResolveContext ResolveContext;
	ResolveContext.RemoteNetTokenStoreState = Context.GetRemoteNetTokenStoreState();
	ResolveContext.NetTokenStore = Context.GetNetTokenStore();

	PackageMap->InitForRead(&PackageMapExports, ResolveContext);

	// Read data
	if (Source.BitCount)
	{
		FNetBitReader Archive(PackageMap, static_cast<uint8*>(Source.Storage), Source.BitCount);
		Property->NetSerializeItem(Archive, PackageMap, reinterpret_cast<void*>(Args.Target));
	}
	else
	{
		Property->ClearValue(reinterpret_cast<void*>(Args.Target));
	}
}

bool FLastResortPropertyNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		if ((Value0.BitCount != Value1.BitCount))
		{
			return false;
		}

		if (!FIrisPackageMapExportsUtil::IsEqual(Context, Value0.QuantizedExports, Value1.QuantizedExports))
		{
			return false;
		}

		const bool bIsEqual = (Value0.BitCount == 0U) || FMemory::Memcmp(Value0.Storage, Value1.Storage, Align((Value0.BitCount + 7U)/8U, AllocationAlignment)) == 0;
		return bIsEqual;
	}
	else
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const FProperty* Property = Config->Property.Get();

		const void* Value0 = reinterpret_cast<const void*>(Args.Source0);
		const void* Value1 = reinterpret_cast<const void*>(Args.Source1);
		const bool bIsEqual = Property->Identical(Value0, Value1);
		return bIsEqual;
	}
}

void FLastResortPropertyNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);

	// Clone captured exports
	FIrisPackageMapExportsUtil::CloneDynamicState(Context, Target.QuantizedExports, Source.QuantizedExports);

	const uint16 ByteCount = static_cast<uint16>(Align((Source.BitCount + 7U)/8U, AllocationAlignment));

	void* Storage = nullptr;
	if (ByteCount > 0)
	{
		Storage = Context.GetInternalContext()->Alloc(ByteCount, AllocationAlignment);
		FMemory::Memcpy(Storage, Source.Storage, ByteCount);
	}
	Target.ByteCapacity = ByteCount;
	Target.BitCount = Source.BitCount;
	Target.Storage = Storage;

}

void FLastResortPropertyNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	return FreeDynamicStateInternal(Context, *reinterpret_cast<QuantizedType*>(Args.Source));
}

void FLastResortPropertyNetSerializer::FreeDynamicStateInternal(FNetSerializationContext& Context, QuantizedType& Value)
{
	// Clear all info

	// Free captured export data.
	FIrisPackageMapExportsUtil::FreeDynamicState(Context, Value.QuantizedExports);
	
	Context.GetInternalContext()->Free(Value.Storage);

	Value.BitCount = 0;
	Value.ByteCapacity = 0;
	Value.Storage = 0;
}

void FLastResortPropertyNetSerializer::GrowDynamicStateInternal(FNetSerializationContext& Context, QuantizedType& Value, uint32 NewBitCount)
{
	checkSlow(NewBitCount > Value.BitCount);

	const uint32 ByteCount = Align((NewBitCount + 7U)/8U, AllocationAlignment);

	// We don't support delta compression for the unknown contents of the bits so we don't need to copy the old data.
	Context.GetInternalContext()->Free(Value.Storage);

	void* Storage = Context.GetInternalContext()->Alloc(ByteCount, AllocationAlignment);

	// Clear the last word to support IsEqual Memcmp optimization.
	const uint32 LastWordIndex = ByteCount/4U - 1U;
	static_cast<uint32*>(Storage)[LastWordIndex] = 0U;

	Value.ByteCapacity = ByteCount;
	Value.BitCount = NewBitCount;
	Value.Storage = Storage;
}

void FLastResortPropertyNetSerializer::AdjustStorageSize(FNetSerializationContext& Context, QuantizedType& Value, uint32 NewBitCount)
{
	const uint32 NewByteCapacity = Align((NewBitCount + 7U)/8U, AllocationAlignment);
	if (NewByteCapacity == 0)
	{
		// Free everything
		FreeDynamicStateInternal(Context, Value);
	}
	else if (NewByteCapacity > Value.ByteCapacity)
	{
		GrowDynamicStateInternal(Context, Value, NewBitCount);
	}
	// If byte capacity is within the allocated capacity we just update the bit count and clear the last word
	else
	{
		Value.BitCount = NewBitCount;

		// Clear the last word to support IsEqual Memcmp optimization.
		const uint32 LastWordIndex = NewByteCapacity/4U - 1U;
		static_cast<uint32*>(Value.Storage)[LastWordIndex] = 0U;
	}
}

bool InitLastResortPropertyNetSerializerConfigFromProperty(FLastResortPropertyNetSerializerConfig& OutConfig, const FProperty* Property)
{
	OutConfig.Property = TFieldPath<FProperty>(const_cast<FProperty*>(Property));
	return Property != nullptr;
}

void FLastResortPropertyNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FNetReferenceCollector& Collector = *reinterpret_cast<UE::Net::FNetReferenceCollector*>(Args.Collector);

	FIrisPackageMapExportsUtil::CollectNetReferences(Context, Value.QuantizedExports, Args.ChangeMaskInfo, Collector);
}


}
