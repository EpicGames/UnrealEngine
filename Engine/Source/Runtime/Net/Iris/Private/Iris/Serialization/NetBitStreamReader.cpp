// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtils.h"
#include "Iris/IrisConfig.h"
#include "HAL/PlatformMemory.h"
#include "Misc/AssertionMacros.h"

#if UE_NETBITSTREAMREADER_VALIDATE
#	define UE_NETBITSTREAMREADER_CHECK(expr) check(expr)
#	define UE_NETBITSTREAMREADER_CHECKF(expr, format, ...) checkf(expr, format, ##__VA_ARGS__)
#else
#	define UE_NETBITSTREAMREADER_CHECK(...) 
#	define UE_NETBITSTREAMREADER_CHECKF(...) 
#endif

namespace UE::Net
{

FNetBitStreamReader::FNetBitStreamReader()
: Buffer(nullptr)
, BufferBitCapacity(0)
, BufferBitStartOffset(0)
, BufferBitPosition(0)
, PendingWord(0)
, OverflowBitCount(0)
, bHasSubstream(0)
, bIsSubstream(0)
, bIsInvalid(0)
{
}

FNetBitStreamReader::~FNetBitStreamReader()
{
	UE_NETBITSTREAMREADER_CHECKF(!bHasSubstream, TEXT("FNetBitStreamReader is destroyed with active substream. "));
}

void FNetBitStreamReader::InitBits(const void* InBuffer, uint32 BitCount)
{
	check(InBuffer != nullptr);
	checkf((UPTRINT(InBuffer) & 3) == 0, TEXT("Buffer needs to be 4-byte aligned."));
	// Re-initializing a substream or while having an active substream is not supported.
	UE_NETBITSTREAMREADER_CHECK(!bHasSubstream && !bIsSubstream);

	Buffer = static_cast<const uint32*>(InBuffer);
	BufferBitCapacity = BitCount;
	BufferBitPosition = 0;
	if (BitCount > 0)
	{
		PendingWord = INTEL_ORDER32(Buffer[0]);
	}
}

uint32 FNetBitStreamReader::ReadBits(uint32 BitCount)
{
	// Must be valid and must not read from main stream if it has a substream. Technically the latter would work, as we're just reading, but it's weird.
	UE_NETBITSTREAMREADER_CHECK((!bIsInvalid) & (!bHasSubstream));

	if (OverflowBitCount != 0)
	{
		return 0U;
	}

	if (BufferBitCapacity - BufferBitPosition < BitCount)
	{
		OverflowBitCount = BitCount - (BufferBitCapacity - BufferBitPosition);
		return 0U;
	}

	const uint32 CurrentBufferBitPosition = BufferBitPosition;
	const uint32 BitCountUsedInWord = BufferBitPosition & 31;
	const uint32 BitCountLeftInWord = 32 - (BufferBitPosition & 31);

	BufferBitPosition += BitCount;

	// If after the read we still have unused bits in the PendingWord we can skip loading a new word.
	if (BitCountLeftInWord > BitCount)
	{
		const uint32 PendingWordMask = ((1U << BitCount) - 1U);
		const uint32 Value = (PendingWord >> BitCountUsedInWord) & PendingWordMask;
		return Value;
	}
	else
	{
		uint32 Value = PendingWord >> BitCountUsedInWord;
		if ((BufferBitPosition & ~31U) < BufferBitCapacity)
		{
			// BitCountToRead will be in range [0, 31] as we've already written at least one bit at this point
			const uint32 BitCountToRead = BitCount - BitCountLeftInWord;
			const uint32 Word = INTEL_ORDER32(Buffer[BufferBitPosition >> 5U]);
			const uint32 WordMask = (1U << BitCountToRead) - 1U;

			Value = ((Word & WordMask) << (BitCountLeftInWord & 31)) | Value;
			PendingWord = Word;
		}

		return Value;
	}
}

void FNetBitStreamReader::ReadBitStream(uint32* InDst, uint32 BitCount)
{
	// Must be valid and must not read from main stream if it has a substream. Technically the latter would work, as we're just reading, but it's weird.
	UE_NETBITSTREAMREADER_CHECK((!bIsInvalid) & (!bHasSubstream));

	if (OverflowBitCount != 0)
	{
		return;
	}

	if (BufferBitCapacity - BufferBitPosition < BitCount)
	{
		OverflowBitCount = BitCount - (BufferBitCapacity - BufferBitPosition);
		return;
	}

	uint32 CurSrcBit = BufferBitPosition;
	const uint32* RESTRICT Src = Buffer;
	uint32* RESTRICT Dst = InDst;
	uint32 DstWordOffset = 0;
	uint32 BitCountToCopy = BitCount;

	// We can adjust the final bit position here as we're only using the above variables from here on
	BufferBitPosition += BitCount;
	// Make sure PendingWord is up to date unless we've reached the end of the stream
	if (BufferBitPosition < BufferBitCapacity)
	{
		PendingWord = INTEL_ORDER32(Src[BufferBitPosition >> 5U]);
	}

	// Copy full words
	if (BitCountToCopy >= 32U)
	{
		// Fast path for byte aligned source buffer.
		if ((CurSrcBit & 7) == 0)
		{
			const uint32 WordCountToCopy = BitCountToCopy >> 5U;
			FPlatformMemory::Memcpy(Dst, reinterpret_cast<const uint8*>(Src) + (CurSrcBit >> 3U), WordCountToCopy*sizeof(uint32));
			DstWordOffset += WordCountToCopy;
		}
		else
		{
			// We know that each 32 bit copy straddles two words from Src as CurSrcBit % 32 != 0, 
			// else the fast path above would be used.
			const uint32 PrevWordShift = CurSrcBit & 31U;
			const uint32 NextWordShift = (32U - CurSrcBit) & 31U;

			// Set up initial Word so we can do a single read in each loop iteration.
			uint32 SrcWordOffset = CurSrcBit >> 5U;
			uint32 PrevWord = INTEL_ORDER32(Src[SrcWordOffset]);
			++SrcWordOffset;
			for (uint32 WordIt = 0, WordEndIt = (BitCountToCopy >> 5U); WordIt != WordEndIt; ++WordIt, ++SrcWordOffset, ++DstWordOffset)
			{
				const uint32 NextWord = INTEL_ORDER32(Src[SrcWordOffset]);
				const uint32 Word = (NextWord << NextWordShift) | (PrevWord >> PrevWordShift);
				Dst[DstWordOffset] = INTEL_ORDER32(Word);
				PrevWord = NextWord;
			}
		}

		const uint32 BitCountCopied = (BitCountToCopy & ~31U);
		CurSrcBit += BitCountCopied;
		BitCountToCopy &= 31U;
	}

	if (BitCountToCopy)
	{
		const uint32 Word = INTEL_ORDER32(Dst[DstWordOffset]);
		const uint32 SrcWord = BitStreamUtils::GetBits(Src, CurSrcBit, BitCountToCopy);
		const uint32 SrcMask = (1U << BitCountToCopy) - 1U;
		const uint32 DstWord = (Word & ~SrcMask) | (SrcWord & SrcMask);
		Dst[DstWordOffset] = INTEL_ORDER32(DstWord);
	}
}

void FNetBitStreamReader::Seek(uint32 BitPosition)
{
	UE_NETBITSTREAMREADER_CHECK((!bIsInvalid) & (!bHasSubstream));

	const uint32 AdjustedBitPosition = BitPosition + BufferBitStartOffset;
	// We handle uint32 overflow as well which makes this code a bit more complicated. The OverflowBitCount may not always end up correct, but will be at least 1.
	if ((AdjustedBitPosition > BufferBitCapacity) | (AdjustedBitPosition < BitPosition))
	{
		OverflowBitCount = FPlatformMath::Max(AdjustedBitPosition, BufferBitCapacity + 1U) - BufferBitCapacity;
		return;
	}

	OverflowBitCount = 0;

	BufferBitPosition = AdjustedBitPosition;
	if ((BufferBitPosition & ~31U) < BufferBitCapacity)
	{
		PendingWord = INTEL_ORDER32(Buffer[BufferBitPosition >> 5U]);
	}
}

void FNetBitStreamReader::DoOverflow()
{
	if (OverflowBitCount == 0)
	{
		Seek(BufferBitCapacity + 1);
	}
}

FNetBitStreamReader FNetBitStreamReader::CreateSubstream(uint32 MaxBitCount)
{
	UE_NETBITSTREAMREADER_CHECK((!bIsInvalid) & (!bHasSubstream));

	// Create a copy of this stream and overwrite the necessary members.
	FNetBitStreamReader Substream = *this;
	Substream.BufferBitStartOffset = BufferBitPosition;
	Substream.bHasSubstream = 0;
	Substream.bIsSubstream = 1;

	bHasSubstream = 1;

	/* If this stream is overflown make sure the substream will always be overflown as well!
	 * We must be careful to ensure that a seek to the beginning of this stream will still cause the substream to be overflown.
	 * We can ignore MaxBitCount completely because no writes will succeed anyway.
	 */
	if (OverflowBitCount)
	{
		Substream.BufferBitCapacity = Substream.BufferBitStartOffset;
		// It's not vital that the OverflowBitCount is set as the user can reset it with a Seek(0) call. In any case no modifications to the bitstream can be done.
		Substream.OverflowBitCount = OverflowBitCount;
	}
	else
	{
		Substream.BufferBitCapacity = BufferBitPosition + FPlatformMath::Min(MaxBitCount, BufferBitCapacity - BufferBitPosition);
	}

	return Substream;
}

void FNetBitStreamReader::CommitSubstream(FNetBitStreamReader& Substream)
{
	// Only accept substreams iff this is the parent and the substream has not overflown and has not previously been commited or discarded.
	if (!ensure(bHasSubstream & (!Substream.bHasSubstream) & (!bIsInvalid) & (!Substream.bIsInvalid) & (Buffer == Substream.Buffer) & (BufferBitPosition == Substream.BufferBitStartOffset)))
	{
		return;
	}

	if (!Substream.IsOverflown())
	{
		BufferBitPosition = Substream.BufferBitPosition;
		if ((Substream.BufferBitPosition & ~31U) < BufferBitCapacity)
		{
			PendingWord = INTEL_ORDER32(Buffer[Substream.BufferBitPosition >> 5U]);
		}
	}

	bHasSubstream = 0;
	Substream.bIsInvalid = 1;
}

void FNetBitStreamReader::DiscardSubstream(FNetBitStreamReader& Substream)
{
	// Only accept substreams iff this is the parent and the substream has not previously been commited or discarded.
	if (!ensure(bHasSubstream & (!Substream.bHasSubstream) & (!bIsInvalid) & (!Substream.bIsInvalid) & (Buffer == Substream.Buffer) & (BufferBitPosition == Substream.BufferBitStartOffset)))
	{
		return;
	}

	bHasSubstream = 0;
	Substream.bIsInvalid = 1;
}

}

#undef UE_NETBITSTREAMREADER_CHECK
#undef UE_NETBITSTREAMREADER_CHECKF
