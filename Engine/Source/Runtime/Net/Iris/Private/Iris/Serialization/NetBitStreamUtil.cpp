// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetBitStreamUtil.h"

#include "Algo/AllOf.h"

#include "Iris/Core/BitTwiddling.h"
#include "Iris/Core/IrisLog.h"

#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/StringNetSerializerUtils.h"

#include "Math/Vector.h"

#include "Net/Core/NetBitArray.h"

namespace UE::Net
{

// Whether a string contains only ASCII codepoints.
static bool ContainsASCIIOnly(FStringView StringView)
{
	return Algo::AllOf(
		StringView,
		[](FStringView::ElementType Char) -> bool
		{
			return (Char & static_cast<FStringView::ElementType>(0x7FU)) == Char;
		}
	);
}

void WritePackedUint64(FNetBitStreamWriter* Writer, uint64 Value)
{
	// As we represent the number of bytes to write with three bits we want bits needed to be >= 1 such that the number of bytes ends up in the range [1, 8].
	const uint32 BitCountNeeded = GetBitsNeeded(Value | 1U);
	const uint32 ByteCountNeeded = (BitCountNeeded + 7U) / 8U;
	const uint32 BitCountToWrite = ByteCountNeeded * 8U;

	Writer->WriteBits(ByteCountNeeded - 1U, 3U);
	if (BitCountToWrite <= 32U)
	{
		Writer->WriteBits(Value & 0xFFFFFFFFU, BitCountToWrite);
	}
	else
	{
		Writer->WriteBits(Value & 0xFFFFFFFFU, 32U);
		Writer->WriteBits(static_cast<uint32>(Value >> 32U), BitCountToWrite - 32U);
	}
}

uint64 ReadPackedUint64(FNetBitStreamReader* Reader)
{
	const uint32 ByteCountToRead = Reader->ReadBits(3U) + 1U;
	const uint32 BitCountToRead = ByteCountToRead * 8U;

	if (BitCountToRead <= 32)
	{
		const uint64 Value = Reader->ReadBits(BitCountToRead);
		return Value;
	}
	else
	{
		uint64 Value = Reader->ReadBits(32U);
		Value |= (static_cast<uint64>(Reader->ReadBits(BitCountToRead - 32U)) << 32U);
		return Value;
	}
}

void WritePackedInt64(FNetBitStreamWriter* Writer, int64 Value)
{
	const uint32 BitCountNeeded = GetBitsNeeded(Value);
	const uint32 ByteCountNeeded = (BitCountNeeded + 7U) / 8U;
	const uint32 BitCountToWrite = ByteCountNeeded * 8U;

	Writer->WriteBits(ByteCountNeeded - 1U, 3U);
	if (BitCountToWrite <= 32U)
	{
		Writer->WriteBits(static_cast<uint64>(Value) & 0xFFFFFFFFU, BitCountToWrite);
	}
	else
	{
		const uint64 UnsignedValue = static_cast<uint64>(Value);
		Writer->WriteBits(UnsignedValue & 0xFFFFFFFFU, 32U);
		Writer->WriteBits(static_cast<uint32>(UnsignedValue >> 32U), BitCountToWrite - 32U);
	}
}

int64 ReadPackedInt64(FNetBitStreamReader* Reader)
{
	const uint32 ByteCountToRead = Reader->ReadBits(3U) + 1U;
	const uint32 BitCountToRead = ByteCountToRead * 8U;

	uint64 UnsignedValue;
	if (BitCountToRead <= 32U)
	{
		UnsignedValue = Reader->ReadBits(BitCountToRead);
	}
	else
	{
		UnsignedValue = Reader->ReadBits(32U);
		UnsignedValue |= (static_cast<uint64>(Reader->ReadBits(BitCountToRead - 32U)) << 32U);
	}

	// Sign-extend the value
	const uint64 Mask = 1ULL << (BitCountToRead - 1U);
	UnsignedValue = (UnsignedValue ^ Mask) - Mask;
	const int64 Value = static_cast<int64>(UnsignedValue);
	return Value;
}

void WritePackedUint32(FNetBitStreamWriter* Writer, uint32 Value)
{
	// As we represent the number of bytes to write with two bits we want bits needed to be >= 1
	const uint32 BitCountNeeded = GetBitsNeeded(Value | 1U);
	const uint32 ByteCountNeeded = (BitCountNeeded + 7U) / 8U;
	const uint32 BitCountToWrite = ByteCountNeeded * 8U;

	Writer->WriteBits(ByteCountNeeded - 1U, 2U);
	Writer->WriteBits(Value, BitCountToWrite);
}

uint32 ReadPackedUint32(FNetBitStreamReader* Reader)
{
	const uint32 ByteCountToRead = Reader->ReadBits(2U) + 1U;
	const uint32 BitCountToRead = ByteCountToRead * 8U;

	const uint32 Value = Reader->ReadBits(BitCountToRead);
	return Value;
}

void WritePackedInt32(FNetBitStreamWriter* Writer, int32 Value)
{
	const uint32 BitCountNeeded = GetBitsNeeded(Value);
	const uint32 ByteCountNeeded = (BitCountNeeded + 7U)/8U;
	const uint32 BitCountToWrite = ByteCountNeeded*8U;
	
	Writer->WriteBits(ByteCountNeeded - 1U, 2U);
	Writer->WriteBits(Value, BitCountToWrite);
}

int32 ReadPackedInt32(FNetBitStreamReader* Reader)
{
	const uint32 ByteCountToRead = Reader->ReadBits(2U) + 1U;
	const uint32 BitCountToRead = ByteCountToRead*8U;
	
	uint32 UnsignedValue = Reader->ReadBits(BitCountToRead);
	// Sign-extend the value
	const uint32 Mask = 1U << (BitCountToRead - 1U);
	UnsignedValue = (UnsignedValue ^ Mask) - Mask;
	const int32 Value = static_cast<int32>(UnsignedValue);
	return Value;
}

void WritePackedUint16(FNetBitStreamWriter* Writer, uint16 Value)
{
	// As we represent the number of bytes to write with one bit we want bits needed to be >= 1
	const uint32 BitCountNeeded = GetBitsNeeded(Value | 1U);
	const uint32 ByteCountNeeded = (BitCountNeeded + 7U) / 8U;
	const uint32 BitCountToWrite = ByteCountNeeded * 8U;

	Writer->WriteBits(ByteCountNeeded - 1U, 1U);
	Writer->WriteBits(Value, BitCountToWrite);
}

uint16 ReadPackedUint16(FNetBitStreamReader* Reader)
{
	const uint32 ByteCountToRead = Reader->ReadBits(1U) + 1U;
	const uint32 BitCountToRead = ByteCountToRead * 8U;

	const uint16 Value = static_cast<uint16>(Reader->ReadBits(BitCountToRead));
	return Value;
}

void WriteString(FNetBitStreamWriter* Writer, FStringView StringView)
{
	using Codec = Private::FStringNetSerializerUtils::TStringCodec<TCHAR>;
	constexpr uint32 SizeAlignment = 4;
	
	const uint32 StringViewLength = StringView.Len();
	if (StringViewLength > 65535)
	{
		UE_LOGF(LogSerialization, Error, "String to write is too long.");
		Writer->WriteBool(false);
		WritePackedUint16(Writer, 0);
		return;
	}

	constexpr uint32 StackAllocatedBufferSize = 1024;
	TArray<uint8, TInlineAllocator<StackAllocatedBufferSize>> EncodingBuffer;
	const bool bIsEncoded = !ContainsASCIIOnly(StringView);
	if (Writer->WriteBool(bIsEncoded))
	{
		if (Writer->IsOverflown())
		{
			return;
		}

		const uint32 SafeEncodedLength = Codec::GetSafeEncodedBufferLength(StringViewLength);
		EncodingBuffer.SetNumUninitialized(Align(SafeEncodedLength, SizeAlignment));

		uint32 OutEncodedLength = 0;
		const bool bEncodingSuccess = Codec::Encode(EncodingBuffer.GetData(), EncodingBuffer.Num(), StringView.GetData(), StringView.Len(), OutEncodedLength);

		if (!bEncodingSuccess || OutEncodedLength > 65535U)
		{
			UE_LOGF(LogSerialization, Error, "String to write is invalid or too long.");
			Writer->WriteBits(0, 16);
			return;
		}

		WritePackedUint16(Writer, static_cast<uint16>(OutEncodedLength));
		Writer->WriteBitStream(reinterpret_cast<uint32*>(EncodingBuffer.GetData()), 0U, OutEncodedLength*8U);
	}
	else
	{
		WritePackedUint16(Writer, static_cast<uint16>(StringViewLength));
		if (Writer->IsOverflown())
		{
			return;
		}

		for (TCHAR Char : StringView)
		{
			Writer->WriteBits(static_cast<uint32>(Char) & 0x7FU, 7);
		}
	}
}

void WriteString(FNetBitStreamWriter* Writer, const FString& String)
{
	return WriteString(Writer, FStringView(String));
}

void ReadString(FNetBitStreamReader* Reader, FString& OutString)
{
	using Codec = Private::FStringNetSerializerUtils::TStringCodec<TCHAR>;
	constexpr uint32 SizeAlignment = 4;
	constexpr uint32 StackAllocatedBufferSize = 1024;

	const bool bIsEncoded = Reader->ReadBool();
	const uint32 Length = ReadPackedUint16(Reader);

	if (Reader->IsOverflown())
	{
		return;
	}

	if (bIsEncoded)
	{
		TArray<uint8, TInlineAllocator<StackAllocatedBufferSize>> EncodedBuffer;
		EncodedBuffer.SetNumUninitialized(Align(Length + 1, SizeAlignment));

		// Read data and null-terminate encoded string.
		Reader->ReadBitStream(reinterpret_cast<uint32*>(EncodedBuffer.GetData()), Length*8U);
		EncodedBuffer[Length] = 0;

		if (Reader->IsOverflown())
		{
			return;
		}

		if (!Codec::IsValidEncoding(EncodedBuffer.GetData(), Length))
		{
			UE_LOGF(LogSerialization, Error, "Received invalid encoded string.");
			Reader->DoOverflow();
			return;
		}

		const uint32 SafeDestLength = Codec::GetSafeDecodedBufferLength(Length + 1);
		OutString.GetCharArray().Reserve(SafeDestLength);

		uint32 ConvertedLength = 0;
		if (!Codec::Decode(OutString.GetCharArray().GetData(), SafeDestLength, EncodedBuffer.GetData(), Length + 1, ConvertedLength))
		{
			UE_LOGF(LogSerialization, Error, "Received invalid encoded string.");
			Reader->DoOverflow();
			return;
		}

		OutString.GetCharArray().SetNumUninitialized(ConvertedLength, EAllowShrinking::No);
	}
	else
	{
		OutString.Reset(Length);
		for (uint32 Index = 0; Index < Length; ++Index)
		{
			OutString.AppendChar(static_cast<FString::ElementType>(Reader->ReadBits(7)));
		}

		if (Reader->IsOverflown())
		{
			UE_LOGF(LogSerialization, Error, "Received truncated ASCII string.");
			return;
		}
	}
}

void WriteVector(FNetBitStreamWriter* Writer, const FVector& Vector)
{
	FNetSerializationContext Context(Writer);

	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FVectorNetSerializer);

	alignas(16) uint8 QuantizedState[32] = {};
	checkSlow(sizeof(QuantizedState) >= Serializer.QuantizedTypeSize);

	FNetQuantizeArgs QuantizeArgs;
	QuantizeArgs.Version = Serializer.Version;
	QuantizeArgs.Source = NetSerializerValuePointer(&Vector);
	QuantizeArgs.Target = NetSerializerValuePointer(&QuantizedState[0]);
	QuantizeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
	Serializer.Quantize(Context, QuantizeArgs);

	FNetSerializeArgs SerializeArgs;
	SerializeArgs.Version = Serializer.Version;
	SerializeArgs.Source = QuantizeArgs.Target;
	SerializeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
	Serializer.Serialize(Context, SerializeArgs);
}

void ReadVector(FNetBitStreamReader* Reader, FVector& Vector)
{
	FNetSerializationContext Context(Reader);

	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FVectorNetSerializer);

	alignas(16) uint8 QuantizedState[32] = {};
	checkSlow(sizeof(QuantizedState) >= Serializer.QuantizedTypeSize);

	FNetDeserializeArgs DeserializeArgs;
	DeserializeArgs.Version = Serializer.Version;
	DeserializeArgs.Target = NetSerializerValuePointer(&QuantizedState[0]);
	DeserializeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
	Serializer.Deserialize(Context, DeserializeArgs);

	FNetDequantizeArgs DequantizeArgs;
	DequantizeArgs.Version = Serializer.Version;
	DequantizeArgs.Source = DeserializeArgs.Target;
	DequantizeArgs.Target = NetSerializerValuePointer(&Vector);
	DequantizeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
	Serializer.Dequantize(Context, DequantizeArgs);
}

void WriteVector(FNetBitStreamWriter* Writer, const FVector& Vector, const FVector& DefaultValue, float Epsilon)
{
	FNetSerializationContext Context(Writer);

	if (Writer->WriteBool(!Vector.Equals(DefaultValue, Epsilon)))
	{
		// Write vector
		WriteVector(Writer, Vector);
	}
}

void ReadVector(FNetBitStreamReader* Reader, FVector& OutVector, const FVector& DefaultValue)
{
	if (Reader->ReadBool())
	{
		ReadVector(Reader, OutVector);
	}
	else
	{
		OutVector = DefaultValue;
	}
}

void WriteRotator(FNetBitStreamWriter* Writer, const FRotator& Rotator)
{
	FNetSerializationContext Context(Writer);

	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FRotatorNetSerializer);

	alignas(16) uint8 QuantizedState[16] = {};
	checkSlow(sizeof(QuantizedState) >= Serializer.QuantizedTypeSize);

	FNetQuantizeArgs QuantizeArgs;
	QuantizeArgs.Version = Serializer.Version;
	QuantizeArgs.Source = NetSerializerValuePointer(&Rotator);
	QuantizeArgs.Target = NetSerializerValuePointer(&QuantizedState[0]);
	QuantizeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
	Serializer.Quantize(Context, QuantizeArgs);

	FNetSerializeArgs SerializeArgs;
	SerializeArgs.Version = Serializer.Version;
	SerializeArgs.Source = QuantizeArgs.Target;
	SerializeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
	Serializer.Serialize(Context, SerializeArgs);
}

void ReadRotator(FNetBitStreamReader* Reader, FRotator& Rotator)
{
	FNetSerializationContext Context(Reader);

	const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FRotatorNetSerializer);

	alignas(16) uint8 QuantizedState[16] = {};
	checkSlow(sizeof(QuantizedState) >= Serializer.QuantizedTypeSize);

	FNetDeserializeArgs DeserializeArgs;
	DeserializeArgs.Version = Serializer.Version;
	DeserializeArgs.Target = NetSerializerValuePointer(&QuantizedState[0]);
	DeserializeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
	Serializer.Deserialize(Context, DeserializeArgs);

	FNetDequantizeArgs DequantizeArgs;
	DequantizeArgs.Version = Serializer.Version;
	DequantizeArgs.Source = DeserializeArgs.Target;
	DequantizeArgs.Target = NetSerializerValuePointer(&Rotator);
	DequantizeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
	Serializer.Dequantize(Context, DequantizeArgs);
}

void WriteRotator(FNetBitStreamWriter* Writer, const FRotator& Rotator, const FRotator& DefaultValue, float Epsilon)
{
	FNetSerializationContext Context(Writer);

	if (Writer->WriteBool(!Rotator.Equals(DefaultValue, Epsilon)))
	{
		// Write vector
		WriteRotator(Writer, Rotator);
	}
}

void ReadRotator(FNetBitStreamReader* Reader, FRotator& OutRotator, const FRotator& DefaultValue)
{
	if (Reader->ReadBool())
	{
		ReadRotator(Reader, OutRotator);
	}
	else
	{
		OutRotator = DefaultValue;
	}
}

namespace Private
{

constexpr uint32 SerializeSparseArrayMaxBitCount = 1024U;
constexpr uint32 SerializeSparseArrayMaxWordCount = SerializeSparseArrayMaxBitCount / 32U;

const uint32 SparseUint32UsingIndices_MaxEncodedIndexBits = 3U;
const uint32 SparseUint32UsingIndices_EncodedIndexBitsHeaderSize = GetBitsNeeded(SparseUint32UsingIndices_MaxEncodedIndexBits);

// Note: This functions expects any stray bits to be filtered out by the caller
// if it is exposed directly, masking of stray bits needs to be added. 
// i.e. something like
//if (BitCount < 32U)
//{
//	Value &= ~uint32(0) >> (uint32(-int32(BitCount)) & (31U));
//}
static void WriteSparseUint32UsingByteMask(FNetBitStreamWriter* Writer, uint32 Value, uint32 BitCount = 32U)
{
	checkSlow((Value & ~(~uint32(0) >> (uint32(-int32(BitCount)) & (31U)))) == 0U);

	uint32 Mask = 0U;
	uint32 ValueToWrite = 0U;
	uint32 ValueBitsToWrite = 0U;
	int32 RemainingBits = BitCount;
	uint32 CurrentMaskBit = 1U;
	
	while (RemainingBits > 0)
	{
		if (const uint32 CurrentByte = Value & 0xffU)
		{
			Mask |= CurrentMaskBit;
			ValueToWrite |= CurrentByte << ValueBitsToWrite;
			ValueBitsToWrite += FMath::Min(8U, (uint32)RemainingBits);
		}
		RemainingBits -= 8;
		Value >>= 8U;
		CurrentMaskBit += CurrentMaskBit;
	}

	// Write mask
	const uint32 NumMaskBits = (BitCount + 7U) / 8U;
	Writer->WriteBits(Mask, NumMaskBits);
	// Write data
	Writer->WriteBits(ValueToWrite, ValueBitsToWrite);
}

static uint32 ReadSparseUint32UsingByteMask(FNetBitStreamReader* Reader, uint32 BitCount = 32U)
{
	const uint32 NumMaskBits = (BitCount + 7U) / 8U;
	const uint32 HighestBitMask = 1U << (NumMaskBits - 1U);

	// Read Mask bits
	const uint32 Mask = Reader->ReadBits(NumMaskBits);
	
	// Calculate bitcount for value based on bits set in mask, if the highest bit is set we need to subtract bits if the last byte is not a full byte
	const uint32 ValueBitsToRead = FMath::CountBits(Mask) * 8U - ((Mask & HighestBitMask) ? 8U - BitCount & 7U : 0U);

	// Read Value bits
	uint32 ValueBits = Reader->ReadBits(ValueBitsToRead);

	uint32 Value = 0U;
	uint32 CurrentMaskBit = 1U;
	uint32 CurrentByteOffset = 0U;
	uint32 CurrentValueByteMask = 0xffU;
	
	for (uint32 It = 0U; It < NumMaskBits; ++It)
	{
		if (Mask & CurrentMaskBit)
		{
			Value |= (ValueBits & 0xffU) << CurrentByteOffset;
			ValueBits >>= 8U;			
		}
		
		CurrentByteOffset += 8U;
		CurrentMaskBit += CurrentMaskBit;
	}

	return Value;
}

// Note: This functions expects any stray bits to be filtered out by the caller
// if it is exposed directly, masking of stray bits needs to be added. 
// i.e. something like
//if (BitCount < 32U)
//{
//	Value &= ~uint32(0) >> (uint32(-int32(BitCount)) & (31U));
//}
static void WriteSparseUint32UsingIndices(FNetBitStreamWriter* Writer, uint32 Value, uint32 BitCount = 32U)
{
	// Note: We expect at least 1 bit to be set so zero bits will take an unoptimal path
	checkSlow(BitCount >= 1U);
	checkSlow((Value & ~(~uint32(0) >> (uint32(-int32(BitCount)) & (31U)))) == 0U);

	const uint32 MaxIndexDeltaBits = BitCount - 1U;
	const uint32 NumBitsSet = FMath::CountBits(Value);

	// If the number of dirty bits is lower SparseUint32UsingIndices_MaxEncodedIndexBits we encode the word using indices for the set bits
	if (NumBitsSet > 0U && NumBitsSet <= SparseUint32UsingIndices_MaxEncodedIndexBits)
	{
		Writer->WriteBits(NumBitsSet, SparseUint32UsingIndices_EncodedIndexBitsHeaderSize);
		uint32 LastWrittenBitIndex = 0U;
		while ((Value != 0))
		{
			const uint32 LeastSignificantBit = Value & uint32(-int32(Value));
			Value ^= LeastSignificantBit;

			// We can deltacompress the index against the previous one to save some bits
			const uint32 RequiredBitCount = GetBitsNeeded(MaxIndexDeltaBits - LastWrittenBitIndex);
			const uint32 BitIndexDelta = FPlatformMath::CountTrailingZeros(LeastSignificantBit) - LastWrittenBitIndex;
			LastWrittenBitIndex += BitIndexDelta;

			Writer->WriteBits(BitIndexDelta, RequiredBitCount);
		}		
	}
	else
	{
		// Fallback on masked approach if too many bits are set
		Writer->WriteBits(0U, SparseUint32UsingIndices_EncodedIndexBitsHeaderSize);
		WriteSparseUint32UsingByteMask(Writer, Value, BitCount);
	}
}

static uint32 ReadSparseUint32UsingIndices(FNetBitStreamReader* Reader, uint32 BitCount = 32U)
{
	const uint32 EncodedBitCount = Reader->ReadBits(SparseUint32UsingIndices_EncodedIndexBitsHeaderSize);
	if (EncodedBitCount > 0U)
	{
		const uint32 MaxDeltaBitCount = BitCount - 1U;

		uint32 Value = 0U;
		uint32 LastReadBitIndex = 0U;
		uint32 CurrentBitMask = 1U;

		for (uint32 It = 0U; It < EncodedBitCount; ++It)
		{
			const uint32 RequiredBitCount = GetBitsNeeded(MaxDeltaBitCount - LastReadBitIndex);
			const uint32 BitIndexDelta = Reader->ReadBits(RequiredBitCount);
			LastReadBitIndex += BitIndexDelta;
			CurrentBitMask <<= BitIndexDelta;

			Value |= CurrentBitMask;
		}

		return Value;
	}
	else
	{
		// Fallback on masked approach if too many bits are set
		return ReadSparseUint32UsingByteMask(Reader, BitCount);
	}
}

template<typename GetDataFunc, typename WriteSparseUint32Func>
void WriteSparseBitArray(FNetBitStreamWriter* Writer, const uint32* Data, uint32 BitCount, GetDataFunc&& GetDataFunction, WriteSparseUint32Func&& WriteSparseUint32Function)
{
	ensure(BitCount <= SerializeSparseArrayMaxBitCount);

	using StorageWordType = FNetBitArrayBase::StorageWordType;
	const uint32 WordBitCount = FNetBitArrayBase::WordBitCount;
	const uint32 WordCount = FNetBitArrayView::CalculateRequiredWordCount(BitCount);
	const StorageWordType LastWordMask = ~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1));
	const uint32 LastWordIt = WordCount - 1;

	// Build mask over words that are all zero or all ones.
	uint32 NonZeroWordMask = uint32(0);
	// Mask over words that are all ones.
	uint32 InvertedWordMask = uint32(0);
	{
		for (uint32 WordIt = 0, CurrentBitMask = 1U; WordIt < WordCount; ++WordIt, CurrentBitMask <<= 1U)
		{
			const StorageWordType CurrentWord = Data[WordIt] & ((WordIt == LastWordIt) ? LastWordMask : ~0U);
			// Set bits outside the valid range as ones when inverting the word to have the possibility of the zero word optimization.
			const StorageWordType InvertedWord = ~(CurrentWord | (WordIt == LastWordIt ? ~LastWordMask : 0U));

			NonZeroWordMask |= (CurrentWord == 0) or (InvertedWord == 0) ? 0U : CurrentBitMask;
			InvertedWordMask |=  InvertedWord == 0 ? CurrentBitMask : 0U;
		}
	}

	// Write Mask
	Writer->WriteBits(NonZeroWordMask, WordCount);
	// Write all ones word mask.
	if (Writer->WriteBool(InvertedWordMask != 0))
	{
		Writer->WriteBits(InvertedWordMask, WordCount);
	}

	// Encode dirty words
	{
		// Full words
		uint32 CurrentMaskBit = 1U;
		uint32 WordIt = 0U;
		uint32 RemainingBits = BitCount;

		while (RemainingBits >= 32U)
		{
			if (NonZeroWordMask & CurrentMaskBit)
			{
				WriteSparseUint32Function(Writer, GetDataFunction(Data[WordIt]), 32U);
			}
			++WordIt;
			CurrentMaskBit <<= 1;
			RemainingBits -= 32U;
		}
		// Last word
		if (RemainingBits && (NonZeroWordMask & CurrentMaskBit))
		{
			const StorageWordType CurrentWord = GetDataFunction(Data[WordIt]) & LastWordMask;
			WriteSparseUint32Function(Writer, CurrentWord, RemainingBits);
		}
	}
}

template<typename GetDataFunc, typename ReadSparseUint32Func>
void ReadSparseBitArray(FNetBitStreamReader* Reader, uint32* OutData, uint32 BitCount, GetDataFunc&& GetDataFunction, ReadSparseUint32Func&& ReadSparseUint32Function)
{
	checkSlow(BitCount <= SerializeSparseArrayMaxBitCount);

	using StorageWordType = FNetBitArrayBase::StorageWordType;

	const uint32 WordBitCount = FNetBitArrayBase::WordBitCount;
	const uint32 WordCount = FNetBitArrayView::CalculateRequiredWordCount(BitCount);
	const StorageWordType LastWordMask = ~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1U));
	const uint32 LastWordIt = WordCount - 1U;

	// Read mask
	uint32 NonZeroWordMask = Reader->ReadBits(WordCount);
	uint32 InvertedWordMask = uint32(0);
	if (Reader->ReadBool())
	{
		InvertedWordMask = Reader->ReadBits(WordCount);
	}

	// Read and decode dirty words
	uint32 CurrentMaskBit = 1U;
	uint32 WordIt = 0U;
	uint32 RemainingBits = BitCount;

	while (RemainingBits >= 32U)
	{
		StorageWordType ReadValue = 0U;
		if (NonZeroWordMask & CurrentMaskBit)
		{
			ReadValue = ReadSparseUint32Function(Reader, 32U);
			ReadValue = GetDataFunction(ReadValue);
		}
		else if (InvertedWordMask & CurrentMaskBit)
		{
			ReadValue = ~0U;
		}
		OutData[WordIt] = ReadValue;

		CurrentMaskBit += CurrentMaskBit;
		RemainingBits -= 32U;
		++WordIt;
	}

	// Last word, make sure we do not overwrite existing data
	if (RemainingBits > 0U)
	{
		StorageWordType ReadValue = 0U;
		if (NonZeroWordMask & CurrentMaskBit)
		{
			ReadValue = ReadSparseUint32Function(Reader, RemainingBits);
			ReadValue = GetDataFunction(ReadValue) & LastWordMask;
		}
		else if (InvertedWordMask & CurrentMaskBit)
		{
			ReadValue = LastWordMask;
		}
		OutData[WordIt] = (OutData[WordIt] & ~LastWordMask) | ReadValue;
	}
}

template<typename WriteSparseUint32Func>
void WriteSparseBitArrayDelta(FNetBitStreamWriter* Writer, const uint32* Data, const uint32* OldData, uint32 BitCount, WriteSparseUint32Func&& WriteSparseUint32Function)
{
	checkSlow(BitCount <= SerializeSparseArrayMaxBitCount);

	using StorageWordType = FNetBitArrayBase::StorageWordType;
	const uint32 WordBitCount = FNetBitArrayBase::WordBitCount;
	const uint32 WordCount = FNetBitArrayView::CalculateRequiredWordCount(BitCount);
	const StorageWordType LastWordMask = ~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1));
	const uint32 LastWordIt = WordCount - 1;

	StorageWordType DeltaData[Private::SerializeSparseArrayMaxWordCount];

	// Build mask
	uint32 DirtyWordMask = uint32(0);
	{
		for (uint32 WordIt = 0, CurrentBitMask = 1U; WordIt < WordCount; ++WordIt, CurrentBitMask <<= 1)
		{
			const StorageWordType CurrentWord = (Data[WordIt] ^ OldData[WordIt]) & ((WordIt == LastWordIt) ? LastWordMask : ~0U);
			DirtyWordMask |= CurrentWord ? CurrentBitMask : 0U;
			DeltaData[WordIt] = CurrentWord;
		}
	}

	// Write Mask
	Writer->WriteBits(DirtyWordMask, WordCount);

	// Encode dirty words
	{
		// Full words
		uint32 CurrentMaskBit = 1U;
		uint32 WordIt = 0U;
		uint32 RemainingBits = BitCount;

		while (RemainingBits >= 32U)
		{
			if (DirtyWordMask & CurrentMaskBit)
			{
				WriteSparseUint32Function(Writer, DeltaData[WordIt], 32U);
			}
			++WordIt;
			CurrentMaskBit <<= 1;
			RemainingBits -= 32U;
		}
		// Last word
		if (RemainingBits && (DirtyWordMask & CurrentMaskBit))
		{
			const StorageWordType CurrentWord = DeltaData[WordIt] & LastWordMask;
			WriteSparseUint32Function(Writer, CurrentWord, RemainingBits);
		}
	}
}

template<typename ReadSparseUint32Func>
void ReadSparseBitArrayDelta(FNetBitStreamReader* Reader, uint32* OutData, const uint32* OldData, uint32 BitCount, ReadSparseUint32Func&& ReadSparseUint32Function)
{
	checkSlow(BitCount <= SerializeSparseArrayMaxBitCount);

	using StorageWordType = FNetBitArrayBase::StorageWordType;

	const uint32 WordBitCount = FNetBitArrayBase::WordBitCount;
	const uint32 WordCount = FNetBitArrayView::CalculateRequiredWordCount(BitCount);
	const StorageWordType LastWordMask = ~StorageWordType(0) >> (uint32(-int32(BitCount)) & (WordBitCount - 1U));
	const uint32 LastWordIt = WordCount - 1U;

	// Read mask
	uint32 DirtyWordMask = Reader->ReadBits(WordCount);

	// Read and decode dirty words
	uint32 CurrentMaskBit = 1U;
	uint32 WordIt = 0U;
	uint32 RemainingBits = BitCount;

	while (RemainingBits >= 32U)
	{
		const StorageWordType ReadValue = DirtyWordMask & CurrentMaskBit ? ReadSparseUint32Function(Reader, 32U) : 0U;
		OutData[WordIt] = OldData[WordIt] ^ ReadValue;

		CurrentMaskBit += CurrentMaskBit;
		RemainingBits -= 32U;
		++WordIt;
	}

	// Last word, make sure we do not overwrite existing data
	if (RemainingBits > 0U)
	{
		const StorageWordType ReadValue = DirtyWordMask & CurrentMaskBit ? ReadSparseUint32Function(Reader, RemainingBits) : 0U;
		OutData[WordIt] = (OutData[WordIt] & ~LastWordMask) | ((OldData[WordIt] ^ ReadValue) & LastWordMask);
	}
}

}

void WriteSparseBitArray(FNetBitStreamWriter* Writer, const uint32* Data, uint32 BitCount, ESparseBitArraySerializationHint Hint)
{
	if (Hint == ESparseBitArraySerializationHint::None)
	{
		auto GetDataFunc = [](const uint32 Value) { return Value; };
		// Support large bit arrays
		uint32 RemainingBits = BitCount;
		while (RemainingBits >= Private::SerializeSparseArrayMaxBitCount)
		{
			Private::WriteSparseBitArray(Writer, Data, Private::SerializeSparseArrayMaxBitCount, GetDataFunc, Private::WriteSparseUint32UsingIndices);
			RemainingBits -= Private::SerializeSparseArrayMaxBitCount;
			Data += Private::SerializeSparseArrayMaxWordCount;
		}
		if (RemainingBits)
		{
			Private::WriteSparseBitArray(Writer, Data, RemainingBits, GetDataFunc, Private::WriteSparseUint32UsingIndices);
		}
	}
	else
	{
		auto GetDataFunc = [](const uint32 Value) { return ~Value; };
		// Support large bit arrays
		uint32 RemainingBits = BitCount;
		while (RemainingBits >= Private::SerializeSparseArrayMaxBitCount)
		{
			Private::WriteSparseBitArray(Writer, Data, Private::SerializeSparseArrayMaxBitCount, GetDataFunc, Private::WriteSparseUint32UsingIndices);
			RemainingBits -= Private::SerializeSparseArrayMaxBitCount;
			Data += Private::SerializeSparseArrayMaxWordCount;
		}
		if (RemainingBits)
		{
			Private::WriteSparseBitArray(Writer, Data, RemainingBits, GetDataFunc, Private::WriteSparseUint32UsingIndices);
		}
	}
}

void ReadSparseBitArray(FNetBitStreamReader* Reader, uint32* OutData, uint32 BitCount, ESparseBitArraySerializationHint Hint)
{
	if (Hint == ESparseBitArraySerializationHint::None)
	{
		auto SetDataFunc = [](const uint32 Value) { return Value; };
		// Support large bit arrays
		uint32 RemainingBits = BitCount;
		while (RemainingBits >= Private::SerializeSparseArrayMaxBitCount)
		{
			Private::ReadSparseBitArray(Reader, OutData, Private::SerializeSparseArrayMaxBitCount, SetDataFunc, Private::ReadSparseUint32UsingIndices);
			RemainingBits -= Private::SerializeSparseArrayMaxBitCount;
			OutData += Private::SerializeSparseArrayMaxWordCount;
		}
		if (RemainingBits)
		{
			Private::ReadSparseBitArray(Reader, OutData, RemainingBits, SetDataFunc, Private::ReadSparseUint32UsingIndices);
		}
	}
	else
	{
		auto SetDataFunc = [](const uint32 Value) { return ~Value; };
		// Support large bit arrays
		uint32 RemainingBits = BitCount;
		while (RemainingBits >= Private::SerializeSparseArrayMaxBitCount)
		{
			Private::ReadSparseBitArray(Reader, OutData, Private::SerializeSparseArrayMaxBitCount, SetDataFunc, Private::ReadSparseUint32UsingIndices);
			RemainingBits -= Private::SerializeSparseArrayMaxBitCount;
			OutData += Private::SerializeSparseArrayMaxWordCount;
		}
		if (RemainingBits)
		{
			Private::ReadSparseBitArray(Reader, OutData, RemainingBits, SetDataFunc, Private::ReadSparseUint32UsingIndices);
		}
	}
}

void WriteSparseBitArrayDelta(FNetBitStreamWriter* Writer, const uint32* Data, const uint32* OldData, uint32 BitCount)
{
	// Support large bit arrays
	uint32 RemainingBits = BitCount;
	while (RemainingBits >= Private::SerializeSparseArrayMaxBitCount)
	{
		Private::WriteSparseBitArrayDelta(Writer, Data, OldData, Private::SerializeSparseArrayMaxBitCount, Private::WriteSparseUint32UsingIndices);
		RemainingBits -= Private::SerializeSparseArrayMaxBitCount;
		Data += Private::SerializeSparseArrayMaxWordCount;
		OldData += Private::SerializeSparseArrayMaxWordCount;
	}
	if (RemainingBits)
	{
		Private::WriteSparseBitArrayDelta(Writer, Data, OldData, RemainingBits, Private::WriteSparseUint32UsingIndices);
	}
}

void ReadSparseBitArrayDelta(FNetBitStreamReader* Reader, uint32* OutData, const uint32* OldData, uint32 BitCount)
{
	// Support large bit arrays
	uint32 RemainingBits = BitCount;
	while (RemainingBits >= Private::SerializeSparseArrayMaxBitCount)
	{
		Private::ReadSparseBitArrayDelta(Reader, OutData, OldData, Private::SerializeSparseArrayMaxBitCount, Private::ReadSparseUint32UsingIndices);
		RemainingBits -= Private::SerializeSparseArrayMaxBitCount;
		OutData += Private::SerializeSparseArrayMaxWordCount;
		OldData += Private::SerializeSparseArrayMaxWordCount;
	}
	if (RemainingBits)
	{
		Private::ReadSparseBitArrayDelta(Reader, OutData, OldData, RemainingBits, Private::ReadSparseUint32UsingIndices);
	}
}

const uint32 NetBitStreamSentinelValue = 0xBAADDEADU;
void WriteSentinelBits(FNetBitStreamWriter* Writer, uint32 BitCount)
{
	Writer->WriteBits(NetBitStreamSentinelValue, BitCount);
}

bool ReadAndVerifySentinelBits(FNetBitStreamReader* Reader, const TCHAR* ErrorString, uint32 BitCount)
{
	const uint32 ReadValue = Reader->ReadBits(BitCount);
	const uint32 CompareMask = ~0U >> (32U - BitCount);
	const uint32 ExpectedValue = (NetBitStreamSentinelValue & CompareMask);
	return ensureAlwaysMsgf(!Reader->IsOverflown() &&  ExpectedValue == ReadValue, TEXT("ReadAndVerifySentinelBits %s failed OverFlow %u Got 0x%u != 0x%u"), ErrorString, Reader->IsOverflown() ? 1U : 0U, ReadValue, ExpectedValue);
}

void ReadBytes(FNetBitStreamReader* Reader, uint8* Destination, uint32 BytesToRead)
{
	if (BytesToRead == 0)
	{
		return;
	}

	uint8* DstData = Destination;
	// Fix up dst alignment
	while (((uintptr_t)(DstData) & 3) && BytesToRead)
	{
		*(DstData++) = static_cast<uint8>(Reader->ReadBits(8U));
		--BytesToRead;
	}
	// Write full words
	const uint32 FullWordsToRead = BytesToRead / 4U;
	if (FullWordsToRead)
	{
		Reader->ReadBitStream(reinterpret_cast<uint32*>(DstData), FullWordsToRead * 32U);
		DstData += FullWordsToRead * 4U;
		BytesToRead -= FullWordsToRead * 4U;
	}
	while (BytesToRead)
	{
		*(DstData++) = static_cast<uint8>(Reader->ReadBits(8U));
		--BytesToRead;
	}
}

void WriteBytes(FNetBitStreamWriter* Writer, const uint8* Src, uint32 BytesToWrite)
{
	if (BytesToWrite == 0)
	{
		return;
	}

	const uint8* SrcData = Src;
	// Fix up src alignment
	while (((uintptr_t)(SrcData) & 3) && BytesToWrite)
	{
		Writer->WriteBits(*(SrcData++), 8U);
		--BytesToWrite;
	}
	// Write full words
	const uint32 FullWordsToWrite = BytesToWrite / 4U;
	if (FullWordsToWrite)
	{
		Writer->WriteBitStream(reinterpret_cast<const uint32*>(SrcData), 0U, FullWordsToWrite * 32);
		SrcData += FullWordsToWrite * 4U;
		BytesToWrite -= FullWordsToWrite * 4U;
	}
	while (BytesToWrite)
	{
		Writer->WriteBits(*(SrcData++), 8U);
		--BytesToWrite;
	}
}

void WriteConditionallyQuantizedVector(UE::Net::FNetBitStreamWriter* Writer, const FVector& Vector, const FVector& DefaultValue, bool bQuantize)
{
	using namespace UE::Net::Private;

	// We use 0.01f for comparing when using quantization, because we will only send a single point of precision anyway.
	// We could probably get away with 0.1f, but that may introduce edge cases for rounding.
	static constexpr float Epsilon_Quantized = 0.01f;
				
	// We use KINDA_SMALL_NUMBER for comparing when not using quantization, because that's the default for FVector::Equals.
	const float Epsilon = bQuantize ? Epsilon_Quantized : UE_KINDA_SMALL_NUMBER;
	
	if (Writer->WriteBool(!Vector.Equals(DefaultValue, Epsilon)))
	{
		Writer->WriteBool(bQuantize);

		const FNetSerializer& Serializer = bQuantize ? UE_NET_GET_SERIALIZER(FVectorNetQuantize10NetSerializer) : UE_NET_GET_SERIALIZER(FVectorNetSerializer);

		FNetSerializationContext Context(Writer);

		alignas(16) uint8 QuantizedState[32] = {};
		checkSlow(sizeof(QuantizedState) >= Serializer.QuantizedTypeSize);

		FNetQuantizeArgs QuantizeArgs;
		QuantizeArgs.Version = Serializer.Version;
		QuantizeArgs.Source = NetSerializerValuePointer(&Vector);
		QuantizeArgs.Target = NetSerializerValuePointer(&QuantizedState[0]);
		QuantizeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
		Serializer.Quantize(Context, QuantizeArgs);

		FNetSerializeArgs SerializeArgs;
		SerializeArgs.Version = Serializer.Version;
		SerializeArgs.Source = QuantizeArgs.Target;
		SerializeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
		Serializer.Serialize(Context, SerializeArgs);
	}
}

void ReadConditionallyQuantizedVector(UE::Net::FNetBitStreamReader* Reader, FVector& OutVector, const FVector& DefaultValue)
{
	using namespace UE::Net::Private;

	if (Reader->ReadBool())
	{
		const bool bIsQuantized = Reader->ReadBool();

		const FNetSerializer& Serializer = bIsQuantized ? UE_NET_GET_SERIALIZER(FVectorNetQuantize10NetSerializer) : UE_NET_GET_SERIALIZER(FVectorNetSerializer);

		FNetSerializationContext Context(Reader);

		alignas(16) uint8 QuantizedState[32] = {};
		checkSlow(sizeof(QuantizedState) >= Serializer.QuantizedTypeSize);

		FNetDeserializeArgs DeserializeArgs;
		DeserializeArgs.Version = Serializer.Version;
		DeserializeArgs.Target = NetSerializerValuePointer(&QuantizedState[0]);
		DeserializeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
		Serializer.Deserialize(Context, DeserializeArgs);

		FNetDequantizeArgs DequantizeArgs;
		DequantizeArgs.Version = Serializer.Version;
		DequantizeArgs.Source = DeserializeArgs.Target;
		DequantizeArgs.Target = NetSerializerValuePointer(&OutVector);
		DequantizeArgs.NetSerializerConfig = NetSerializerConfigParam(Serializer.DefaultConfig);
		Serializer.Dequantize(Context, DequantizeArgs);
	}
	else
	{
		OutVector = DefaultValue;
	}
}

} // end namespace UE::Net
