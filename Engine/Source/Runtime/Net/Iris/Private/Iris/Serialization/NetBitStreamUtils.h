// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include <stdlib.h>
#include "Misc/ByteSwap.h"

namespace UE::Net::BitStreamUtils
{

// BitCount is in range [1, 31]
inline uint32 GetBits(const uint32* Src, uint32 SrcBit, uint32 BitCount)
{
	const uint32 ShiftAmount0 = SrcBit & 31U;
	 // Only masking with 31 to avoid undefined behavior. Otherwise 32U - ShiftAmount0 could have been used because Word1 would be masked away anyway.
	const uint32 ShiftAmount1 = (32U - SrcBit) & 31U;

	const uint32 WordOffset0 = SrcBit >> 5U;
	const uint32 WordOffset1 = (SrcBit + BitCount - 1) >> 5U;

	const uint32 Word0 = INTEL_ORDER32(Src[WordOffset0]) >> ShiftAmount0;
	const uint32 Word1 = INTEL_ORDER32(Src[WordOffset1]) << ShiftAmount1;

	// WordOffset1 is either WordOffset0 or WordOffset0+1. By subtracting WordOffset0 from WordOffset1
	// the desired mask 0xFFFFFFFF is produced if the offsets differ and 0 if they are the same.
	const uint32 Word1Mask = WordOffset0 - WordOffset1;
	const uint32 WordMask = (1U << BitCount) - 1U;

	const uint32 Word = ((Word1 & Word1Mask) | Word0) & WordMask;
	return Word;
}

}
