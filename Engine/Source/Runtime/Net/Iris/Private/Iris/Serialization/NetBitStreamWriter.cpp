// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamUtils.h"
#include "Iris/IrisConfigInternal.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformMemory.h"
#include "Misc/AssertionMacros.h"
#include "Templates/AlignmentTemplates.h"

#if UE_NETBITSTREAMWRITER_VALIDATE
#	define UE_NETBITSTREAMWRITER_CHECK(expr) check(expr)
#	define UE_NETBITSTREAMWRITER_CHECKF(expr, format, ...) checkf(expr, format, ##__VA_ARGS__)
#else
#	define UE_NETBITSTREAMWRITER_CHECK(...) 
#	define UE_NETBITSTREAMWRITER_CHECKF(...) 
#endif

namespace UE::Net
{

FNetBitStreamWriter::FNetBitStreamWriter()
: Buffer(nullptr)
, BufferBitCapacity(0)
, BufferBitStartOffset(0)
, BufferBitPosition(0)
, PendingWord(0)
, OverflowBitCount(0)
, bHasSubstream(0)
, bIsSubstream(0)
, bIsInvalid(0)
{
}

FNetBitStreamWriter::~FNetBitStreamWriter()
{
	UE_NETBITSTREAMWRITER_CHECKF(!bHasSubstream, TEXT("%s"), TEXT("FNetBitStreamWriter is destroyed with active substream. Substreams must be commited or discarded."));
	CommitWrites();
}

void FNetBitStreamWriter::InitBytes(void* InBuffer, uint32 ByteCount)
{
	UE_NETBITSTREAMWRITER_CHECK(InBuffer != nullptr);
	UE_NETBITSTREAMWRITER_CHECKF((uintptr_t(InBuffer) & 3) == 0, TEXT("Buffer needs to be 4-byte aligned."));
	UE_NETBITSTREAMWRITER_CHECKF((ByteCount & 3) == 0, TEXT("Buffer capacity needs to be a multiple of 4."));
	UE_NETBITSTREAMWRITER_CHECK(!bHasSubstream && !bIsSubstream);

	Buffer = static_cast<uint32*>(InBuffer);
	BufferBitCapacity = ByteCount*8U;
	BufferBitPosition = 0U;
	OverflowBitCount = 0U;

	if (ByteCount >= 4)
	{
		PendingWord = INTEL_ORDER32(Buffer[0]);
	}
}

void FNetBitStreamWriter::WriteBits(uint32 Value, uint32 BitCount)
{
	UE_NETBITSTREAMWRITER_CHECK((!bIsInvalid) & (!bHasSubstream));

	if (OverflowBitCount != 0)
	{
		return;
	}

	if (BufferBitCapacity - BufferBitPosition < BitCount)
	{
		OverflowBitCount = BitCount - (BufferBitCapacity - BufferBitPosition);
		return;
	}

	const uint32 CurrentBufferBitPosition = BufferBitPosition;
	const uint32 BitCountUsedInWord = BufferBitPosition & 31;
	const uint32 BitCountLeftInWord = 32 - (BufferBitPosition & 31);

	BufferBitPosition += BitCount;

	// If after the write we still have unused bits in the PendingWord we can skip storing and loading a new word.
	if (BitCountLeftInWord > BitCount)
	{
		const uint32 ValueMask = ((1U << BitCount) - 1U) << BitCountUsedInWord;
		PendingWord = ((Value << BitCountUsedInWord) & ValueMask) | (PendingWord & ~ValueMask);
	}
	else
	{
		// Both BitCountLeftInWord and BitCount is in range [1, 32]
		// BitCountUsedInWord is in range [0, 31]

		// We know that we're going to fill an entire word to start with. We can use that
		// fact to avoid unnecessary masking of the input Value.
		const uint32 PendingWordMask = (1U << BitCountUsedInWord) - 1U;
		const uint32 FirstWord = (Value << BitCountUsedInWord) | (PendingWord & PendingWordMask);
		Buffer[CurrentBufferBitPosition >> 5U] = INTEL_ORDER32(FirstWord);

		// If we're at the end of the buffer we cannot load a new word as that can cause
		// a read access violation. We also know that we've written everything we should have
		// as we check for overflow very early in this function.
		// Substreams may have a BufferBitCapacity which isn't evenly divisible by 32.
		// It's ok, and necessary, for such a substream to read up to the rounded up capacity.
		if (BufferBitPosition < Align(BufferBitCapacity, 32U))
		{
			// BitCountToWrite will be in range [0, 31] as we've already written at least one bit at this point
			const uint32 BitCountToWrite = BitCount - BitCountLeftInWord;
			const uint32 ValueMask = (1U << BitCountToWrite) - 1U; // Zero if BitCountToWrite == 0

			const uint32 SecondWord = INTEL_ORDER32(Buffer[BufferBitPosition >> 5U]);
			PendingWord = (SecondWord & ~ValueMask) | ((Value >> (BitCountLeftInWord & 31U)) & ValueMask);
		}
	}
}

void FNetBitStreamWriter::WriteBitStream(const uint32* InSrc, uint32 SrcBitOffset, uint32 BitCount)
{
	UE_NETBITSTREAMWRITER_CHECK((!bIsInvalid) & (!bHasSubstream));

	if ((OverflowBitCount != 0) || (BitCount == 0))
	{
		return;
	}

	if (BufferBitCapacity - BufferBitPosition < BitCount)
	{
		OverflowBitCount = BitCount - (BufferBitCapacity - BufferBitPosition);
		return;
	}

	const uint32* RESTRICT Src = InSrc;
	uint32 CurSrcBit = SrcBitOffset;
	uint32* RESTRICT Dst = Buffer;
	uint32 CurDstBit = BufferBitPosition;
	uint32 BitCountToCopy = BitCount;

	// We can adjust the final bit position here as we're only using CurDstBit from here on
	BufferBitPosition += BitCount;

	// Fill pending word so we can store a word at a time in the main loop
	if (const uint32 BitCountUsedInWord = (CurDstBit & 31U))
	{
		const uint32 BitCountToWrite = FPlatformMath::Min(32U - BitCountUsedInWord, BitCount);
		const uint32 SrcWord = BitStreamUtils::GetBits(Src, CurSrcBit, BitCountToWrite);

		const uint32 Mask = ((1U << BitCountToWrite) - 1U) << BitCountUsedInWord;
		const uint32 StoreWord = (SrcWord << BitCountUsedInWord) | (PendingWord & ~Mask);
		Dst[CurDstBit >> 5U] = INTEL_ORDER32(StoreWord);

		CurSrcBit += BitCountToWrite;
		CurDstBit += BitCountToWrite;
		BitCountToCopy -= BitCountToWrite;
	}

	// Copy full words
	if (BitCountToCopy >= 32U)
	{
		// Fast path for byte aligned source buffer.
		if ((CurSrcBit & 7) == 0)
		{
			const size_t WordCountToCopy = BitCountToCopy >> 5U;
			FPlatformMemory::Memcpy(Dst + (CurDstBit >> 5U), reinterpret_cast<const uint8*>(Src) + (CurSrcBit >> 3U), WordCountToCopy*sizeof(uint32));
		}
		else
		{
			// We know that each 32 bit copy straddles two words from Src as CurSrcBit % 32 != 0, 
			// else the fast path above would be used. Also note that DstSrcBit % 32 == 0
			// which allows us to perform a single store in each loop iteration.
			const uint32 PrevWordShift = CurSrcBit & 31U;
			const uint32 NextWordShift = (32U - CurSrcBit) & 31U;

			// Set up initial Word so we can do a single read in each loop iteration.
			uint32 SrcWordOffset = CurSrcBit >> 5U;
			uint32 PrevWord = INTEL_ORDER32(Src[SrcWordOffset]);
			++SrcWordOffset;
			uint32 DstWordOffset = (CurDstBit >> 5U);
			for (uint32 WordIt = 0, WordEndIt = (BitCountToCopy >> 5U); WordIt != WordEndIt; ++WordIt, ++SrcWordOffset, ++DstWordOffset)
			{
				const uint32 NextWord = INTEL_ORDER32(Src[SrcWordOffset]);
				const uint32 Word = (NextWord << NextWordShift) | (PrevWord >> PrevWordShift);
				Dst[DstWordOffset] = INTEL_ORDER32(Word);
				PrevWord = NextWord;
			}
		}

		const uint32 BitCountCopied = (BitCountToCopy & ~31U);
		CurSrcBit += BitCountCopied;
		CurDstBit += BitCountCopied;
		BitCountToCopy &= 31U;
	}

	if (BitCountToCopy)
	{
		// Bear in mind that we've already made sure that CurDstBit % 32 == 0 if we're entering this path.
		const uint32 Word = INTEL_ORDER32(Dst[CurDstBit >> 5U]);
		const uint32 SrcWord = BitStreamUtils::GetBits(Src, CurSrcBit, BitCountToCopy);
		const uint32 SrcMask = (1U << BitCountToCopy) - 1U;
		PendingWord = (Word & ~SrcMask) | (SrcWord & SrcMask);
	}
	else
	{
		if (CurDstBit < Align(BufferBitCapacity, 32U))
		{
			PendingWord = INTEL_ORDER32(Dst[CurDstBit >> 5U]);
		}
	}
}

void FNetBitStreamWriter::CommitWrites()
{
	if ((!bIsInvalid) & (BufferBitPosition < Align(BufferBitCapacity, 32)))
	{
		Buffer[BufferBitPosition >> 5U] = INTEL_ORDER32(PendingWord);
	}
}

void FNetBitStreamWriter::Seek(uint32 BitPosition)
{
	UE_NETBITSTREAMWRITER_CHECK((!bIsInvalid) & (!bHasSubstream));

	const uint32 AdjustedBitPosition = BitPosition + BufferBitStartOffset;
	// We handle uint32 overflow as well which makes this code a bit more complicated. The OverflowBitCount may not always end up correct, but will be at least 1.
	if ((AdjustedBitPosition > BufferBitCapacity) | (AdjustedBitPosition < BitPosition))
	{
		OverflowBitCount = FPlatformMath::Max(AdjustedBitPosition, BufferBitCapacity + 1U) - BufferBitCapacity;
		return;
	}

	OverflowBitCount = 0;

	const uint32 AlignedBufferBitCapacity = Align(BufferBitCapacity, 32U);
	// Commit data in PendingWord
	if (BufferBitPosition < AlignedBufferBitCapacity)
	{
		Buffer[BufferBitPosition >> 5U] = INTEL_ORDER32(PendingWord);
	}

	// Populate PendingWord with new data from new position
	BufferBitPosition = AdjustedBitPosition;
	if (BufferBitPosition < AlignedBufferBitCapacity)
	{
		PendingWord = INTEL_ORDER32(Buffer[BufferBitPosition >> 5U]);
	}
}

void FNetBitStreamWriter::DoOverflow()
{
	if (OverflowBitCount == 0)
	{
		Seek(BufferBitCapacity + 1);
	}
}


FNetBitStreamWriter FNetBitStreamWriter::CreateSubstream(uint32 MaxBitCount)
{
	UE_NETBITSTREAMWRITER_CHECK((!bIsInvalid) & (!bHasSubstream));

	// Create a copy of this stream and overwrite the necessary members.
	FNetBitStreamWriter Substream = *this;
	Substream.BufferBitStartOffset = BufferBitPosition;
	Substream.bHasSubstream = 0;
	Substream.bIsSubstream = 1;

	bHasSubstream = 1;

	/* If this stream is overflown make sure the substream will always be overflown as well!
	 * We must be careful to ensure that a seek to the beginning of this stream will still cause the substream to be overflown.
	 * We can ignore MaxBitCount completely because no writes will succeed anyway.
	 */
	if (OverflowBitCount)
	{
		Substream.BufferBitCapacity = Substream.BufferBitStartOffset;
		// It's not vital that the OverflowBitCount is set as the user can reset it with a Seek(0) call. In any case no modifications to the bitstream can be done.
		Substream.OverflowBitCount = OverflowBitCount;
	}
	else
	{
		Substream.BufferBitCapacity = BufferBitPosition + FPlatformMath::Min(MaxBitCount, BufferBitCapacity - BufferBitPosition);
	}

	return Substream;
}

void FNetBitStreamWriter::CommitSubstream(FNetBitStreamWriter& Substream)
{
	/* Only accept substreams iff this is the parent and the substream has not overflown
	 * and has not previously been commited or discarded.
	 */
	if (!ensure(bHasSubstream & (!Substream.bHasSubstream) & (!bIsInvalid) & (!Substream.bIsInvalid) & (Buffer == Substream.Buffer) & (BufferBitPosition == Substream.BufferBitStartOffset)))
	{
		return;
	}

	if (!Substream.IsOverflown())
	{
		Substream.CommitWrites();
		BufferBitPosition = Substream.BufferBitPosition;
		if (Substream.BufferBitPosition < Align(BufferBitCapacity, 32U))
		{
			PendingWord = INTEL_ORDER32(Buffer[Substream.BufferBitPosition >> 5U]);
		}
	}

	bHasSubstream = 0;
	Substream.bIsInvalid = 1;
}

void FNetBitStreamWriter::DiscardSubstream(FNetBitStreamWriter& Substream)
{
	/* Only accept substreams iff this is the parent and the substream has not overflown
	 * and has not previously been commited or discarded. The substream may not have active substreams.
	 */
	if (!ensure(bHasSubstream & (!Substream.bHasSubstream) & (!bIsInvalid) & (!Substream.bIsInvalid) & (Buffer == Substream.Buffer) & (BufferBitPosition == Substream.BufferBitStartOffset)))
	{
		return;
	}

	bHasSubstream = 0;
	Substream.bIsInvalid = 1;
}

}

#undef UE_NETBITSTREAMWRITER_CHECK
#undef UE_NETBITSTREAMWRITER_CHECKF
