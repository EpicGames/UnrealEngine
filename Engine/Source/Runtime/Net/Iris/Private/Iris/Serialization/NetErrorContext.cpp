// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetErrorContext.h"

namespace UE::Net
{

const FName GNetError_BitStreamOverflow("BitStream overflow");
const FName GNetError_BitStreamError("BitStream error");
const FName GNetError_ArraySizeTooLarge("Array size is too large");
const FName GNetError_InvalidNetHandle("Invalid NetHandle");
const FName GNetError_BrokenNetHandle("Broken NetHandle");
const FName GNetError_InvalidValue("Invalid value");
const FName GNetError_InternalError("Internal error");
const FName GNetError_InvalidDataStream("Invalid DataStream");

void FNetErrorContext::SetError(const FName InError)
{
	ensureMsgf(!InError.IsNone(), TEXT("Clearing an error is not allowed. Error '%s' will remain."), ToCStr(Error.ToString()));
	if (HasError())
	{
		return;
	}

	Error = InError;
}

}
