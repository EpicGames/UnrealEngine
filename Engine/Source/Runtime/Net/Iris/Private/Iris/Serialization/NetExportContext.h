// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Net/Core/NetToken/NetToken.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Core/NetObjectReference.h"

namespace UE::Net::Private
{

class FNetExportContext
{
public:
	typedef TArray<FNetRefHandle, TInlineAllocator<32>> FExportsArray;
	typedef TArray<FNetToken, TInlineAllocator<32>> FNetTokenExportsArray;
	typedef TArray<FNetObjectReference, TInlineAllocator<32>> FPendingExportArray;
	typedef TArray<FNetToken, TInlineAllocator<32>> FNetTokenPendingExportArray;

	struct FAcknowledgedExports
	{
		TSet<FNetRefHandle> AcknowledgedExportedHandles;
		TSet<FNetToken> AcknowledgedExportedNetTokens;
	};

	struct FBatchExports
	{
		void Reset()
		{
			HandlesExportedInCurrentBatch.Empty();
			NetTokensExportedInCurrentBatch.Empty();
			ReferencesPendingExportInCurrentBatch.Empty();
			NetTokensPendingExportInCurrentBatch.Empty();
		}

		bool HasPendingExports() const { return (ReferencesPendingExportInCurrentBatch.Num() > 0) || (NetTokensPendingExportInCurrentBatch.Num() > 0); }

		// Exports in the current batch
		FExportsArray HandlesExportedInCurrentBatch;
		FNetTokenExportsArray NetTokensExportedInCurrentBatch;
		FPendingExportArray ReferencesPendingExportInCurrentBatch;
		FNetTokenPendingExportArray NetTokensPendingExportInCurrentBatch;
	};

public:

	FNetExportContext(const FAcknowledgedExports& InAcknowledgedExports, FBatchExports& BatchExports);

	// Returns true if the Handle is acknowledged as delivered or if it is exported in the current batch
	bool IsExported(FNetRefHandle Handle) const;

	// Returns true if the Handle is acknowledged as delivered or if it is exported in the current batch
	bool IsExported(FNetToken Token) const;

	// Add a Handle to the current export batch
	void AddExported(FNetRefHandle Handle);

	// Add a Handle to the current export batch
	void AddExported(FNetToken Token);

	// Add a reference to the current pending exports arr.
	void AddPendingExport(const FNetObjectReference& Ref);

	// Add a NetToken to the current pending export array
	void AddPendingExport(FNetToken Token);

	// Add NetTokens to the current pending export array
	void AddPendingExports(TArrayView<const FNetToken> NetTokens);

	// Returns true if the Reference is in PendingExports array
	bool IsPendingExport(const FNetObjectReference& Ref) const;

	// Clear the list of pending exports
	void ClearPendingExports()
	{ 
		BatchExports.ReferencesPendingExportInCurrentBatch.Empty(); 
		BatchExports.NetTokensPendingExportInCurrentBatch.Empty();
	}

	// Get current batch exports
	const FNetExportContext::FBatchExports& GetBatchExports() const { return BatchExports; }

private:
	friend class FNetExportRollbackScope;

	// Acknowledged exports
	const FAcknowledgedExports& AcknowledgedExports;

	// Exports for the current batch which we can treat as exported within the batch
	FBatchExports& BatchExports;
};

// Rollback scope to be able to rollback exports with bitstream
class FNetExportRollbackScope
{
public:
	explicit FNetExportRollbackScope(FNetSerializationContext& InContext);
	~FNetExportRollbackScope();

	void Rollback();

private:
	FNetSerializationContext& Context;
	int32 StartNumNetHandleExports;
	int32 StartNumNetTokenExports;
	int32 StartNumPendingExports;
	int32 StartNumNetTokensPendingExports;
};

inline FNetExportRollbackScope::FNetExportRollbackScope(FNetSerializationContext& InContext)
: Context(InContext)
{
	const FNetExportContext* ExportContext = Context.GetExportContext();

	StartNumNetHandleExports = ExportContext ? ExportContext->BatchExports.HandlesExportedInCurrentBatch.Num() : 0;
	StartNumNetTokenExports = ExportContext ? ExportContext->BatchExports.NetTokensExportedInCurrentBatch.Num() : 0;
	StartNumPendingExports = ExportContext ? ExportContext->BatchExports.ReferencesPendingExportInCurrentBatch.Num() : 0;
	StartNumNetTokensPendingExports = ExportContext ? ExportContext->BatchExports.NetTokensPendingExportInCurrentBatch.Num() : 0;
}

inline void FNetExportRollbackScope::Rollback()
{
	if (const FNetExportContext* ExportContext = Context.GetExportContext())
	{ 
		ExportContext->BatchExports.HandlesExportedInCurrentBatch.SetNum(StartNumNetHandleExports);
		ExportContext->BatchExports.NetTokensExportedInCurrentBatch.SetNum(StartNumNetTokenExports);
		ExportContext->BatchExports.NetTokensPendingExportInCurrentBatch.SetNum(StartNumNetTokensPendingExports);
		ExportContext->BatchExports.ReferencesPendingExportInCurrentBatch.SetNum(StartNumPendingExports);
	}
}

inline FNetExportRollbackScope::~FNetExportRollbackScope()
{
	// Trigger rollback if we have encountered an error
	if (Context.HasErrorOrOverflow())
	{
		Rollback();
	}
}

inline FNetExportContext::FNetExportContext(const FAcknowledgedExports& InAcknowledgedExports, FBatchExports& InBatchExports)
	: AcknowledgedExports(InAcknowledgedExports)
	, BatchExports(InBatchExports)
{
}

inline bool FNetExportContext::IsExported(FNetRefHandle Handle) const
{
	return AcknowledgedExports.AcknowledgedExportedHandles.Contains(Handle) || (BatchExports.HandlesExportedInCurrentBatch.Find(Handle) != INDEX_NONE);
}

inline void FNetExportContext::AddExported(FNetRefHandle Handle)
{
	BatchExports.HandlesExportedInCurrentBatch.Add(Handle);
}

inline bool FNetExportContext::IsExported(FNetToken Token) const
{
	return AcknowledgedExports.AcknowledgedExportedNetTokens.Contains(Token) || (BatchExports.NetTokensExportedInCurrentBatch.Find(Token) != INDEX_NONE);
}

inline void FNetExportContext::AddExported(FNetToken Token)
{
	BatchExports.NetTokensExportedInCurrentBatch.Add(Token);
}

inline void FNetExportContext::AddPendingExport(const FNetObjectReference& Ref)
{
	BatchExports.ReferencesPendingExportInCurrentBatch.AddUnique(Ref);
}

inline void FNetExportContext::AddPendingExport(FNetToken NetToken)
{
	BatchExports.NetTokensPendingExportInCurrentBatch.AddUnique(NetToken);
}

inline void FNetExportContext::AddPendingExports(TArrayView<const FNetToken> NetTokens)
{
	for (const FNetToken& NetToken : NetTokens)
	{
		AddPendingExport(NetToken);
	}
}

inline bool FNetExportContext::IsPendingExport(const FNetObjectReference& Ref) const
{
	return BatchExports.ReferencesPendingExportInCurrentBatch.Contains(Ref);
}


}
