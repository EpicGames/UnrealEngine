// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetJournal.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"

namespace UE::Net
{

FString FNetJournal::Print(const UReplicationSystem* ReplicationSystem) const
{
	using namespace UE::Net::Private;

	const FNetRefHandleManager* NetRefHandleManager = ReplicationSystem ? &ReplicationSystem->GetReplicationSystemInternal()->GetNetRefHandleManager() : nullptr;

	FString Result;
	const uint32 StoredCount = FMath::Min(NumEntries, JournalSize);
	const uint32 StartIndex = (NumEntries - StoredCount);
	Result.Appendf(TEXT("ErrorContext:\n"));

	FNetRefHandle LastNetRefHandle;
	for (uint32 EntryIt = 0U; EntryIt < StoredCount; ++EntryIt)
	{
		const FJournalEntry& Entry = Entries[(StartIndex + EntryIt) & JournalMask];
		if (LastNetRefHandle != Entry.NetRefHandle)
		{
			Result.Appendf(TEXT("%s\n"), NetRefHandleManager ? *NetRefHandleManager->PrintObjectFromNetRefHandle(Entry.NetRefHandle) : *Entry.NetRefHandle.ToString());
			LastNetRefHandle = Entry.NetRefHandle;
		}
		Result.Appendf(TEXT("%u: - BitOffset: %u:%s\n"), EntryIt, Entry.BitOffset, Entry.Name);		
	}
	return Result;
}

}
