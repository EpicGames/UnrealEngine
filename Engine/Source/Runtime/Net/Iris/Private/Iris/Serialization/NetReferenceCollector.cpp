// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetReferenceCollector.h"

#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Iris/Serialization/QuantizedObjectReference.h"

namespace UE::Net
{

void FNetReferenceCollector::Add(const FNetReferenceInfo& ReferenceInfo, const FQuantizedObjectReference& Reference, const FNetSerializerChangeMaskParam& ChangeMaskInfo)
{
	if (Reference.IsRemoteReference())
	{
		return;
	}

	const FNetObjectReference& NetReference = Reference.NetReference;

	if (!NetReference.IsValid() && !EnumHasAnyFlags(Traits, ENetReferenceCollectorTraits::IncludeInvalidReferences))
	{
		return;
	}

	if (!NetReference.CanBeExported() && EnumHasAnyFlags(Traits, ENetReferenceCollectorTraits::OnlyCollectReferencesThatCanBeExported))
	{
		return;
	}

	FReferenceInfo RefInfo;
	RefInfo.Info = ReferenceInfo;
	RefInfo.Reference = NetReference;
	RefInfo.ChangeMaskInfo = ChangeMaskInfo;

	ReferenceInfos.Add(MoveTemp(RefInfo));
}

}
