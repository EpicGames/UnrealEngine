// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternalNetSerializers.h"
#include "Iris/Serialization/EnumNetSerializers.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/IntRangeNetSerializerUtils.h"
#include "Iris/Serialization/InternalEnumNetSerializers.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"

namespace UE::Net
{

struct FNetRoleNetSerializer
{
	static const uint32 Version = 0;
	// Can't use same value optimization due to role downgrading.
	static constexpr bool bUseDefaultDelta = false;

	typedef uint8 SourceType;
	typedef uint8 QuantizedType;
	typedef FNetRoleNetSerializerConfig ConfigType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static FNetRoleNetSerializerConfig CachedConfig;
};

FNetRoleNetSerializerConfig FNetRoleNetSerializer::CachedConfig;

UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FNetRoleNetSerializer);


void FNetRoleNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const SourceType Value = *reinterpret_cast<const SourceType*>(Args.Source);
	if (Context.GetInternalContext()->bAlwaysSwapRolesOnReplication)
	{
		FNetSerializeArgs SwapArgs = Args;

		SourceType* OtherRolePtr = reinterpret_cast<SourceType*>(Args.Source + Config->RelativeInternalOffsetToOtherRole);
		const SourceType OtherRoleValue = *OtherRolePtr;

		if (Context.GetInternalContext()->bDowngradeAutonomousProxyRole & (OtherRoleValue == Config->AutonomousProxyValue))
		{
			*OtherRolePtr = Config->SimulatedProxyValue;
		}

		SwapArgs.Source = NetSerializerValuePointer(OtherRolePtr);
		Private::FIntRangeNetSerializerBase<SourceType, ConfigType>::Serialize(Context, SwapArgs);

		*OtherRolePtr = OtherRoleValue;
	}
	else if (Context.GetInternalContext()->bDowngradeAutonomousProxyRole & (Value == Config->AutonomousProxyValue))
	{
		FNetSerializeArgs DowngradedRoleArgs = Args;
		DowngradedRoleArgs.Source = NetSerializerValuePointer(&Config->SimulatedProxyValue);
		Private::FIntRangeNetSerializerBase<SourceType, ConfigType>::Serialize(Context, DowngradedRoleArgs);
	}
	else
	{
		Private::FIntRangeNetSerializerBase<SourceType, ConfigType>::Serialize(Context, Args);
	}
}

void FNetRoleNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	Private::FIntRangeNetSerializerBase<SourceType, ConfigType>::Deserialize(Context, Args);
}

void FNetRoleNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	Private::FIntRangeNetSerializerBase<SourceType, ConfigType>::Quantize(Context, Args);
}

void FNetRoleNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);

	// Dequantize to the other role. This will result in the desired role swap.
	FNetDequantizeArgs RoleSwappingArgs = Args;
	RoleSwappingArgs.Target += Config->RelativeExternalOffsetToOtherRole;
	Private::FIntRangeNetSerializerBase<SourceType, ConfigType>::Dequantize(Context, RoleSwappingArgs);
}

bool FNetRoleNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	return Private::FIntRangeNetSerializerBase<SourceType, ConfigType>::IsEqual(Context, Args);
}

bool FNetRoleNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);

	const SourceType Value = *reinterpret_cast<const SourceType*>(Args.Source);

	// Detect values outside of the valid range. This check needs to be performed before the enum check due to the generated _MAX value.
	const SourceType ClampedValue = FMath::Clamp(Value, Config->LowerBound, Config->UpperBound);
	if (Value != ClampedValue)
	{
		return false;
	}

	return Config->Enum->IsValidEnumValue(Value);
}

bool PartialInitNetRoleSerializerConfig(FNetRoleNetSerializerConfig& OutConfig, const UEnum* Enum)
{
	if (FNetRoleNetSerializer::CachedConfig.BitCount == 0)
	{
		FEnumUint8NetSerializerConfig EnumConfig;
		if (!Private::InitEnumNetSerializerConfig(EnumConfig, Enum))
		{
			check(false);
			return false;
		}

		// Because of role downgrading we currently assume that quantize is a no-op so we don't have to requantize.
		check(EnumConfig.LowerBound == 0);

		const int64 AutonomousProxyValue = Enum->GetValueByNameString(TEXT("ROLE_AutonomousProxy"), EGetByNameFlags::CaseSensitive);
		check(AutonomousProxyValue > 0 && AutonomousProxyValue < 256);
		const int64 SimulatedProxyValue = Enum->GetValueByNameString(TEXT("ROLE_SimulatedProxy"), EGetByNameFlags::CaseSensitive);
		check(SimulatedProxyValue > 0 && SimulatedProxyValue < 256);
		
		FNetRoleNetSerializer::CachedConfig.LowerBound = EnumConfig.LowerBound;
		FNetRoleNetSerializer::CachedConfig.UpperBound = EnumConfig.UpperBound;
		FNetRoleNetSerializer::CachedConfig.BitCount = EnumConfig.BitCount;
		FNetRoleNetSerializer::CachedConfig.AutonomousProxyValue = static_cast<uint8>(AutonomousProxyValue);
		FNetRoleNetSerializer::CachedConfig.SimulatedProxyValue = static_cast<uint8>(SimulatedProxyValue);
		FNetRoleNetSerializer::CachedConfig.Enum = Enum;
	}

	OutConfig = FNetRoleNetSerializer::CachedConfig;
	return true;
}

}
