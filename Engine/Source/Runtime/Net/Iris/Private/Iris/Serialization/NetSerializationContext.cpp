// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"

namespace UE::Net
{

bool FNetSerializationContext::IsBitStreamOverflown() const
{
	if (BitStreamReader != nullptr && BitStreamReader->IsOverflown())
	{
		return true;
	}

	if (BitStreamWriter != nullptr && BitStreamWriter->IsOverflown())
	{
		return true;
	}

	return false;
}

void FNetSerializationContext::AddReadJournalEntry(const TCHAR* Name)
{
	if (!HasErrorOrOverflow())
	{
		ReadJournal.AddEntry(Name, BitStreamReader->GetPosBits(), ErrorContext.GetObjectHandle());
	}
}

void FNetSerializationContext::AddReadJournalEntry(const FNetDebugName* DebugName)
{
	if (!HasErrorOrOverflow())
	{
		ReadJournal.AddEntry(DebugName->Name, BitStreamReader->GetPosBits(), ErrorContext.GetObjectHandle());
	}
}

FString FNetSerializationContext::PrintReadJournal()
{
	return ReadJournal.Print(InternalContext ? InternalContext->ReplicationSystem : nullptr);
}

void FNetSerializationContext::SetBitStreamOverflow()
{
	if (BitStreamReader != nullptr && !BitStreamReader->IsOverflown())
	{
		BitStreamReader->DoOverflow();
	}

	if (BitStreamWriter != nullptr && !BitStreamWriter->IsOverflown())
	{
		BitStreamWriter->DoOverflow();
	}
}

UObject* FNetSerializationContext::GetLocalConnectionUserData(uint32 ConnectionId) const
{
	if (InternalContext == nullptr)
	{
		return nullptr;
	}

	const UReplicationSystem* ReplicationSystem = InternalContext->ReplicationSystem;
	if (ReplicationSystem == nullptr)
	{
		return nullptr;
	}

	if (ConnectionId == UE::Net::InvalidConnectionId)
	{
		return nullptr;
	}

	UObject* UserData = ReplicationSystem->GetConnectionUserData(ConnectionId);
	return UserData;
}

const FNetTokenStore* FNetSerializationContext:: GetNetTokenStore() const
{
	if (InternalContext == nullptr)
	{
		return nullptr;
	}

	const UReplicationSystem* ReplicationSystem = InternalContext->ReplicationSystem;

	return ReplicationSystem ? ReplicationSystem->GetNetTokenStore() : nullptr;
}

FNetTokenStore* FNetSerializationContext::GetNetTokenStore()
{
	if (InternalContext == nullptr)
	{
		return nullptr;
	}

	UReplicationSystem* ReplicationSystem = InternalContext->ReplicationSystem;

	return ReplicationSystem ? ReplicationSystem->GetNetTokenStore() : nullptr;
}

const UE::Net::FNetTokenStoreState* FNetSerializationContext::GetRemoteNetTokenStoreState() const
{
	return InternalContext ? InternalContext->ResolveContext.RemoteNetTokenStoreState : nullptr;
}


}
