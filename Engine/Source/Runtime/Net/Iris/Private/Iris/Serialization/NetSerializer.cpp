// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetSerializer.h"

FNetSerializerConfig::~FNetSerializerConfig()
{
}
