// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetSerializerArrayStorage.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"

namespace UE::Net::AllocationPolicies
{

void* FElementAllocationPolicy::Realloc(FNetSerializationContext& Context, void* Original, SIZE_T Size, uint32 Alignment)
{
	return Context.GetInternalContext()->Realloc(Original, Size, Alignment);
}

void FElementAllocationPolicy::Free(FNetSerializationContext& Context, void* Ptr)
{
	return Context.GetInternalContext()->Free(Ptr);
}

}


namespace UE::Net
{

void FNetSerializerAlignedStorage::AdjustSize(FNetSerializationContext& Context, SizeType InNum, SizeType InAlignment)
{
	if (!InNum)
	{
		Free(Context);
		return;
	}

	// If the allocation isn't properly aligned or if the allocation is too small we make a new allocation.
	if ((InNum > StorageMaxCapacity) || !IsAligned(Data, InAlignment))
	{
		void* NewData = Context.GetInternalContext()->Alloc(InNum, InAlignment);
		FMemory::Memzero(NewData, InNum);
		// Copy old data
		if (StorageNum > 0)
		{
			FMemory::Memcpy(NewData, Data, StorageNum);
		}
		Context.GetInternalContext()->Free(Data);

		Data = static_cast<uint8*>(NewData);
		StorageNum = InNum;
		StorageMaxCapacity = InNum;
		StorageAlignment = InAlignment;
	}
	// Requested data size fits the current allocation
	else
	{
		// Clear capacity we're not using anymore. If we're growing we don't need to clear as it has already been cleared.
		if (InNum < StorageNum)
		{
			FMemory::Memzero(static_cast<void*>(Data + InNum), StorageNum - InNum);
		}
		StorageNum = InNum;
		// Use the requested alignment as the StorageAlignment. This allows Clone to allocate using the minimum required alignment.
		StorageAlignment = InAlignment;
	}
}

void FNetSerializerAlignedStorage::Free(FNetSerializationContext& Context)
{
	if (Data != nullptr)
	{
		Context.GetInternalContext()->Free(Data);
	}

	Data = nullptr;
	StorageNum = 0;
	StorageMaxCapacity = 0;
	StorageAlignment = 0;
}

void FNetSerializerAlignedStorage::Clone(FNetSerializationContext& Context, const FNetSerializerAlignedStorage& Source)
{
	// Only allocate and copy the exact amount of memory needed.
	if (Source.StorageNum > 0)
	{
		Data = static_cast<uint8*>(Context.GetInternalContext()->Alloc(Source.StorageNum, Source.StorageAlignment));
		StorageNum = Source.StorageNum;
		StorageMaxCapacity = Source.StorageNum;
		StorageAlignment = Source.StorageAlignment;
		FMemory::Memcpy(Data, Source.Data, Source.StorageNum);
	}
	else
	{
		Data = nullptr;
		StorageNum = 0;
		StorageMaxCapacity = 0;
		StorageAlignment = 0;
	}
}

}
