// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/InternalNetSerializerDelegates.h"

namespace UE::Net
{

using namespace UE::Net::Private;

FNetSerializerRegistryDelegates::FNetSerializerRegistryDelegates(EFlags Flags)
: PreFreezeDelegate(FInternalNetSerializerDelegates::GetPreFreezeNetSerializerRegistryDelegate().AddRaw(this, &FNetSerializerRegistryDelegates::PreFreezeNetSerializerRegistry))
, PostFreezeDelegate(FInternalNetSerializerDelegates::GetPostFreezeNetSerializerRegistryDelegate().AddRaw(this, &FNetSerializerRegistryDelegates::PostFreezeNetSerializerRegistry))
{
	if ((Flags & EFlags::ShouldBindLoadedModulesUpdatedDelegate) != 0U)
	{
		LoadedModulesUpdatedDelegate = FInternalNetSerializerDelegates::GetLoadedModulesUpdatedDelegate().AddRaw(this, &FNetSerializerRegistryDelegates::LoadedModulesUpdated);
	}
}

FNetSerializerRegistryDelegates::FNetSerializerRegistryDelegates()
: FNetSerializerRegistryDelegates(EFlags::None)
{
}

FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	if (PreFreezeDelegate.IsValid())
	{
		FInternalNetSerializerDelegates::GetPreFreezeNetSerializerRegistryDelegate().Remove(PreFreezeDelegate);
	}
	if (PostFreezeDelegate.IsValid())
	{
		FInternalNetSerializerDelegates::GetPostFreezeNetSerializerRegistryDelegate().Remove(PostFreezeDelegate);
	}
	if (LoadedModulesUpdatedDelegate.IsValid())
	{
		FInternalNetSerializerDelegates::GetLoadedModulesUpdatedDelegate().Remove(LoadedModulesUpdatedDelegate);
	}
}

void FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
}

void FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
}

void FNetSerializerRegistryDelegates::OnLoadedModulesUpdated()
{
}

void FNetSerializerRegistryDelegates::PreFreezeNetSerializerRegistry()
{
	OnPreFreezeNetSerializerRegistry();
	FInternalNetSerializerDelegates::GetPreFreezeNetSerializerRegistryDelegate().Remove(PreFreezeDelegate);
	PreFreezeDelegate.Reset();
}

void FNetSerializerRegistryDelegates::PostFreezeNetSerializerRegistry()
{
	OnPostFreezeNetSerializerRegistry();
	FInternalNetSerializerDelegates::GetPostFreezeNetSerializerRegistryDelegate().Remove(PostFreezeDelegate);
	PostFreezeDelegate.Reset();
}

void FNetSerializerRegistryDelegates::LoadedModulesUpdated()
{
	OnLoadedModulesUpdated();
}

}
