// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetSerializers.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetSerializers)

FStructNetSerializerConfig::FStructNetSerializerConfig()
: FNetSerializerConfig()
{
	ConfigTraits = ENetSerializerConfigTraits::NeedDestruction;
}

FStructNetSerializerConfig::~FStructNetSerializerConfig() = default;
