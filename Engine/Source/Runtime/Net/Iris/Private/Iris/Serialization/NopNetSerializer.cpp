// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/NetSerializers.h"

namespace UE::Net
{

struct FNopNetSerializer
{
	static const uint32 Version = 0;

	typedef void SourceType;
	typedef FNopNetSerializerConfig ConfigType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&) {}
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&) {}

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&) {}
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&) {}

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&) {}
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&) {}

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&) { return true; }
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&) { return true; }

	static FNopNetSerializerConfig DefaultConfig;
};

FNopNetSerializerConfig FNopNetSerializer::DefaultConfig;

UE_NET_IMPLEMENT_SERIALIZER(FNopNetSerializer);

}
