// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/Core/IrisLog.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Iris/Serialization/InternalNetSerializer.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/QuantizedObjectReference.h"
#include "Iris/Serialization/QuantizedRemoteObjectReference.h"
#include "Iris/Serialization/RemoteObjectReferenceNetSerializer.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ObjectReferenceCache.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "UObject/RemoteObjectTransfer.h"
#include "UObject/RemoteObjectTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectNetSerializer)

namespace UE::Net
{

// Validate size & alignment of public quantized storage type
static_assert(sizeof(FObjectNetSerializerQuantizedReferenceStorage) >= sizeof(FQuantizedObjectReference), "FObjectNetSerializerQuantizedReferenceStorage::Storage must be large enough to store FQuantizedObjectReference");
static_assert(alignof(FObjectNetSerializerQuantizedReferenceStorage) >= alignof(FQuantizedObjectReference), "FObjectNetSerializerQuantizedReferenceStorage::Storage must be aligned to store FQuantizedObjectReference");

void WriteNetRefHandle(FNetSerializationContext& Context, const FNetRefHandle Handle)
{
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	if (Handle.IsValid())
	{
		Writer->WriteBool(true);
		WritePackedUint64(Writer, Handle.GetId());
	}
	else
	{
		Writer->WriteBool(false);
	}
}

FNetRefHandle ReadNetRefHandle(FNetSerializationContext& Context)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	if (Reader->ReadBool())
	{
		const uint64 NetId = ReadPackedUint64(Reader);
		if (!Reader->IsOverflown())
		{
			return Private::FNetRefHandleManager::MakeNetRefHandleFromId(NetId);
		}
	}
	return FNetRefHandle();
}

void ReadFullNetObjectReference(FNetSerializationContext& Context, FNetObjectReference& Reference)
{
	// Read full ref for now
	Context.GetInternalContext()->ObjectReferenceCache->ReadFullReference(Context, Reference);
}

void WriteFullNetObjectReference(FNetSerializationContext& Context, const FNetObjectReference& Reference)
{
	// Write full ref for now
	Context.GetInternalContext()->ObjectReferenceCache->WriteFullReference(Context, Reference);
}

}

namespace UE::Net::Private
{

static void ReadNetObjectReference(FNetSerializationContext& Context, FNetObjectReference& Reference)
{
	FInternalNetSerializationContext* InternalContext = Context.GetInternalContext();

	if (InternalContext->bInlineObjectReferenceExports == 0U)
	{
		InternalContext->ObjectReferenceCache->ReadReference(Context, Reference);
	}
	else
	{
		InternalContext->ObjectReferenceCache->ReadFullReference(Context, Reference);
	}
}

static void WriteNetObjectReference(FNetSerializationContext& Context, const FNetObjectReference& Reference)
{
	FInternalNetSerializationContext* InternalContext = Context.GetInternalContext();

	if (InternalContext->bInlineObjectReferenceExports == 0U)
	{
		InternalContext->ObjectReferenceCache->WriteReference(Context, Reference);
	}
	else
	{
		InternalContext->ObjectReferenceCache->WriteFullReference(Context, Reference);
	}
}

static void AllocateRemoteReferenceStorage(FQuantizedObjectReference& QuantizedReference, FInternalNetSerializationContext& InternalContext)
{
	QuantizedReference.RemoteReferencePtr = static_cast<FQuantizedRemoteObjectReference*>(InternalContext.Alloc(sizeof(FQuantizedRemoteObjectReference), alignof(FQuantizedRemoteObjectReference)));
	FMemory::Memset(QuantizedReference.RemoteReferencePtr, 0, sizeof(FQuantizedRemoteObjectReference));
}

template<typename T>
struct FObjectNetSerializerBase
{
	typedef T SourceType;
	typedef FQuantizedObjectReference QuantizedType;

	static constexpr bool bHasDynamicState = true;
	static constexpr bool bUseSerializerIsEqual = true;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

protected:

	// Raw pointer getters/setters
	static UObject* GetRawPtrValue(UObject* Value) { return Value; }
	static TObjectPtr<UObject> GetObjectPtrValue(UObject* Value) { return Value; }
	static UObject* GetValidatedRawPtrValue(UObject* Value, const UClass* PropertyClass);

	static void SetRawPtrValue(UObject*& Dst, UObject* Value) { Dst = Value; }

	// TObjectPtr getters/setters
	static UObject* GetRawPtrValue(const TObjectPtr<UObject>& Value) { return Value; }
	static const TObjectPtr<UObject>& GetObjectPtrValue(const TObjectPtr<UObject>& Value) { return Value; }

	static void SetRawPtrValue(TObjectPtr<UObject>& Dst, UObject* Value) { Dst = Value; }
	static void SetObjectPtrValue(TObjectPtr<UObject>& Dst, const FRemoteObjectReference& Value) { Dst = TObjectPtr<UObject>(Value.ToObjectPtr()); }

	// Weak pointer getters/setters
	static UObject* GetRawPtrValue(const TWeakObjectPtr<UObject>& Value) { return Value.Get(); }
	static const TWeakObjectPtr<UObject>& GetObjectPtrValue(const TWeakObjectPtr<UObject>& Value) { return Value; }

	static void SetRawPtrValue(TWeakObjectPtr<UObject>& Dst, UObject* Value) { Dst = TWeakObjectPtr<UObject>(Value); }
#if UE_WITH_REMOTE_OBJECT_HANDLE
	static void SetObjectPtrValue(TWeakObjectPtr<UObject>& Dst, const FRemoteObjectReference& Value) { Dst = TWeakObjectPtr<UObject>(Value.GetRemoteId()); }
#endif

	// For FScriptInterface we have a custom Dequantize so we only provide a value getter.
	static UObject* GetRawPtrValue(const FScriptInterface& Value) { return Value.GetObject(); }
	static const TObjectPtr<UObject>& GetObjectPtrValue(FScriptInterface& Value) { return Value.GetObjectRef(); }

	static UObject* ResolveObjectReference(FNetSerializationContext&, const FNetObjectReference&);

	inline static const FNetSerializer* RemoteObjectReferenceNetSerializer = &UE_NET_GET_SERIALIZER(FRemoteObjectReferenceNetSerializer);
	inline static const FNetSerializerConfig* RemoteObjectReferenceNetSerializerConfig = UE_NET_GET_SERIALIZER_DEFAULT_CONFIG(FRemoteObjectReferenceNetSerializer);
};

}

namespace UE::Net
{

struct FObjectNetSerializer : public Private::FObjectNetSerializerBase<UObject*>
{
	static const uint32 Version = 0;
	typedef FObjectNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;
};

UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FObjectNetSerializer);

struct FObjectPtrNetSerializer : public Private::FObjectNetSerializerBase<TObjectPtr<UObject>>
{
	static const uint32 Version = 0;
	typedef FObjectPtrNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
};

UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FObjectPtrNetSerializer);

struct FWeakObjectNetSerializer : public Private::FObjectNetSerializerBase<TWeakObjectPtr<UObject>>
{
	static const uint32 Version = 0;
	typedef FWeakObjectNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;
};

UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FWeakObjectNetSerializer);

struct FScriptInterfaceNetSerializer : public Private::FObjectNetSerializerBase<FScriptInterface>
{
	static const uint32 Version = 0;
	typedef FScriptInterfaceNetSerializerConfig ConfigType;

	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);
};

UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FScriptInterfaceNetSerializer);

// Use a simple traits class to distinguish between raw UObject pointers and TObjectPtrs in the serializer implementation.
// Raw pointers are never serialized as remote references. Using traits reduces boilerplate code that would be needed
// in two separate, but very similar, net serializer implementations.
template<class T>
struct TObjectNetSerializerTraits
{
	// If true, serializer is allowed to use the remote reference path for the template parameter type.
	// UE_WITH_REMOTE_OBJECT_HANDLE is required to use remote references.
#if UE_WITH_REMOTE_OBJECT_HANDLE
	static constexpr bool bCanSerializeRemoteReference = true;
#else
	static constexpr bool bCanSerializeRemoteReference = false;
#endif
};

template<>
struct TObjectNetSerializerTraits<UObject*>
{
	static constexpr bool bCanSerializeRemoteReference = false;
};

}

namespace UE::Net::Private
{

template<typename T>
void FObjectNetSerializerBase<T>::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	if (!TObjectNetSerializerTraits<T>::bCanSerializeRemoteReference && Source.IsRemoteReference())
	{
		UE_LOGF(LogIris, Error, "FObjectNetSerializerBase::Serialize: traits disallow remote references but trying to serialize a remote reference: %ls", *Source.ToString());
		Context.SetError(GNetError_InvalidNetHandle);
		return;
	}

	if (Source.IsNetReference())
	{
		Writer->WriteBool(true);
		WriteNetObjectReference(Context, Source.NetReference);
	}
	else
	{
		Writer->WriteBool(false);

		FNetSerializeArgs SerializeArgs = Args;
		SerializeArgs.NetSerializerConfig = RemoteObjectReferenceNetSerializerConfig;
		SerializeArgs.Source = NetSerializerValuePointer(Source.RemoteReferencePtr);
		RemoteObjectReferenceNetSerializer->Serialize(Context, SerializeArgs);
	}
}

template<typename T>
void FObjectNetSerializerBase<T>::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const bool bIsNetObjectReference = Reader->ReadBool();

	if (!TObjectNetSerializerTraits<T>::bCanSerializeRemoteReference && !bIsNetObjectReference)
	{
		UE_LOGF(LogIris, Error, "FObjectNetSerializerBase::Deserialize: traits disallow remote references but trying to deserialize a remote reference");
		Context.SetError(GNetError_InvalidNetHandle);
		return;
	}

	if (bIsNetObjectReference)
	{
		FNetObjectReference Ref;
		// Read full ref for now
		ReadNetObjectReference(Context, Ref);
		Target.SetNetReference(Context, Args, Ref);
	}
	else
	{
		// It's a remote object reference
		if (!Target.RemoteReferencePtr)
		{
			AllocateRemoteReferenceStorage(Target, *Context.GetInternalContext());
		}

		FNetDeserializeArgs InternalArgs = Args;
		InternalArgs.NetSerializerConfig = RemoteObjectReferenceNetSerializerConfig;
		InternalArgs.Target = NetSerializerValuePointer(Target.RemoteReferencePtr);
		RemoteObjectReferenceNetSerializer->Deserialize(Context, InternalArgs);
	}
}

template<typename T>
void FObjectNetSerializerBase<T>::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	T& Source = *reinterpret_cast<T*>(Args.Source);

	// Notice that we quantize to default here if initializing for default state, this is due to the fact that
	// object references cannot be quantized for default state as they will use locally assigned ids which will differ between server and client
	// $TODO: Jira: UE-221750 It should now be possible to implement support for this, need to track if stored quantized data is local or not, as long as dequantized
	// state is correct it should work.

	const FInternalNetSerializationContext* InternalContext = Context.GetInternalContext();
	if (TObjectNetSerializerTraits<T>::bCanSerializeRemoteReference && InternalContext->bSerializeObjectReferencesAsRemoteIds)
	{
		if (!Target.RemoteReferencePtr)
		{
			AllocateRemoteReferenceStorage(Target, *Context.GetInternalContext());
		}

		FRemoteObjectReference SourceReference = Context.IsInitializingDefaultState() ? FRemoteObjectReference() : FRemoteObjectReference(GetObjectPtrValue(Source));

		FNetQuantizeArgs InternalArgs = Args;
		InternalArgs.NetSerializerConfig = RemoteObjectReferenceNetSerializerConfig;
		InternalArgs.Source = NetSerializerValuePointer(&SourceReference);
		InternalArgs.Target = NetSerializerValuePointer(Target.RemoteReferencePtr);
		RemoteObjectReferenceNetSerializer->Quantize(Context, InternalArgs);
	}
	else
	{
		UObject* SourceObject = GetRawPtrValue(Source);
		UReplicationSystem* ReplicationSystem = InternalContext->ReplicationSystem;

		FNetObjectReference Reference;
		if (!Context.IsInitializingDefaultState())
		{
			if (InternalContext->bForceObjectPathSerialization)
			{
				Reference = InternalContext->ObjectReferenceCache->GetOrCreateObjectReferenceUsingPath(SourceObject, ReplicationSystem);
			}
			else
			{
				Reference = InternalContext->ObjectReferenceCache->GetOrCreateObjectReference(SourceObject, ReplicationSystem);
			}
		}

		Target.SetNetReference(Context, Args, Reference);
	}
}

template<typename T>
void FObjectNetSerializerBase<T>::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const FObjectNetSerializerConfig& Config = *static_cast<const FObjectNetSerializerConfig*>(Args.NetSerializerConfig);
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	T& Target = *reinterpret_cast<T*>(Args.Target);

	if (Source.IsNetReference())
	{
		UObject* DequantizedObject = ResolveObjectReference(Context, Source.NetReference);
		SetRawPtrValue(Target, GetValidatedRawPtrValue(DequantizedObject, Config.PropertyClass));
		return;
	}

	// Source is a remote object reference
	if constexpr (TObjectNetSerializerTraits<T>::bCanSerializeRemoteReference)
	{
		FRemoteObjectReference DequantizedRemoteReference;

		FNetDequantizeArgs InternalArgs = Args;
		InternalArgs.NetSerializerConfig = RemoteObjectReferenceNetSerializerConfig;
		InternalArgs.Source = NetSerializerValuePointer(Source.RemoteReferencePtr);
		InternalArgs.Target = NetSerializerValuePointer(&DequantizedRemoteReference);
		RemoteObjectReferenceNetSerializer->Dequantize(Context, InternalArgs);

		SetObjectPtrValue(Target, DequantizedRemoteReference);
	}
	else
	{
		UE_LOGF(LogIris, Error, "FObjectNetSerializerBase::Dequantize: source is a remote ref but target type doesn't support remote refs.");
		Context.SetError(GNetError_InvalidNetHandle);
		return;
	}
}

template<typename T>
bool FObjectNetSerializerBase<T>::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		return Value0 == Value1;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		return Value0 == Value1;
	}
}

template<typename T>
void FObjectNetSerializerBase<T>::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	QuantizedType& SourceValue = *reinterpret_cast<QuantizedType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	if (SourceValue.IsRemoteReference())
	{
		// Need to deep copy source to target. Target memory is in undefined state, so allocate new target dynamic state
		AllocateRemoteReferenceStorage(TargetValue, *Context.GetInternalContext());

		FNetCloneDynamicStateArgs InternalArgs = Args;
		InternalArgs.NetSerializerConfig = RemoteObjectReferenceNetSerializerConfig;
		InternalArgs.Source = NetSerializerValuePointer(SourceValue.RemoteReferencePtr);
		InternalArgs.Target = NetSerializerValuePointer(TargetValue.RemoteReferencePtr);
		RemoteObjectReferenceNetSerializer->CloneDynamicState(Context, InternalArgs);
	}
}

template<typename T>
void FObjectNetSerializerBase<T>::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& SourceValue = *reinterpret_cast<QuantizedType*>(Args.Source);

	SourceValue.FreeRemoteReference(Context, Args);
}

template<typename T>
UObject* FObjectNetSerializerBase<T>::GetValidatedRawPtrValue(UObject* Value, const UClass* PropertyClass)
{
	if (PropertyClass && Value && !Value->IsA(PropertyClass))
	{
		UE_LOGF(LogIris, Warning, "Forged object: got %ls, expecting %ls", *Value->GetFullName(), *PropertyClass->GetFullName());
		return nullptr;
	}
	else
	{
		return Value;
	}
}

template<typename T>
UObject* FObjectNetSerializerBase<T>::ResolveObjectReference(FNetSerializationContext& Context, const FNetObjectReference& NetObjectReference)
{
	const FInternalNetSerializationContext* InternalContext = Context.GetInternalContext();

	UObject* Object = InternalContext->ObjectReferenceCache ? InternalContext->ObjectReferenceCache->ResolveObjectReference(NetObjectReference, InternalContext->ResolveContext) : nullptr;
	return Object;
}

}

namespace UE::Net
{

// FObjectPtrNetSerializer implementation
bool FObjectPtrNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		return Value0 == Value1;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		// Avoid FObjectPtr operator == which passes by value and may mark objects as reachable. We can compare a formerly valid valid pointer that had its memory deleted or re-used.
		// We don't care about UE_WITH_OBJECT_HANDLE_TYPE_SAFETY being defined or not.
		return Value0.GetHandle() == Value1.GetHandle();
	}
}

// FScriptInterfaceNetSerializer implementation
void FScriptInterfaceNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	UObject* Object = nullptr;
	
	if (Source.IsNetReference())
	{
		Object = ResolveObjectReference(Context, Source.NetReference);
	}
	else
	{
		FRemoteObjectReference DequantizedRemoteReference;

		FNetDequantizeArgs InternalArgs = Args;
		InternalArgs.NetSerializerConfig = RemoteObjectReferenceNetSerializerConfig;
		InternalArgs.Source = NetSerializerValuePointer(Source.RemoteReferencePtr);
		InternalArgs.Target = NetSerializerValuePointer(&DequantizedRemoteReference);
		RemoteObjectReferenceNetSerializer->Dequantize(Context, InternalArgs);

		Object = DequantizedRemoteReference.Resolve();
	}

	if (Object != nullptr)
	{
		const FScriptInterfaceNetSerializerConfig& Config = *static_cast<const FScriptInterfaceNetSerializerConfig*>(Args.NetSerializerConfig);
		UClass* ConfigInterfaceClass = Config.PropertyClass;
		void* Interface = Object->GetInterfaceAddress(ConfigInterfaceClass);

		Target.SetObject(Object);
		Target.SetInterface(Interface);
	}
	else
	{
		// Setting a null object clears the interface as well.
		Target.SetObject(Object);
	}
}

bool FScriptInterfaceNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Value = *reinterpret_cast<const SourceType*>(Args.Source);
	if (UObject* Object = Value.GetObject())
	{
		void* Interface = Value.GetInterface();		
		const FScriptInterfaceNetSerializerConfig& Config = *static_cast<const FScriptInterfaceNetSerializerConfig*>(Args.NetSerializerConfig);
		UClass* ConfigInterfaceClass = Config.PropertyClass;
		if (Interface != Object->GetInterfaceAddress(ConfigInterfaceClass))
		{
			return false;
		}
	}
	else
	{
		if (Value.GetInterface() != nullptr)
		{
			return false;
		}
	}

	return true;
}

}
