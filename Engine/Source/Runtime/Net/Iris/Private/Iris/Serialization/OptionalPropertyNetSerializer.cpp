// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/ReplicationState/InternalPropertyReplicationState.h"
#include "Iris/Serialization/NetSerializerArrayStorage.h"
#include "Iris/Serialization/InternalNetSerializerUtils.h"
#include "Iris/Serialization/InternalNetSerializers.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "UObject/PropertyOptional.h"

namespace UE::Net
{

struct FOptionalPropertyNetSerializerQuantizedData
{
	FNetSerializerAlignedStorage OptionalPropertyData;
};

}

template<> struct TIsPODType<UE::Net::FOptionalPropertyNetSerializerQuantizedData> { enum { Value = true }; };

namespace UE::Net
{

struct FOptionalPropertyNetSerializer
{
public:
	static const uint32 Version = 0;

	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bHasCustomNetReference = true;

	typedef void SourceType; // Dummy
	typedef FOptionalPropertyNetSerializerQuantizedData QuantizedType;
	typedef FOptionalPropertyNetSerializerConfig ConfigType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

	static void Apply(FNetSerializationContext&, const FNetApplyArgs&);

private:

	// Frees dynamic memory allocated and reset the entire quantized state to default.
	static void Reset(FNetSerializationContext&, const ConfigType*, QuantizedType&);
	static void InternalFreeOptionalPropertyData(FNetSerializationContext&, const ConfigType*, QuantizedType&);
};

UE_NET_IMPLEMENT_SERIALIZER(FOptionalPropertyNetSerializer);

void FOptionalPropertyNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	const bool bIsSet = Source.OptionalPropertyData.GetData() != nullptr;
	if (Writer->WriteBool(bIsSet))
	{
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetSerializeArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = NetSerializerValuePointer(Source.OptionalPropertyData.GetData());
		MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

		Serializer->Serialize(Context, MemberArgs);
	}
}

void FOptionalPropertyNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const bool bIsSet = Reader->ReadBool();
	if (bIsSet)
	{
		if (Target.OptionalPropertyData.Num() == 0)
		{
			Target.OptionalPropertyData.AdjustSize(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
		}

		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetDeserializeArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Target = NetSerializerValuePointer(Target.OptionalPropertyData.GetData());
		MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

		Serializer->Deserialize(Context, MemberArgs);
	}
	else
	{
		// Reset target
		Reset(Context, Config, Target);
	}
}

void FOptionalPropertyNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& Prev = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	const bool bIsSourceSet = Source.OptionalPropertyData.Num() > 0;
	const bool bIsPrevSet = Prev.OptionalPropertyData.Num() > 0;

	// If we cannot delta compress just call serialize
	if (Writer->WriteBool(bIsSourceSet != bIsPrevSet))
	{
		Serialize(Context, Args);
		return;
	}
	// Nothing to delta serialize
	if (!bIsSourceSet)
	{
		return;
	}

	const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];
	const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

	FNetSerializeDeltaArgs MemberArgs;
	MemberArgs.Version = 0;
	MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
	MemberArgs.Source = NetSerializerValuePointer(Source.OptionalPropertyData.GetData());
	MemberArgs.Prev = NetSerializerValuePointer(Prev.OptionalPropertyData.GetData());
	MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

	Serializer->SerializeDelta(Context, MemberArgs);
}

void FOptionalPropertyNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	if (const bool bIsSetDiffers = Reader->ReadBool())
	{
		Deserialize(Context, Args);
		return;
	}

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& Prev = *reinterpret_cast<const QuantizedType*>(Args.Prev);
	const bool bIsPrevSet = Prev.OptionalPropertyData.Num() > 0;

	// If not set, reset target.
	if (!bIsPrevSet)
	{
		Reset(Context, Config, Target);
		return;
	}

	if (Target.OptionalPropertyData.Num() == 0)
	{
		Target.OptionalPropertyData.AdjustSize(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
	}

	const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];
	const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

	FNetDeserializeDeltaArgs MemberArgs;
	MemberArgs.Version = 0;
	MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
	MemberArgs.Target = NetSerializerValuePointer(Target.OptionalPropertyData.GetData());
	MemberArgs.Prev = NetSerializerValuePointer(Prev.OptionalPropertyData.GetData());
	MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

	Serializer->DeserializeDelta(Context, MemberArgs);
}

void FOptionalPropertyNetSerializer::Reset(FNetSerializationContext& Context, const ConfigType* Config, QuantizedType& Value)
{
	InternalFreeOptionalPropertyData(Context, Config, Value);
	Value.OptionalPropertyData.Free(Context);
	FMemory::Memzero(&Value, sizeof(QuantizedType));
}

void FOptionalPropertyNetSerializer::InternalFreeOptionalPropertyData(FNetSerializationContext& Context, const ConfigType* Config, QuantizedType& Value)
{
	if (Value.OptionalPropertyData.Num() > 0 && EnumHasAnyFlags(Config->StateDescriptor->Traits, EReplicationStateTraits::HasDynamicState))
	{
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Config->StateDescriptor->MemberSerializerDescriptors[0];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetFreeDynamicStateArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = NetSerializerValuePointer(Value.OptionalPropertyData.GetData());

		Serializer->FreeDynamicState(Context, MemberArgs);
	}
}

void FOptionalPropertyNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;

	const FOptionalProperty* OptionalProperty = Config->Property.Get();
	if (OptionalProperty->IsSet(reinterpret_cast<const void*>(Args.Source)))
	{
		if (Target.OptionalPropertyData.Num() == 0)
		{
			Target.OptionalPropertyData.AdjustSize(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
		}
		
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetQuantizeArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = NetSerializerValuePointer(OptionalProperty->GetValuePointerForRead(reinterpret_cast<const void*>(Args.Source)));
		MemberArgs.Target = NetSerializerValuePointer(Target.OptionalPropertyData.GetData());
		MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

		Serializer->Quantize(Context, MemberArgs);

		return;
	}
	else
	{
		// Reset target
		Reset(Context, Config, Target);
	}
}

void FOptionalPropertyNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;

	const FOptionalProperty* OptionalProperty = Config->Property.Get();
	
	if (Source.OptionalPropertyData.Num() > 0)
	{
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetDequantizeArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = NetSerializerValuePointer(Source.OptionalPropertyData.GetData());
		MemberArgs.Target = NetSerializerValuePointer(OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(reinterpret_cast<void*>(Args.Target)));
		MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

		Serializer->Dequantize(Context, MemberArgs);

		if (!Context.HasError())
		{
			return;
		}
	}

	OptionalProperty->MarkUnset(reinterpret_cast<void*>(Args.Target));
}

void FOptionalPropertyNetSerializer::Apply(FNetSerializationContext& Context, const FNetApplyArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;

	const FOptionalProperty* OptionalProperty = Config->Property.Get();	
	if (OptionalProperty->IsSet(reinterpret_cast<const void*>(Args.Source)))
	{
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		UE::Net::Private::InternalApplyPropertyValue(Descriptor, 0, static_cast<uint8*>(OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(reinterpret_cast<void*>(Args.Target))), static_cast<uint8*>(OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(reinterpret_cast<void*>(Args.Source))));
	}
	else
	{
		OptionalProperty->MarkUnset(reinterpret_cast<void*>(Args.Target));
	}
}


bool FOptionalPropertyNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Source0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Source1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		const bool bIsSource0Set = Source0.OptionalPropertyData.Num() > 0;
		const bool bIsSource1Set = Source1.OptionalPropertyData.Num() > 0;

		if (bIsSource0Set != bIsSource1Set)
		{
			return false;
		}
		if (bIsSource0Set)
		{
			const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];
			const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

			FNetIsEqualArgs MemberArgs = Args;
			MemberArgs.Version = 0;
			MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			MemberArgs.Source0 = NetSerializerValuePointer(Source0.OptionalPropertyData.GetData());
			MemberArgs.Source1 = NetSerializerValuePointer(Source1.OptionalPropertyData.GetData());
			MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

			return Serializer->IsEqual(Context, MemberArgs);
		}
	}
	else
	{
		const FOptionalProperty* OptionalProperty = Config->Property.Get();
		const bool bIsEqual = OptionalProperty->Identical(reinterpret_cast<const void*>(Args.Source0), reinterpret_cast<const void*>(Args.Source1), 0);

		return bIsEqual;
	}
	return true;
}

bool FOptionalPropertyNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	if (Descriptor == nullptr)
	{
		return false;
	}

	const FOptionalProperty* OptionalProperty = Config->Property.Get();
	if (OptionalProperty->IsSet(reinterpret_cast<const void*>(Args.Source)))
	{
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetValidateArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = NetSerializerValuePointer(OptionalProperty->GetValuePointerForReadOrReplace(reinterpret_cast<const void*>(Args.Source)));

		if (!Serializer->Validate(Context, MemberArgs))
		{
			return false;
		}
	}

	return true;
}

void FOptionalPropertyNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	Target.OptionalPropertyData.Clone(Context, Source.OptionalPropertyData);

	// Forward call if necessary
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;

	const bool bIsSourceSet = Source.OptionalPropertyData.Num() > 0;
	if (bIsSourceSet && EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState))
	{
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetCloneDynamicStateArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = NetSerializerValuePointer(Source.OptionalPropertyData.GetData());
		MemberArgs.Target = NetSerializerValuePointer(Target.OptionalPropertyData.GetData());

		Serializer->CloneDynamicState(Context, MemberArgs);
	}
}

void FOptionalPropertyNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);

	if (!Value.OptionalPropertyData.GetData())
	{
		return;
	}

	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	if (EnumHasAnyFlags(Descriptor->MemberTraitsDescriptors[0].Traits, EReplicationStateMemberTraits::HasDynamicState))
	{
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = Descriptor->MemberSerializerDescriptors[0];

		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetFreeDynamicStateArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = NetSerializerValuePointer(Value.OptionalPropertyData.GetData());

		Serializer->FreeDynamicState(Context, MemberArgs);
	}
	
	Value.OptionalPropertyData.Free(Context);
}

void FOptionalPropertyNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);

	if (!Descriptor->HasObjectReference() || Value.OptionalPropertyData.Num() == 0)
	{
		return;
	}

	Private::FReplicationStateOperationsInternal::CollectReferences(Context, *reinterpret_cast<FNetReferenceCollector*>(Args.Collector), Args.ChangeMaskInfo, Value.OptionalPropertyData.GetData(), Descriptor);
}

}
