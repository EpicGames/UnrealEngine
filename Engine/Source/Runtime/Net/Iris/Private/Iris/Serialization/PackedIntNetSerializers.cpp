// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/PackedIntNetSerializers.h"
#include "Iris/Core/BitTwiddling.h"
#include "Iris/Serialization/BitPacking.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PackedIntNetSerializers)

namespace UE::Net
{

struct FPackedInt64NetSerializerBase
{
	static constexpr SIZE_T DeltaBitCountTableEntryCount = 3;
	// Bit counts aiming to have small value changes use few bits.
	static constexpr uint8 DeltaBitCountTable[] = {0, 14, 32};
};

struct FPackedInt64NetSerializer : public FPackedInt64NetSerializerBase
{
	static const uint32 Version = 0;

	typedef int64 SourceType;
	typedef FPackedInt64NetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);
};
UE_NET_IMPLEMENT_SERIALIZER(FPackedInt64NetSerializer);

struct FPackedUint64NetSerializer : public FPackedInt64NetSerializerBase
{
	static const uint32 Version = 0;

	typedef uint64 SourceType;
	typedef FPackedUint64NetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);
};
UE_NET_IMPLEMENT_SERIALIZER(FPackedUint64NetSerializer);

struct FPackedInt32NetSerializerBase
{
	static constexpr SIZE_T DeltaBitCountTableEntryCount = 3;
	// Bit counts aiming to have small value changes use few bits.
	static constexpr uint8 DeltaBitCountTable[] = {0, 4, 14};
};

struct FPackedInt32NetSerializer : public FPackedInt32NetSerializerBase
{
	static const uint32 Version = 0;

	typedef int32 SourceType;
	typedef FPackedInt32NetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);
};
UE_NET_IMPLEMENT_SERIALIZER(FPackedInt32NetSerializer);

struct FPackedUint32NetSerializer : public FPackedInt32NetSerializerBase
{
	static const uint32 Version = 0;

	typedef uint32 SourceType;
	typedef FPackedUint32NetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);
};
UE_NET_IMPLEMENT_SERIALIZER(FPackedUint32NetSerializer);

// FPackedInt64NetSerializer implementation
void FPackedInt64NetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const int64 Value = *reinterpret_cast<const int64*>(Args.Source);

	const uint32 BitCountNeeded = GetBitsNeeded(Value);
	const uint32 ByteCountNeeded = (BitCountNeeded + 7U) / 8U;
	const uint32 BitCountToWrite = ByteCountNeeded * 8U;

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(ByteCountNeeded - 1U, 3U);
	if (BitCountToWrite <= 32U)
	{
		Writer->WriteBits(static_cast<uint64>(Value) & 0xFFFFFFFFU, BitCountToWrite);
	}
	else
	{
		const uint64 UnsignedValue = static_cast<uint64>(Value);
		Writer->WriteBits(UnsignedValue & 0xFFFFFFFFU, 32U);
		Writer->WriteBits(static_cast<uint32>(UnsignedValue >> 32U), BitCountToWrite - 32U);
	}
}

void FPackedInt64NetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const uint32 ByteCountToRead = Reader->ReadBits(3U) + 1U;
	const uint32 BitCountToRead = ByteCountToRead * 8U;

	uint64 UnsignedValue;
	if (BitCountToRead <= 32U)
	{
		UnsignedValue = Reader->ReadBits(BitCountToRead);
	}
	else
	{
		UnsignedValue = Reader->ReadBits(32U);
		UnsignedValue |= (static_cast<uint64>(Reader->ReadBits(BitCountToRead - 32U)) << 32U);
	}

	const uint64 Mask = 1ULL << (BitCountToRead - 1U);
	UnsignedValue = (UnsignedValue ^ Mask) - Mask;
	const int64 Value = static_cast<int64>(UnsignedValue);

	int64* Target = reinterpret_cast<int64*>(Args.Target);
	*Target = Value;
}

void FPackedInt64NetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const int64 Value = *reinterpret_cast<const int64*>(Args.Source);
	const int64 PrevValue = *reinterpret_cast<const int64*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	SerializeIntDelta(*Writer, Value, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 64U);
}

void FPackedInt64NetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	int64& Target = *reinterpret_cast<int64*>(Args.Target);
	const int64 PrevValue = *reinterpret_cast<const int64*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	DeserializeIntDelta(*Reader, Target, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 64U);
}

// FPackedUint64NetSerializer implementation
void FPackedUint64NetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const uint64 Value = *reinterpret_cast<const uint64*>(Args.Source);

	// As we represent the number of bytes to write with two bits we want bits needed to be >= 1
	const uint32 BitCountNeeded = GetBitsNeeded(Value | 1U);
	const uint32 ByteCountNeeded = (BitCountNeeded + 7U) / 8U;
	const uint32 BitCountToWrite = ByteCountNeeded * 8U;

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(ByteCountNeeded - 1U, 3U);
	if (BitCountToWrite <= 32U)
	{
		Writer->WriteBits(Value & 0xFFFFFFFFU, BitCountToWrite);
	}
	else
	{
		Writer->WriteBits(Value & 0xFFFFFFFFU, 32U);
		Writer->WriteBits(static_cast<uint32>(Value >> 32U), BitCountToWrite - 32U);
	}
}

void FPackedUint64NetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const uint32 ByteCountToRead = Reader->ReadBits(3U) + 1U;
	const uint32 BitCountToRead = ByteCountToRead * 8U;

	uint64 Value;
	if (BitCountToRead <= 32)
	{
		Value = Reader->ReadBits(BitCountToRead);
	}
	else
	{
		Value = Reader->ReadBits(32U);
		Value |= (static_cast<uint64>(Reader->ReadBits(BitCountToRead - 32U)) << 32U);
	}

	uint64* Target = reinterpret_cast<uint64*>(Args.Target);
	*Target = Value;
}

void FPackedUint64NetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const uint64 Value = *reinterpret_cast<const uint64*>(Args.Source);
	const uint64 PrevValue = *reinterpret_cast<const uint64*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	SerializeUintDelta(*Writer, Value, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 64U);
}

void FPackedUint64NetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	uint64& Target = *reinterpret_cast<uint64*>(Args.Target);
	const uint64 PrevValue = *reinterpret_cast<const uint64*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	DeserializeUintDelta(*Reader, Target, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 64U);
}

// FPackedInt32NetSerializer implementation
void FPackedInt32NetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const int32 Value = *reinterpret_cast<const int32*>(Args.Source);

	const uint32 BitCountNeeded = GetBitsNeeded(Value);
	const uint32 ByteCountNeeded = (BitCountNeeded + 7U)/8U;
	const uint32 BitCountToWrite = ByteCountNeeded*8U;
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(ByteCountNeeded - 1U, 2U);
	Writer->WriteBits(Value, BitCountToWrite);
}

void FPackedInt32NetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const uint32 ByteCountToRead = Reader->ReadBits(2U) + 1U;
	const uint32 BitCountToRead = ByteCountToRead*8U;
	
	uint32 UnsignedValue = Reader->ReadBits(BitCountToRead);
	const uint32 Mask = 1U << (BitCountToRead - 1U);
	UnsignedValue = (UnsignedValue ^ Mask) - Mask;
	const int32 Value = static_cast<int32>(UnsignedValue);
		
	int32* Target = reinterpret_cast<int32*>(Args.Target);
	*Target = Value;
}

void FPackedInt32NetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const int32 Value = *reinterpret_cast<const int32*>(Args.Source);
	const int32 PrevValue = *reinterpret_cast<const int32*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	SerializeIntDelta(*Writer, Value, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 32U);
}

void FPackedInt32NetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	int32& Target = *reinterpret_cast<int32*>(Args.Target);
	const int32 PrevValue = *reinterpret_cast<const int32*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	DeserializeIntDelta(*Reader, Target, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 32U);
}

// FPackedUint32NetSerializer implementation
void FPackedUint32NetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const uint32 Value = *reinterpret_cast<const uint32*>(Args.Source);

	// As we represent the number of bytes to write with two bits we want bits needed to be >= 1
	const uint32 BitCountNeeded = GetBitsNeeded(Value | 1U);
	const uint32 ByteCountNeeded = (BitCountNeeded + 7U)/8U;
	const uint32 BitCountToWrite = ByteCountNeeded*8U;
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(ByteCountNeeded - 1U, 2U);
	Writer->WriteBits(Value, BitCountToWrite);
}

void FPackedUint32NetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const uint32 ByteCountToRead = Reader->ReadBits(2U) + 1U;
	const uint32 BitCountToRead = ByteCountToRead*8U;
	
	const uint32 Value = Reader->ReadBits(BitCountToRead);
		
	uint32* Target = reinterpret_cast<uint32*>(Args.Target);
	*Target = Value;
}

void FPackedUint32NetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const uint32 Value = *reinterpret_cast<const uint32*>(Args.Source);
	const uint32 PrevValue = *reinterpret_cast<const uint32*>(Args.Prev);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	SerializeUintDelta(*Writer, Value, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 32U);
}

void FPackedUint32NetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	uint32& Target = *reinterpret_cast<uint32*>(Args.Target);
	const uint32 PrevValue = *reinterpret_cast<const uint32*>(Args.Prev);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	DeserializeUintDelta(*Reader, Target, PrevValue, DeltaBitCountTable, DeltaBitCountTableEntryCount, 32U);
}

}
