// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/PackedVectorNetSerializers.h"
#include "Iris/Serialization/BitPacking.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Core/BitTwiddling.h"
#include "Math/Vector.h"
#include "Logging/LogMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PackedVectorNetSerializers)

namespace UE::Net::Private
{

// Supports both 64- and 32-bit floating point types.
struct FPackedVectorNetSerializerBase
{
	struct FQuantizedType
	{
		uint64 X;
		uint64 Y;
		uint64 Z;
		/*
		 * Extra info stores whether the value is scaled when ComponentBitCount > 0,
		 * or if the source component type is 64 or 32 bits when ComponentBitCount == 0.
		 * Info: [ExtraInfo][ComponentBitCount]
		 * Bit:          [6][           543210]
		 */
		uint32 ComponentBitCountAndExtraInfo;
		uint32 Unused;
	};

	typedef FVector SourceType;
	typedef FQuantizedType QuantizedType;

	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

protected:
	enum Constants : unsigned
	{
		ComponentBitCountMask = 63U,
		ExtraInfoMask = 64U,
		// Depending on ComponentBitCount the ExtraInfo has different meanings.
		// For ComponentBitCount > 0 the ExtraInfo indicates whether the value is scaled or not.
		IsScaledValueMask = ExtraInfoMask,
		// For ComponentBitCount == 0 the ExtraInfo indicates whether the value is 64 bits per component or not.
		Is64BitScalarType = IsScaledValueMask,
	};

	enum DeltaConstants : uint32
	{
		XDiffersMask = 1U,
		YDiffersMask = 2U,
		ZDiffersMask = 4U,

		XYZDiffersMask = 7U,
		
	};

	static void Serialize(uint32 ScaleBitCount, FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(uint32 ScaleBitCount, FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(uint32 ScaleBitCount, FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(uint32 ScaleBitCount, FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(uint32 ScaleBitCount, FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(uint32 ScaleBitCount, FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(uint32 ScaleBitCount, FNetSerializationContext&, const FNetIsEqualArgs& Args);

	// Round negative values towards -infinity and positive values towards +infinity.
	static int32 RoundFloatToInt(float Value)
	{
		return int32(Value + FPlatformMath::Sign(Value)*0.5f);
	}

	static int64 RoundFloatToInt(double Value)
	{
		return int64(Value + FPlatformMath::Sign(Value)*0.5);
	}

};

template<typename InConfigType, uint32 InScaleBitCount>
struct FVectorNetQuantizeNetSerializerBase : public FPackedVectorNetSerializerBase
{
	static constexpr uint32 Version = 0U;

	typedef InConfigType ConfigType;

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
};

template<typename InConfigType, uint32 ScaleBitCount>
const typename FVectorNetQuantizeNetSerializerBase<InConfigType, ScaleBitCount>::ConfigType FVectorNetQuantizeNetSerializerBase<InConfigType, ScaleBitCount>::DefaultConfig;

}

namespace UE::Net
{

struct FVectorNetQuantizeNetSerializer : public Private::FVectorNetQuantizeNetSerializerBase<FVectorNetQuantizeNetSerializerConfig, 0U>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FVectorNetQuantizeNetSerializer);

// 
struct FVectorNetQuantize10NetSerializer : public Private::FVectorNetQuantizeNetSerializerBase<FVectorNetQuantize10NetSerializerConfig, 3U>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FVectorNetQuantize10NetSerializer);

struct FVectorNetQuantize100NetSerializer : public Private::FVectorNetQuantizeNetSerializerBase<FVectorNetQuantize100NetSerializerConfig, 7U>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FVectorNetQuantize100NetSerializer);

struct FVectorNetQuantizeNormalNetSerializer
{
	static constexpr uint32 Version = 0U;

	struct FQuantizedType
	{
		uint32 X;
		uint32 Y;
		uint32 Z;
		uint32 Unused;
	};

	typedef FVector SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FVectorNetQuantizeNormalNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

private:
	static constexpr uint32 PrecisionBitCount = 16U;
};

const FVectorNetQuantizeNormalNetSerializer::ConfigType FVectorNetQuantizeNormalNetSerializer::DefaultConfig;

UE_NET_IMPLEMENT_SERIALIZER(FVectorNetQuantizeNormalNetSerializer);

}

namespace UE::Net::Private
{

void FPackedVectorNetSerializerBase::Serialize(uint32 ScaleBitCount, FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(Value.ComponentBitCountAndExtraInfo, 7U);

	uint32 ComponentBitCount = Value.ComponentBitCountAndExtraInfo & ComponentBitCountMask;
	if (ComponentBitCount == 0U)
	{
		ComponentBitCount = Value.ComponentBitCountAndExtraInfo & Is64BitScalarType ? 64U : 32U;
	}

	if (ComponentBitCount <= 32U)
	{
		Writer->WriteBits(static_cast<uint32>(Value.X), ComponentBitCount);
		Writer->WriteBits(static_cast<uint32>(Value.Y), ComponentBitCount);
		Writer->WriteBits(static_cast<uint32>(Value.Z), ComponentBitCount);
	}
	else
	{
		const uint32 BitCountForHighBits = ComponentBitCount - 32U;
		Writer->WriteBits(static_cast<uint32>(Value.X), 32U);
		Writer->WriteBits(static_cast<uint32>(Value.X >> 32U), BitCountForHighBits);
		Writer->WriteBits(static_cast<uint32>(Value.Y), 32U);
		Writer->WriteBits(static_cast<uint32>(Value.Y >> 32U), BitCountForHighBits);
		Writer->WriteBits(static_cast<uint32>(Value.Z), 32U);
		Writer->WriteBits(static_cast<uint32>(Value.Z >> 32U), BitCountForHighBits);
	}
}

void FPackedVectorNetSerializerBase::Deserialize(uint32 ScaleBitCount, FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType TempValue = {};

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	TempValue.ComponentBitCountAndExtraInfo = Reader->ReadBits(7U);
	const uint32 ComponentBitCount = TempValue.ComponentBitCountAndExtraInfo & ComponentBitCountMask;
	if (ComponentBitCount > 0)
	{
		if (ComponentBitCount <= 32U)
		{
			TempValue.X = Reader->ReadBits(ComponentBitCount);
			TempValue.Y = Reader->ReadBits(ComponentBitCount);
			TempValue.Z = Reader->ReadBits(ComponentBitCount);
		}
		else
		{
			const uint32 BitCountForHighBits = ComponentBitCount - 32U;
			TempValue.X  = Reader->ReadBits(32U);
			TempValue.X |= static_cast<uint64>(Reader->ReadBits(BitCountForHighBits)) << 32U;
			TempValue.Y  = Reader->ReadBits(32U);
			TempValue.Y |= static_cast<uint64>(Reader->ReadBits(BitCountForHighBits)) << 32U;
			TempValue.Z  = Reader->ReadBits(32U);
			TempValue.Z |= static_cast<uint64>(Reader->ReadBits(BitCountForHighBits)) << 32U;
		}

		// Sign-extend
		const uint64 Mask = (1ULL << (ComponentBitCount - 1U));
		TempValue.X = (TempValue.X ^ Mask) - Mask;
		TempValue.Y = (TempValue.Y ^ Mask) - Mask;
		TempValue.Z = (TempValue.Z ^ Mask) - Mask;
	}
	else
	{
		if (TempValue.ComponentBitCountAndExtraInfo & Is64BitScalarType)
		{
			TempValue.X = Reader->ReadBits(32U);
			TempValue.X |= static_cast<uint64>(Reader->ReadBits(32U)) << 32U;
			TempValue.Y = Reader->ReadBits(32U);
			TempValue.Y |= static_cast<uint64>(Reader->ReadBits(32U)) << 32U;
			TempValue.Z = Reader->ReadBits(32U);
			TempValue.Z |= static_cast<uint64>(Reader->ReadBits(32U)) << 32U;

			FVector3d Vector;
			memcpy(&Vector, &TempValue.X, 3U*sizeof(double)); //-V512
			if (Vector.ContainsNaN())
			{
				// While we detect and zero out nan data at quantize time if we get bad data here it
				// indicates serialization issues and client should disconnect.
				Context.SetError(GNetError_InvalidValue);
				return;
			}
		}
		else
		{
			uint32 Components[3];
			Components[0] = Reader->ReadBits(32U);
			Components[1] = Reader->ReadBits(32U);
			Components[2] = Reader->ReadBits(32U);

			FVector3f Vector;
			memcpy(&Vector, &Components, sizeof(Components));
			if (Vector.ContainsNaN())
			{
				// While we detect and zero out nan data at quantize time if we get bad data here it
				// indicates serialization issues and client should disconnect.
				Context.SetError(GNetError_InvalidValue);
				return;
			}

			TempValue.X = Components[0];
			TempValue.Y = Components[1];
			TempValue.Z = Components[2];
		}
	}

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = TempValue;
}

void FPackedVectorNetSerializerBase::SerializeDelta(uint32 ScaleBitCount, FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	const QuantizedType& PrevValue = *reinterpret_cast<QuantizedType*>(Args.Prev);
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// Try to figure out whether it's worthwhile to delta compress or not.
	const uint32 ExtraInfo = Value.ComponentBitCountAndExtraInfo & ExtraInfoMask;
	const uint32 PrevExtraInfo = PrevValue.ComponentBitCountAndExtraInfo & ExtraInfoMask;

	if (ExtraInfo == PrevExtraInfo)
	{
		const uint32 ComponentBitCount = Value.ComponentBitCountAndExtraInfo & ComponentBitCountMask;
		const uint32 PrevComponentBitCount = PrevValue.ComponentBitCountAndExtraInfo & ComponentBitCountMask;
		// If both values are packed we can see if the integer delta is small.
		if (ComponentBitCount > 0U && PrevComponentBitCount > 0U)
		{
			const int64 DeltaValues[3] = 
			{
				static_cast<int64>(Value.X - PrevValue.X),
				static_cast<int64>(Value.Y - PrevValue.Y),
				static_cast<int64>(Value.Z - PrevValue.Z)
			};

			const uint32 XDiffers = (DeltaValues[0] != 0 ? 1U : 0U);
			const uint32 YDiffers = (DeltaValues[1] != 0 ? 1U : 0U);
			const uint32 ZDiffers = (DeltaValues[2] != 0 ? 1U : 0U);
			const uint32 ChangedComponentCount = XDiffers + YDiffers + ZDiffers;

			const uint32 DeltaComponentBitCount = FPlatformMath::Max(GetBitsNeeded(DeltaValues[0]), FPlatformMath::Max(GetBitsNeeded(DeltaValues[1]), GetBitsNeeded(DeltaValues[2])));

			constexpr uint32 BitCountForComponent = 6U;
			constexpr uint32 BitCountForExtraInfo = 1U;
			constexpr uint32 BitCountForChangedMask = 3U;
			const uint32 EstimatedStandardBitCount = BitCountForComponent +  BitCountForExtraInfo + 3U*ComponentBitCount;
			const uint32 EstimatedCompressedBitCount = (ChangedComponentCount != 0 ? BitCountForComponent : 0U) + BitCountForChangedMask + ChangedComponentCount*DeltaComponentBitCount;
			if (EstimatedCompressedBitCount < EstimatedStandardBitCount)
			{
				constexpr uint32 IsUsingDeltaCompression = 1U;
				const uint32 ChangedComponentsMask = ((XDiffers*XDiffersMask) | (YDiffers*YDiffersMask) | (ZDiffers*ZDiffersMask));

				Writer->WriteBits(IsUsingDeltaCompression, 1U);
				Writer->WriteBits(ChangedComponentsMask, 3U);
				if (ChangedComponentsMask != 0U)
				{
					Writer->WriteBits(DeltaComponentBitCount, 6U);
					if (DeltaComponentBitCount <= 32U)
					{
						for (SIZE_T ValueIndex = 0, ValueEndIndex = 3, ComponentMask = XDiffersMask; ValueIndex != ValueEndIndex; ++ValueIndex, ComponentMask += ComponentMask)
						{
							if (ChangedComponentsMask & ComponentMask)
							{
								Writer->WriteBits(static_cast<uint32>(static_cast<int32>(DeltaValues[ValueIndex])), DeltaComponentBitCount);
							}
						}
					}
					else
					{
						const uint32 BitCountForHighBits = DeltaComponentBitCount - 32U;
						for (SIZE_T ValueIndex = 0, ValueEndIndex = 3, ComponentMask = XDiffersMask; ValueIndex != ValueEndIndex; ++ValueIndex, ComponentMask += ComponentMask)
						{
							if (ChangedComponentsMask & ComponentMask)
							{
								const uint64 UnsignedValue = static_cast<uint64>(DeltaValues[ValueIndex]);
								Writer->WriteBits(static_cast<uint32>(UnsignedValue), 32U);
								Writer->WriteBits(static_cast<uint32>(UnsignedValue >> 32U), BitCountForHighBits);
							}
						}
					}
				}

				// We have serialized the delta compressed data.
				return;
			}
		}
		// Both values are full precision of the same type.
		else if (ComponentBitCount == PrevComponentBitCount)
		{
			// Always say we're using delta compression. This will save at least 3 bits.
			// Check whether any of the components are equal for massive savings.
			const uint32 XDiffers = (Value.X != PrevValue.X ? XDiffersMask : 0U);
			const uint32 YDiffers = (Value.Y != PrevValue.Y ? YDiffersMask : 0U);
			const uint32 ZDiffers = (Value.Z != PrevValue.Z ? ZDiffersMask : 0U);
			const uint32 ChangedComponentsMask = (XDiffers | YDiffers | ZDiffers);

			constexpr uint32 IsUsingDeltaCompression = 1U;
			Writer->WriteBits(IsUsingDeltaCompression, 1U);
			Writer->WriteBits(ChangedComponentsMask, 3U);
			if (ChangedComponentsMask != 0U)
			{
				// 64-bit values
				if (ExtraInfo)
				{
					if (XDiffers)
					{
						Writer->WriteBits(static_cast<uint32>(Value.X), 32U);
						Writer->WriteBits(static_cast<uint32>(Value.X >> 32U), 32U);
					}
					if (YDiffers)
					{
						Writer->WriteBits(static_cast<uint32>(Value.Y), 32U);
						Writer->WriteBits(static_cast<uint32>(Value.Y >> 32U), 32U);
					}
					if (ZDiffers)
					{
						Writer->WriteBits(static_cast<uint32>(Value.Z), 32U);
						Writer->WriteBits(static_cast<uint32>(Value.Z >> 32U), 32U);
					}
				}
				// 32-bit values
				else
				{
					if (XDiffers)
					{
						Writer->WriteBits(static_cast<uint32>(Value.X), 32U);
					}
					if (YDiffers)
					{
						Writer->WriteBits(static_cast<uint32>(Value.Y), 32U);
					}
					if (ZDiffers)
					{
						Writer->WriteBits(static_cast<uint32>(Value.Z), 32U);
					}
				}
			}

			// We have serialized the delta compressed data.
			return;
		}
	}

	// If we end up here we couldn't delta compress.
	{
		Writer->WriteBits(0U, 1U);
		Serialize(ScaleBitCount, Context, Args);
	}
}

void FPackedVectorNetSerializerBase::DeserializeDelta(uint32 ScaleBitCount, FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	if (Reader->ReadBits(1U) == 0)
	{
		return Deserialize(ScaleBitCount, Context, Args);
	}

	const QuantizedType& PrevValue = *reinterpret_cast<QuantizedType*>(Args.Prev);
	QuantizedType TempValue = PrevValue;
	// Handle packed values
	if (PrevValue.ComponentBitCountAndExtraInfo & ComponentBitCountMask)
	{
		const uint32 ChangedComponentsMask = Reader->ReadBits(3U);
		if (ChangedComponentsMask != 0U)
		{
			const uint32 DeltaComponentBitCount = Reader->ReadBits(6U);
			const uint64 Mask = (1ULL << (DeltaComponentBitCount - 1U));
			if (DeltaComponentBitCount <= 32U)
			{
				if (ChangedComponentsMask & XDiffersMask)
				{
					uint64 DX = Reader->ReadBits(DeltaComponentBitCount);
					DX = (DX ^ Mask) - Mask;
					TempValue.X += DX;
				}

				if (ChangedComponentsMask & YDiffersMask)
				{
					uint64 DY = Reader->ReadBits(DeltaComponentBitCount);
					DY = (DY ^ Mask) - Mask;
					TempValue.Y += DY;
				}

				if (ChangedComponentsMask & ZDiffersMask)
				{
					uint64 DZ = Reader->ReadBits(DeltaComponentBitCount);
					DZ = (DZ ^ Mask) - Mask;
					TempValue.Z += DZ;
				}
			}
			else
			{
				const uint32 BitCountForHighBits = DeltaComponentBitCount - 32U;
				if (ChangedComponentsMask & XDiffersMask)
				{
					uint64 DX;
					DX = Reader->ReadBits(32U);
					DX |= static_cast<uint64>(Reader->ReadBits(BitCountForHighBits)) << 32U;
					DX = (DX ^ Mask) - Mask;
					TempValue.X += DX;
				}

				if (ChangedComponentsMask & YDiffersMask)
				{
					uint64 DY;
					DY = Reader->ReadBits(32U);
					DY |= static_cast<uint64>(Reader->ReadBits(BitCountForHighBits)) << 32U;
					DY = (DY ^ Mask) - Mask;
					TempValue.Y += DY;
				}

				if (ChangedComponentsMask & ZDiffersMask)
				{
					uint64 DZ;
					DZ = Reader->ReadBits(32U);
					DZ |= static_cast<uint64>(Reader->ReadBits(BitCountForHighBits)) << 32U;
					DZ = (DZ ^ Mask) - Mask;
					TempValue.Z += DZ;
				}
			}

			// We must re-calculate the bit count needed for the new values of the components
			// in case this data will be serialized later.
			const uint32 NewComponentBitCount = FPlatformMath::Max(GetBitsNeeded(static_cast<int64>(TempValue.X)), FPlatformMath::Max(GetBitsNeeded(static_cast<int64>(TempValue.Y)), GetBitsNeeded(static_cast<int64>(TempValue.Z))));
			TempValue.ComponentBitCountAndExtraInfo = (TempValue.ComponentBitCountAndExtraInfo & ExtraInfoMask) | NewComponentBitCount;
		}
	}
	// Handle full precision values
	else
	{
		const uint32 ChangedComponentsMask = Reader->ReadBits(3U);
		if (PrevValue.ComponentBitCountAndExtraInfo & Is64BitScalarType)
		{
			if (ChangedComponentsMask & XDiffersMask)
			{
				TempValue.X = Reader->ReadBits(32U);
				TempValue.X |= static_cast<uint64>(Reader->ReadBits(32U)) << 32U;
			}
			if (ChangedComponentsMask & YDiffersMask)
			{
				TempValue.Y = Reader->ReadBits(32U);
				TempValue.Y |= static_cast<uint64>(Reader->ReadBits(32U)) << 32U;
			}
			if (ChangedComponentsMask & ZDiffersMask)
			{
				TempValue.Z = Reader->ReadBits(32U);
				TempValue.Z |= static_cast<uint64>(Reader->ReadBits(32U)) << 32U;
			}

			FVector3d Vector;
			memcpy(&Vector, &TempValue.X, 3U*sizeof(double)); //-V512
			if (Vector.ContainsNaN())
			{
				// While we detect and zero out nan data at quantize time if we get bad data here it
				// indicates serialization issues and client should disconnect.
				Context.SetError(GNetError_InvalidValue);
				return;
			}
		}
		else
		{
			uint32 Components[3];
			if (ChangedComponentsMask & XDiffersMask)
			{
				Components[0] = Reader->ReadBits(32U);
			}
			else
			{
				Components[0] = static_cast<uint32>(TempValue.X);
			}

			if (ChangedComponentsMask & YDiffersMask)
			{
				Components[1] = Reader->ReadBits(32U);
			}
			else
			{
				Components[1] = static_cast<uint32>(TempValue.Y);
			}

			if (ChangedComponentsMask & ZDiffersMask)
			{
				Components[2] = Reader->ReadBits(32U);
			}
			else
			{
				Components[2] = static_cast<uint32>(TempValue.Z);
			}

			FVector3f Vector;
			memcpy(&Vector, &Components, sizeof(Components));
			if (Vector.ContainsNaN())
			{
				// While we detect and zero out nan data at quantize time if we get bad data here it
				// indicates serialization issues and client should disconnect.
				Context.SetError(GNetError_InvalidValue);
				return;
			}

			TempValue.X = Components[0];
			TempValue.Y = Components[1];
			TempValue.Z = Components[2];
		}
	}

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = TempValue;
}

void FPackedVectorNetSerializerBase::Quantize(uint32 ScaleBitCount, FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	using ScalarType = decltype(SourceType::X);
	constexpr SIZE_T ScalarTypeSize = sizeof(ScalarType);
	using IntType = typename TSignedIntType<ScalarTypeSize>::Type;
	using UintType = typename TUnsignedIntType<ScalarTypeSize>::Type;

	// Beyond 2^MaxExponentForScaling scaling cannot improve the precision as the next floating point value is at least 1.0 more. 
	constexpr uint32 MaxExponentForScaling = ScalarTypeSize == 4 ? 23U : 52U;
	constexpr ScalarType MaxValueToScale = ScalarType(IntType(1) << MaxExponentForScaling);

	// Rounding of large values can introduce additional precision errors and the extra bandwidth cost to serialize with full precision is small.
	constexpr uint32 MaxExponentAfterScaling = ScalarTypeSize == 4 ? 30U : 62U;
	constexpr ScalarType MaxScaledValue = ScalarType(IntType(1) << MaxExponentAfterScaling);

	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	if (Source.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("%s"), TEXT("PackedVectorNetSerializeBase::Quantize Value isn't finite. Clearing for safety."));		

		constexpr uint32 ComponentBitCountForZeroValue = 1U;

		QuantizedType TempValue = {};
		TempValue.ComponentBitCountAndExtraInfo = IsScaledValueMask | ComponentBitCountForZeroValue;
		Target = TempValue;

		return;
	}

	const ScalarType Scale = ScalarType(IntType(1) << ScaleBitCount);
	
	SourceType ScaledValue;
	// Avoid NaN checks called by FVector operators.
	ScaledValue.X = Source.X*Scale;
	ScaledValue.Y = Source.Y*Scale;
	ScaledValue.Z = Source.Z*Scale;

	if (ScaledValue.GetAbsMax() < MaxScaledValue)
	{
		const bool bUseScaledValue = Source.GetAbsMin() < MaxValueToScale;

		IntType X;
		IntType Y;
		IntType Z;
		if (bUseScaledValue)
		{
			X = RoundFloatToInt(ScaledValue.X);
			Y = RoundFloatToInt(ScaledValue.Y);
			Z = RoundFloatToInt(ScaledValue.Z);
		}
		else
		{
			X = RoundFloatToInt(Source.X);
			Y = RoundFloatToInt(Source.Y);
			Z = RoundFloatToInt(Source.Z);
		}

		const uint32 ComponentBitCount = FPlatformMath::Max(GetBitsNeeded(X), FPlatformMath::Max(GetBitsNeeded(Y), GetBitsNeeded(Z)));

		QuantizedType TempValue = {};
		TempValue.X = static_cast<uint64>(static_cast<int64>(X));
		TempValue.Y = static_cast<uint64>(static_cast<int64>(Y));
		TempValue.Z = static_cast<uint64>(static_cast<int64>(Z));
		TempValue.ComponentBitCountAndExtraInfo = (bUseScaledValue ? IsScaledValueMask : 0U) | ComponentBitCount;

		Target = TempValue;
	}
	else
	{
		// Value needs full precision
		QuantizedType TempValue = {};
		TempValue.X = *reinterpret_cast<const UintType*>(&Source.X);
		TempValue.Y = *reinterpret_cast<const UintType*>(&Source.Y);
		TempValue.Z = *reinterpret_cast<const UintType*>(&Source.Z);
		TempValue.ComponentBitCountAndExtraInfo = (ScalarTypeSize == 8U ? Is64BitScalarType : 0U);

		Target = TempValue;
	}
}

void FPackedVectorNetSerializerBase::Dequantize(uint32 ScaleBitCount, FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	using ScalarType = decltype(SourceType::X);
	using IntType = typename TSignedIntType<sizeof(ScalarType)>::Type;

	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	const uint32 ComponentBitCount = Source.ComponentBitCountAndExtraInfo & ComponentBitCountMask;
	if (ComponentBitCount > 0)
	{
		SourceType TempValue;
		TempValue.X = ScalarType(int64(Source.X));
		TempValue.Y = ScalarType(int64(Source.Y));
		TempValue.Z = ScalarType(int64(Source.Z));

		if (Source.ComponentBitCountAndExtraInfo & IsScaledValueMask)
		{
			const ScalarType InvScale = ScalarType(1)/ScalarType(IntType(1) << ScaleBitCount);
			Target = TempValue*InvScale;
		}
		else
		{
			Target = TempValue;
		}
	}
	else
	{
		if (Source.ComponentBitCountAndExtraInfo & Is64BitScalarType)
		{
			if constexpr (std::is_same<SourceType, FVector3d>::value)
			{
				memcpy(&Target.X, &Source.X, 3U*sizeof(double)); //-V512
			}
			else
			{
				FVector3d Vector;
				memcpy(&Vector.X, &Source.X, 3U*sizeof(double)); //-V512
				Target = SourceType(Vector);
			}
		}
		else
		{
			uint32 Components[3];
			Components[0] = static_cast<uint32>(Source.X);
			Components[1] = static_cast<uint32>(Source.Y);
			Components[2] = static_cast<uint32>(Source.Z);

			if constexpr (std::is_same<SourceType, FVector3f>::value)
			{
				memcpy(&Target.X, &Components[0], 3U*sizeof(float)); //-V512
			}
			else
			{
				FVector3f Vector;
				memcpy(&Vector.X, &Components[0], 3U*sizeof(float)); //-V512
				Target = SourceType(Vector);
			}
		}
	}
}

bool FPackedVectorNetSerializerBase::IsEqual(uint32 ScaleBitCount, FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	QuantizedType QuantizedValue0;
	QuantizedType QuantizedValue1;

	if (Args.bStateIsQuantized)
	{
		QuantizedValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		QuantizedValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
	}
	else
	{
		FNetQuantizeArgs QuantizeArgs = {};
		QuantizeArgs.NetSerializerConfig = Args.NetSerializerConfig;

		QuantizeArgs.Source = NetSerializerValuePointer(Args.Source0);
		QuantizeArgs.Target = NetSerializerValuePointer(&QuantizedValue0);
		Quantize(ScaleBitCount, Context, QuantizeArgs);

		QuantizeArgs.Source = NetSerializerValuePointer(Args.Source1);
		QuantizeArgs.Target = NetSerializerValuePointer(&QuantizedValue1);
		Quantize(ScaleBitCount, Context, QuantizeArgs);
	}

	return ((QuantizedValue0.X == QuantizedValue1.X) & (QuantizedValue0.Y == QuantizedValue1.Y) & (QuantizedValue0.Z == QuantizedValue1.Z) & (QuantizedValue0.ComponentBitCountAndExtraInfo == QuantizedValue1.ComponentBitCountAndExtraInfo));
}

bool FPackedVectorNetSerializerBase::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	// While we can properly send any value we want to be able to inform the user of bad values such as NaN and infinite.
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	return !Source.ContainsNaN();
}

// FVectorNetQuantizeNetSerializerBase implementation
template<typename InConfigType, uint32 ScaleBitCount>
void FVectorNetQuantizeNetSerializerBase<InConfigType, ScaleBitCount>::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	return FPackedVectorNetSerializerBase::Serialize(ScaleBitCount, Context, Args);
}

template<typename InConfigType, uint32 ScaleBitCount>
void FVectorNetQuantizeNetSerializerBase<InConfigType, ScaleBitCount>::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	return FPackedVectorNetSerializerBase::Deserialize(ScaleBitCount, Context, Args);
}

template<typename InConfigType, uint32 ScaleBitCount>
void FVectorNetQuantizeNetSerializerBase<InConfigType, ScaleBitCount>::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	return FPackedVectorNetSerializerBase::SerializeDelta(ScaleBitCount, Context, Args);
}

template<typename InConfigType, uint32 ScaleBitCount>
void FVectorNetQuantizeNetSerializerBase<InConfigType, ScaleBitCount>::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	return FPackedVectorNetSerializerBase::DeserializeDelta(ScaleBitCount, Context, Args);
}

template<typename InConfigType, uint32 ScaleBitCount>
void FVectorNetQuantizeNetSerializerBase<InConfigType, ScaleBitCount>::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	return FPackedVectorNetSerializerBase::Quantize(ScaleBitCount, Context, Args);
}

template<typename InConfigType, uint32 ScaleBitCount>
void FVectorNetQuantizeNetSerializerBase<InConfigType, ScaleBitCount>::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	return FPackedVectorNetSerializerBase::Dequantize(ScaleBitCount, Context, Args);
}

template<typename InConfigType, uint32 ScaleBitCount>
bool FVectorNetQuantizeNetSerializerBase<InConfigType, ScaleBitCount>::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	return FPackedVectorNetSerializerBase::IsEqual(ScaleBitCount, Context, Args);
}

}

namespace UE::Net
{

// FVectorNetQuantizeNormalNetSerializer implementation
void FVectorNetQuantizeNormalNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	SerializeSignedUnitFloat(*Writer, Value.X, PrecisionBitCount);
	SerializeSignedUnitFloat(*Writer, Value.Y, PrecisionBitCount);
	SerializeSignedUnitFloat(*Writer, Value.Z, PrecisionBitCount);
}

void FVectorNetQuantizeNormalNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	QuantizedType TempValue = {};
	TempValue.X = DeserializeSignedUnitFloat(*Reader, PrecisionBitCount);
	TempValue.Y = DeserializeSignedUnitFloat(*Reader, PrecisionBitCount);
	TempValue.Z = DeserializeSignedUnitFloat(*Reader, PrecisionBitCount);

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = TempValue;
}

void FVectorNetQuantizeNormalNetSerializer::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	QuantizedType TempValue = {};
	TempValue.X = QuantizeSignedUnitFloat(static_cast<float>(Source.X), PrecisionBitCount);
	TempValue.Y = QuantizeSignedUnitFloat(static_cast<float>(Source.Y), PrecisionBitCount);
	TempValue.Z = QuantizeSignedUnitFloat(static_cast<float>(Source.Z), PrecisionBitCount);

	Target = TempValue;
}

void FVectorNetQuantizeNormalNetSerializer::Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	SourceType TempValue;
	TempValue.X = DequantizeSignedUnitFloat(Source.X, PrecisionBitCount);
	TempValue.Y = DequantizeSignedUnitFloat(Source.Y, PrecisionBitCount);
	TempValue.Z = DequantizeSignedUnitFloat(Source.Z, PrecisionBitCount);

	Target = TempValue;
}

bool FVectorNetQuantizeNormalNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		// By comparing even the unused bits we allow the compiler to emit more optimized code.
		return ((Value0.X == Value1.X) & (Value0.Y == Value1.Y) & (Value0.Z == Value1.Z) & (Value0.Unused == Value1.Unused));
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		if (QuantizeSignedUnitFloat(static_cast<float>(SourceValue0.X), PrecisionBitCount) != QuantizeSignedUnitFloat(static_cast<float>(SourceValue1.X), PrecisionBitCount))
		{
			return false;
		}

		if (QuantizeSignedUnitFloat(static_cast<float>(SourceValue0.Y), PrecisionBitCount) != QuantizeSignedUnitFloat(static_cast<float>(SourceValue1.Y), PrecisionBitCount))
		{
			return false;
		}

		if (QuantizeSignedUnitFloat(static_cast<float>(SourceValue0.Z), PrecisionBitCount) != QuantizeSignedUnitFloat(static_cast<float>(SourceValue1.Z), PrecisionBitCount))
		{
			return false;
		}

		return true;
	}
}

bool FVectorNetQuantizeNormalNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Value = *reinterpret_cast<SourceType*>(Args.Source);
	if (Value.ContainsNaN())
	{
		return false;
	}

	SourceType TempValue;
	TempValue.X = FMath::Clamp(Value.X, -1.0f, 1.0f);
	TempValue.Y = FMath::Clamp(Value.Y, -1.0f, 1.0f);
	TempValue.Z = FMath::Clamp(Value.Z, -1.0f, 1.0f);

	return TempValue == Value;
}

}
