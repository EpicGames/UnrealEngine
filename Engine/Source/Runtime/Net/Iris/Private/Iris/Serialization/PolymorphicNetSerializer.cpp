// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/PolymorphicNetSerializer.h"
#include "Iris/Serialization/PolymorphicNetSerializerImpl.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "UObject/UObjectIterator.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolymorphicNetSerializer)

namespace UE::Net
{

static bool bDumpPolymorphicTypeRegistry = false;
static FAutoConsoleVariableRef CVarDumpPolymorphicTypeRegistry(
	TEXT("net.Iris.DumpPolymorphicTypeRegistry"),
	bDumpPolymorphicTypeRegistry,
	TEXT("When enabled, logs the full type-to-index mapping for polymorphic net serializer caches whenever they are updated. Useful for diagnosing client/server type index mismatches."),
	ECVF_Default);

const FName NetError_PolymorphicStructNetSerializer_InvalidStructType("Invalid struct type");

FName FPolymorphicNetSerializerScriptStructCache::MakeStructName(const UScriptStruct* ScriptStruct)
{
	if (ScriptStruct == nullptr)
	{
		return NAME_None;
	}
	return FName(*ScriptStruct->GetPathName());
}

void FPolymorphicNetSerializerScriptStructCache::InitForType(const UScriptStruct* InScriptStruct)
{
	IRIS_PROFILER_SCOPE(FPolymorphicNetSerializerScriptStructCache_InitForType);

	TMap<FName, FTypeInfo> UpdatedRegisteredTypes;
	UpdatedRegisteredTypes.Reserve(RegisteredTypes.Num());

	CommonTraits = EReplicationStateTraits::None;

	bool bFoundNewScriptStructs = false;

	// Find all script structs of this type and add them to the list and build descriptor
	// (not sure of a better way to do this but it should only happen at startup)
	TArray<UObject*> Structs;
	Structs.Reserve(EstimatedScriptStructCount);

	constexpr bool bIncludeDerivedClasses = true;
	GetObjectsOfClass(UScriptStruct::StaticClass(), Structs, bIncludeDerivedClasses, RF_ClassDefaultObject, GetObjectIteratorDefaultInternalExclusionFlags(EInternalObjectFlags::None));
	EstimatedScriptStructCount = Structs.Max();
	for (const UObject* Object : Structs)
	{
		const UScriptStruct* Struct = static_cast<const UScriptStruct*>(Object);
		if (Struct->IsChildOf(InScriptStruct))
		{
			FTypeInfo Entry;
			const FName StructName = MakeStructName(Struct);
			Entry.StructName = StructName;
			Entry.ScriptStruct = Struct;

			// See if we already had a descriptor for the struct
			if (const FTypeInfo* ExistingInfo = RegisteredTypes.Find(StructName))
			{
				Entry.Descriptor = ExistingInfo->Descriptor;
			}
			else
			{
				// Get or create descriptor
				FReplicationStateDescriptorBuilder::FParameters Params;
				Entry.Descriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(Struct, Params);
				bFoundNewScriptStructs = true;
			}

			if (Entry.Descriptor.IsValid())
			{
				CommonTraits |= (Entry.Descriptor->Traits & EReplicationStateTraits::HasObjectReference);

				if (FTypeInfo* Existing = UpdatedRegisteredTypes.Find(StructName))
				{
					UE_LOGF(LogIris, Error, "FPolymorphicNetSerializerScriptStructCache::InitForType Duplicate StructName %ls for base %ls (existing %ls, new %ls)",
						ToCStr(Entry.StructName.ToString()),
						ToCStr(InScriptStruct->GetName()),
						ToCStr(Existing->ScriptStruct->GetPathName()),
						ToCStr(Struct->GetPathName()));
				}
				else
				{
					UpdatedRegisteredTypes.Add(StructName, MoveTemp(Entry));
				}
			}
			else
			{
				UE_LOGF(LogIris, Error, "FPolymorphicNetSerializerScriptStructCache::InitForType Failed to create descriptor for type %ls when building cache for base %ls", ToCStr(Struct->GetName()), ToCStr(InScriptStruct->GetName()));
			}
		}
	}

	// Log if we updated the types
	const bool bWasUpdated = (UpdatedRegisteredTypes.Num() != RegisteredTypes.Num()) || bFoundNewScriptStructs;
	UE_CLOGF(bWasUpdated, LogIris, Log, "FPolymorphicNetSerializerScriptStructCache::InitForType Updated CachedTypeNames for base %ls, NumCachedTypes: %d", ToCStr(InScriptStruct->GetName()), UpdatedRegisteredTypes.Num());

	RegisteredTypes = MoveTemp(UpdatedRegisteredTypes);

	if (bWasUpdated && bDumpPolymorphicTypeRegistry)
	{
		UE_LOGF(LogIris, Log, "PolymorphicTypeCache for base %ls: %d types registered", ToCStr(InScriptStruct->GetName()), RegisteredTypes.Num());
		for (const TPair<FName, FTypeInfo>& Pair : RegisteredTypes)
		{
			const FTypeInfo& Info = Pair.Value;
			UE_LOGF(LogIris, Log, "  [%ls] StructName %ls -> %ls (HasObjRef=%ls)",
				ToCStr(InScriptStruct->GetName()),
				ToCStr(Info.StructName.ToString()),
				ToCStr(Info.ScriptStruct->GetName()),
				EnumHasAnyFlags(Info.Descriptor->Traits, EReplicationStateTraits::HasObjectReference) ? TEXT("Yes") : TEXT("No"));
		}
	}
}

}

namespace UE::Net::Private
{

void* FPolymorphicStructNetSerializerInternal::Alloc(FNetSerializationContext& Context, SIZE_T Size, SIZE_T Alignment)
{
	return Context.GetInternalContext()->Alloc(Size, Alignment);
}

void FPolymorphicStructNetSerializerInternal::Free(FNetSerializationContext& Context, void* Ptr)
{
	return Context.GetInternalContext()->Free(Ptr);
}

void FPolymorphicStructNetSerializerInternal::CollectReferences(FNetSerializationContext& Context, FNetReferenceCollector& Collector, const FNetSerializerChangeMaskParam& OuterChangeMaskInfo, const uint8* RESTRICT SrcInternalBuffer,  const FReplicationStateDescriptor* Descriptor)
{
	return FReplicationStateOperationsInternal::CollectReferences(Context, Collector, OuterChangeMaskInfo, SrcInternalBuffer, Descriptor);
}

void FPolymorphicStructNetSerializerInternal::CloneQuantizedState(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor)
{
	return FReplicationStateOperationsInternal::CloneQuantizedState(Context, DstInternalBuffer, SrcInternalBuffer, Descriptor);
}

}
