// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/QuantizedObjectReference.h"

#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/QuantizedRemoteObjectReference.h"
#include "Iris/Serialization/RemoteObjectReferenceNetSerializer.h"

namespace UE::Net
{

bool FQuantizedObjectReference::operator==(const FQuantizedObjectReference& RHS) const
{
	if (!IsRemoteReference() && !RHS.IsRemoteReference())
	{
		return NetReference == RHS.NetReference;
	}
	else if (IsRemoteReference() && RHS.IsRemoteReference())
	{
		return *RemoteReferencePtr == *RHS.RemoteReferencePtr;
	}

	return false;
}

bool FQuantizedObjectReference::IsValid() const
{
	if (IsRemoteReference())
	{
		return RemoteReferencePtr->IsValid();
	}
	else
	{
		return NetReference.IsValid();
	}
}

void FQuantizedObjectReference::FreeRemoteReference(FNetSerializationContext& Context, const FNetSerializerBaseArgs& Args)
{
	if (RemoteReferencePtr)
	{
		FNetFreeDynamicStateArgs InternalArgs;
		InternalArgs.ChangeMaskInfo = Args.ChangeMaskInfo;
		InternalArgs.Version = Args.Version;
		InternalArgs.NetSerializerConfig = UE_NET_GET_SERIALIZER_DEFAULT_CONFIG(FRemoteObjectReferenceNetSerializer);
		InternalArgs.Source = NetSerializerValuePointer(RemoteReferencePtr);
		UE_NET_GET_SERIALIZER(FRemoteObjectReferenceNetSerializer).FreeDynamicState(Context, InternalArgs);

		Context.GetInternalContext()->Free(RemoteReferencePtr);
		RemoteReferencePtr = nullptr;
	}
}

FString FQuantizedObjectReference::ToString() const
{
	if (IsNetReference())
	{
		return NetReference.ToString();
	}
	else
	{
		const FRemoteObjectId TempId = FRemoteObjectId::CreateFromInt(RemoteReferencePtr->ObjectId);
		return FString::Printf(TEXT("Remote: %s"), *TempId.ToString());
	}
}

}
