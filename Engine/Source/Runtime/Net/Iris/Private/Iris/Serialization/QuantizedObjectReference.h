// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Note: This is intended for internal use and should not be included or used outside of NetSerializers having to deal with object references.
 
#include "Iris/Core/NetObjectReference.h"
#include "Templates/IsPODType.h"

namespace UE::Net
{

class FNetSerializationContext;
struct FNetSerializerBaseArgs;

struct FQuantizedRemoteObjectReference;

// The quantized state can hold either an FNetObjectReference or a FQuantizedRemoteObjectReference.
// The FQuantizedRemoteObjectReference is stored as dynamic state in order to reduce required size
// when it's not used.
// If RemoteReferencePtr is null, this state is using the FNetObjectReference.
// If RemoteReferencePtr is non-null, this state is using the remote reference.
struct FQuantizedObjectReference
{
	// Assign an FNetObjectReference to this state. Called from within serializer functions.
	void SetNetReference(FNetSerializationContext& Context, const FNetSerializerBaseArgs& Args, const FNetObjectReference& InNetReference)
	{
		FreeRemoteReference(Context, Args);
		NetReference = InNetReference;
	}

	bool IsNetReference() const
	{
		return !IsRemoteReference();
	}

	bool IsRemoteReference() const
	{
		return RemoteReferencePtr != nullptr;
	}

	bool operator==(const FQuantizedObjectReference& RHS) const;
	bool IsValid() const;
	void FreeRemoteReference(FNetSerializationContext& Context, const FNetSerializerBaseArgs& Args);
	FString ToString() const;

	FNetObjectReference NetReference;
	FQuantizedRemoteObjectReference* RemoteReferencePtr = nullptr;
};

}

template <> struct TIsPODType<UE::Net::FQuantizedObjectReference> { enum { Value = true }; };
