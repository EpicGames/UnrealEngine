// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

namespace UE::Net
{

// Quantized/POD state for FRemoteObjectReference
struct FQuantizedRemoteObjectReference
{
	uint64 ObjectId;
	uint16 ServerId;
	alignas(8) uint8 QuantizedPathNameStruct[32];

	bool IsValid() const
	{
		return ObjectId != 0 && ServerId != 0;
	}

	bool operator==(const FQuantizedRemoteObjectReference& Other) const
	{
		// Equal only if the object itself is the same
		return ObjectId == Other.ObjectId;
	}
};

}
