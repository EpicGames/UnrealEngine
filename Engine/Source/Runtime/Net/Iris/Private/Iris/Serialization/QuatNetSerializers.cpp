// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/QuatNetSerializers.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Math/UnrealMathUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(QuatNetSerializers)

namespace UE::Net::Private
{

template<typename T>
struct FUnitQuatNetSerializerBase
{
	using FloatType = decltype(T::X);
	using UintType = typename TUnsignedIntType<sizeof(FloatType)>::Type;

	enum EQuantizedFlags : uint32
	{
		XIsNotZero = 1U,
		YIsNotZero = XIsNotZero << 1U,
		ZIsNotZero = YIsNotZero << 1U,

		XIsNegative = ZIsNotZero << 1U,
		YIsNegative = XIsNegative << 1U,
		ZIsNegative = YIsNegative << 1U,
		WIsNegative = ZIsNegative << 1U,
	};

	struct FQuantizedData
	{
		UintType X;
		UintType Y;
		UintType Z;
		uint32 Flags;
	};

	typedef T SourceType;
	typedef FQuantizedData QuantizedType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

private:
	static UintType FloatAsUint(FloatType Value);
	static FloatType UintAsFloat(UintType Value);

	static T GetUnitQuat(const T& Value);

	static constexpr uint32 SignificandBitCount = (sizeof(FloatType) == 4U ? 23U : 52U);
	static constexpr UintType FloatOneAsUint = (sizeof(FloatType) == 4U ? 0x3F800000U : 0x3FF0000000000000ULL);
	static constexpr uint32 SignBitShiftAmount = (sizeof(FloatType) == 4U ? 31U : 63U);
	static constexpr UintType SignBit = UintType(1) << SignBitShiftAmount;
};

template<typename T>
inline typename FUnitQuatNetSerializerBase<T>::UintType FUnitQuatNetSerializerBase<T>::FloatAsUint(FloatType Value)
{
	union FFloatAsUint
	{
		FloatType Float;
		UintType Uint;
	};
	
	FFloatAsUint FloatAsUint;
	FloatAsUint.Float = Value;
	return FloatAsUint.Uint;
}

template<typename T>
inline typename FUnitQuatNetSerializerBase<T>::FloatType FUnitQuatNetSerializerBase<T>::UintAsFloat(UintType Value)
{
	union FFloatAsUint
	{
		FloatType Float;
		UintType Uint;
	};

	FFloatAsUint FloatAsUint;
	FloatAsUint.Uint = Value;
	return FloatAsUint.Float;
}

template<typename T>
T FUnitQuatNetSerializerBase<T>::GetUnitQuat(const T& Value)
{
	constexpr FloatType SmallNumber = FloatType(UE_SMALL_NUMBER);
	T UnitQuat = Value;
	if (UnitQuat.SizeSquared() <= SmallNumber)
	{
		UnitQuat = T::Identity;
	}
	else
	{
		// All transmitted quaternions must be unit quaternions, in which case we can deduce the value of W.
		if (!UnitQuat.IsNormalized())
		{
			UnitQuat.Normalize();
		}

		UnitQuat.X = FMath::Clamp(UnitQuat.X, -FloatType(1), FloatType(1));
		UnitQuat.Y = FMath::Clamp(UnitQuat.Y, -FloatType(1), FloatType(1));
		UnitQuat.Z = FMath::Clamp(UnitQuat.Z, -FloatType(1), FloatType(1));
	}

	return UnitQuat;
}

template<typename T>
void FUnitQuatNetSerializerBase<T>::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(Value.Flags, 7U);
	if constexpr (sizeof(FloatType) == 4U)
	{
		if (Value.Flags & EQuantizedFlags::XIsNotZero)
		{
			Writer->WriteBits(Value.X, SignificandBitCount);
		}
		if (Value.Flags & EQuantizedFlags::YIsNotZero)
		{
			Writer->WriteBits(Value.Y, SignificandBitCount);
		}
		if (Value.Flags & EQuantizedFlags::ZIsNotZero)
		{
			Writer->WriteBits(Value.Z, SignificandBitCount);
		}
	}
	else
	{
		constexpr uint32 BitCountForHighBits = SignificandBitCount - 32U;

		if (Value.Flags & EQuantizedFlags::XIsNotZero)
		{
			Writer->WriteBits(static_cast<uint32>(Value.X), 32U);
			Writer->WriteBits(static_cast<uint32>(Value.X >> 32U), BitCountForHighBits);
		}
		if (Value.Flags & EQuantizedFlags::YIsNotZero)
		{
			Writer->WriteBits(static_cast<uint32>(Value.Y), 32U);
			Writer->WriteBits(static_cast<uint32>(Value.Y >> 32U), BitCountForHighBits);
		}
		if (Value.Flags & EQuantizedFlags::ZIsNotZero)
		{
			Writer->WriteBits(static_cast<uint32>(Value.Z), 32U);
			Writer->WriteBits(static_cast<uint32>(Value.Z >> 32U), BitCountForHighBits);
		}
	}
}

template<typename T>
void FUnitQuatNetSerializerBase<T>::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	QuantizedType TempValue = {};
	TempValue.Flags = Reader->ReadBits(7U);
	if constexpr (sizeof(FloatType) == 4U)
	{
		constexpr UintType IncreaseExponent = UintType(1) << SignificandBitCount;

		if (TempValue.Flags & EQuantizedFlags::XIsNotZero)
		{
			TempValue.X = Reader->ReadBits(SignificandBitCount);
			if (TempValue.X == 0)
			{
				TempValue.X = IncreaseExponent;
			}
		}

		if (TempValue.Flags & EQuantizedFlags::YIsNotZero)
		{
			TempValue.Y = Reader->ReadBits(SignificandBitCount);
			if (TempValue.Y == 0)
			{
				TempValue.Y = IncreaseExponent;
			}
		}

		if (TempValue.Flags & EQuantizedFlags::ZIsNotZero)
		{
			TempValue.Z = Reader->ReadBits(SignificandBitCount);
			if (TempValue.Z == 0)
			{
				TempValue.Z = IncreaseExponent;
			}
		}
	}
	else
	{
		constexpr UintType IncreaseExponent = UintType(1) << SignificandBitCount;
		constexpr uint32 BitCountForHighBits = SignificandBitCount - 32U;
		if (TempValue.Flags & EQuantizedFlags::XIsNotZero)
		{
			TempValue.X = Reader->ReadBits(32U);
			TempValue.X |= static_cast<uint64>(Reader->ReadBits(BitCountForHighBits)) << 32U;
			if (TempValue.X == 0)
			{
				TempValue.X = IncreaseExponent;
			}
		}

		if (TempValue.Flags & EQuantizedFlags::YIsNotZero)
		{
			TempValue.Y = Reader->ReadBits(32U);
			TempValue.Y |= static_cast<uint64>(Reader->ReadBits(BitCountForHighBits)) << 32U;
			if (TempValue.Y == 0)
			{
				TempValue.Y = IncreaseExponent;
			}
		}

		if (TempValue.Flags & EQuantizedFlags::ZIsNotZero)
		{
			TempValue.Z = Reader->ReadBits(32U);
			TempValue.Z |= static_cast<uint64>(Reader->ReadBits(BitCountForHighBits)) << 32U;
			if (TempValue.Z == 0)
			{
				TempValue.Z = IncreaseExponent;
			}
		}
	}

	// After adding the integer representation of 1.0f we will have all values in the [1.0f, 2.0f] range when interpreted as floats.
	TempValue.X += FloatOneAsUint;
	TempValue.Y += FloatOneAsUint;
	TempValue.Z += FloatOneAsUint;

	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);
	Value = TempValue;
}

template<typename T>
void FUnitQuatNetSerializerBase<T>::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	SourceType TempSource = Source;
	QuantizedType TempValue = {};

	if (TempSource.SizeSquared() <= SMALL_NUMBER)
	{
		TempSource = SourceType(FloatType(0), FloatType(0), FloatType(0), FloatType(1));
	}
	else
	{
		// All transmitted quaternions must be unit quaternions, in which case we can deduce the value of W.
		if (!TempSource.IsNormalized())
		{
			TempSource.Normalize();
		}

		TempSource.X = FMath::Clamp(TempSource.X, -FloatType(1), FloatType(1));
		TempSource.Y = FMath::Clamp(TempSource.Y, -FloatType(1), FloatType(1));
		TempSource.Z = FMath::Clamp(TempSource.Z, -FloatType(1), FloatType(1));
	}
	
	TempValue.Flags |= EQuantizedFlags::XIsNegative*(FloatAsUint(TempSource.X) >> SignBitShiftAmount);
	TempValue.Flags |= EQuantizedFlags::YIsNegative*(FloatAsUint(TempSource.Y) >> SignBitShiftAmount);
	TempValue.Flags |= EQuantizedFlags::ZIsNegative*(FloatAsUint(TempSource.Z) >> SignBitShiftAmount);
	TempValue.Flags |= EQuantizedFlags::WIsNegative*(FloatAsUint(TempSource.W) >> SignBitShiftAmount);

	// Rebase the X, Y and Z components to end up in the range [1.0, 2.0], which allows us to not replicate the exponent
	// except for a bit to differentiate between 1.0 and 2.0.
	TempSource.X = FGenericPlatformMath::Abs(TempSource.X) + FloatType(1);
	TempSource.Y = FGenericPlatformMath::Abs(TempSource.Y) + FloatType(1);
	TempSource.Z = FGenericPlatformMath::Abs(TempSource.Z) + FloatType(1);

	// If the value is 1.0 after rebasing then we denote that to avoid replicating the significand.
	TempValue.Flags |= EQuantizedFlags::XIsNotZero*(FloatAsUint(TempSource.X) != FloatOneAsUint);
	TempValue.Flags |= EQuantizedFlags::YIsNotZero*(FloatAsUint(TempSource.Y) != FloatOneAsUint);
	TempValue.Flags |= EQuantizedFlags::ZIsNotZero*(FloatAsUint(TempSource.Z) != FloatOneAsUint);

	TempValue.X = FloatAsUint(TempSource.X);
	TempValue.Y = FloatAsUint(TempSource.Y);
	TempValue.Z = FloatAsUint(TempSource.Z);

	Target = TempValue;
}

template<typename T>
void FUnitQuatNetSerializerBase<T>::Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	// Rebase to [0.0, 1.0]
	SourceType TempValue;
	TempValue.X = UintAsFloat(Source.X) - FloatType(1);
	TempValue.Y = UintAsFloat(Source.Y) - FloatType(1);
	TempValue.Z = UintAsFloat(Source.Z) - FloatType(1);
	TempValue.W = FloatType(0);
	// Deduce W
	TempValue.W = FPlatformMath::Sqrt(FMath::Max(FloatType(0), FloatType(1) - TempValue.SizeSquared()));

	// Apply signs.
	static_assert((EQuantizedFlags::XIsNegative >> 3U) == 1U, "EQuantizedFlags are broken.");
	static_assert((EQuantizedFlags::YIsNegative >> 4U) == 1U, "EQuantizedFlags are broken.");
	static_assert((EQuantizedFlags::ZIsNegative >> 5U) == 1U, "EQuantizedFlags are broken.");
	static_assert((EQuantizedFlags::WIsNegative >> 6U) == 1U, "EQuantizedFlags are broken.");

	TempValue.X = UintAsFloat(FloatAsUint(TempValue.X) | (UintType(Source.Flags & EQuantizedFlags::XIsNegative) << (SignBitShiftAmount - 3U)));
	TempValue.Y = UintAsFloat(FloatAsUint(TempValue.Y) | (UintType(Source.Flags & EQuantizedFlags::YIsNegative) << (SignBitShiftAmount - 4U)));
	TempValue.Z = UintAsFloat(FloatAsUint(TempValue.Z) | (UintType(Source.Flags & EQuantizedFlags::ZIsNegative) << (SignBitShiftAmount - 5U)));
	TempValue.W = UintAsFloat(FloatAsUint(TempValue.W) | (UintType(Source.Flags & EQuantizedFlags::WIsNegative) << (SignBitShiftAmount - 6U)));

	Target = TempValue;
}

template<typename T>
bool FUnitQuatNetSerializerBase<T>::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		return ((Value0.X == Value1.X) & (Value0.Y == Value1.Y) & (Value0.Z == Value1.Z) & (Value0.Flags == Value1.Flags));
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		// Lightweight version of the quantization. We do not rebase the values in this case.
		SourceType UnitQuat0 = GetUnitQuat(SourceValue0);
		SourceType UnitQuat1 = GetUnitQuat(SourceValue1);

		return ((UnitQuat0.X == UnitQuat1.X) & (UnitQuat0.Y == UnitQuat1.Y) & (UnitQuat0.Z == UnitQuat1.Z)
			& ((FloatAsUint(UnitQuat0.W) & SignBit) == (FloatAsUint(UnitQuat1.W) & SignBit)));
	}
}

template<typename T>
bool FUnitQuatNetSerializerBase<T>::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Value = *reinterpret_cast<SourceType*>(Args.Source);
	// Require unit quats.
	if (!Value.IsNormalized())
	{
		return false;
	}

	return true; 
}

}

namespace UE::Net
{

static_assert(std::is_same<decltype(FQuat::X), float>::value || std::is_same<decltype(FQuat::X), double>::value, "Unknown floating point type in FQuat.");
static_assert(std::is_same<decltype(FQuat4f::X), float>::value, "Unknown floating point type in FQuat4f.");
static_assert(std::is_same<decltype(FQuat4d::X), double>::value, "Unknown floating point type in FQuat4d.");

struct FUnitQuatNetSerializer : public Private::FUnitQuatNetSerializerBase<FQuat>
{
	static const uint32 Version = 0;

	typedef FUnitQuatNetSerializerConfig ConfigType;
	static const ConfigType DefaultConfig;

};
const FUnitQuatNetSerializer::ConfigType FUnitQuatNetSerializer::DefaultConfig;
UE_NET_IMPLEMENT_SERIALIZER(FUnitQuatNetSerializer);

struct FUnitQuat4fNetSerializer : public Private::FUnitQuatNetSerializerBase<FQuat4f>
{
	static const uint32 Version = 0;

	typedef FUnitQuat4fNetSerializerConfig ConfigType;
	static const ConfigType DefaultConfig;

};
const FUnitQuat4fNetSerializer::ConfigType FUnitQuat4fNetSerializer::DefaultConfig;
UE_NET_IMPLEMENT_SERIALIZER(FUnitQuat4fNetSerializer);

struct FUnitQuat4dNetSerializer : public Private::FUnitQuatNetSerializerBase<FQuat4d>
{
	static const uint32 Version = 0;

	typedef FUnitQuat4dNetSerializerConfig ConfigType;
	static const ConfigType DefaultConfig;

};
const FUnitQuat4dNetSerializer::ConfigType FUnitQuat4dNetSerializer::DefaultConfig;
UE_NET_IMPLEMENT_SERIALIZER(FUnitQuat4dNetSerializer);

}
