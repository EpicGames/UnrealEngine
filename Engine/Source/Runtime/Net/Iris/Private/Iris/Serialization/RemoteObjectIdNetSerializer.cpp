// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteObjectIdNetSerializer.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "UObject/RemoteObjectTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RemoteObjectIdNetSerializer)

namespace UE::Net
{

struct FRemoteObjectIdNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Types
	typedef FRemoteObjectId SourceType;
	typedef uint64 QuantizedType;
	typedef FRemoteObjectIdNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static_assert(sizeof(QuantizedType) == sizeof(FRemoteObjectId::Id), "Quantized RemoteObjectId type is not the correct size.");

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FRemoteObjectIdNetSerializer);

const FRemoteObjectIdNetSerializer::ConfigType FRemoteObjectIdNetSerializer::DefaultConfig;

void FRemoteObjectIdNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	WriteUint64(Writer, Value);
}

void FRemoteObjectIdNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	Value = ReadUint64(Reader);
}

void FRemoteObjectIdNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	FRemoteObjectId GlobalizedId = SourceValue.GetGlobalized();
	TargetValue = GlobalizedId.Id;

	FRemoteObjectId LocalizedId;
	LocalizedId.Id = TargetValue;
	checkf(LocalizedId.GetServerId().Id != (uint32)ERemoteServerIdConstants::Local, TEXT("Local server id should never be serialized"));
}

void FRemoteObjectIdNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	QuantizedType& QuantizedValue = *reinterpret_cast<QuantizedType*>(Args.Source);

	FRemoteObjectId LocalizedId;
	LocalizedId.Id = QuantizedValue;
	checkf(LocalizedId.GetServerId().Id != (uint32)ERemoteServerIdConstants::Local, TEXT("Local server id should never be serialized"));
	TargetValue = LocalizedId.GetLocalized();
}

bool FRemoteObjectIdNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& QuantizedValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& QuantizedValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		return QuantizedValue0 == QuantizedValue1;
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		return SourceValue0 == SourceValue1;
	}
}

bool FRemoteObjectIdNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);

	// Server ID and serial number should be within the valid range
	const bool bServerIdValid = SourceValue.GetServerId().GetIdNumber() <= MAX_REMOTE_OBJECT_SERVER_ID;
	const bool bSerialNumberValid = SourceValue.SerialNumber <= MAX_REMOTE_OBJECT_SERIAL_NUMBER;

	return bServerIdValid && bSerialNumberValid;
}

}
