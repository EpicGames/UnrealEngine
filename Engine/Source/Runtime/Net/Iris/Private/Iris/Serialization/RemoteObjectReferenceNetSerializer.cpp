// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteObjectReferenceNetSerializer.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/QuantizedRemoteObjectReference.h"
#include "UObject/RemoteObjectTransfer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RemoteObjectReferenceNetSerializer)

namespace UE::Net
{

struct FRemoteObjectReferenceNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bHasCustomNetReference = true;

	// Types
	typedef FRemoteObjectReference SourceType;
	typedef FQuantizedRemoteObjectReference QuantizedType;
	typedef FRemoteObjectReferenceNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static_assert(sizeof(QuantizedType::ServerId) * 8 >= REMOTE_OBJECT_SERVER_ID_BIT_SIZE, "Quantized ServerId is not large enough to store maximum server ID");

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	private:
		virtual void OnPostFreezeNetSerializerRegistry() override;
	};

	static FRemoteObjectReferenceNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	static FStructNetSerializerConfig RemoteObjectPathNameNetSerializerConfig;
	static const FNetSerializer* RemoteObjectPathNameNetSerializer;
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FRemoteObjectReferenceNetSerializer);

const FRemoteObjectReferenceNetSerializer::ConfigType FRemoteObjectReferenceNetSerializer::DefaultConfig;

FRemoteObjectReferenceNetSerializer::FNetSerializerRegistryDelegates FRemoteObjectReferenceNetSerializer::NetSerializerRegistryDelegates;
FStructNetSerializerConfig FRemoteObjectReferenceNetSerializer::RemoteObjectPathNameNetSerializerConfig;
const FNetSerializer* FRemoteObjectReferenceNetSerializer::RemoteObjectPathNameNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);

void FRemoteObjectReferenceNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	static_assert(sizeof(Value.ObjectId) == 8, "Size of ObjectId expected to be 8 bytes");
	Writer->WriteBits(static_cast<uint32>(Value.ObjectId), 32U);
	Writer->WriteBits(static_cast<uint32>(Value.ObjectId >> 32U), 32U);

	Writer->WriteBits(Value.ServerId, REMOTE_OBJECT_SERVER_ID_BIT_SIZE);

	// Forward serialization of the FRemoteObjectPathName to struct serializer
	FNetSerializeArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &RemoteObjectPathNameNetSerializerConfig;
	InternalArgs.Source = NetSerializerValuePointer(&Value.QuantizedPathNameStruct[0]);
	RemoteObjectPathNameNetSerializer->Serialize(Context, InternalArgs);
}

void FRemoteObjectReferenceNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	Value.ObjectId = Reader->ReadBits(32U);
	Value.ObjectId |= static_cast<uint64>(Reader->ReadBits(32U)) << 32U;

	Value.ServerId = static_cast<uint16>(Reader->ReadBits(REMOTE_OBJECT_SERVER_ID_BIT_SIZE));

	FNetDeserializeArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &RemoteObjectPathNameNetSerializerConfig;
	InternalArgs.Target = NetSerializerValuePointer(&Value.QuantizedPathNameStruct[0]);
	RemoteObjectPathNameNetSerializer->Deserialize(Context, InternalArgs);
}

void FRemoteObjectReferenceNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	static_assert(sizeof(Value.ObjectId) == 8, "Size of ObjectId expected to be 8 bytes");
	Writer->WriteBits(static_cast<uint32>(Value.ObjectId), 32U);
	Writer->WriteBits(static_cast<uint32>(Value.ObjectId >> 32U), 32U);

	Writer->WriteBits(Value.ServerId, REMOTE_OBJECT_SERVER_ID_BIT_SIZE);

	// Forward serialization of the FRemoteObjectPathName to struct serializer
	FNetSerializeDeltaArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &RemoteObjectPathNameNetSerializerConfig;
	InternalArgs.Source = NetSerializerValuePointer(&Value.QuantizedPathNameStruct[0]);
	RemoteObjectPathNameNetSerializer->SerializeDelta(Context, InternalArgs);
}

void FRemoteObjectReferenceNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	Value.ObjectId = Reader->ReadBits(32U);
	Value.ObjectId |= static_cast<uint64>(Reader->ReadBits(32U)) << 32U;

	Value.ServerId = static_cast<uint16>(Reader->ReadBits(REMOTE_OBJECT_SERVER_ID_BIT_SIZE));

	FNetDeserializeDeltaArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &RemoteObjectPathNameNetSerializerConfig;
	InternalArgs.Target = NetSerializerValuePointer(&Value.QuantizedPathNameStruct[0]);
	RemoteObjectPathNameNetSerializer->DeserializeDelta(Context, InternalArgs);
}

void FRemoteObjectReferenceNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	TargetValue.ObjectId = SourceValue.GetRemoteId().GetIdNumber();
	TargetValue.ServerId = static_cast<uint16>(SourceValue.GetSharingServerId().GetIdNumber());

	FRemoteObjectPathName PathName;
	if (SourceValue.GetRemoteId().IsValid())
	{
		if (UObject* ExistingObject = StaticFindObjectFastInternal(SourceValue.GetRemoteId()))
		{
			UE::RemoteObject::Transfer::RegisterSharedObject(ExistingObject);
			PathName = FRemoteObjectPathName(ExistingObject);
		}
	}

	FNetQuantizeArgs InternalArgs = Args;
	InternalArgs.Source = NetSerializerValuePointer(&PathName);
	InternalArgs.Target = NetSerializerValuePointer(&TargetValue.QuantizedPathNameStruct[0]);
	InternalArgs.NetSerializerConfig = &RemoteObjectPathNameNetSerializerConfig;
	return RemoteObjectPathNameNetSerializer->Quantize(Context, InternalArgs);
}

void FRemoteObjectReferenceNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	QuantizedType& QuantizedValue = *reinterpret_cast<QuantizedType*>(Args.Source);

	FRemoteObjectId ObjectId;
	ObjectId.Id = QuantizedValue.ObjectId;

	FRemoteObjectPathName PathName;

	FNetDequantizeArgs InternalArgs = Args;
	InternalArgs.Source = NetSerializerValuePointer(&QuantizedValue.QuantizedPathNameStruct[0]);
	InternalArgs.Target = NetSerializerValuePointer(&PathName);
	InternalArgs.NetSerializerConfig = &RemoteObjectPathNameNetSerializerConfig;

	RemoteObjectPathNameNetSerializer->Dequantize(Context, InternalArgs);

	TargetValue.NetDequantize(ObjectId.GetLocalized(), FRemoteServerId::FromIdNumber(QuantizedValue.ServerId).GetLocalized(), PathName);
}

bool FRemoteObjectReferenceNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& QuantizedValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& QuantizedValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		return QuantizedValue0 == QuantizedValue1;
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		return SourceValue0 == SourceValue1;
	}
}

bool FRemoteObjectReferenceNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);

	// The remote reference's ObjectId and ServerId should either both be valid or both be invalid
	return SourceValue.GetRemoteId().IsValid() == SourceValue.GetSharingServerId().IsValid();
}

void FRemoteObjectReferenceNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	// Remote references don't need to go through this collection process.
}

void FRemoteObjectReferenceNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	QuantizedType& SourceValue = *reinterpret_cast<QuantizedType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	TargetValue.ObjectId = SourceValue.ObjectId;
	TargetValue.ServerId = SourceValue.ServerId;

	FNetCloneDynamicStateArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &RemoteObjectPathNameNetSerializerConfig;
	InternalArgs.Source = NetSerializerValuePointer(&SourceValue.QuantizedPathNameStruct[0]);
	InternalArgs.Target = NetSerializerValuePointer(&TargetValue.QuantizedPathNameStruct[0]);
	return RemoteObjectPathNameNetSerializer->CloneDynamicState(Context, InternalArgs);
}

void FRemoteObjectReferenceNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& SourceValue = *reinterpret_cast<QuantizedType*>(Args.Source);

	FNetFreeDynamicStateArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &RemoteObjectPathNameNetSerializerConfig;
	InternalArgs.Source = NetSerializerValuePointer(&SourceValue.QuantizedPathNameStruct[0]);
	return RemoteObjectPathNameNetSerializer->FreeDynamicState(Context, InternalArgs);
}

void FRemoteObjectReferenceNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	UStruct* Struct = TBaseStructure<FRemoteObjectPathName>::Get();
	RemoteObjectPathNameNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(Struct);
	const FReplicationStateDescriptor* Descriptor = RemoteObjectPathNameNetSerializerConfig.StateDescriptor.GetReference();
	check(Descriptor != nullptr);

	// Validate our assumptions regarding quantized state size and alignment.
	static_assert(offsetof(QuantizedType, QuantizedPathNameStruct) == 16U, "Expected buffer for struct to be at offset 16 of QuantizedType.");
	if (sizeof(QuantizedType::QuantizedPathNameStruct) < Descriptor->InternalSize || alignof(QuantizedType) < Descriptor->InternalAlignment)
	{
		LowLevelFatalError(TEXT("QuantizedType::QuantizedStruct for FRemoteObjectReferenceNetSerializer has size %u and alignment %u but requires size %u and alignment %u."), uint32(sizeof(QuantizedType::QuantizedPathNameStruct)), uint32(alignof(QuantizedType)), uint32(Descriptor->InternalSize), uint32(Descriptor->InternalAlignment));
	}
}

}
