// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteServerIdNetSerializer.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "UObject/RemoteObjectTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RemoteServerIdNetSerializer)

namespace UE::Net
{

struct FRemoteServerIdNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Types
	typedef FRemoteServerId SourceType;
	typedef uint16 QuantizedType;
	typedef FRemoteServerIdNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static_assert(sizeof(QuantizedType) * 8U >= REMOTE_OBJECT_SERVER_ID_BIT_SIZE, "Quantized ServerId is not large enough to store maximum server ID");

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FRemoteServerIdNetSerializer);

const FRemoteServerIdNetSerializer::ConfigType FRemoteServerIdNetSerializer::DefaultConfig;

void FRemoteServerIdNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	Writer->WriteBits(Value, REMOTE_OBJECT_SERVER_ID_BIT_SIZE);
}

void FRemoteServerIdNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	Value = static_cast<uint16>(Reader->ReadBits(REMOTE_OBJECT_SERVER_ID_BIT_SIZE));
}

void FRemoteServerIdNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	FRemoteServerId GlobalizedId = SourceValue.GetGlobalized();
	TargetValue = static_cast<QuantizedType>(GlobalizedId.Id);

	checkf(TargetValue != static_cast<QuantizedType>(ERemoteServerIdConstants::Local), TEXT("Local server id should never be serialized"));
}

void FRemoteServerIdNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	QuantizedType& QuantizedValue = *reinterpret_cast<QuantizedType*>(Args.Source);

	checkf(QuantizedValue != static_cast<QuantizedType>(ERemoteServerIdConstants::Local), TEXT("Local server id should never be serialized"));

	FRemoteServerId LocalizedId;
	LocalizedId.Id = QuantizedValue;
	TargetValue = LocalizedId.GetLocalized();
}

bool FRemoteServerIdNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& QuantizedValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& QuantizedValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		return QuantizedValue0 == QuantizedValue1;
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		return SourceValue0 == SourceValue1;
	}
}

bool FRemoteServerIdNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);

	// Server IDs should be within the valid range
	return SourceValue.GetIdNumber() <= MAX_REMOTE_OBJECT_SERVER_ID;
}

}
