// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/InternalNetSerializer.h"
#include "RemoteServerIdNetSerializer.generated.h"

USTRUCT()
struct FRemoteServerIdNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{
	UE_NET_DECLARE_SERIALIZER_INTERNAL(FRemoteServerIdNetSerializer, IRISCORE_API);
}
