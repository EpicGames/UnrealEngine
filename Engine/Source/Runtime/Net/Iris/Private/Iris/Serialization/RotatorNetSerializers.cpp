// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/RotatorNetSerializers.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Math/Rotator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RotatorNetSerializers)

namespace UE::Net
{

template<typename RotatorType>
struct FRotatorAsShortNetSerializerBase
{
	struct FQuantizedType
	{
		// Using three bits to indicate whether the components are zero or not.
		uint16 XYZIsNotZero;
		uint16 X;
		uint16 Y;
		uint16 Z;
	};

	typedef RotatorType SourceType;
	typedef FQuantizedType QuantizedType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

protected:
	using ScalarType = decltype(SourceType::Pitch);

	enum Constants : uint16
	{
		XDiffersMask = 1U,
		YDiffersMask = 2U,
		ZDiffersMask = 4U,

		XYZDiffersMask = 7U,
	};
};

template<typename RotatorType>
struct FRotatorAsByteNetSerializerBase
{
	struct FQuantizedType
	{
		// Using three bits to indicate whether the components are zero or not.
		uint8 XYZIsNotZero;
		uint8 X;
		uint8 Y;
		uint8 Z;
	};

	typedef RotatorType SourceType;
	typedef FQuantizedType QuantizedType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

protected:
	using ScalarType = decltype(SourceType::Pitch);

	enum Constants : uint16
	{
		XDiffersMask = 1U,
		YDiffersMask = 2U,
		ZDiffersMask = 4U,

		XYZDiffersMask = 7U,
	};
};

// FRotatorAsShortNetSerializerBase implementation
template<typename T>
void FRotatorAsShortNetSerializerBase<T>::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(Value.XYZIsNotZero, 3U);
	if (Value.XYZIsNotZero & XDiffersMask)
	{
		Writer->WriteBits(Value.X, 16U);
	}
	if (Value.XYZIsNotZero & YDiffersMask)
	{
		Writer->WriteBits(Value.Y, 16U);
	}
	if (Value.XYZIsNotZero & ZDiffersMask)
	{
		Writer->WriteBits(Value.Z, 16U);
	}
}

template<typename T>
void FRotatorAsShortNetSerializerBase<T>::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	QuantizedType Value;
	const uint16 XYZIsNotZero = static_cast<uint16>(Reader->ReadBits(3U));
	Value.XYZIsNotZero = XYZIsNotZero;
	Value.X = static_cast<uint16>(XYZIsNotZero & XDiffersMask ? Reader->ReadBits(16U) : 0U);
	Value.Y = static_cast<uint16>(XYZIsNotZero & YDiffersMask ? Reader->ReadBits(16U) : 0U);
	Value.Z = static_cast<uint16>(XYZIsNotZero & ZDiffersMask ? Reader->ReadBits(16U) : 0U);

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = Value;
}

template<typename T>
void FRotatorAsShortNetSerializerBase<T>::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	// Per component equality. With the current scaling a 10 degree change would require 12 bits to replicate
	// due to 11 bits for the value and 1 for the sign. Add a small lookup table index for that and you're
	// up to 14 bits instead of 1+16 bits.

	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	const QuantizedType& PrevValue = *reinterpret_cast<QuantizedType*>(Args.Prev);

	const uint16 DX = Value.X - PrevValue.X;
	const uint16 DY = Value.Y - PrevValue.Y;
	const uint16 DZ = Value.Z - PrevValue.Z;

	uint32 XYZDiffers = 0;
	XYZDiffers |= (DX != 0) ? uint32(XDiffersMask) : uint32(0);
	XYZDiffers |= (DY != 0) ? uint32(YDiffersMask) : uint32(0);
	XYZDiffers |= (DZ != 0) ? uint32(ZDiffersMask) : uint32(0);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(XYZDiffers, 3U);
	if (XYZDiffers & XDiffersMask)
	{
		Writer->WriteBits(DX, 16U);
	}
	if (XYZDiffers & YDiffersMask)
	{
		Writer->WriteBits(DY, 16U);
	}
	if (XYZDiffers & ZDiffersMask)
	{
		Writer->WriteBits(DZ, 16U);
	}
}

template<typename T>
void FRotatorAsShortNetSerializerBase<T>::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const QuantizedType& PrevValue = *reinterpret_cast<QuantizedType*>(Args.Prev);
	QuantizedType TempValue = PrevValue;

	const uint32 XYZDiffers = Reader->ReadBits(3U);
	if (XYZDiffers & XDiffersMask)
	{
		const uint16 DX = static_cast<uint16>(Reader->ReadBits(16U));
		TempValue.X += DX;
	}
	if (XYZDiffers & YDiffersMask)
	{
		const uint16 DY = static_cast<uint16>(Reader->ReadBits(16U));
		TempValue.Y += DY;
	}
	if (XYZDiffers & ZDiffersMask)
	{
		const uint16 DZ = static_cast<uint16>(Reader->ReadBits(16U));
		TempValue.Z += DZ;
	}

	// Reconstruct flags
	uint16 XYZIsNotZero = 0U;
	XYZIsNotZero |= (TempValue.X != 0 ? XDiffersMask : 0U);
	XYZIsNotZero |= (TempValue.Y != 0 ? YDiffersMask : 0U);
	XYZIsNotZero |= (TempValue.Z != 0 ? ZDiffersMask : 0U);
	TempValue.XYZIsNotZero = XYZIsNotZero;

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = TempValue;
}

template<typename T>
void FRotatorAsShortNetSerializerBase<T>::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	QuantizedType TempValue = {};

	TempValue.X = SourceType::CompressAxisToShort(Source.Pitch);
	TempValue.Y = SourceType::CompressAxisToShort(Source.Yaw);
	TempValue.Z = SourceType::CompressAxisToShort(Source.Roll);

	TempValue.XYZIsNotZero |= (TempValue.X != 0) ? XDiffersMask : uint16(0);
	TempValue.XYZIsNotZero |= (TempValue.Y != 0) ? YDiffersMask : uint16(0);
	TempValue.XYZIsNotZero |= (TempValue.Z != 0) ? ZDiffersMask : uint16(0);

	Target = TempValue;
}

template<typename T>
void FRotatorAsShortNetSerializerBase<T>::Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	SourceType TempValue;

	TempValue.Pitch = SourceType::DecompressAxisFromShort(Source.X);
	TempValue.Yaw = SourceType::DecompressAxisFromShort(Source.Y);
	TempValue.Roll = SourceType::DecompressAxisFromShort(Source.Z);

	Target = TempValue;
}

template<typename T>
bool FRotatorAsShortNetSerializerBase<T>::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& SourceValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& SourceValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		const bool bXIsEqual = (SourceValue0.X == SourceValue1.X);
		const bool bYIsEqual = (SourceValue0.Y == SourceValue1.Y);
		const bool bZIsEqual = (SourceValue0.Z == SourceValue1.Z);
		const bool bFlagsAreEqual = (SourceValue0.XYZIsNotZero == SourceValue1.XYZIsNotZero);
		return bXIsEqual & bYIsEqual & bZIsEqual & bFlagsAreEqual;
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		const bool bPitchIsEqual = (SourceValue0.Pitch == SourceValue1.Pitch);
		const bool bYawIsEqual = (SourceValue0.Yaw == SourceValue1.Yaw);
		const bool bRollIsEqual = (SourceValue0.Roll == SourceValue1.Roll);
		return bPitchIsEqual & bYawIsEqual & bRollIsEqual;
	}
}

template<typename T>
bool FRotatorAsShortNetSerializerBase<T>::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Value = *reinterpret_cast<SourceType*>(Args.Source);
	return !Value.ContainsNaN();
}

// FRotatorAsByteNetSerializerBase implementation
template<typename T>
void FRotatorAsByteNetSerializerBase<T>::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(Value.XYZIsNotZero, 3U);
	if (Value.XYZIsNotZero & XDiffersMask)
	{
		Writer->WriteBits(Value.X, 8U);
	}
	if (Value.XYZIsNotZero & YDiffersMask)
	{
		Writer->WriteBits(Value.Y, 8U);
	}
	if (Value.XYZIsNotZero & ZDiffersMask)
	{
		Writer->WriteBits(Value.Z, 8U);
	}
}

template<typename T>
void FRotatorAsByteNetSerializerBase<T>::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	QuantizedType Value;
	const uint8 XYZIsNotZero = static_cast<uint8>(Reader->ReadBits(3U));
	Value.XYZIsNotZero = XYZIsNotZero;
	Value.X = static_cast<uint8>(XYZIsNotZero & XDiffersMask ? Reader->ReadBits(8U) : 0U);
	Value.Y = static_cast<uint8>(XYZIsNotZero & YDiffersMask ? Reader->ReadBits(8U) : 0U);
	Value.Z = static_cast<uint8>(XYZIsNotZero & ZDiffersMask ? Reader->ReadBits(8U) : 0U);

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = Value;
}

template<typename T>
void FRotatorAsByteNetSerializerBase<T>::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	const QuantizedType& PrevValue = *reinterpret_cast<QuantizedType*>(Args.Prev);

	uint32 XYZDiffers = 0;
	XYZDiffers |= (Value.X != PrevValue.X) ? uint32(XDiffersMask) : uint32(0);
	XYZDiffers |= (Value.Y != PrevValue.Y) ? uint32(YDiffersMask) : uint32(0);
	XYZDiffers |= (Value.Z != PrevValue.Z) ? uint32(ZDiffersMask) : uint32(0);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(XYZDiffers, 3U);
	if (XYZDiffers & XDiffersMask)
	{
		Writer->WriteBits(Value.X, 8U);
	}
	if (XYZDiffers & YDiffersMask)
	{
		Writer->WriteBits(Value.Y, 8U);
	}
	if (XYZDiffers & ZDiffersMask)
	{
		Writer->WriteBits(Value.Z, 8U);
	}
}

template<typename T>
void FRotatorAsByteNetSerializerBase<T>::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	const QuantizedType& PrevValue = *reinterpret_cast<QuantizedType*>(Args.Prev);
	QuantizedType TempValue = PrevValue;

	const uint32 XYZDiffers = Reader->ReadBits(3U);
	if (XYZDiffers & XDiffersMask)
	{
		TempValue.X = static_cast<uint8>(Reader->ReadBits(8U));
	}
	if (XYZDiffers & YDiffersMask)
	{
		TempValue.Y = static_cast<uint8>(Reader->ReadBits(8U));
	}
	if (XYZDiffers & ZDiffersMask)
	{
		TempValue.Z = static_cast<uint8>(Reader->ReadBits(8U));
	}

	// Reconstruct flags
	uint8 XYZIsNotZero = 0U;
	XYZIsNotZero |= static_cast<uint8>(TempValue.X != 0 ? XDiffersMask : 0U);
	XYZIsNotZero |= static_cast<uint8>(TempValue.Y != 0 ? YDiffersMask : 0U);
	XYZIsNotZero |= static_cast<uint8>(TempValue.Z != 0 ? ZDiffersMask : 0U);
	TempValue.XYZIsNotZero = XYZIsNotZero;

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	Target = TempValue;
}

template<typename T>
void FRotatorAsByteNetSerializerBase<T>::Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	QuantizedType TempValue = {};

	TempValue.X = SourceType::CompressAxisToByte(Source.Pitch);
	TempValue.Y = SourceType::CompressAxisToByte(Source.Yaw);
	TempValue.Z = SourceType::CompressAxisToByte(Source.Roll);

	TempValue.XYZIsNotZero |= (TempValue.X != 0) ? XDiffersMask : uint8(0);
	TempValue.XYZIsNotZero |= (TempValue.Y != 0) ? YDiffersMask : uint8(0);
	TempValue.XYZIsNotZero |= (TempValue.Z != 0) ? ZDiffersMask : uint8(0);

	Target = TempValue;
}

template<typename T>
void FRotatorAsByteNetSerializerBase<T>::Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	SourceType TempValue;
	TempValue.Pitch = SourceType::DecompressAxisFromByte(Source.X);
	TempValue.Yaw = SourceType::DecompressAxisFromByte(Source.Y);
	TempValue.Roll = SourceType::DecompressAxisFromByte(Source.Z);

	Target = TempValue;
}

template<typename T>
bool FRotatorAsByteNetSerializerBase<T>::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& SourceValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& SourceValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		const bool bXIsEqual = (SourceValue0.X == SourceValue1.X);
		const bool bYIsEqual = (SourceValue0.Y == SourceValue1.Y);
		const bool bZIsEqual = (SourceValue0.Z == SourceValue1.Z);
		const bool bFlagsAreEqual = (SourceValue0.XYZIsNotZero == SourceValue1.XYZIsNotZero);
		return bXIsEqual & bYIsEqual & bZIsEqual & bFlagsAreEqual;
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		const bool bPitchIsEqual = (SourceValue0.Pitch == SourceValue1.Pitch);
		const bool bYawIsEqual = (SourceValue0.Yaw == SourceValue1.Yaw);
		const bool bRollIsEqual = (SourceValue0.Roll == SourceValue1.Roll);
		return bPitchIsEqual & bYawIsEqual & bRollIsEqual;
	}
}

template<typename T>
bool FRotatorAsByteNetSerializerBase<T>::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Value = *reinterpret_cast<SourceType*>(Args.Source);
	return !Value.ContainsNaN();
}

struct FRotatorNetSerializer : public FRotatorAsShortNetSerializerBase<FRotator>
{
	static constexpr uint32 Version = 0;

	typedef FRotatorNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;
};
UE_NET_IMPLEMENT_SERIALIZER(FRotatorNetSerializer);

struct FRotatorAsByteNetSerializer : public FRotatorAsByteNetSerializerBase<FRotator>
{
	static constexpr uint32 Version = 0;

	typedef FRotatorAsByteNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;
};
UE_NET_IMPLEMENT_SERIALIZER(FRotatorAsByteNetSerializer);

struct FRotatorAsShortNetSerializer : public FRotatorAsShortNetSerializerBase<FRotator>
{
	static constexpr uint32 Version = 0;

	typedef FRotatorAsShortNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;
};
UE_NET_IMPLEMENT_SERIALIZER(FRotatorAsShortNetSerializer);

struct FRotator3fNetSerializer : public FRotatorAsShortNetSerializerBase<FRotator3f>
{
	static constexpr uint32 Version = 0;

	typedef FRotator3fNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;
};
UE_NET_IMPLEMENT_SERIALIZER(FRotator3fNetSerializer);

struct FRotator3dNetSerializer : public FRotatorAsShortNetSerializerBase<FRotator3d>
{
	static constexpr uint32 Version = 0;

	typedef FRotator3dNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;
};
UE_NET_IMPLEMENT_SERIALIZER(FRotator3dNetSerializer);

}
