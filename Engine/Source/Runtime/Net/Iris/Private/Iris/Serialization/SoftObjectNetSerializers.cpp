// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/SoftObjectNetSerializers.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/Serialization/InternalNetSerializer.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Iris/Serialization/QuantizedObjectReference.h"
#include "Iris/Serialization/StringNetSerializerUtils.h"
#include "Templates/IsPODType.h"
#include "UObject/SoftObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoftObjectNetSerializers)


namespace UE::Net
{

struct FFSoftObjectNetSerializerQuantizedData
{
	Private::FStringNetSerializerBase::QuantizedType Path;
	FQuantizedObjectReference ObjectReference;
	bool bIsObject;
	uint8 Padding[7];
};

}

template<> struct TIsPODType<UE::Net::FFSoftObjectNetSerializerQuantizedData> { enum { Value = true }; };

namespace UE::Net
{

struct FSoftObjectNetSerializer : public Private::FStringNetSerializerBase
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true;
	static constexpr bool bHasCustomNetReference = true;

	// Types
	typedef FSoftObjectPtr SourceType;
	typedef FFSoftObjectNetSerializerQuantizedData QuantizedType;
	typedef FSoftObjectNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	// Implementation
	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);
	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	typedef Private::FStringNetSerializerBase Super;

	inline static const FNetSerializer* ObjectNetSerializer = &UE_NET_GET_SERIALIZER(FObjectNetSerializer);
	inline static const FNetSerializerConfig* ObjectNetSerializerConfig = UE_NET_GET_SERIALIZER_DEFAULT_CONFIG(FObjectNetSerializer);
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FSoftObjectNetSerializer);

struct FSoftObjectPathNetSerializer : public Private::FStringNetSerializerBase
{
	// Version
	static const uint32 Version = 0;

	// Types
	typedef FSoftObjectPath SourceType;
	typedef FSoftObjectPathNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	// Implementation
	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FSoftObjectPathNetSerializer);

struct FSoftClassPathNetSerializer : public Private::FStringNetSerializerBase
{
	// Version
	static const uint32 Version = 0;

	// Types
	typedef FSoftClassPath SourceType;
	typedef FSoftClassPathNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	// Implementation
	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FSoftClassPathNetSerializer);
const FSoftClassPathNetSerializer::ConfigType FSoftClassPathNetSerializer::DefaultConfig;

// FSoftObjectNetSerializer
void FSoftObjectNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	QuantizedType TempValue = {};

	const UObject* Object = Source.Get();
	// Use path whenever possible. Only resort to object serialization for non-stably named objects.
	const bool bIsObject = Object && !Object->IsFullNameStableForNetworking();
	if (bIsObject && !Target.bIsObject)
	{
		FNetFreeDynamicStateArgs FreeArgs;
		FreeArgs.Source = reinterpret_cast<NetSerializerValuePointer>(&Target.Path);
		Super::FreeDynamicState(Context, FreeArgs);
	}

	TempValue.bIsObject = bIsObject;
	if (bIsObject)
	{
		FNetQuantizeArgs QuantizeArgs = Args;
		QuantizeArgs.NetSerializerConfig = reinterpret_cast<NetSerializerConfigParam>(ObjectNetSerializerConfig);
		QuantizeArgs.Source = reinterpret_cast<NetSerializerValuePointer>(&Object);
		QuantizeArgs.Target = reinterpret_cast<NetSerializerValuePointer>(&TempValue.ObjectReference);
		ObjectNetSerializer->Quantize(Context, QuantizeArgs);
	}
	else
	{
		const FSoftObjectPath& SoftObjectPath = Source.GetUniqueID();
		const FString& SoftObjectPathString = SoftObjectPath.ToString();

		FNetQuantizeArgs QuantizeArgs = Args;
		QuantizeArgs.Source = 0;
		QuantizeArgs.Target = reinterpret_cast<NetSerializerValuePointer>(&TempValue.Path);
		FStringNetSerializerBase::Quantize(Context, QuantizeArgs, SoftObjectPathString);
	}

	Target = TempValue;
}

void FSoftObjectNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	if (Source.bIsObject)
	{
		UObject* Object = nullptr;

		FNetDequantizeArgs DequantizeArgs = Args;
		DequantizeArgs.NetSerializerConfig = ObjectNetSerializerConfig;
		DequantizeArgs.Source = reinterpret_cast<NetSerializerValuePointer>(&Source.ObjectReference);
		DequantizeArgs.Target = reinterpret_cast<NetSerializerValuePointer>(&Object);
		ObjectNetSerializer->Dequantize(Context, DequantizeArgs);
		
		// This will release any previous weak object and create the unique ID as well. 
		Target = Object;
	}
	else
	{
		FString SoftObjectPathString;
		Super::Dequantize(Context, Args, SoftObjectPathString);

		Target.ResetWeakPtr();
		Target.GetUniqueID().SetPath(MoveTemp(SoftObjectPathString));
	}

	Target.GetUniqueID().FixupForPIE();
}

void FSoftObjectNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	if (Writer->WriteBool(Source.bIsObject))
	{
		FNetSerializeArgs SerializeArgs = Args;
		SerializeArgs.NetSerializerConfig = ObjectNetSerializerConfig;
		SerializeArgs.Source = reinterpret_cast<NetSerializerValuePointer>(&Source.ObjectReference);
		ObjectNetSerializer->Serialize(Context, SerializeArgs);
	}
	else
	{
		FNetSerializeArgs SerializeArgs = Args;
		SerializeArgs.NetSerializerConfig = 0;
		SerializeArgs.Source = reinterpret_cast<NetSerializerValuePointer>(&Source.Path);
		Super::Serialize(Context, SerializeArgs);
	}
}

void FSoftObjectNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	QuantizedType TempValue = {};
	TempValue.bIsObject = Reader->ReadBool();
	if (TempValue.bIsObject)
	{
		FNetDeserializeArgs DeserializeArgs = Args;
		DeserializeArgs.NetSerializerConfig = ObjectNetSerializerConfig;
		DeserializeArgs.Target = reinterpret_cast<NetSerializerValuePointer>(&TempValue.ObjectReference);
		ObjectNetSerializer->Deserialize(Context, DeserializeArgs);

		// Free previous path allocation
		if (!Target.bIsObject)
		{
			FNetFreeDynamicStateArgs FreeArgs = {};
			FreeArgs.Source = reinterpret_cast<NetSerializerValuePointer>(&Target.Path);
			Super::FreeDynamicState(Context, FreeArgs);
		}
	}
	else
	{
		// Reuse memory.
		TempValue.Path = Target.Path;

		FNetDeserializeArgs DeserializeArgs = Args;
		DeserializeArgs.NetSerializerConfig = 0;
		DeserializeArgs.Target = reinterpret_cast<NetSerializerValuePointer>(&TempValue.Path);
		Super::Deserialize(Context, DeserializeArgs);
	}

	Target = TempValue;
}

void FSoftObjectNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	NetSerializeDeltaDefault<FSoftObjectNetSerializer::Serialize, FSoftObjectNetSerializer::IsEqual>(Context, Args);
}

void FSoftObjectNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	NetDeserializeDeltaDefault<sizeof(QuantizedType), FSoftObjectNetSerializer::Deserialize, FSoftObjectNetSerializer::FreeDynamicState, FSoftObjectNetSerializer::CloneDynamicState>(Context, Args);
}

bool FSoftObjectNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Source0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Source1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (Source0.bIsObject != Source1.bIsObject)
		{
			return false;
		}

		if (Source0.bIsObject)
		{
			FNetIsEqualArgs EqualArgs = Args;
			EqualArgs.NetSerializerConfig = ObjectNetSerializerConfig;
			EqualArgs.Source0 = reinterpret_cast<NetSerializerValuePointer>(&Source0.Path);
			EqualArgs.Source1 = reinterpret_cast<NetSerializerValuePointer>(&Source1.Path);
			return ObjectNetSerializer->IsEqual(Context, EqualArgs);
		}
		else
		{
			FNetIsEqualArgs EqualArgs = Args;
			EqualArgs.Source0 = reinterpret_cast<NetSerializerValuePointer>(&Source0.Path);
			EqualArgs.Source1 = reinterpret_cast<NetSerializerValuePointer>(&Source1.Path);
			return Super::IsQuantizedEqual(Context, Args);
		}
	}
	else
	{
		const SourceType& Source0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Source1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		const FString& SoftObjectPathString0 = Source0.GetUniqueID().ToString();
		const FString& SoftObjectPathString1 = Source1.GetUniqueID().ToString();
		// Might not need case sensitive string compare, but not having it would require the quantization to
		// transform the string to lowercase first.
		return SoftObjectPathString0.Equals(SoftObjectPathString1, ESearchCase::CaseSensitive);
	}
}

bool FSoftObjectNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	const FSoftObjectPath& SoftObjectPath = Source.GetUniqueID();
	const FString& SoftObjectPathString = SoftObjectPath.ToString();
	return FStringNetSerializerBase::Validate(Context, Args, SoftObjectPathString);
}

void FSoftObjectNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	Target = Source;
	if (!Source.bIsObject)
	{
		FNetCloneDynamicStateArgs CloneArgs = Args;
		CloneArgs.NetSerializerConfig = 0;
		CloneArgs.Source = NetSerializerValuePointer(&Source.Path);
		CloneArgs.Target = NetSerializerValuePointer(&Target.Path);
		Super::CloneDynamicState(Context, CloneArgs);
	}
}

void FSoftObjectNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& Source = *reinterpret_cast<QuantizedType*>(Args.Source);

	if (!Source.bIsObject)
	{
		FNetFreeDynamicStateArgs FreeArgs = Args;
		FreeArgs.NetSerializerConfig = 0;
		FreeArgs.Source = NetSerializerValuePointer(&Source.Path);
		Super::FreeDynamicState(Context, FreeArgs);
	}
}

void FSoftObjectNetSerializer::CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);

	if (Source.bIsObject)
	{
		const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
		FNetReferenceCollector& Collector = *reinterpret_cast<FNetReferenceCollector*>(Args.Collector);

		const FNetReferenceInfo ReferenceInfo(FNetReferenceInfo::EResolveType::ResolveOnClient);
		Collector.Add(ReferenceInfo, Value.ObjectReference, Args.ChangeMaskInfo);
	}
}

// FSoftObjectPathNetSerializer
void FSoftObjectPathNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	const FString& SoftObjectPathString = Source.ToString();
	return FStringNetSerializerBase::Quantize(Context, Args, SoftObjectPathString);
}

void FSoftObjectPathNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	FString SoftObjectPathString;
	FStringNetSerializerBase::Dequantize(Context, Args, SoftObjectPathString);

	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);
	Target.SetPath(SoftObjectPathString);
}

bool FSoftObjectPathNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		return IsQuantizedEqual(Context, Args);
	}
	else
	{
		const SourceType& Source0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Source1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		const FString& SoftObjectPathString0 = Source0.ToString();
		const FString& SoftObjectPathString1 = Source1.ToString();
		return SoftObjectPathString0.Equals(SoftObjectPathString1, ESearchCase::CaseSensitive);
	}
}

bool FSoftObjectPathNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	const FString& SoftObjectPathString = Source.ToString();
	return FStringNetSerializerBase::Validate(Context, Args, SoftObjectPathString);
}

// FSoftClassPathNetSerializer
void FSoftClassPathNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	const FString& SoftClassPathString = Source.ToString();
	return FStringNetSerializerBase::Quantize(Context, Args, SoftClassPathString);
}

void FSoftClassPathNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	FString SoftClassPathString;
	FStringNetSerializerBase::Dequantize(Context, Args, SoftClassPathString);

	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);
	Target.SetPath(SoftClassPathString);
}

bool FSoftClassPathNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		return IsQuantizedEqual(Context, Args);
	}
	else
	{
		const SourceType& Source0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Source1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		const FString& SoftClassPathString0 = Source0.ToString();
		const FString& SoftClassPathString1 = Source1.ToString();
		return SoftClassPathString0.Equals(SoftClassPathString1, ESearchCase::CaseSensitive);
	}
}

bool FSoftClassPathNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	const FString& SoftClassPathString0 = Source.ToString();
	return Private::FStringNetSerializerBase::Validate(Context, Args, SoftClassPathString0);
}

}

