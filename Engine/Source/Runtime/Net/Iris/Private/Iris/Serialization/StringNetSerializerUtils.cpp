// Copyright Epic Games, Inc. All Rights Reserved.

#include "StringNetSerializerUtils.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Misc/CString.h"

namespace UE::Net
{

const FName GNetError_CorruptString("Corrupt string");

}

namespace UE::Net::Private
{

template class FStringNetSerializerUtils::TStringCodec<TCHAR>;

void FStringNetSerializerBase::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	Writer->WriteBits(Value.bIsEncoded, 1U);
	WritePackedUint32(Writer, (Value.ElementCount > 0 ? Value.ElementCount - 1 : 0));
	if (Value.ElementCount > 0)
	{
		Writer->WriteBitStream(static_cast<uint32*>(Value.ElementStorage), 0U, (Value.ElementCount - 1U) * 8U);
	}
}

void FStringNetSerializerBase::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	const bool bIsEncoded = Reader->ReadBool();
	const uint32 NewElementCount = ReadPackedUint32(Reader);
	if ((NewElementCount > 65535U) | (bIsEncoded & (NewElementCount == 0)))
	{
		Context.SetError(GNetError_ArraySizeTooLarge);
		return;
	}

	Target.bIsEncoded = bIsEncoded;
	FStringNetSerializerUtils::AdjustArraySize<QuantizedType, uint8>(Context, Target, static_cast<uint16>(NewElementCount > 0 ? NewElementCount + 1 : 0));
	if (NewElementCount > 0)
	{
		Reader->ReadBitStream(static_cast<uint32*>(Target.ElementStorage), NewElementCount*8U);
		// Note that we've allocated space for terminating character.
		static_cast<uint8*>(Target.ElementStorage)[NewElementCount] = 0;
		if (bIsEncoded && !FStringNetSerializerUtils::TStringCodec<TCHAR>::IsValidEncoding(static_cast<uint8*>(Target.ElementStorage), NewElementCount))
		{
			Context.SetError(GNetError_CorruptString);
			return;
		}
	}
}

void FStringNetSerializerBase::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FStringNetSerializerUtils::CloneDynamicState<QuantizedType, ANSICHAR>(Context, Target, Source);
}

void FStringNetSerializerBase::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	FStringNetSerializerUtils::FreeDynamicState<QuantizedType, ANSICHAR>(Context, Value);
}

void FStringNetSerializerBase::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args, const FString& Source)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	const bool bIsEncoded = !FCString::IsPureAnsi(*Source, Source.Len());
	if (bIsEncoded)
	{
		const uint32 StringLength = Source.Len() + 1U;
		constexpr uint32 MaxStringLength = 65535U/3U;
		if (StringLength > MaxStringLength)
		{
			Context.SetError(GNetError_ArraySizeTooLarge);
			return;
		}
		FStringNetSerializerUtils::AdjustArraySize<QuantizedType, uint8>(Context, Target, static_cast<uint16>(3U*StringLength));
		Target.bIsEncoded = true;

		uint32 OutDestLen = 0;
		uint8* EncodingBuffer = static_cast<uint8*>(Target.ElementStorage);
		const bool bEncodingSuccess = FStringNetSerializerUtils::TStringCodec<FString::ElementType>::Encode(EncodingBuffer, 3U*StringLength, GetData(Source), StringLength, OutDestLen);
		Target.ElementCount = static_cast<uint16>(OutDestLen);
	}
	else
	{
		const uint32 StringLength = Source.Len();
		if (StringLength + 1U > 65535U)
		{
			Context.SetError(GNetError_ArraySizeTooLarge);
			return;
		}
		FStringNetSerializerUtils::AdjustArraySize<QuantizedType, uint8>(Context, Target, static_cast<uint16>(StringLength > 0 ? StringLength + 1 : 0));
		Target.bIsEncoded = false;

		if (StringLength > 0)
		{
			const TCHAR* SourceString = GetData(Source);
			uint8* TargetString = static_cast<uint8*>(Target.ElementStorage);
			for (uint32 It = 0, EndIt = StringLength + 1; It != EndIt; ++It)
			{
				TargetString[It] = SourceString[It] & 0xFF;
			}
		}
	}
}

void FStringNetSerializerBase::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args, FString& Target)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	if (Source.bIsEncoded)
	{
		const uint32 SourceLen = Source.ElementCount;
		TArray<TCHAR, TInlineAllocator<4096U/sizeof(TCHAR)>> TempString;
		TempString.AddUninitialized(FStringNetSerializerUtils::TStringCodec<WIDECHAR>::GetSafeDecodedBufferLength(SourceLen));

		uint32 OutLength = 0;
		if (!FStringNetSerializerUtils::TStringCodec<WIDECHAR>::Decode(TempString.GetData(), TempString.Num(), static_cast<uint8*>(Source.ElementStorage), SourceLen, OutLength))
		{
			Target.Empty();
			Context.SetError(GNetError_CorruptString);
			return;
		}

		if (OutLength == 0 || TempString.GetData()[OutLength - 1] != 0)
		{
			OutLength += 1U;
		}

		Target = FString::ConstructFromPtrSize(TempString.GetData(), int32(OutLength - 1U));
	}
	else
	{
		Target = FString::ConstructFromPtrSize(static_cast<ANSICHAR*>(Source.ElementStorage), (Source.ElementCount > 0U ? Source.ElementCount - 1U : 0U));
	}
}

bool FStringNetSerializerBase::IsQuantizedEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args)
{
	const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
	const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
	if ((Value0.bIsEncoded != Value1.bIsEncoded) | (Value0.ElementCount != Value1.ElementCount))
	{
		return false;
	}

	const bool bIsEqual = FMemory::Memcmp(Value0.ElementStorage, Value1.ElementStorage, Value0.ElementCount) == 0;
	return bIsEqual;
}

bool FStringNetSerializerBase::Validate(FNetSerializationContext&, const FNetValidateArgs&, const FString& Source)
{
	// Validate length of string. Current limit is 65535 bytes for the encoded string.
	if (FStringNetSerializerUtils::TStringCodec<TCHAR>::GetSafeEncodedBufferLength(Source.Len() + 1U) > 65535U)
	{
		return false;
	}

	return true;
}

}
