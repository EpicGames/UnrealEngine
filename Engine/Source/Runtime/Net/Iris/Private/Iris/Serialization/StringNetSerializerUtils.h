// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"

namespace UE::Net
{

extern const FName GNetError_CorruptString;

}

namespace UE::Net::Private
{

class FStringNetSerializerUtils
{
public:
	// Note: Keep the allocations a multiple of 4 and at least 4 byte aligned to allow use of NetBitstreamWriter::WriteBitStream

	// More compact format than UTF-8. A codepoint will use at most 24 bits, versus 32 bits for UTF-8.
	template<typename InWideCharType>
	class TStringCodec
	{
	public:
		typedef uint32 Codepoint;
		typedef uint8 EncodeType;
		typedef InWideCharType WideCharType;

		static bool Encode(EncodeType* Dest, uint32 DestLen, const WideCharType* Source, uint32 SourceLen, uint32& OutDestLen)
		{
			return EncodeImpl(Dest, DestLen, Source, SourceLen, OutDestLen);
		}

		// Decodes into a WideCharType string. It is mandatory to call IsValidEncoding() before Decode().
		static bool Decode(WideCharType* Dest, uint32 DestLen, const EncodeType* Source, uint32 SourceLen, uint32& OutDestLen)
		{
			return DecodeImpl(Dest, DestLen, Source, SourceLen, OutDestLen);
		}

		static uint32 GetSafeEncodedBufferLength(uint32 DecodedLen)
		{
			return 3*DecodedLen;
		}

		// Returns the length in characters of type WideCharType that is needed to safely decode EncodedLen
		static uint32 GetSafeDecodedBufferLength(uint32 EncodedLen)
		{
			if constexpr (sizeof(WideCharType) == 2)
			{
				// If the machine sending had 32-bit characters there may be codepoints that are encoded as 3 bytes but require 4 bytes in the destination buffer.
				// Worst case scenario every codepoint requires 4 bytes in the destination buffer.
				return (EncodedLen * 4U) / 3U;
			}
			else
			{
				return EncodedLen;
			}
		}

		// This checks for very serious errors such as a codepoint encoded into more than three bytes, or codepoint passing end of buffer.
		static bool IsValidEncoding(const EncodeType* Encoding, uint32 EncodeLen)
		{
			uint32 Continue = 0;
			for (uint32 EncodeIt = 0, EncodeEndIt = EncodeLen; EncodeIt != EncodeEndIt; ++EncodeIt)
			{
				uint32 EncodedCount = 0;
				do
				{
					if ((++EncodedCount > 3U) | (EncodeIt == EncodeEndIt))
					{
						return false;
					}

					EncodeType Code = Encoding[EncodeIt];
					Continue = (Code & 0x80) ? 1U : 0U;
					EncodeIt += Continue;
				} while (Continue);
			}

			return true;
		}

	private:

		static bool IsInvalidCodepoint(uint32 Codepoint)
		{
			// Valid codepoints are in range [0, 0x10FFFF], except for 0xFFFE and 0xFFFF
			return (Codepoint > 0x10FFFFU) | ((Codepoint - 0xFFFEU) <= 1U);
		}

		static bool IsSurrogate(uint32 Codepoint)
		{
			return (Codepoint >= 0xD800U) & (Codepoint <= 0xDFFFU);
		}

		static bool IsHighSurrogate(WideCharType Char)
		{
			return (Char >= WideCharType(0xD800U)) & (Char <= WideCharType(0xDBFFU));
		}

		static bool IsLowSurrogate(WideCharType Char)
		{
			return (Char >= WideCharType(0xDC00U)) & (Char <= WideCharType(0xDFFFU));
		}

		static uint32 GetCodepointFromSurrogates(WideCharType HighSurrogate, WideCharType LowSurrogate)
		{
			return (uint32(uint16(HighSurrogate) - 0xD800U) << 10U) + (uint32(uint16(LowSurrogate) - 0xDC00U) + 0x10000U);
		}

		static bool IsInNeedOfSurrogates(uint32 Codepoint)
		{
			return (Codepoint >= 0x10000U) & (Codepoint <= 0x10FFFFU);
		}

		static WideCharType GetHighSurrogate(uint32 Codepoint)
		{
			return static_cast<WideCharType>(((Codepoint & 0xFFFFU) >> 10U) + 0xD800U);
		}

		static WideCharType GetLowSurrogate(uint32 Codepoint)
		{
			return static_cast<WideCharType>((Codepoint & 0x3FFU) + 0xDC00U);
		}

		/**
		 * The 4-byte character encoder yields at most three bytes per codepoint.
		 * It checks for some invalid codepoints and replaces them with the replacement character.
		 */
		template<typename T = WideCharType, typename TEnableIf<sizeof(T) == 4, char>::Type CharSize = 4>
		static bool EncodeImpl(EncodeType* Dest, uint32 DestLen, const WideCharType* Source, uint32 SourceLen, uint32& OutDestLen)
		{
			constexpr WideCharType LastCodePoint = 0x10FFFF;
			constexpr WideCharType ReplacementCharacter = 0xFFFD;

			// Unless the destination length is safe let's not encode. Encoding would require OOB checks.
			if (DestLen < 3U*SourceLen)
			{
				return false;
			}

			uint32 DestIt = 0;
			uint32 SourceIt = 0;
			for (const uint32 SourceEndIt = SourceLen; SourceIt < SourceEndIt; ++SourceIt)
			{
				WideCharType Char = Source[SourceIt];
				Char = (Char > LastCodePoint ? ReplacementCharacter : Char);

				if (Char < WideCharType(0x80))
				{
					Dest[DestIt++] = static_cast<EncodeType>(Char);
				}
				else if (Char < WideCharType(0x400))
				{
					Dest[DestIt + 0] = EncodeType(0x80) | static_cast<EncodeType>(Char >> 7U);
					Dest[DestIt + 1] = (Char & 0x7F);
					DestIt += 2;
				}
				else
				{
					Dest[DestIt + 0] = EncodeType(0x80) | static_cast<EncodeType>(Char >> 14U);
					Dest[DestIt + 1] = EncodeType(0x80) | static_cast<EncodeType>((Char >> 7U) & 0x7F);
					Dest[DestIt + 2] = (Char & 0x7F);
					DestIt += 3;
				}
			}

			OutDestLen = DestIt;
			return SourceIt == SourceLen;
		}

		/**
		 * The 2-byte character encoder yields at most three bytes per codepoint.
		 * It combines surrogate pairs so they can be encoded as three bytes instead of three bytes each.
		 */
		template<typename T = WideCharType, typename TEnableIf<sizeof(T) == 2, char>::Type CharSize = 2>
		static bool EncodeImpl(EncodeType* Dest, uint32 DestLen, const WideCharType* Source, uint32 SourceLen, uint32& OutDestLen)
		{
			// Unless the destination length is safe let's not encode. Encoding would require OOB checks.
			if (DestLen < 3U*SourceLen)
			{
				return false;
			}

			uint32 DestIt = 0;
			uint32 SourceIt = 0;
			for (const uint32 SourceEndIt = SourceLen; SourceIt < SourceEndIt; ++SourceIt)
			{
				const WideCharType Char = Source[SourceIt];
				const WideCharType NextChar = Source[FMath::Min(SourceIt + 1U, SourceEndIt - 1U)];
				/**
				 * A pair of surrogates would be encoded as 6 bytes. So we construct the 4-byte codepoint and encode them as 3 bytes.
				 * We let the decoder deal with invalid surrogate sequences, such as wrong order or a single surrogate.
				 */
				if (IsHighSurrogate(Char) && IsLowSurrogate(NextChar))
				{
					const uint32 Codepoint = GetCodepointFromSurrogates(Char, NextChar);
					Dest[DestIt + 0] = EncodeType(0x80) | static_cast<EncodeType>(Codepoint >> 14U);
					Dest[DestIt + 1] = EncodeType(0x80) | static_cast<EncodeType>((Codepoint >> 7U) & 0x7F);
					Dest[DestIt + 2] = (Codepoint & 0x7F);
					DestIt += 3;
					++SourceIt;
				}
				else if (Char < WideCharType(0x80))
				{
					Dest[DestIt++] = static_cast<EncodeType>(Char);
				}
				else if (Char < WideCharType(0x400))
				{
					Dest[DestIt + 0] = EncodeType(0x80) | static_cast<EncodeType>(Char >> 7U);
					Dest[DestIt + 1] = (Char & 0x7F);
					DestIt += 2;
				}
				else
				{
					Dest[DestIt + 0] = EncodeType(0x80) | static_cast<EncodeType>(Char >> 14U);
					Dest[DestIt + 1] = EncodeType(0x80) | static_cast<EncodeType>((Char >> 7U) & 0x7F);
					Dest[DestIt + 2] = (Char & 0x7F);
					DestIt += 3;
				}
			}

			OutDestLen = DestIt;
			return SourceIt == SourceLen;
		}

		// 4-byte destination character type version
		template<typename T = WideCharType, typename TEnableIf<sizeof(T) == 4, char>::Type CharSize = 4>
		static bool DecodeImpl(WideCharType* Dest, uint32 DestLen, const EncodeType* Source, uint32 SourceLen, uint32& OutDestLen)
		{
			if (SourceLen == 0)
			{
				OutDestLen = 0;
				return true;
			}

			if (DestLen < GetSafeDecodedBufferLength(SourceLen))
			{
				return false;
			}

			constexpr WideCharType ReplacementCharacter = 0xFFFD;

			uint32 SurrogateCount = 0;
			uint32 DestIt = 0;
			bool bIsErrorDetected = false;
			for (uint32 SourceIt = 0, SourceEndIt = SourceLen, DestEndIt = DestLen; (SourceIt < SourceEndIt) & (DestIt < DestEndIt) & (!bIsErrorDetected); ++SourceIt)
			{
				EncodeType Byte;
				uint32 Codepoint;
				uint8 Mask;

				// Decode first byte
				Byte = Source[SourceIt];
				Codepoint = Byte & 0x7FU;

				// Optionally decode second byte. Current state will remain unchanged unless the most significant bit in Byte is set.
				Mask = int8(Byte) >> 7U;
				SourceIt += 1U & Mask;
				if (SourceIt >= SourceEndIt)
				{
					bIsErrorDetected = true;
					break;
				}

				Byte = Source[SourceIt];
				Codepoint = (Codepoint << (7U & Mask)) | (Byte & 0x7FU);

				// Optionally decode third byte. Current state will remain unchanged unless the most significant bit in Byte is set.
				Mask = int8(Byte) >> 7U;
				SourceIt += 1U & Mask;
				if (SourceIt >= SourceEndIt)
				{
					bIsErrorDetected = true;
					break;
				}
				Byte = Source[SourceIt];
				Codepoint = (Codepoint << (7U & Mask)) | (Byte & 0x7FU);

				const bool bIsInvalidCodepoint = IsInvalidCodepoint(Codepoint);
				bIsErrorDetected = bIsErrorDetected | bIsInvalidCodepoint;
				SurrogateCount += IsSurrogate(Codepoint);

				Dest[DestIt++] = bIsInvalidCodepoint ? ReplacementCharacter : static_cast<WideCharType>(Codepoint);
			}

			// Make sure decoding is null terminated
			if (Dest[DestIt - 1] != 0)
			{
				bIsErrorDetected = true;
				// This may overwrite the last character
				uint32 LastIndex = FMath::Min(DestIt, DestLen - 1U);
				Dest[LastIndex] = 0;
				DestIt = LastIndex + 1U;
			}

			uint32 CurrentLength = DestIt;
			if (SurrogateCount > 0)
			{
				bIsErrorDetected = bIsErrorDetected || !CombineSurrogatesInPlace(SurrogateCount, Dest, DestLen, CurrentLength);
			}

			OutDestLen = CurrentLength;
			return !bIsErrorDetected;
		}

		// 2-byte destination character type version
		template<typename T = WideCharType, typename TEnableIf<sizeof(T) == 2, char>::Type CharSize = 2>
		static bool DecodeImpl(WideCharType* Dest, uint32 DestLen, const EncodeType* Source, uint32 SourceLen, uint32& OutDestLen)
		{
			if (SourceLen == 0)
			{
				OutDestLen = 0;
				return true;
			}

			if (DestLen < GetSafeDecodedBufferLength(SourceLen))
			{
				return false;
			}

			constexpr WideCharType ReplacementCharacter = 0xFFFD;

			uint32 NeedSurrogatesCount = 0;
			uint32 DestIt = 0;
			bool bIsErrorDetected = false;
			for (uint32 SourceIt = 0, SourceEndIt = SourceLen, DestEndIt = DestLen; (SourceIt < SourceEndIt) & (DestIt < DestEndIt) & (!bIsErrorDetected); ++SourceIt)
			{
				EncodeType Byte;
				uint32 Codepoint;
				uint8 Mask;

				// Decode first byte
				Byte = Source[SourceIt];
				Codepoint = Byte & 0x7FU;

				// Optionally decode second byte. Current state will remain unchanged unless the most significant bit in Byte is set.
				Mask = int8(Byte) >> 7U;
				SourceIt += 1U & Mask;
				if (SourceIt >= SourceEndIt)
				{
					bIsErrorDetected = true;
					break;
				}
				Byte = Source[SourceIt];
				Codepoint = (Codepoint << (7U & Mask)) | (Byte & 0x7FU);

				// Optionally decode third byte. Current state will remain unchanged unless the most significant bit in Byte is set.
				Mask = int8(Byte) >> 7U;
				SourceIt += 1U & Mask;
				if (SourceIt >= SourceEndIt)
				{
					bIsErrorDetected = true;
					break;
				}
				Byte = Source[SourceIt];
				Codepoint = (Codepoint << (7U & Mask)) | (Byte & 0x7FU);

				const bool bIsInvalidCodePoint = IsInvalidCodepoint(Codepoint);
				const bool bIsInNeedOfSurrogates = IsInNeedOfSurrogates(Codepoint);
				bIsErrorDetected = bIsErrorDetected | bIsInvalidCodePoint;

				// Replace bad character with replacement character
				Codepoint = bIsInvalidCodePoint ? ReplacementCharacter : Codepoint;

				// Note that there's no error checking for single surrogates. We just ignore them as the code point will not be in need of surrogates.
				if (bIsInNeedOfSurrogates)
				{
					// One character is safe to write from the loop condition, but two could be one too many.
					if (DestIt + 1U >= DestLen)
					{
						bIsErrorDetected = true;
						break;
					}
					Dest[DestIt + 0] = GetHighSurrogate(Codepoint);
					Dest[DestIt + 1] = GetLowSurrogate(Codepoint);
					DestIt += 2U;
				}
				else
				{
					Dest[DestIt++] = bIsInvalidCodePoint ? ReplacementCharacter : static_cast<WideCharType>(Codepoint);
				}
			}

			OutDestLen = DestIt;
			return !bIsErrorDetected;
		}

		template<typename T = WideCharType, typename TEnableIf<sizeof(T) == 4, char>::Type CharSize = 4>
		static bool CombineSurrogatesInPlace(uint32 SurrogateCount, WideCharType* Buffer, uint32 BufferCapacity, uint32& InOutBufferLen)
		{
			const int32 InBufferLen = static_cast<int32>(InOutBufferLen);
			const int32 NewLength = StringConv::InlineCombineSurrogates_Buffer<WideCharType>(Buffer, InBufferLen);

			const int32 ExpectedNewLength = InBufferLen - static_cast<int32>(SurrogateCount/2U);
			const bool bSuccess = (NewLength == ExpectedNewLength) & ((SurrogateCount & 1) == 0);

			InOutBufferLen = static_cast<uint32>(NewLength);
			return bSuccess;
		}
	};

	// NetSerializer helpers
	template<typename QuantizedType, typename ElementType>
	static void CloneDynamicState(FNetSerializationContext& Context, QuantizedType& Target, const QuantizedType& Source)
	{
		// Copy whatever other parameters may be present
		Target = Source;

		constexpr SIZE_T ElementSize = sizeof(ElementType);
		constexpr SIZE_T ElementAlignment = Align(alignof(ElementType), 4U);

		void* ElementStorage = nullptr;
		if (Source.ElementCount > 0)
		{
			ElementStorage = Context.GetInternalContext()->Alloc(Align(ElementSize*Source.ElementCount, 4U), ElementAlignment);
			FMemory::Memcpy(ElementStorage, Source.ElementStorage, ElementSize*Source.ElementCount);
		}
		Target.ElementCapacityCount = Source.ElementCount;
		Target.ElementCount = Source.ElementCount;
		Target.ElementStorage = ElementStorage;
	}

	template<typename QuantizedType, typename ElementType>
	static void FreeDynamicState(FNetSerializationContext& Context, QuantizedType& Array)
	{
		Context.GetInternalContext()->Free(Array.ElementStorage);

		// Clear all info
		Array = QuantizedType();
	}

	template<typename QuantizedType, typename ElementType>
	static void GrowDynamicState(FNetSerializationContext& Context, QuantizedType& Array, uint16 NewElementCount)
	{
		checkSlow(NewElementCount > Array.ElementCapacityCount);

		constexpr SIZE_T ElementSize = sizeof(ElementType);
		constexpr SIZE_T ElementAlignment = Align(alignof(ElementType), 4U);

		// We don't support delta compression for strings so we don't need to copy anything
		Context.GetInternalContext()->Free(Array.ElementStorage);

		void* NewElementStorage = Context.GetInternalContext()->Alloc(Align(ElementSize*NewElementCount, 4U), ElementAlignment);
		// We don't need to clear the memory.
		//FMemory::Memzero(NewElementStorage, ElementSize*NewElementCount);

		Array.ElementCapacityCount = NewElementCount;
		Array.ElementCount = NewElementCount;
		Array.ElementStorage = NewElementStorage;
	}

	template<typename QuantizedType, typename ElementType>
	static void AdjustArraySize(FNetSerializationContext& Context, QuantizedType& Array, uint16 NewElementCount)
	{
		if (NewElementCount == 0)
		{
			// Free everything
			FreeDynamicState<QuantizedType, ElementType>(Context, Array);
		}
		else if (NewElementCount > Array.ElementCapacityCount)
		{
			GrowDynamicState<QuantizedType, ElementType>(Context, Array, NewElementCount);
		}
		// If element count is within the allocated capacity we just change the number of elements
		else
		{
			Array.ElementCount = NewElementCount;
		}
	}
};

extern template class FStringNetSerializerUtils::TStringCodec<TCHAR>;

}

namespace UE::Net
{

struct FNetSerializeArgs;
struct FNetDeserializeArgs;
struct FNetCloneDynamicStateArgs;
struct FNetFreeDynamicStateArgs;
struct FNetQuantizeArgs;
struct FNetDequantizeArgs;
struct FNetIsEqualArgs;
struct FNetValidateArgs;

}

namespace UE::Net::Private
{

struct FStringNetSerializerBase
{
	// Traits
	static constexpr bool bHasDynamicState = true;

	// Types
	struct FQuantizedType
	{
		// Whether the stored string is encoded or ANSI
		uint32 bIsEncoded : 1U;

		// How many elements the current allocation can hold.
		uint16 ElementCapacityCount;
		// How many elements are valid
		uint16 ElementCount;
		void* ElementStorage;
	};

	typedef FQuantizedType QuantizedType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

protected:
	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&, const FString& Source);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&, FString& Target);

	static bool IsQuantizedEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&, const FString& Source);
};

}
