// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/StringNetSerializers.h"
#include "Iris/Serialization/StringNetSerializerUtils.h"
#include "Iris/Serialization/InternalNetSerializer.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Core/BitTwiddling.h"
#include "Containers/StringConv.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "Iris/ReplicationSystem/NameTokenStore.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationSystemInternal.h"
#include "Iris/Serialization/InternalNetSerializationContext.h"
#include "Iris/Serialization/NetExportContext.h"
#include "Net/Core/Trace/NetTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StringNetSerializers)

static_assert(sizeof(ANSICHAR) == sizeof(uint8), "ANSICHAR is expected to be one byte.");

namespace UE::Net
{

struct FNameNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bHasDynamicState = true;

	// Types

	// Crafted such that zeroed memory will represent FName(NAME_None)
	struct FQuantizedType
	{
		// If the FName is hardcoded or not
		uint32 bIsString : 1U;
		// When serializing/deserializing the number is expressed as (MAX_int32 - number)
		uint32 bEncodeNumberFromIntMax : 1U;
		// For FNames replicated as strings this indicates whether the stored string is encoded or ANSI
		uint32 bIsEncoded : 1U;

		// If bIsString this is the Number, otherwise it's the EName
		int32 ENameOrNumber;

		// How many elements the current allocation can hold.
		uint16 ElementCapacityCount;
		// How many elements are valid
		uint16 ElementCount;
		void* ElementStorage;
	};

	static_assert(GetNameNetSerializerSafeQuantizedSize() >= sizeof(FQuantizedType));

	typedef FName SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FNameNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	//
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

private:
	static const uint32 BitCountNeededForEName;
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FNameNetSerializer);
const FNameNetSerializer::ConfigType FNameNetSerializer::DefaultConfig;
const uint32 FNameNetSerializer::BitCountNeededForEName = UE::Net::GetBitsNeeded(MAX_NETWORKED_HARDCODED_NAME);

struct FStringNetSerializer : public Private::FStringNetSerializerBase
{
	// Version
	static const uint32 Version = 0;

	typedef FString SourceType;
	typedef FStringNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	//
	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);
};
UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FStringNetSerializer);
const FStringNetSerializer::ConfigType FStringNetSerializer::DefaultConfig;

// FNameNetSerializer
void FNameNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	if (Writer->WriteBool(Value.bIsString))
	{
		// Write the Number
		Writer->WriteBits(Value.bEncodeNumberFromIntMax, 1U);
		const int32 Number = Value.bEncodeNumberFromIntMax ? (MAX_int32 - Value.ENameOrNumber) : Value.ENameOrNumber;
		WritePackedInt32(Writer, Number);

		// Write the string
		Writer->WriteBits(Value.bIsEncoded, 1U);
		WritePackedUint32(Writer, Value.ElementCount - 1U);
		Writer->WriteBitStream(static_cast<uint32*>(Value.ElementStorage), 0U, (Value.ElementCount - 1U)*8U);
	}
	else
	{
		Writer->WriteBits(static_cast<uint32>(Value.ENameOrNumber), BitCountNeededForEName);
	}
}

void FNameNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	using namespace Private;

	// Unexpected, but consistent with Serialize.
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const uint32 CurrentElementCount = Target.ElementCount;

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	if (const bool bIsString = Reader->ReadBool())
	{
		const bool bEncodeNumberFromIntMax = Reader->ReadBool();
		int32 Number = ReadPackedInt32(Reader);
		Number = bEncodeNumberFromIntMax ? (MAX_int32 - Number) : Number;
		
		const bool bIsEncoded = Reader->ReadBool();
		const uint32 NewElementCount = ReadPackedUint32(Reader) + 1U;
		if (NewElementCount == 0 || NewElementCount > (NAME_SIZE + 1)*3U)
		{
			Context.SetError(GNetError_ArraySizeTooLarge);
			return;
		}

		Target.bIsString = 1;
		Target.bEncodeNumberFromIntMax = bEncodeNumberFromIntMax;
		Target.bIsEncoded = bIsEncoded;
		Target.ENameOrNumber = Number;
		FStringNetSerializerUtils::AdjustArraySize<QuantizedType, uint8>(Context, Target, static_cast<uint16>(NewElementCount));
		Reader->ReadBitStream(static_cast<uint32*>(Target.ElementStorage), (NewElementCount - 1U)*8U);
		static_cast<uint8*>(Target.ElementStorage)[NewElementCount - 1] = 0;
		if (bIsEncoded && !FStringNetSerializerUtils::TStringCodec<WIDECHAR>::IsValidEncoding(static_cast<uint8*>(Target.ElementStorage), NewElementCount - 1U))
		{
			Context.SetError(GNetError_CorruptString);
			return;
		}
	}
	else
	{
		const uint32 ENameNumber = Reader->ReadBits(BitCountNeededForEName);
		if (!ShouldReplicateAsInteger(EName(ENameNumber), FName(EName(ENameNumber))))
		{
			Context.SetError(GNetError_BitStreamError);
			return;
		}

		constexpr uint32 NewElementCount = 0U;
		FStringNetSerializerUtils::AdjustArraySize<QuantizedType, uint8>(Context, Target, NewElementCount);

		Target.bIsString = 0U;
		Target.bEncodeNumberFromIntMax = 0;
		Target.bIsEncoded = 0;
		Target.ENameOrNumber = ENameNumber;
	}
}

void FNameNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	using namespace Private;

	const SourceType SourceName = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetName = *reinterpret_cast<QuantizedType*>(Args.Target);

	const EName* AsEName = (SourceName.GetNumber() == NAME_NO_NUMBER_INTERNAL ? SourceName.ToEName() : nullptr);
	const bool bIsString = (AsEName == nullptr || !ShouldReplicateAsInteger(*AsEName, SourceName));
	if (bIsString)
	{
		const FNameEntry* DisplayNameEntry = SourceName.GetDisplayNameEntry();
		TargetName.bIsString = 1;
		TargetName.bIsEncoded = DisplayNameEntry->IsWide();
		TargetName.ENameOrNumber = SourceName.GetNumber();
		TargetName.bEncodeNumberFromIntMax = GetBitsNeeded(MAX_int32 - TargetName.ENameOrNumber) < GetBitsNeeded(TargetName.ENameOrNumber);
		// Encode the string if needed
		if (TargetName.bIsEncoded)
		{
			WIDECHAR TempWideBuffer[NAME_SIZE];
			SourceName.GetPlainWIDEString(TempWideBuffer);

			// Our codec uses up to 3 bytes per codepoint
			const uint32 NameLength = DisplayNameEntry->GetNameLength() + 1U;
			constexpr uint32 MaxArrayCount = 65536U/3U;
			if (NameLength > MaxArrayCount)
			{
				Context.SetError(GNetError_ArraySizeTooLarge);
				return;
			}
			FStringNetSerializerUtils::AdjustArraySize<QuantizedType, uint8>(Context, TargetName, static_cast<uint16>(3U*NameLength));

			uint32 OutDestLen = 0;
			uint8* EncodingBuffer = static_cast<uint8*>(TargetName.ElementStorage);
			const bool bEncodingSuccess = FStringNetSerializerUtils::TStringCodec<WIDECHAR>::Encode(EncodingBuffer, 3U*NameLength, TempWideBuffer, NameLength, OutDestLen);
			if (bEncodingSuccess)
			{
				TargetName.ElementCount = static_cast<uint16>(OutDestLen);
			}
			else
			{
				ensureMsgf(bEncodingSuccess, TEXT("Failed to encode string '%s'"), TempWideBuffer);
				TargetName.ElementCount = 1;
				EncodingBuffer[0] = 0;
			}
		}
		else
		{
			// For debugging purposes we store null terminated strings.
			const uint32 NewElementCount = DisplayNameEntry->GetNameLength() + 1U;
			constexpr uint32 MaxArrayCount = 65536U;
			if (NewElementCount > MaxArrayCount)
			{
				Context.SetError(GNetError_ArraySizeTooLarge);
				return;
			}
			FStringNetSerializerUtils::AdjustArraySize<QuantizedType, uint8>(Context, TargetName, static_cast<uint16>(NewElementCount));

			// At this time it's impossible to avoid the double copy unless we use a fixed excessive storage
			ANSICHAR TempAnsiBuffer[NAME_SIZE];
			SourceName.GetPlainANSIString(TempAnsiBuffer);

			FMemory::Memcpy(TargetName.ElementStorage, TempAnsiBuffer, NewElementCount*sizeof(ANSICHAR));
		}
	}
	else
	{
		constexpr uint16 NewElementCount = 0U;
		FStringNetSerializerUtils::AdjustArraySize<QuantizedType, uint8>(Context, TargetName, NewElementCount);

		TargetName.bIsString = 0;
		// Don't really care what the bit is set to, but these names are ANSI.
		TargetName.bIsEncoded = 0;
		// Again, this value doesn't matter for hardcoded names as we know they will start from 0
		TargetName.bEncodeNumberFromIntMax = 0;
		// The EName we do care about!
		TargetName.ENameOrNumber = static_cast<int32>(static_cast<uint32>(*AsEName));
	}
}

void FNameNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	using namespace Private;

	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	if (Source.bIsString)
	{
		if (Source.bIsEncoded)
		{
			const uint32 SourceLen = Source.ElementCount;

			TArray<WIDECHAR, TInlineAllocator<4096U/sizeof(WIDECHAR)>> TempString;
			TempString.AddUninitialized(FStringNetSerializerUtils::TStringCodec<WIDECHAR>::GetSafeDecodedBufferLength(SourceLen));

			uint32 OutLength = 0;
			if (!FStringNetSerializerUtils::TStringCodec<WIDECHAR>::Decode(TempString.GetData(), TempString.Num(), static_cast<uint8*>(Source.ElementStorage), SourceLen, OutLength))
			{
				// IsValidEncoding() has been called before this, but it only checks for the most serious errors. If the string contains invalid codepoints it will be considered a fail.
				Target = FName();
				Context.SetError(GNetError_CorruptString);
				return;
			}

			if (OutLength == 0 || TempString.GetData()[OutLength - 1] != 0)
			{
				// Empty strings should be replicated using optimized path and
				// strings are always null-terminated in the decoding pass, even when errors occur.
				Context.SetError(GNetError_CorruptString);
				return;
			}
			Target = FName(int32(OutLength - 1), TempString.GetData(), Source.ENameOrNumber);
		}
		else
		{
			Target = FName(int32(Source.ElementCount - 1), static_cast<ANSICHAR*>(Source.ElementStorage), Source.ENameOrNumber);
		}
	}
	else
	{
		Target = EName(static_cast<uint32>(Source.ENameOrNumber));
	}
}

bool FNameNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		// Bits will cancel out when values are equal.
		const bool bIsEqual = ((Value0.bIsString ^ Value1.bIsString) | (Value0.ENameOrNumber ^ Value1.ENameOrNumber) | (Value0.ElementCount ^ Value1.ElementCount)) == 0;
		if (!bIsEqual)
		{
			return false;
		}

		/**
		 * Ideally we would have some process agnostic fast way to compare the FNames. There's currently no such thing.
		 * We could store a hash as well in the quantized state but currently we don't. This method shouldn't be called
		 * often anyway and the above checks should catch most differences.
		 */
		if (Value0.bIsString)
		{
			if (Value0.bIsEncoded)
			{
				SourceType SourceValue0;
				SourceType SourceValue1;

				FNetDequantizeArgs DequantizeArgs = {};
				DequantizeArgs.NetSerializerConfig = Args.NetSerializerConfig;

				DequantizeArgs.Source = Args.Source0;
				DequantizeArgs.Target = NetSerializerValuePointer(&SourceValue0);
				Dequantize(Context, DequantizeArgs);

				DequantizeArgs.Source = Args.Source1;
				DequantizeArgs.Target = NetSerializerValuePointer(&SourceValue1);
				Dequantize(Context, DequantizeArgs);

				return SourceValue0 == SourceValue1;
			}
			else
			{
				return FCStringAnsi::Strnicmp(static_cast<const ANSICHAR*>(Value0.ElementStorage), static_cast<const ANSICHAR*>(Value1.ElementStorage), Value0.ElementCount) == 0;;
			}
		}

		return true;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		const bool bIsEqual = (Value0 == Value1);
		return bIsEqual;
	}
}

bool FNameNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Value = *reinterpret_cast<const SourceType*>(Args.Source);
	const bool bIsValid = Value.IsValid();
	return bIsValid;
}

void FNameNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	return Private::FStringNetSerializerUtils::CloneDynamicState<QuantizedType, ANSICHAR>(Context, Target, Source);
}

void FNameNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	return Private::FStringNetSerializerUtils::FreeDynamicState<QuantizedType, ANSICHAR>(Context, Value);
}

// FStringNetSerializer
void FStringNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	return FStringNetSerializerBase::Quantize(Context, Args, Source);
}

void FStringNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);
	return FStringNetSerializerBase::Dequantize(Context, Args, Target);
}

bool FStringNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		return IsQuantizedEqual(Context, Args);
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<SourceType*>(Args.Source1);
		return Value0.Equals(Value1, ESearchCase::CaseSensitive);
	}
}

bool FStringNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	return FStringNetSerializerBase::Validate(Context, Args, Source);
}

// FNameAsNetTokenNetSerializer
// Naive implementation ignoring numbers so it will reexport name part for each number.
// Good thing is that it works for all different configurations of FNames
struct FNameAsNetTokenNetSerializerQuantizedType
{
	// NetToken
	FNetToken NetToken;

	// If bIsString this is the Number, otherwise it's the EName
	int32 ENameOrNumber;

	// If the FName is hardcoded or not
	uint32 bIsString : 1U;
	// When serializing/deserializing the number is expressed as (MAX_int32 - number)
	uint32 bEncodeNumberFromIntMax : 1U;
};

static_assert(GetNameNetSerializerSafeQuantizedSize() >= sizeof(FNameAsNetTokenNetSerializerQuantizedType));

} // End of namespace UE::Net

template<> struct TIsPODType<UE::Net::FNameAsNetTokenNetSerializerQuantizedType> { enum { Value = true }; };

namespace UE::Net
{

struct FNameAsNetTokenNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Types

	// Crafted such that zeroed memory will represent FName(NAME_None)
	typedef FNameAsNetTokenNetSerializerQuantizedType FQuantizedType;
	typedef FName SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FNameAsNetTokenNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	//
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

private:

	// Utility methods, consolidate with other changes to NetTokenStore as next step.
	static FName ResolveNetToken(FNetSerializationContext&, FNetToken NetToken);

	static const uint32 BitCountNeededForEName;
};

UE_NET_IMPLEMENT_SERIALIZER_INTERNAL(FNameAsNetTokenNetSerializer);
const FNameAsNetTokenNetSerializer::ConfigType FNameAsNetTokenNetSerializer::DefaultConfig;
const uint32 FNameAsNetTokenNetSerializer::BitCountNeededForEName = UE::Net::GetBitsNeeded(MAX_NETWORKED_HARDCODED_NAME);

void FNameAsNetTokenNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	using namespace UE::Net::Private;

	// For now we ignore this in default state hash
	// We could probably output lowercase hash to allow validation of defaultstate.
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	if (Writer->WriteBool(Value.bIsString))
	{
		// Always write the token
		Context.GetNetTokenStore()->WriteNetTokenWithKnownType<FNameTokenStore>(Context, Value.NetToken);

		// Export or add to pending exports for later export
		FNetTokenStore::AppendExport(Context, Value.NetToken);
	}
	else
	{
		Writer->WriteBits(static_cast<uint32>(Value.ENameOrNumber), BitCountNeededForEName);
	}
}

void FNameAsNetTokenNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	using namespace Private;

	// Unexpected, but consistent with Serialize.
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	if (const bool bIsString = Reader->ReadBool())
	{
		Target.bIsString = 1;

		// Always Read the token
		FNetToken NetToken = Context.GetNetTokenStore()->ReadNetTokenWithKnownType<FNameTokenStore>(Context);

		if (Reader->IsOverflown())
		{
			return;
		}

		Target.NetToken = NetToken;
	}
	else
	{
		const uint32 ENameNumber = Reader->ReadBits(BitCountNeededForEName);
		if (!ShouldReplicateAsInteger(EName(ENameNumber), FName(EName(ENameNumber))))
		{
			Context.SetError(GNetError_BitStreamError);
			return;
		}

		Target.bIsString = 0U;
		Target.bEncodeNumberFromIntMax = 0;
		Target.ENameOrNumber = ENameNumber;
	
		Target.NetToken = FNetToken();
	}
}

void FNameAsNetTokenNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	using namespace Private;

	const SourceType SourceName = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetName = *reinterpret_cast<QuantizedType*>(Args.Target);

	const EName* AsEName = (SourceName.GetNumber() == NAME_NO_NUMBER_INTERNAL ? SourceName.ToEName() : nullptr);
	const bool bIsString = (AsEName == nullptr || !ShouldReplicateAsInteger(*AsEName, SourceName));
	if (bIsString)
	{
		TargetName.bIsString = 1;
		TargetName.ENameOrNumber = 0;
		TargetName.bEncodeNumberFromIntMax = 0;

		// Store as NetToken
		FNameTokenStore* NameTokenStore = Context.GetNetTokenStore()->GetDataStore<FNameTokenStore>();
		TargetName.NetToken = NameTokenStore->GetOrCreateToken(SourceName);
	}
	else
	{
		TargetName.bIsString = 0;
		// Again, this value doesn't matter for hardcoded names as we know they will start from 0
		TargetName.bEncodeNumberFromIntMax = 0;
		// The EName we do care about!
		TargetName.ENameOrNumber = static_cast<int32>(static_cast<uint32>(*AsEName));

		TargetName.NetToken = FNetToken();
	}
}

FName FNameAsNetTokenNetSerializer::ResolveNetToken(FNetSerializationContext& Context, FNetToken NetToken)
{
	using namespace UE::Net::Private;

	FInternalNetSerializationContext* InternalContext = Context.GetInternalContext();
	FNameTokenStore* NameTokenStore = Context.GetNetTokenStore()->GetDataStore<FNameTokenStore>();
	return NameTokenStore->ResolveToken(NetToken, InternalContext->ResolveContext.RemoteNetTokenStoreState);
}

void FNameAsNetTokenNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	using namespace Private;

	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	if (Source.bIsString)
	{
		// Resolve from RemoteNetToken
		Target = FNameAsNetTokenNetSerializer::ResolveNetToken(Context, Source.NetToken);
	}
	else
	{
		Target = EName(static_cast<uint32>(Source.ENameOrNumber));
	}
}

bool FNameAsNetTokenNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	using namespace Private;

	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		// Bits will cancel out when values are equal.
		const bool bIsEqual = ((Value0.bIsString ^ Value1.bIsString) | (Value0.ENameOrNumber ^ Value1.ENameOrNumber)) == 0;
		if (!bIsEqual)
		{
			return false;
		}

		// Need to compare actual FNames to properly compare non-auth and auth token
		if (Value0.bIsString)
		{
			if (Value0.NetToken.IsAssignedByAuthority() != Value1.NetToken.IsAssignedByAuthority())
			{
				const FName Name0 = FNameAsNetTokenNetSerializer::ResolveNetToken(Context, Value0.NetToken);
				const FName Name1 = FNameAsNetTokenNetSerializer::ResolveNetToken(Context, Value1.NetToken);
				
				if (Name0 != Name1)
				{
					return false;
				}
			}
			else if (Value0.NetToken != Value1.NetToken)
			{
				return false;
			}
		}

		return true;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<const SourceType*>(Args.Source1);
		const bool bIsEqual = (Value0 == Value1);
		return bIsEqual;
	}
}

bool FNameAsNetTokenNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Value = *reinterpret_cast<const SourceType*>(Args.Source);
	const bool bIsValid = Value.IsValid();
	return bIsValid;
}

}
