// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Iris/ReplicationSystem/ReplicationOperationsInternal.h"
#include "Iris/Serialization/InternalNetSerializerUtils.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"

namespace UE::Net
{

struct FStructNetSerializer
{
public:
	static const uint32 Version = 0;

	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing

	typedef void SourceType; // Dummy

	typedef FStructNetSerializerConfig ConfigType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	static void CloneStructMember(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
};

UE_NET_IMPLEMENT_SERIALIZER(FStructNetSerializer);

void FStructNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;

	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetSerializeArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = Args.Source + MemberDescriptor.InternalMemberOffset;
		// Currently we pass on the changemask info unmodified to support fastarrays, but if we decide to support other serializers utilizing additional changemask we need to extend this
		MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Serializer->Name, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		Serializer->Serialize(Context, MemberArgs);
	}
}

void FStructNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetDeserializeArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Target = Args.Target + MemberDescriptor.InternalMemberOffset;
		// Currently we pass on the changemask info unmodified to support fastarrays, but if we decide to support other serializers utilizing additional changemask we need to extend this
		MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Serializer->Name, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		Serializer->Deserialize(Context, MemberArgs);
		if (Context.HasErrorOrOverflow())
		{
			// $IRIS TODO Provide information which member is failing to deserialize (though could be red herring)
			return;
		}
	}
}

void FStructNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	/**
	  * $IRIS TODO UE-130963 The struct could hint whether we should check for equality before serializing.
	  * If the outer replication state already has a bit for state changes then this would be unnecessary,
	  * but if this struct itself contains struct members we'd want those to have hints. The solution
	  * would be to have a check for structs when a member is serialized and check and write whether the
	  * structs are equal before recursing into this function if they are not.
	  */

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;

	const uint32 MemberCount = Descriptor->MemberCount;

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Serializer->Name, *Context.GetBitStreamWriter(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		if (IsStructNetSerializer(Serializer))
		{
			FNetIsEqualArgs MemberEqualArgs;
			MemberEqualArgs.bStateIsQuantized = true;
			MemberEqualArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			MemberEqualArgs.Source0 = Args.Source + MemberDescriptor.InternalMemberOffset;
			MemberEqualArgs.Source1 = Args.Prev + MemberDescriptor.InternalMemberOffset;
			MemberEqualArgs.ChangeMaskInfo = Args.ChangeMaskInfo;
			if (Writer->WriteBool(IsEqual(Context, MemberEqualArgs)))
			{
				continue;
			}
		}

		{
			FNetSerializeDeltaArgs MemberArgs;
			MemberArgs.Version = 0;
			MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			MemberArgs.Source = Args.Source + MemberDescriptor.InternalMemberOffset;
			MemberArgs.Prev = Args.Prev + MemberDescriptor.InternalMemberOffset;
			// Currently we pass on the changemask info unmodified to support fastarrays, but if we decide to support other serializers utilizing additional changemask we need to extend this
			MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

			Serializer->SerializeDelta(Context, MemberArgs);
		}
	}
}

void FStructNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberTraitsDescriptor* MemberTraitsDescriptors = Descriptor->MemberTraitsDescriptors;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors = Descriptor->MemberDebugDescriptors;

	const uint32 MemberCount = Descriptor->MemberCount;

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(MemberDebugDescriptors[MemberIt].DebugName, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::Verbose);
		UE_NET_TRACE_DYNAMIC_NAME_SCOPE(Serializer->Name, *Context.GetBitStreamReader(), Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		if (IsStructNetSerializer(Serializer))
		{
			if (Reader->ReadBool())
			{
				FNetCloneDynamicStateArgs CloneArgs;
				CloneArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
				CloneArgs.Version = Args.Version;
				CloneArgs.Source = Args.Prev + MemberDescriptor.InternalMemberOffset;
				CloneArgs.Target = Args.Target + MemberDescriptor.InternalMemberOffset;
				CloneStructMember(Context, CloneArgs);
				continue;
			}
		}

		{
			FNetDeserializeDeltaArgs MemberArgs;
			MemberArgs.Version = 0;
			MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			MemberArgs.Target = Args.Target + MemberDescriptor.InternalMemberOffset;
			MemberArgs.Prev = Args.Prev + MemberDescriptor.InternalMemberOffset;
			// Currently we pass on the changemask info unmodified to support fastarrays, but if we decide to support other serializers utilizing additional changemask we need to extend this
			MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

			Serializer->DeserializeDelta(Context, MemberArgs);
			if (Context.HasErrorOrOverflow())
			{
				// $IRIS TODO Provide information which member is failing to deserialize (though could be red herring)
				return;
			}
		}
	}
}

void FStructNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetQuantizeArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = Args.Source + MemberDescriptor.ExternalMemberOffset;
		MemberArgs.Target = Args.Target + MemberDescriptor.InternalMemberOffset;

		// Currently we pass on the changemask info unmodified to support fastarrays, but if we decide to support other serializers utilizing additional changemask we need to extend this
		MemberArgs.ChangeMaskInfo = Args.ChangeMaskInfo;

		Serializer->Quantize(Context, MemberArgs);
	}
}

void FStructNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetDequantizeArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = Args.Source + MemberDescriptor.InternalMemberOffset;
		MemberArgs.Target = Args.Target + MemberDescriptor.ExternalMemberOffset;

		Serializer->Dequantize(Context, MemberArgs);
		if (Context.HasError())
		{
			// $IRIS TODO Provide information which member is failing to dequantize
			return;
		}
	}
}

bool FStructNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	
	// Optimized case for quantized state without dynamic state
	if (Args.bStateIsQuantized && !EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState))
	{
		return !FPlatformMemory::Memcmp(reinterpret_cast<const void*>(Args.Source0), reinterpret_cast<const void*>(Args.Source1), Descriptor->InternalSize);
	}

	// Per member equality
	{
		const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
		const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
		const uint32 MemberCount = Descriptor->MemberCount;

		FNetIsEqualArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.bStateIsQuantized = Args.bStateIsQuantized;

		const bool bIsQuantized = Args.bStateIsQuantized;
		for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
		{
			const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
			const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
			const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

			MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
			const uint32 MemberOffset = (bIsQuantized ? MemberDescriptor.InternalMemberOffset : MemberDescriptor.ExternalMemberOffset);
			MemberArgs.Source0 = Args.Source0 + MemberOffset;
			MemberArgs.Source1 = Args.Source1 + MemberOffset;

			if (!Serializer->IsEqual(Context, MemberArgs))
			{
				return false;
			}
		}
	}

	return true;
}

bool FStructNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	if (Descriptor == nullptr)
	{
		return false;
	}

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];
		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;

		FNetValidateArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = Args.Source + MemberDescriptor.ExternalMemberOffset;

		if (!Serializer->Validate(Context, MemberArgs))
		{
			return false;
		}
	}

	return true;
}

void FStructNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	// If no member has dynamic state then there's nothing for us to do
	if (!EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState))
	{
		return;
	}

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberTraitsDescriptor* MemberTraitsDescriptors = Descriptor->MemberTraitsDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberTraitsDescriptor& MemberTraitsDescriptor = MemberTraitsDescriptors[MemberIt];
		if (!EnumHasAnyFlags(MemberTraitsDescriptor.Traits, EReplicationStateMemberTraits::HasDynamicState))
		{
			continue;
		}

		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];

		FNetCloneDynamicStateArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = Args.Source + MemberDescriptor.InternalMemberOffset;
		MemberArgs.Target = Args.Target + MemberDescriptor.InternalMemberOffset;

		Serializer->CloneDynamicState(Context, MemberArgs);
	}
}

void FStructNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	// If no member has dynamic state then there's nothing for us to do
	if (!EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState))
	{
		return;
	}

	const FReplicationStateMemberDescriptor* MemberDescriptors = Descriptor->MemberDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors = Descriptor->MemberSerializerDescriptors;
	const FReplicationStateMemberTraitsDescriptor* MemberTraitsDescriptors = Descriptor->MemberTraitsDescriptors;
	const uint32 MemberCount = Descriptor->MemberCount;

	for (uint32 MemberIt = 0; MemberIt < MemberCount; ++MemberIt)
	{
		const FReplicationStateMemberTraitsDescriptor& MemberTraitsDescriptor = MemberTraitsDescriptors[MemberIt];
		if (!EnumHasAnyFlags(MemberTraitsDescriptor.Traits, EReplicationStateMemberTraits::HasDynamicState))
		{
			continue;
		}

		const FReplicationStateMemberSerializerDescriptor& MemberSerializerDescriptor = MemberSerializerDescriptors[MemberIt];
		const FNetSerializer* Serializer = MemberSerializerDescriptor.Serializer;
		const FReplicationStateMemberDescriptor& MemberDescriptor = MemberDescriptors[MemberIt];

		FNetFreeDynamicStateArgs MemberArgs;
		MemberArgs.Version = 0;
		MemberArgs.NetSerializerConfig = MemberSerializerDescriptor.SerializerConfig;
		MemberArgs.Source = Args.Source + MemberDescriptor.InternalMemberOffset;

		Serializer->FreeDynamicState(Context, MemberArgs);
	}
}

void FStructNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	// User implemented forwarding serializers don't have access to ReplicationStateOperationsInternal	
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const FReplicationStateDescriptor* Descriptor = Config->StateDescriptor;
	Private::FReplicationStateOperationsInternal::CollectReferences(Context, *reinterpret_cast<FNetReferenceCollector*>(Args.Collector), Args.ChangeMaskInfo, reinterpret_cast<const uint8*>(Args.Source), Descriptor);
}

void FStructNetSerializer::CloneStructMember(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& CloneArgs)
{
	const FReplicationStateDescriptor* StateDescriptor = static_cast<const ConfigType*>(CloneArgs.NetSerializerConfig)->StateDescriptor;
	const bool bNeedFreeingAndCloning = EnumHasAnyFlags(StateDescriptor->Traits, EReplicationStateTraits::HasDynamicState);

	if (bNeedFreeingAndCloning)
	{
		FNetFreeDynamicStateArgs FreeArgs;
		FreeArgs.NetSerializerConfig = CloneArgs.NetSerializerConfig;
		FreeArgs.Version = CloneArgs.Version;
		FreeArgs.Source = CloneArgs.Target;
		FreeDynamicState(Context, FreeArgs);
	}

	FPlatformMemory::Memcpy(reinterpret_cast<void*>(CloneArgs.Target), reinterpret_cast<const void*>(CloneArgs.Source), StateDescriptor->InternalSize);

	if (bNeedFreeingAndCloning)
	{
		CloneDynamicState(Context, CloneArgs);
	}
}

}
