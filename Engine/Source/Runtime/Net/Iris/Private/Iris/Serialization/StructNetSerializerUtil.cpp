// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/StructNetSerializerUtil.h"

#include "Iris/Serialization/NetSerializers.h"
#include "Iris/Serialization/NetSerializerArrayStorage.h"

namespace UE::Net
{
	void WriteStruct(FNetSerializationContext& Context, NetSerializerValuePointer InValue, const FReplicationStateDescriptor* Descriptor)
	{
		if (!ensureAlwaysMsgf(Descriptor, TEXT("Replication State Descriptor cannot be null")))
		{
			return;
		}
		const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FStructNetSerializer);
		FStructNetSerializerConfig StructConfig;
		StructConfig.StateDescriptor = Descriptor;
		
		FNetSerializerAlignedStorage QuantizedStorage;
		QuantizedStorage.AdjustSize(Context, StructConfig.StateDescriptor->InternalSize, StructConfig.StateDescriptor->InternalAlignment);
		{
			FNetQuantizeArgs QuantizeArgs;
			QuantizeArgs.Source = InValue;
			QuantizeArgs.Target = NetSerializerValuePointer(QuantizedStorage.GetData());
			QuantizeArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
			Serializer.Quantize(Context, QuantizeArgs);
		}

		{
			FNetSerializeArgs SerializeArgs;
			SerializeArgs.Version = Serializer.Version;
			SerializeArgs.Source = NetSerializerValuePointer(QuantizedStorage.GetData());
			SerializeArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
			Serializer.Serialize(Context, SerializeArgs);
		}
		QuantizedStorage.Free(Context);
	}
	
	void ReadStruct(FNetSerializationContext& Context, NetSerializerValuePointer OutValue, const FReplicationStateDescriptor* Descriptor)
	{
		if (!ensureAlwaysMsgf(Descriptor, TEXT("Replication State Descriptor cannot be null")))
		{
			return;
		}
		const FNetSerializer& Serializer = UE_NET_GET_SERIALIZER(FStructNetSerializer);
		FStructNetSerializerConfig StructConfig;
		StructConfig.StateDescriptor = Descriptor;
		FNetSerializerAlignedStorage QuantizedStorage;
		QuantizedStorage.AdjustSize(Context, StructConfig.StateDescriptor->InternalSize, StructConfig.StateDescriptor->InternalAlignment);
		{
			FNetDeserializeArgs SerializeArgs;
			SerializeArgs.Version = Serializer.Version;
			SerializeArgs.Target = NetSerializerValuePointer(QuantizedStorage.GetData());
			SerializeArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
			Serializer.Deserialize(Context, SerializeArgs);
		}
		
		{
			FNetDequantizeArgs DequantizeArgs;
			DequantizeArgs.Source = NetSerializerValuePointer(QuantizedStorage.GetData());
			DequantizeArgs.Target = OutValue;
			DequantizeArgs.NetSerializerConfig = NetSerializerConfigParam(&StructConfig);
			Serializer.Dequantize(Context, DequantizeArgs);
		}
		
		QuantizedStorage.Free(Context);
	}
}
