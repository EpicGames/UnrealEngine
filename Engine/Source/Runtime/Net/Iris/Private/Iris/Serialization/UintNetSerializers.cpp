// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/UintNetSerializers.h"
#include "Iris/Serialization/IntNetSerializerBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UintNetSerializers)

namespace UE::Net
{

struct FUint8NetSerializer : public Private::FIntNetSerializerBase<uint8, FUint8NetSerializerConfig>
{
	inline static const FUint8NetSerializerConfig DefaultConfig = FUint8NetSerializerConfig(uint8(8));
};
UE_NET_IMPLEMENT_SERIALIZER(FUint8NetSerializer);

struct FUint16NetSerializer : public Private::FIntNetSerializerBase<uint16, FUint16NetSerializerConfig>
{
	inline static const FUint16NetSerializerConfig DefaultConfig = FUint16NetSerializerConfig(uint8(16));
};
UE_NET_IMPLEMENT_SERIALIZER(FUint16NetSerializer);

struct FUint32NetSerializer : public Private::FIntNetSerializerBase<uint32, FUint32NetSerializerConfig>
{
	inline static const FUint32NetSerializerConfig DefaultConfig = FUint32NetSerializerConfig(uint8(32));
};
UE_NET_IMPLEMENT_SERIALIZER(FUint32NetSerializer);

struct FUint64NetSerializer : public Private::FIntNetSerializerBase<uint64, FUint64NetSerializerConfig>
{
	inline static const FUint64NetSerializerConfig DefaultConfig = FUint64NetSerializerConfig(uint8(64));
};
UE_NET_IMPLEMENT_SERIALIZER(FUint64NetSerializer);

}
