// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/UintRangeNetSerializers.h"
#include "Iris/Serialization/IntRangeNetSerializerUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UintRangeNetSerializers)

namespace UE::Net
{

struct FUint8RangeNetSerializer : public Private::FIntRangeNetSerializerBase<uint8, FUint8RangeNetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FUint8RangeNetSerializer);

struct FUint16RangeNetSerializer : public Private::FIntRangeNetSerializerBase<uint16, FUint16RangeNetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FUint16RangeNetSerializer);

struct FUint32RangeNetSerializer : public Private::FIntRangeNetSerializerBase<uint32, FUint32RangeNetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FUint32RangeNetSerializer);

struct FUint64RangeNetSerializer : public Private::FIntRangeNetSerializerBase<uint64, FUint64RangeNetSerializerConfig>
{
};
UE_NET_IMPLEMENT_SERIALIZER(FUint64RangeNetSerializer);

}
