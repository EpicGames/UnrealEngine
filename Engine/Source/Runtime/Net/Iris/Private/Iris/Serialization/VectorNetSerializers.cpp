// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Serialization/VectorNetSerializers.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Math/Vector.h"
#include "Traits/IntType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VectorNetSerializers)

namespace UE::Net::Private
{

template<typename T>
struct FFloatTripletNetSerializer
{
	using UintType = typename TUnsignedIntType<sizeof(T)>::Type;

	static const uint32 Version = 0;

	/**
	 * We are interested in the bit representation of the floats, not IEEE 754 behavior. This is particularly
	 * relevant for IsEqual where for example -0.0f == +0.0f if the values were treated as floats
	 * rather than the bit representation of the floats. By using integer types we can avoid
	 * implementing some functions and use the default implementations instead.
	 */
	struct FFloatTriplet
	{
		UintType X;
		UintType Y;
		UintType Z;

	public:
		bool operator==(const FFloatTriplet& Other) const { return X == Other.X && Y == Other.Y && Z == Other.Z; }
	};

	typedef FFloatTriplet SourceType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);
};

template<typename T>
void FFloatTripletNetSerializer<T>::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const FFloatTriplet& Value = *reinterpret_cast<const FFloatTriplet*>(Args.Source);

	const UintType X = Value.X;
	const UintType Y = Value.Y;
	const UintType Z = Value.Z;

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	if constexpr (sizeof(T) == 4)
	{
		if (Writer->WriteBool(X != 0))
		{
			Writer->WriteBits(X, 32U);
		}
		if (Writer->WriteBool(Y != 0))
		{
			Writer->WriteBits(Y, 32U);
		}
		if (Writer->WriteBool(Z != 0))
		{
			Writer->WriteBits(Z, 32U);
		}
	}
	else
	{
		if (Writer->WriteBool(X != 0))
		{
			Writer->WriteBits(static_cast<uint32>(X), 32U);
			Writer->WriteBits(static_cast<uint32>(X >> 32U), 32U);
		}
		if (Writer->WriteBool(Y != 0))
		{
			Writer->WriteBits(static_cast<uint32>(Y), 32U);
			Writer->WriteBits(static_cast<uint32>(Y >> 32U), 32U);
		}
		if (Writer->WriteBool(Z != 0))
		{
			Writer->WriteBits(static_cast<uint32>(Z), 32U);
			Writer->WriteBits(static_cast<uint32>(Z >> 32U), 32U);
		}
	}
}

template<typename T>
void FFloatTripletNetSerializer<T>::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	UintType X = 0;
	UintType Y = 0;
	UintType Z = 0;

	if constexpr (sizeof(T) == 4U)
	{
		if (Reader->ReadBool())
		{
			X = Reader->ReadBits(32U);
		}

		if (Reader->ReadBool())
		{
			Y = Reader->ReadBits(32U);
		}

		if (Reader->ReadBool())
		{
			Z = Reader->ReadBits(32U);
		}
	}
	else
	{
		if (Reader->ReadBool())
		{
			X = Reader->ReadBits(32U);
			X |= static_cast<UintType>(Reader->ReadBits(32U)) << 32U;
		}

		if (Reader->ReadBool())
		{
			Y = Reader->ReadBits(32U);
			Y |= static_cast<UintType>(Reader->ReadBits(32U)) << 32U;
		}

		if (Reader->ReadBool())
		{
			Z = Reader->ReadBits(32U);
			Z |= static_cast<UintType>(Reader->ReadBits(32U)) << 32U;
		}
	}

	FFloatTriplet& Value = *reinterpret_cast<FFloatTriplet*>(Args.Target);
	Value.X = X;
	Value.Y = Y;
	Value.Z = Z;
}

}

namespace UE::Net
{

static_assert(sizeof(decltype(FVector::X)) == 4U || sizeof(decltype(FVector::X)) == 8U, "Unknown floating point type in FVector.");

struct FVectorNetSerializer : public Private::FFloatTripletNetSerializer<decltype(FVector::X)>
{
	static const uint32 Version = 0;

	typedef FVectorNetSerializerConfig ConfigType;
	static const ConfigType DefaultConfig;

};
const FVectorNetSerializer::ConfigType FVectorNetSerializer::DefaultConfig;
UE_NET_IMPLEMENT_SERIALIZER(FVectorNetSerializer);

struct FVector3fNetSerializer : public Private::FFloatTripletNetSerializer<float>
{
	static const uint32 Version = 0;

	typedef FVector3fNetSerializerConfig ConfigType;
	static const ConfigType DefaultConfig;
};

const FVector3fNetSerializer::ConfigType FVector3fNetSerializer::DefaultConfig;
UE_NET_IMPLEMENT_SERIALIZER(FVector3fNetSerializer);

struct FVector3dNetSerializer : public Private::FFloatTripletNetSerializer<double>
{
	static const uint32 Version = 0;

	typedef FVector3dNetSerializerConfig ConfigType;
	static const ConfigType DefaultConfig;
};

const FVector3dNetSerializer::ConfigType FVector3dNetSerializer::DefaultConfig;
UE_NET_IMPLEMENT_SERIALIZER(FVector3dNetSerializer);

}
