// Copyright Epic Games, Inc. All Rights Reserved.

#include "Iris/Stats/NetStats.h"
#include "Iris/Stats/NetStatsContext.h"
#include "Iris/Core/IrisCsv.h"
#include "Iris/Core/IrisProfiler.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Async/Fundamental/Scheduler.h"

DEFINE_LOG_CATEGORY(LogNetStats);

namespace UE::Net::Private
{

static bool bCVARShouldIncludeSubObjectWithRoot = true;
static FAutoConsoleVariableRef CShouldIncludeSubObjectWithRoot(
	TEXT("net.Iris.Stats.ShouldIncludeSubObjectWithRoot"),
	bCVARShouldIncludeSubObjectWithRoot,
	TEXT("If enabled SubObjects will reports stats with RootObject, if set to false SubObjects will be treated as separate objects."
	));

// Per type stats
CSV_DEFINE_CATEGORY(IrisPreUpdateMS, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisPreUpdateCount, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisPollMS, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisPollCount, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisPollWasteMS, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisPollWasteCount, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisQuantizeMS, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisQuantizeCount, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisWriteMS, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisWriteCount, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisWriteKBytes, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisWriteWasteMS, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisWriteWasteCount, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisWriteWasteKBytes, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisWriteCreationInfoCount, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisWriteCreationInfoKBytes, WITH_SERVER_CODE);
CSV_DEFINE_CATEGORY(IrisWriteExportsCount, WITH_SERVER_CODE);

void FNetSendStats::Accumulate(const FNetSendStats& Other)
{
#if UE_NET_IRIS_CSV_STATS
	UE::TScopeLock Lock(CS);

	Stats.ScheduledForReplicationRootObjectCount += Other.Stats.ScheduledForReplicationRootObjectCount;
	Stats.ReplicatedRootObjectCount += Other.Stats.ReplicatedRootObjectCount;
	Stats.ReplicatedObjectCount += Other.Stats.ReplicatedObjectCount;
	Stats.ReplicatedDestructionInfoCount += Other.Stats.ReplicatedDestructionInfoCount;
	Stats.DeltaCompressedObjectCount += Other.Stats.DeltaCompressedObjectCount;
	Stats.ReplicatedObjectStatesMaskedOut += Other.Stats.ReplicatedObjectStatesMaskedOut;
	Stats.ActiveHugeObjectCount += Other.Stats.ActiveHugeObjectCount;
	Stats.HugeObjectsWaitingForAckCount += Other.Stats.HugeObjectsWaitingForAckCount;
	Stats.HugeObjectsStallingCount += Other.Stats.HugeObjectsStallingCount;

	Stats.HugeObjectWaitingForAckTimeInSeconds += Other.Stats.HugeObjectWaitingForAckTimeInSeconds;
	Stats.HugeObjectStallingTimeInSeconds += Other.Stats.HugeObjectStallingTimeInSeconds;
#endif 
}

void FNetSendStats::Reset()
{
#if UE_NET_IRIS_CSV_STATS
	UE::TScopeLock Lock(CS);
	Stats = FStats();
#endif
}

void FNetSendStats::ReportCsvStats()
{
#if CSV_PROFILER_STATS
	UE::TScopeLock Lock(CS);

	// Calculate connection averages for some stats
	if (Stats.ReplicatingConnectionCount > 0)
	{
		const float ConnectionCountFloat = float(Stats.ReplicatingConnectionCount);
		CSV_CUSTOM_STAT(Iris, AvgScheduledForReplicationRootObjectCount, float(Stats.ScheduledForReplicationRootObjectCount)/ConnectionCountFloat, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgReplicatedRootObjectCount, float(Stats.ReplicatedRootObjectCount)/ConnectionCountFloat, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgReplicatedObjectCount, float(Stats.ReplicatedObjectCount)/ConnectionCountFloat, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgReplicatedDestructionInfoCount, float(Stats.ReplicatedDestructionInfoCount)/ConnectionCountFloat, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgReplicatedObjectStatesMaskedOut, float(Stats.ReplicatedObjectStatesMaskedOut)/ConnectionCountFloat, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgDeltaCompressedObjectCount, float(Stats.DeltaCompressedObjectCount)/ConnectionCountFloat, ECsvCustomStatOp::Set);
	}
	else
	{
		CSV_CUSTOM_STAT(Iris, AvgScheduledForReplicationRootObjectCount, 0.0f, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgReplicatedRootObjectCount, 0.0f, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgReplicatedObjectCount, 0.0f, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgReplicatedDestructionInfoCount, 0.0f, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgReplicatedHugeObjectCount, 0.0f, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgDeltaCompressedObjectCount, 0.0f, ECsvCustomStatOp::Set);
	}

	// Huge object counts
	CSV_CUSTOM_STAT(Iris, ActiveHugeObjectCount, Stats.ActiveHugeObjectCount, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(Iris, HugeObjectsWaitingForAckCount, Stats.HugeObjectsWaitingForAckCount, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(Iris, HugeObjectsStallingCount, Stats.HugeObjectsStallingCount, ECsvCustomStatOp::Set);

	CSV_CUSTOM_STAT(Iris, ReplicatingConnectionCount, Stats.ReplicatingConnectionCount, ECsvCustomStatOp::Set);

	CSV_CUSTOM_STAT(Iris, HugeObjectWaitingForAckTimeInSeconds, Stats.HugeObjectWaitingForAckTimeInSeconds, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(Iris, HugeObjectStallingTimeInSeconds, Stats.HugeObjectStallingTimeInSeconds, ECsvCustomStatOp::Set);
#endif
}

FNetStatsContext::FNetStatsContext() = default;
FNetStatsContext::~FNetStatsContext() = default;

FNetTypeStats::FNetTypeStats()
{
	ParentStatsContext = CreateNetStatsContext();
	// Add default TypeStats
	GetOrCreateTypeStats(TEXT("Undefined"));
	GetOrCreateTypeStats(TEXT("OOBChannel"));
}

FNetTypeStats::~FNetTypeStats()
{
	check(!bIsInParallelPhase);
	CleanupChildNetStatsContexts();
	delete ParentStatsContext;
}

void FNetTypeStats::Init(FInitParams& InitParams)
{
	NetRefHandleManager = InitParams.NetRefHandleManager;
	ResetStats();
}

void FNetTypeStats::PreUpdateSetup()
{
	//Create child contexts if needed
	LowLevelTasks::FScheduler& Scheduler = LowLevelTasks::FScheduler::Get();
	const int32 MaxParallelThreads = Scheduler.GetMaxNumWorkers() + 1;//Include GameThread as Tasks can also run there in addition to Worker Threads
	const int32 NewChildCount = MaxParallelThreads - ChildStatsContext.Num();
	UE_CLOGF(NewChildCount > 0, LogNetStats, Log, "FNetTypeStats Creating %i CreateChildNetStatsContexts. Total: %i", NewChildCount, MaxParallelThreads);
	for (int32 i = 0; i < NewChildCount; ++i)
	{
		CreateChildNetStatsContext();
	}
}

void FNetTypeStats::ResetStats()
{
	UpdateContext(*ParentStatsContext);
	for (FNetStatsContext* ChildStatsContextPtr : ChildStatsContext)
	{
		if (ensure(ChildStatsContextPtr))
		{
			UpdateContext(*ChildStatsContextPtr);
		}
	}
}

int32 FNetTypeStats::GetOrCreateTypeStats(FName Name)
{
	const int32 ExistingIndex = TypeStatsNames.Find(Name);
	if (ExistingIndex != INDEX_NONE)
	{
		return ExistingIndex;
	}

	checkf(!bIsInParallelPhase, TEXT("Trying to expand TypeStatsNames from within a parallel phase. This is not supported as the net object list should never expand during a parallel phase"));

	//We're adding a new stat, which means all ChildStatsContexts will need to be expanded to match TypeStatsData
	const int32 NewTypeStatsIndex = int32(TypeStatsNames.Num());
	TypeStatsNames.Add(Name);
	ParentStatsContext->TypeStatsData.SetNum(TypeStatsNames.Num());
	for (int32 i = 0; i < ChildStatsContext.Num(); ++i)
	{
		ChildStatsContext[i]->TypeStatsData.SetNum(TypeStatsNames.Num());
	}

	return NewTypeStatsIndex;
}

void FNetTypeStats::UpdateContext(FNetStatsContext& Context)
{
	Context.ResetStats(TypeStatsNames.Num());
	Context.NetRefHandleManager = NetRefHandleManager;
	Context.bShouldIncludeSubObjectWithRoot = bCVARShouldIncludeSubObjectWithRoot;
}

FNetStatsContext* FNetTypeStats::CreateNetStatsContext()
{
	FNetStatsContext* Context = new FNetStatsContext;
	UpdateContext(*Context);
	return Context;
}

FNetStatsContext* FNetTypeStats::CreateChildNetStatsContext()
{
	IRIS_PROFILER_SCOPE(FNetTypeStats_CreateChildNetStatsContext);

	UE::TScopeLock Lock(ChildStatsContextCS);

	//Set up new StatsContext for this ThreadId
	FNetStatsContext* NewChildStatsContext = new FNetStatsContext;
	NewChildStatsContext->ResetStats(TypeStatsNames.Num());
	NewChildStatsContext->NetRefHandleManager = ParentStatsContext->NetRefHandleManager;
	NewChildStatsContext->bShouldIncludeSubObjectWithRoot = bCVARShouldIncludeSubObjectWithRoot;

	ChildStatsContext.Add(NewChildStatsContext);
	return NewChildStatsContext;
}

FNetStatsContext* FNetTypeStats::AcquireChildNetStatsContext()
{
	UE::TScopeLock Lock(ChildStatsContextCS);
	
	for (FNetStatsContext* StatsContext : ChildStatsContext)
	{
		if (StatsContext->CanAcquire())
		{
			StatsContext->Acquire();
			return StatsContext;
		}
	}

	//Fallback to create a new context if we've run out (PreUpdateSetup should have created enough)
	ensureMsgf(false, TEXT("Too many tasks accessing ChildNetStatsContext, creating a new one. This may cause unexpected memory usage. Current count: %i"), ChildStatsContext.Num());
	FNetStatsContext* NewContext = CreateChildNetStatsContext();
	if (ensure(NewContext))
	{
		NewContext->Acquire();
	}

	return NewContext;
}

void FNetTypeStats::ReleaseChildNetStatsContext(FNetStatsContext* StatsContext)
{
	if(ensureMsgf(StatsContext, TEXT("ReleaseChildNetStatsContext was passed a null StatsContext. Possibility of net stats data being lost.")))
	{
		UE::TScopeLock Lock(ChildStatsContextCS);
		check(!StatsContext->CanAcquire());
		StatsContext->Release();
	}
}

void FNetTypeStats::CleanupChildNetStatsContexts()
{
	for (int32 i = 0; i < ChildStatsContext.Num(); i++)
	{
		delete ChildStatsContext[i];
	}
	ChildStatsContext.Empty();
}

void FNetTypeStats::AccumulateChildrenToParent()
{
	IRIS_PROFILER_SCOPE(FNetTypeStats_AccumulateChildrenToParent);

	checkf(!bIsInParallelPhase, TEXT("Trying to expand Accumulate ChildStatsContexts from within a parallel phase. This is not supported as this expects ChildStatsContexts to remain constant."));

	for(TArray<FNetStatsContext*>::TIterator It = ChildStatsContext.CreateIterator(); It; ++It)
	{
		FNetStatsContext* StatsContext = *It;
		if (ensure(StatsContext))
		{
			Accumulate(*StatsContext);
		}
	}
}

void FNetTypeStats::Accumulate(FNetStatsContext& Context)
{
	IRIS_PROFILER_SCOPE(FNetTypeStats_Accumulate);

	// Skip default context as that is our target.
	if (&Context == ParentStatsContext)
	{
		return;
	}

	if (!ensureMsgf(Context.TypeStatsData.Num() <= TypeStatsNames.Num(), TEXT("Invalid Context")))
	{
		return;
	}

	// Accumulate stats
	const FNetTypeStatsData* Src = Context.TypeStatsData.GetData();
	FNetTypeStatsData* Dst = ParentStatsContext->TypeStatsData.GetData();
	for (int32 StatsIndex = 0, EndIndex = FMath::Min(Context.TypeStatsData.Num(), ParentStatsContext->TypeStatsData.Num()); StatsIndex < EndIndex; ++StatsIndex)
	{
		Dst[StatsIndex].Accumulate(Src[StatsIndex]);
	}

	Context.ResetStats(TypeStatsNames.Num());
}

#define UE_NET_STATS_RECORD_TYPESTATS_TIME(StatsName, ValueName, StatsData) FCsvProfiler::RecordCustomStat(StatsName, CSV_CATEGORY_INDEX(Iris##ValueName##MS), FGenericPlatformTime::ToMilliseconds64(StatsData.Values[FNetTypeStatsData::EStatsIndex::ValueName].Time), ECsvCustomStatOp::Set)
#define UE_NET_STATS_RECORD_TYPESTATS_COUNT(StatsName, ValueName, StatsData) FCsvProfiler::RecordCustomStat(StatsName, CSV_CATEGORY_INDEX(Iris##ValueName##Count), static_cast<int32>(StatsData.Values[FNetTypeStatsData::EStatsIndex::ValueName].Count) , ECsvCustomStatOp::Set)
#define UE_NET_STATS_RECORD_TYPESTATS_BITS(StatsName, ValueName, StatsData) FCsvProfiler::RecordCustomStat(StatsName, CSV_CATEGORY_INDEX(Iris##ValueName##KBytes), float((StatsData.Values[FNetTypeStatsData::EStatsIndex::ValueName].Bits + 7U) / 8) / 1000.f , ECsvCustomStatOp::Set)

void FNetTypeStats::ReportCSVStats()
{
#if UE_NET_IRIS_CSV_STATS

	FCsvProfiler* Profiler = FCsvProfiler::Get();
	bIsEnabled = Profiler->IsCapturing();

	if (bIsEnabled)
	{
		IRIS_PROFILER_SCOPE(FNetTypeStats_ReportCSVStats);

		// Accumulate stats from child contexts into the parent one
		AccumulateChildrenToParent();

		// Report stats for this frame
		FNetTypeStatsData* TypeStatsDatas = ParentStatsContext->TypeStatsData.GetData();
		for (int32 StatsIndex = 0; StatsIndex < TypeStatsNames.Num(); ++StatsIndex)
		{
			const FName StatsName = TypeStatsNames[StatsIndex];
			FNetTypeStatsData& TypeStatsData = TypeStatsDatas[StatsIndex];

			// Report, we could do a loop here but we might end up not wanting to report all collected stats.
			UE_NET_STATS_RECORD_TYPESTATS_TIME(StatsName, PreUpdate, TypeStatsData);
			UE_NET_STATS_RECORD_TYPESTATS_COUNT(StatsName, PreUpdate, TypeStatsData);

			UE_NET_STATS_RECORD_TYPESTATS_TIME(StatsName, Poll, TypeStatsData);
			UE_NET_STATS_RECORD_TYPESTATS_COUNT(StatsName, Poll, TypeStatsData);

			UE_NET_STATS_RECORD_TYPESTATS_TIME(StatsName, PollWaste, TypeStatsData);
			UE_NET_STATS_RECORD_TYPESTATS_COUNT(StatsName, PollWaste, TypeStatsData);

			UE_NET_STATS_RECORD_TYPESTATS_TIME(StatsName, Quantize, TypeStatsData);
			UE_NET_STATS_RECORD_TYPESTATS_COUNT(StatsName, Quantize, TypeStatsData);

			UE_NET_STATS_RECORD_TYPESTATS_TIME(StatsName, Write, TypeStatsData);
			UE_NET_STATS_RECORD_TYPESTATS_COUNT(StatsName, Write, TypeStatsData);
			UE_NET_STATS_RECORD_TYPESTATS_BITS(StatsName, Write, TypeStatsData);

			UE_NET_STATS_RECORD_TYPESTATS_TIME(StatsName, WriteWaste, TypeStatsData);
			UE_NET_STATS_RECORD_TYPESTATS_COUNT(StatsName, WriteWaste, TypeStatsData);
			UE_NET_STATS_RECORD_TYPESTATS_BITS(StatsName, WriteWaste, TypeStatsData);

			UE_NET_STATS_RECORD_TYPESTATS_BITS(StatsName, WriteCreationInfo, TypeStatsData);
			UE_NET_STATS_RECORD_TYPESTATS_COUNT(StatsName, WriteCreationInfo, TypeStatsData);

			UE_NET_STATS_RECORD_TYPESTATS_COUNT(StatsName, WriteExports, TypeStatsData);

			// Reset
			TypeStatsData.Reset();
		}
	}

#endif
}

#undef UE_NET_STATS_RECORD_TYPESTATS_TIME
#undef UE_NET_STATS_RECORD_TYPESTATS_COUNT
#undef UE_NET_STATS_RECORD_TYPESTATS_BITS

void FReplicationStats::ReportCSVStats()
{
#if UE_NET_IRIS_CSV_STATS
	if (SampleCount)
	{
		const double SampleCountDbl = float(SampleCount);
		CSV_CUSTOM_STAT(Iris, AvgPendingObjectCount, double(PendingObjectCount) / SampleCountDbl, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgPendingDependentObjectCount, double(PendingDependentObjectCount) / SampleCountDbl, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgHugeObjectSendQueue, double(HugeObjectSendQueue) / SampleCountDbl, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, MaxPendingObjectCount, int32(MaxPendingObjectCount), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, MaxPendingDependentObjectCount, int32(MaxPendingDependentObjectCount), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, MaxHugeObjectSendQueue, int32(MaxHugeObjectSendQueue), ECsvCustomStatOp::Set);
	}
	else
	{
		CSV_CUSTOM_STAT(Iris, AvgPendingObjectCount, double(0), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgPendingDependentObjectCount, double(0), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, AvgHugeObjectSendQueue, double(0), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, MaxPendingObjectCount, int32(0), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, MaxPendingDependentObjectCount, int32(0), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(Iris, MaxHugeObjectSendQueue, int32(0), ECsvCustomStatOp::Set);
	}
#endif
}

} // end namespace UE::Net::Private
