// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/CriticalSection.h"
#include "Logging/LogMacros.h"
#include "Containers/Array.h"
#include "Misc/TransactionallySafeCriticalSection.h"

DECLARE_LOG_CATEGORY_EXTERN(LogNetStats, Log, All);

namespace UE::Net::Private
{
	class FNetRefHandleManager;
	class FNetStatsContext;
}

namespace UE::Net::Private
{

/**
 * Send stats for Iris replication reported to the CSV profiler. Mostly of interest on the server side due to the server authoritative network model.
 * Its intended use is to do thread local tracking to an instance and then use the Accumulate function for thread safe updating of the ReplicationSystem owned instance.
 * The ReplicationSystem owned instance is the one reporting to the CSV profiler.
 */
class FNetSendStats
{
public:
	FNetSendStats() = default;
	FNetSendStats(const FNetSendStats&) = delete;
	FNetSendStats& operator=(const FNetSendStats&) = delete;

	/** Set number of objects scheduled for replication. */
	void SetNumberOfRootObjectsScheduledForReplication(uint32 Count);

	/** Add number of replicated root objects. */
	void AddNumberOfReplicatedRootObjects(uint32 Count);

	/** Add number of replicated objects, including subobjects. */
	void AddNumberOfReplicatedObjects(uint32 Count);

	/** Add number of replicated destruction infos. */
	void AddNumberOfReplicatedDestructionInfos(uint32 Count);

	/** Add number of replicated objects, including subobjects, using delta compression. */
	void AddNumberOfDeltaCompressedReplicatedObjects(uint32 Count);

	/** Add number of replicated object states masked out such that no state is replicated for the object. The object may still replicate attachments. */
	void AddNumberOfReplicatedObjectStatesMaskedOut(uint32 Count);

	/** Get the number of replicated root objects. */
	uint32 GetNumberOfReplicatedRootObjects() const;

	/** Get the number of replicated objects, including subobjects. */
	uint32 GetNumberOfReplicatedObjects() const;

	/** Set the number of huge objects in sending or waiting to be acked. */
	void SetNumberOfActiveHugeObjects(uint32 Count);

	/** Add time in seconds waiting for completely sent huge object to be acked. */
	void AddHugeObjectWaitingTime(double Seconds);

	/** Add time in seconds waiting to be able to continue sending huge object. */
	void AddHugeObjectStallTime(double Seconds);

	/** Add stats from another instance. */
	IRISCORE_API void Accumulate(const FNetSendStats& Stats);

	/** Reset stats. */
	IRISCORE_API void Reset();

	/** Report the stats to the CSV profiler. Does nothing if CSV profiler support is compiled out. */
	IRISCORE_API void ReportCsvStats();

	/** Set number of replicating connections. */
	void SetNumberOfReplicatingConnections(uint32 Count);

private:
	// Helper struct to facilitate reset of stats.
	struct FStats
	{
		double HugeObjectWaitingForAckTimeInSeconds = 0;
		double HugeObjectStallingTimeInSeconds = 0;

		int32 ScheduledForReplicationRootObjectCount = 0;
		int32 ReplicatedRootObjectCount = 0;
		int32 ReplicatedObjectCount = 0;
		int32 ReplicatedDestructionInfoCount = 0;
		int32 DeltaCompressedObjectCount = 0;
		int32 ReplicatedObjectStatesMaskedOut = 0;
		int32 ActiveHugeObjectCount = 0;
		int32 HugeObjectsWaitingForAckCount = 0;
		int32 HugeObjectsStallingCount = 0;
		int32 ReplicatingConnectionCount = 0;
	};

	FTransactionallySafeCriticalSection CS;
	FStats Stats;
};

inline void FNetSendStats::SetNumberOfRootObjectsScheduledForReplication(uint32 Count)
{
	Stats.ScheduledForReplicationRootObjectCount = Count;
}

inline void FNetSendStats::AddNumberOfReplicatedRootObjects(uint32 Count)
{
	Stats.ReplicatedRootObjectCount += Count;
}

inline void FNetSendStats::AddNumberOfReplicatedObjects(uint32 Count)
{
	Stats.ReplicatedObjectCount += Count;
}

inline void FNetSendStats::AddNumberOfDeltaCompressedReplicatedObjects(uint32 Count)
{
	Stats.DeltaCompressedObjectCount += Count;
}

inline void FNetSendStats::AddNumberOfReplicatedObjectStatesMaskedOut(uint32 Count)
{
	Stats.ReplicatedObjectStatesMaskedOut += Count;
}

inline void FNetSendStats::AddNumberOfReplicatedDestructionInfos(uint32 Count)
{
	Stats.ReplicatedDestructionInfoCount += Count;
}

inline uint32 FNetSendStats::GetNumberOfReplicatedRootObjects() const
{
	return Stats.ReplicatedRootObjectCount;
}

inline uint32 FNetSendStats::GetNumberOfReplicatedObjects() const
{
	return Stats.ReplicatedObjectCount;
}

inline void FNetSendStats::SetNumberOfActiveHugeObjects(uint32 Count)
{
	Stats.ActiveHugeObjectCount = static_cast<int32>(Count);
}

inline void FNetSendStats::AddHugeObjectWaitingTime(double Seconds)
{
	++Stats.HugeObjectsWaitingForAckCount;
	Stats.HugeObjectWaitingForAckTimeInSeconds += Seconds;
}

inline void FNetSendStats::AddHugeObjectStallTime(double Seconds)
{
	++Stats.HugeObjectsStallingCount;
	Stats.HugeObjectStallingTimeInSeconds += Seconds;
}

inline void FNetSendStats::SetNumberOfReplicatingConnections(uint32 Count)
{
	Stats.ReplicatingConnectionCount = Count;
}

/**
 * Stats defined per object type for Iris replication reported to the CSV profiler. Mostly of interest on the server side due to the server authoritative network model.
 * Currently we use a single NetStatsContext when collecting the stats, when we go wide we need to extend this to use separate contexts for different threads.
 */
class FNetTypeStats
{
public:

	struct FInitParams
	{
		FNetRefHandleManager* NetRefHandleManager = nullptr;
	};

	// Preset stats type indices
	static constexpr int32 DefaultTypeStatsIndex = 0U;
	static constexpr int32 OOBChannelTypeStatsIndex = 1U;

public:
	FNetTypeStats();
	FNetTypeStats(const FNetTypeStats&) = delete;
	FNetTypeStats& operator=(const FNetTypeStats&) = delete;
	~FNetTypeStats();

	void Init(FInitParams& InitParams);

	/** Called once a frame before stats collection starts in order to set up ChildStatsContexts */
	void PreUpdateSetup();

	/** Reset stats */
	void ResetStats();
		
	/** Returns the TypeStatIndex associated with the Name or creates a new one if it does not exist */
	int32 GetOrCreateTypeStats(FName Name);

	/** Get parent context if stats is enabled. Should not be called when in a parallel phase, as the ChildContexts should be used instead. */
	FNetStatsContext* GetNetStatsContext() { ensure(!bIsInParallelPhase); return IsEnabled() ?  ParentStatsContext : nullptr; }

	/** Updated every frame based on the state of the CSVProfiler */
	bool IsEnabled() const { return bIsEnabled; }

	/** Accumulate stats from context to main context */
	void Accumulate(FNetStatsContext& Context);

	/** ReportCSVStats and reset context */
	void ReportCSVStats();

	/** Wipes the Child NetStatsContext Map, ready for the next frame */
	void CleanupChildNetStatsContexts();

	/** For each Child NetStatsContext, accumulate its values into the Parent NetStatsContext */
	void AccumulateChildrenToParent();

	/** Returns the next available ChildNetStatsContext which isn't being used by another task. (Thread-safe) */
	FNetStatsContext* AcquireChildNetStatsContext();

	/** Relinquishes a ChildNetStatsContext so it can be used by another task. (Thread-safe) */
	void ReleaseChildNetStatsContext(FNetStatsContext* StatsContext);

	/** Sets a flag used to guard against using non thread safe operations when we're running parallel tasks. */
	void SetIsInParallelPhase(const bool InParallelPhase) { bIsInParallelPhase = InParallelPhase; }

private:
	FNetStatsContext* CreateNetStatsContext();

	/** Creates a new FNetStatsContext and adds it to ChildStatsContext */
	FNetStatsContext* CreateChildNetStatsContext();

	void UpdateContext(FNetStatsContext& Context);

	FNetStatsContext* ParentStatsContext = nullptr;

	/** Critical Section used to prevent multiple threads from accessing ChildStatsContext array simultaneously */
	FTransactionallySafeCriticalSection ChildStatsContextCS;

	/** NetStatsContexts that are used by sub tasks during a parallel phase. Are accumulated to ParentStatsContext at the end of a frame. */
	TArray<FNetStatsContext*> ChildStatsContext;

	FNetRefHandleManager* NetRefHandleManager = nullptr;
	TArray<FName> TypeStatsNames;
	bool bIsEnabled = false;
	bool bIsInParallelPhase = false;
};

struct FReplicationStats
{
	/** Report the stats to the CSV profiler. Does nothing if CSV profiler support is compiled out. */
	void ReportCSVStats();

	void Accumulate(const FReplicationStats& Stats)
	{
		PendingObjectCount += Stats.PendingObjectCount;
		PendingDependentObjectCount += Stats.PendingDependentObjectCount;
		HugeObjectSendQueue += Stats.HugeObjectSendQueue;
		MaxPendingObjectCount = FMath::Max(MaxPendingObjectCount, Stats.MaxPendingObjectCount);
		MaxPendingDependentObjectCount = FMath::Max(MaxPendingDependentObjectCount, Stats.MaxPendingDependentObjectCount);
		MaxHugeObjectSendQueue = FMath::Max(MaxHugeObjectSendQueue, Stats.MaxHugeObjectSendQueue);
		SampleCount += Stats.SampleCount;
	}

	uint64 PendingObjectCount;
	uint64 PendingDependentObjectCount;
	uint64 HugeObjectSendQueue;
	uint32 MaxPendingObjectCount;
	uint32 MaxPendingDependentObjectCount;
	uint32 MaxHugeObjectSendQueue;
	uint32 SampleCount;
};

} // end namespace UE::Net::Private
