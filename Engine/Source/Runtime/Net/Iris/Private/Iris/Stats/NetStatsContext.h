// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Iris/Stats/NetStats.h"
#include "ProfilingDebugging/CsvProfilerConfig.h"
#include "Iris/ReplicationSystem/NetRefHandleManager.h"
#include "Iris/ReplicationSystem/ReplicationProtocol.h"
#include "Net/Core/Trace/NetTrace.h"

namespace UE::Net::Private
{

struct FNetTypeStatsData
{
	enum EStatsIndex : unsigned
	{
		PreUpdate = 0U,
		Poll,
		PollWaste,
		Quantize,
		Write,
		WriteWaste,
		WriteCreationInfo,
		WriteExports,
		Count
	};

	struct FStatsValue
	{
		void Accumulate(const FStatsValue& Other)
		{
			Time += Other.Time;
			Bits += Other.Bits;
			Count += Other.Count;
		}
		uint64 Time = uint64(0);
		uint32 Bits = 0U;
		uint32 Count = 0U;
	};

	/** Zero out all stats */
	void Reset()
	{
		*this = FNetTypeStatsData();
	}

	/** Accumulate all stats */
	void Accumulate(const FNetTypeStatsData& Other)
	{
		for (int32 Index = 0; Index < EStatsIndex::Count; ++Index)
		{
			Values[Index].Accumulate(Other.Values[Index]);
		}
	}

	FStatsValue Values[EStatsIndex::Count];
};

class FNetStatsContext
{
public:
	FNetStatsContext();
	~FNetStatsContext();
	FNetStatsContext(const FNetStatsContext&) = delete;
	FNetStatsContext& operator=(const FNetStatsContext&) = delete;

	void ResetStats(int32 NumTypeStats);
	
	// Helper to lookup TypeStatsIndex from object, we currently store the NetTypeStatsIndex in the ReplicationProtocol but nothing prevents us from storing it elsewhere
	// Depending on config subobjects report their stats with the root, for those cases we do not bump the count of root objects.
	// returns the FNetTypeStatsData associated with the protocol used by the object references by the InternalIndex
	static FNetTypeStatsData& GetTypeStatsDataForObject(FNetStatsContext& Context, FInternalNetRefIndex InternalIndex, uint32& OutUpdateCount);
	static FNetTypeStatsData& GetTypeStatsDataForObject(FNetStatsContext& Context, FInternalNetRefIndex InternalIndex);
	static FNetTypeStatsData& GetTypeStatsData(FNetStatsContext& Context, const FNetRefHandleManager::FReplicatedObjectData& ObjectData, bool bTreatAsRoot);

	bool CanAcquire() const
	{
		return !bIsAcquiredByTask;
	}

protected:
	void Acquire()
	{
		check(!bIsAcquiredByTask);
		bIsAcquiredByTask = true;
	}
	void Release()
	{
		check(bIsAcquiredByTask);
		bIsAcquiredByTask = false;
	}

private:
	friend class FNetTypeStats;

	const FNetRefHandleManager* NetRefHandleManager = nullptr;
	TArray<FNetTypeStatsData> TypeStatsData;
	bool bShouldIncludeSubObjectWithRoot = false;
	bool bIsAcquiredByTask = false;//True if actively being used by a parallel task and cannot be acquired by another one
};

class FNetStatsTimer : public FNetTimer
{
public:
	FNetStatsTimer(FNetStatsContext* InStatsContext)
	: FNetTimer(InStatsContext != nullptr)
	, NetStatsContext(InStatsContext)
	{
	}

	// If we have a context, this timer is enabled 
	FNetStatsContext* GetNetStatsContext() const { return NetStatsContext; }
	
private:
	FNetStatsContext* NetStatsContext;
};

inline FNetTypeStatsData& FNetStatsContext::GetTypeStatsData(FNetStatsContext& Context, const FNetRefHandleManager::FReplicatedObjectData& ObjectData, bool bTreatAsRoot)
{
	int32 TypeStatsIndex = FNetTypeStats::DefaultTypeStatsIndex;
	if (bTreatAsRoot)
	{
		TypeStatsIndex = ObjectData.Protocol ? ObjectData.Protocol->TypeStatsIndex : FNetTypeStats::OOBChannelTypeStatsIndex;
	}
	else
	{
		const FNetRefHandleManager::FReplicatedObjectData& RootObjectData = Context.NetRefHandleManager->GetReplicatedObjectDataNoCheck(ObjectData.SubObjectRootIndex);
		TypeStatsIndex = RootObjectData.Protocol ? RootObjectData.Protocol->TypeStatsIndex : FNetTypeStats::OOBChannelTypeStatsIndex;
	}

	return Context.TypeStatsData[TypeStatsIndex];
}

inline FNetTypeStatsData& FNetStatsContext::GetTypeStatsDataForObject(FNetStatsContext& Context, FInternalNetRefIndex InternalIndex)
{
	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = Context.NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);
	const bool bTreatAsRoot = !Context.bShouldIncludeSubObjectWithRoot || !ObjectData.IsSubObject();
	return GetTypeStatsData(Context, ObjectData, bTreatAsRoot);
}


inline FNetTypeStatsData& FNetStatsContext::GetTypeStatsDataForObject(FNetStatsContext& Context, FInternalNetRefIndex InternalIndex, uint32& OutUpdateCount)
{
	const FNetRefHandleManager::FReplicatedObjectData& ObjectData = Context.NetRefHandleManager->GetReplicatedObjectDataNoCheck(InternalIndex);		
	const bool bTreatAsRoot = !Context.bShouldIncludeSubObjectWithRoot || !ObjectData.IsSubObject();
	OutUpdateCount = bTreatAsRoot ? 1U : 0U;
	return GetTypeStatsData(Context, ObjectData, bTreatAsRoot);
}


inline void FNetStatsContext::ResetStats(int32 NumTypeStats)
{ 
	TypeStatsData.SetNum(NumTypeStats);
	for (FNetTypeStatsData& TypeStats : TypeStatsData)
	{
		TypeStats.Reset();
	}
} 

}

// Wrap usage of stats in macros so we can compile it out
#if UE_NET_IRIS_CSV_STATS

#define UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT(Timer, StatName, ObjectIndex) \
	do { \
		if (UE::Net::Private::FNetStatsContext* LocalNetStatsContext = Timer.GetNetStatsContext()) \
		{ \
			uint32 CountIncrement = 0U; \
			UE::Net::Private::FNetTypeStatsData& StatsData = UE::Net::Private::FNetStatsContext::GetTypeStatsDataForObject(*LocalNetStatsContext, ObjectIndex, CountIncrement); \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName].Time += Timer.GetCyclesSinceStart(); \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName].Count += CountIncrement; \
		} \
	} while (0)

#define UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT_AS_WASTE(Timer, StatName, ObjectIndex) \
	do { \
		if (UE::Net::Private::FNetStatsContext* LocalNetStatsContext = Timer.GetNetStatsContext()) \
		{ \
			const uint64 DeltaTimeForStat = Timer.GetCyclesSinceStart(); \
			uint32 CountIncrement = 0U; \
			UE::Net::Private::FNetTypeStatsData& StatsData = UE::Net::Private::FNetStatsContext::GetTypeStatsDataForObject(*LocalNetStatsContext, ObjectIndex, CountIncrement); \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName].Time += DeltaTimeForStat; \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName].Count += CountIncrement; \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName##Waste].Time += DeltaTimeForStat; \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName##Waste].Count += CountIncrement; \
		} \
	} while (0)

#define UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_AND_COUNT_FOR_OBJECT(NetStatsContext, BitCount, StatName, ObjectIndex) \
	do { \
		if (NetStatsContext) \
		{ \
			uint32 CountIncrement = 0U; \
			UE::Net::Private::FNetTypeStatsData& StatsData = UE::Net::Private::FNetStatsContext::GetTypeStatsDataForObject(*NetStatsContext, ObjectIndex, CountIncrement); \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName].Bits += BitCount; \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName].Count += CountIncrement; \
		} \
	} while (0)

#define UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_FOR_OBJECT(NetStatsContext, BitCount, StatName, ObjectIndex) \
	do { \
		if (NetStatsContext) \
		{ \
			UE::Net::Private::FNetTypeStatsData& StatsData = UE::Net::Private::FNetStatsContext::GetTypeStatsDataForObject(*NetStatsContext, ObjectIndex); \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName].Bits += BitCount; \
		} \
	} while (0)

#define UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_FOR_OBJECT_AS_WASTE(NetStatsContext, BitCount, StatName, ObjectIndex) \
	do { \
		if (NetStatsContext) \
		{ \
			UE::Net::Private::FNetTypeStatsData& StatsData = UE::Net::Private::FNetStatsContext::GetTypeStatsDataForObject(*NetStatsContext, ObjectIndex); \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName].Bits += BitCount; \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName##Waste].Bits += BitCount; \
		} \
	} while (0)

// Increment stat count by 1
#define UE_NET_IRIS_STATS_INCREMENT_FOR_OBJECT(NetStatsContext, StatName, ObjectIndex) \
	do { \
		if (NetStatsContext) \
		{ \
			UE::Net::Private::FNetTypeStatsData& StatsData = UE::Net::Private::FNetStatsContext::GetTypeStatsDataForObject(*NetStatsContext, ObjectIndex); \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName].Count++; \
		} \
	} while (0)

// Increment stat count by any amount
#define UE_NET_IRIS_STATS_ADD_COUNT_FOR_OBJECT(NetStatsContext, StatName, ObjectIndex, CountToAdd) \
	do { \
		if (NetStatsContext) \
		{ \
			UE::Net::Private::FNetTypeStatsData& StatsData = UE::Net::Private::FNetStatsContext::GetTypeStatsDataForObject(*NetStatsContext, ObjectIndex); \
			StatsData.Values[UE::Net::Private::FNetTypeStatsData::EStatsIndex::StatName].Count += CountToAdd; \
		} \
	} while (0)

#else

#define UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT(...)
#define UE_NET_IRIS_STATS_ADD_TIME_AND_COUNT_FOR_OBJECT_AS_WASTE(...)
#define UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_AND_COUNT_FOR_OBJECT(...)
#define UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_FOR_OBJECT(...)
#define UE_NET_IRIS_STATS_ADD_BITS_WRITTEN_FOR_OBJECT_AS_WASTE(...)
#define UE_NET_IRIS_STATS_INCREMENT_FOR_OBJECT(...)
#define UE_NET_IRIS_STATS_ADD_COUNT_FOR_OBJECT(...)

#endif

// this is needed for net traces as well
#if UE_NET_IRIS_CSV_STATS || UE_NET_TRACE_ENABLED

#define UE_NET_IRIS_STATS_TIMER(TimerName, NetStatsContext) UE::Net::Private::FNetStatsTimer TimerName(NetStatsContext);

#else

#define UE_NET_IRIS_STATS_TIMER(...)

#endif
