// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/StringFwd.h"
#include "Delegates/Delegate.h"
#include "UObject/NameTypes.h"

namespace UE::Net
{

/**
 * FIrisDynamicConfig is a system designed for Iris to handle runtime dynamic configuration of features based on dynamic loading/unloading or activation/deactivation of executable code, such as a GameFeaturePlugin.
 * It will only support specific configuration sections rather than being generic like DefaultEngine.ini. This means that each subsystem will decide exactly what will be supported in terms of dynamic configuration.
 */
class UE_EXPERIMENTAL(5.8, "Iris dynamic config API is experimental and subject to change.") FIrisDynamicConfig
{
public:
	/** Register an Iris config file for dynamically loaded or activated executable code. Make sure to use a unique name so that the config can be unloaded properly. Config must be unregistered via UnregisterDynamicConfig except during app shutdown. */
	IRISCORE_API static void RegisterDynamicConfig(FName UniqueConfigName, const FString& FilePath);

	/** Register an Iris config buffer for dynamically loaded or activated executable code. Make sure to use a unique name and virtual file path so that the config can be unloaded properly. Config must be unregistered via UnregisterDynamicConfig except during app shutdown. */
	IRISCORE_API static void RegisterDynamicConfigBuffer(FName UniqueConfigName, const FString& VirtualFilePath, const FString& ConfigContents);

	/** Register an Iris config buffer for dynamically loaded or activated executable code. Make sure to use a unique name and virtual file path so that the config can be unloaded properly. The contents will be converted to an FString internally. Config must be unregistered via UnregisterDynamicConfig except during app shutdown. */
	IRISCORE_API static void RegisterDynamicConfigBuffer(FName UniqueConfigName, const FString& VirtualFilePath, FUtf8StringView ConfigContents);

	/** Unregister the config file or buffer associated with the name. */
	IRISCORE_API static void UnregisterDynamicConfig(FName UniqueConfigName);

	using FOnIrisDynamicConfigChange = TMulticastDelegate<void(const TSet<FString>& ModifiedSections)>;
	/** Dynamic config loaded/unloaded delegate registration/unregistration */
	IRISCORE_API static FOnIrisDynamicConfigChange::RegistrationType& OnIrisDynamicConfigChange();

	/** Retrieve the contents for a config section. */
	IRISCORE_API static void GetSection(const TCHAR* SectionName, TArray<FString>& OutKeyValuePairs);

	/** Retrieve the values for an array in a config section. */
	IRISCORE_API static void GetArray(const TCHAR* SectionName, const TCHAR* ArrayName, TArray<FString>& OutValues);
};

}
