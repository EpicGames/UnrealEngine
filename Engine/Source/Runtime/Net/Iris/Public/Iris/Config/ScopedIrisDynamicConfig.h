// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Config/IrisDynamicConfig.h"
#include "Containers/Set.h"

namespace UE::Net
{

/**
 * FScopedIrisDynamicConfig is a helper class for FIrisDynamicConfig that unregisters all registered dynamic configs in its destructor.
 * Configs can be unregistered manually if desired. The API mimics FIrisDynamicConfig when it comes to registering and unregistering individual configs.
 */
class UE_EXPERIMENTAL(5.8, "Iris dynamic config API is experimental and subject to change.") FScopedIrisDynamicConfig
{
public:
	~FScopedIrisDynamicConfig()
	{
		UnregisterAllDynamicConfigs();
	}

	void RegisterDynamicConfig(FName UniqueConfigName, const FString& FilePath)
	{
		RegisteredConfigs.Add(UniqueConfigName);
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		FIrisDynamicConfig::RegisterDynamicConfig(UniqueConfigName, FilePath);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	}

	void RegisterDynamicConfigBuffer(FName UniqueConfigName, const FString& VirtualFilePath, const FString& ConfigContents)
	{
		RegisteredConfigs.Add(UniqueConfigName);
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		FIrisDynamicConfig::RegisterDynamicConfigBuffer(UniqueConfigName, VirtualFilePath, ConfigContents);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	}

	void RegisterDynamicConfigBuffer(FName UniqueConfigName, const FString& VirtualFilePath, FUtf8StringView ConfigContents)
	{
		RegisteredConfigs.Add(UniqueConfigName);
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		FIrisDynamicConfig::RegisterDynamicConfigBuffer(UniqueConfigName, VirtualFilePath, ConfigContents);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	}

	void UnregisterDynamicConfig(FName UniqueConfigName)
	{
		RegisteredConfigs.Remove(UniqueConfigName);
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		FIrisDynamicConfig::UnregisterDynamicConfig(UniqueConfigName);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	}

	void UnregisterAllDynamicConfigs()
	{
		for (FName ConfigName : RegisteredConfigs)
		{
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
			FIrisDynamicConfig::UnregisterDynamicConfig(ConfigName);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		}
		RegisteredConfigs.Empty();
	}
private:
	TSet<FName, DefaultKeyFuncs<FName>, TInlineSetAllocator<1>> RegisteredConfigs;
};

}
