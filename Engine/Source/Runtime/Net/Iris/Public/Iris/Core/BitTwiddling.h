// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "Templates/EnableIf.h"
#include "Templates/IsSigned.h"
#include "Traits/IntType.h"

namespace UE::Net
{

/**
 * GetBitsNeeded returns the number of bits needed to serialize the value and be able to deserialize it to the
 * original value. For signed integers this means there always need to be room for the sign bit. On the
 * receiving side the most significant bit received will be propagated to all higher bits. For unsigned values
 * no consideration needs to be taken for sign.
 *
 * Examples: 
 * int(-1) will return 1
 * int(0) will return 1
 * int(1) will return 2
 * unsigned(0) will return 0
 * unsigned(1) will return 1
 *
 * @param Value An integer value.
 * @return The number of bits needed to be able to properly reconstruct the value.
 *         For signed integers a sign bit is assumed to be propagated to all higher bits.
 */
template<typename T, typename TEnableIf<TIsSigned<T>::Value && sizeof(T) <= 4U, int32>::Type X = -1>
constexpr uint32 GetBitsNeeded(const T Value)
{
	typedef typename TUnsignedIntType<sizeof(T)>::Type SmallUnsignedType;

	/* The algorithm for signed integers works as described below.
	 *
	 * 1. Replicate the sign bit to the right, by assuming right - shift on signed integers propagates the sign - bit.
	 *    This will create a bit pattern consisting of all ones for negative numbersand all zeros for positive numbers.
	 * 2. Exclusive - or the bit - pattern with the original value.This will do nothing for positive numbers, but for
	 *    negative this will clear out all the top bits that were set to 1 and then set the most significant zero bit to 1.
	 * 3. The resulting value from step 2 makes it possible to find the most significant bit set apart from sign - bits that
	 *    can easily be derived from the bit above.The number of bits needed can now be calculated as 1 for the sign
	 *    plus the number of bits in the type minus how many zero bits in the value after the most significant set bit.
	 */
	const uint32 MassagedValue = uint32(SmallUnsignedType(T(Value ^ (Value >> (sizeof(T)*8 - 1U)))));
	return 33U - static_cast<uint32>(FPlatformMath::CountLeadingZeros(MassagedValue));
}

template<typename T, typename TEnableIf<!TIsSigned<T>::Value && sizeof(T) <= 4U, uint32>::Type X = 1U>
constexpr uint32 GetBitsNeeded(const T Value)
{
	return 32U - static_cast<uint32>(FPlatformMath::CountLeadingZeros(uint32(Value)));
}

template<typename T, typename TEnableIf<TIsSigned<T>::Value && sizeof(T) == 8U, int64>::Type X = -1LL>
constexpr uint32 GetBitsNeeded(const T Value)
{
	const uint64 MassagedValue = uint64(Value ^ (Value >> 63U));
	return 65U - static_cast<uint32>(FPlatformMath::CountLeadingZeros64(MassagedValue));
}

template<typename T, typename TEnableIf<!TIsSigned<T>::Value && sizeof(T) == 8U, uint64>::Type X = 1ULL>
constexpr uint32 GetBitsNeeded(const T Value)
{
	return 64U - static_cast<uint32>(FPlatformMath::CountLeadingZeros64(Value));
}

/**
 * GetBitsNeededForRange returns the number of bits needed to represent any value in the 
 * range [LowerBound, UpperBound]. This number is calculated as the number of bits needed
 * to express Value - LowerBound treated as an unsigned value.
 * @param LowerBound The lowest integer value in the range.
 * @param UpperBound The highest value in the range.
 * @return The number of bits needed to represent any value in the range, assuming user always knows 
 * @note It is assumed UpperBound >= LowerBound.
 */
template<typename T, typename TEnableIf<TIsSigned<T>::Value && sizeof(T) <= 4U, int32>::Type X = -1>
constexpr uint32 GetBitsNeededForRange(const T LowerBound, const T UpperBound)
{
	using UnsignedType = typename TUnsignedIntType<sizeof(T)>::Type;
	const uint32 Range = uint32(UnsignedType(UnsignedType(UpperBound) - UnsignedType(LowerBound)));
	return 32U - static_cast<uint32>(FPlatformMath::CountLeadingZeros(Range));
}

template<typename T, typename TEnableIf<!TIsSigned<T>::Value && sizeof(T) <= 4U, uint32>::Type X = 1U>
constexpr uint32 GetBitsNeededForRange(const T LowerBound, const T UpperBound)
{
	const uint32 Range = uint32(UpperBound) - uint32(LowerBound);
	return 32U - static_cast<uint32>(FPlatformMath::CountLeadingZeros(Range));
}

template<typename T, typename TEnableIf<sizeof(T) == 8U, uint64>::Type X = 1ULL>
constexpr uint32 GetBitsNeededForRange(const T LowerBound, const T UpperBound)
{
	const uint64 Range = uint64(UpperBound) - uint64(LowerBound);
	return 64U - static_cast<uint32>(FPlatformMath::CountLeadingZeros64(Range));
}

/** GetLeastSignificantBit returns the least significant bit set in Value, or 0 if none is set. */
template<typename T, typename TEnableIf<!TIsSigned<T>::Value, int>::Type X = 1>
constexpr T GetLeastSignificantBit(const T Value)
{
	using SignedT = typename TSignedIntType<sizeof(T)>::Type;
	const T LeastSignificantBit = Value & T(-SignedT(Value));
	return LeastSignificantBit;
}

/** Return the largest unsigned value that can be stored in BitCount numbers of bits. */
constexpr inline uint64 LargestBitValueForBits(uint64 BitCount)
{
	if (BitCount == 0)
	{
		return 0ULL;
	}

	if (BitCount > 64)
	{
		return ~0ULL;
	}

	return ~0ULL >> (64 - BitCount);
}

}
