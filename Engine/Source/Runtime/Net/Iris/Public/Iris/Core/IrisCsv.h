// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/CsvProfiler.h"
#include "Iris/IrisConfig.h"
#include "Iris/Core/IrisProfiler.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(IRISCORE_API, Iris);

#if UE_NET_IRIS_CSV_STATS
	#define IRIS_CSV_PROFILER_SCOPE(CsvCategory, x) \
	CSV_SCOPED_TIMING_STAT(CsvCategory, x); \
	IRIS_PROFILER_SCOPE(x)
#else
	#define IRIS_CSV_PROFILER_SCOPE(CsvCategory, x) IRIS_PROFILER_SCOPE(x)
#endif

