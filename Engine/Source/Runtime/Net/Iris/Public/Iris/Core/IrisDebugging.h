// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Net/Core/NetHandle/NetHandle.h"

class UReplicationSystem;
namespace UE::Net
{
	struct FReplicationProtocol;
	struct FReplicationInstanceProtocol;

	typedef uint32 FReplicationProtocolIdentifier;

	typedef uint32 FInternalNetRefIndex;
}

#if !UE_BUILD_SHIPPING

namespace UE::Net::IrisDebugHelper
{

/** Dummy methods binding pointers to external methods to avoid them from being stripped by compiler */
IRISCORE_API uint64 Init();

/** Trigger a breakpoint and return true if the object contains the current debug name */
IRISCORE_API bool BreakOnObjectName(UObject* Object);

/** Trigger a breakpoint and return true if the NetRefHandle is the current debug NetRefHandle */
IRISCORE_API bool BreakOnNetRefHandle(FNetRefHandle NetRefHandle);

/** Trigger a breakpoint and return true if the name contains the debug RPC string */
IRISCORE_API bool BreakOnRPCName(FName RPCName);

/** Trigger a breakpoint and return true if the index is the current debug index */
IRISCORE_API bool BreakOnInternalNetRefIndex(UE::Net::FInternalNetRefIndex InternalIndex);

/** Returns true if the object name contains the current debug name, will return true if no debug name is set */
IRISCORE_API bool FilterDebuggedObject(UObject* Object);

/** Returns the internal index set via SetIrisDebugInternalIndex */
IRISCORE_API UE::Net::FInternalNetRefIndex GetDebugNetInternalIndex();

/** Returns the NetRefHandle set via SetIrisDebugNetRefHandle */
IRISCORE_API FNetRefHandle GetDebugNetRefHandle();

/** Output state data to StringBuilder for the specified Handle */
void NetObjectStateToString(FStringBuilderBase& StringBuilder, FNetRefHandle RefHandle);

/** Find all handles references registered for a protocol and output to StringBuilder */
void NetObjectProtocolReferencesToString(FStringBuilderBase& StringBuilder, FReplicationProtocolIdentifier ProtocolId, uint32 ReplicationSystemId);

// Helper functions for debugging state data exposed as extern "C" that are callable from immediate and watch window in the debugger

/** Get the ReplicationSystem with the Id */
extern "C" IRISCORE_API UReplicationSystem* GetReplicationSystemForDebug(uint32 Id);

/** DebugOutputObject state for Handle specified by NetRefHandleId and ReplicationSystemId. */
extern "C" IRISCORE_API void DebugOutputNetObjectState(uint64 NetRefHandleId, uint32 ReplicationSystemId);

/**
 * Variant of NetObjectStateToString that can be used from breakpoints and in watch window to print the current state of a NetRefHandle.
 * NOTE: Use only for debugging as this variant uses a static buffer which is not thread safe.
 */
 //$IRIS TODO: Make console command to log these for object X or class X
extern "C" IRISCORE_API const TCHAR* DebugNetObjectStateToString(uint32 NetRefHandleId, uint32 ReplicationSystemId);

/** Find all handles references registered for a protocol and output to DebugOutput in debugger */
extern "C" IRISCORE_API void DebugOutputNetObjectProtocolReferences(FReplicationProtocolIdentifier ProtocolId, uint32 ReplicationSystemId);

/** Get info about replicated object */
struct FNetReplicatedObjectDebugInfo
{
	const FNetRefHandle* RefHandle;
	UE::Net::FInternalNetRefIndex InternalNetRefIndex;
	const UReplicationSystem* ReplicationSystem;
	const FReplicationProtocol* Protocol;
	const FReplicationInstanceProtocol* InstanceProtocol;
	const UObject* Object;
};

/** Look up replicated handle from Instance pointer and return debug information. This variant searches all active replication systems and returns information for the first one replicating the Instance. */
extern "C" IRISCORE_API FNetReplicatedObjectDebugInfo DebugNetObject(UObject* Instance);

/** Look up replicated handle from Instance pointer and return debug information. This variant only searches the ReplicationSystem with the specified id. */
extern "C" IRISCORE_API FNetReplicatedObjectDebugInfo DebugNetObjectById(UObject* Instance, uint32 ReplicationSystemId);

/** Look up replicated handle and return debug information */
extern "C" IRISCORE_API FNetReplicatedObjectDebugInfo DebugNetRefHandle(FNetRefHandle Handle);

/** Look up replicated handle specified by handle id and replicationsystem id and return debug information */
extern "C" IRISCORE_API FNetReplicatedObjectDebugInfo DebugNetRefHandleById(uint64 NetRefHandleId, uint32 ReplicationSystemId);

/** Look up replicated handle specified by an internal index and replicationsystem id and return debug information */
extern "C" IRISCORE_API FNetReplicatedObjectDebugInfo DebugInternalNetRefIndex(UE::Net::FInternalNetRefIndex InternalIndex, uint32 ReplicationSystemId);

/** Variant of DebugOutputNetObjectProtocolReferences that can be used from breakpoints or in watch window to find all handles references registered for a protocol and output to DebugOutput in debugger
 * NOTE: Use only for debugging as this variant uses a static buffer which is not thread safe	
*/
extern "C" IRISCORE_API const TCHAR* DebugNetObjectProtocolReferencesToString(FReplicationProtocolIdentifier ProtocolId, uint32 ReplicationSystemId);

/** Set the Object Name to break on */
extern "C" IRISCORE_API void SetIrisDebugObjectName(const ANSICHAR* NameBuffer);

/** Set the NetHandle to break on */
extern "C" IRISCORE_API void SetIrisDebugNetRefHandle(uint64 NetHandleId);

/** Set the InternalIndex to break on */
extern "C" IRISCORE_API void SetIrisDebugInternalNetRefIndex(UE::Net::FInternalNetRefIndex InternalIndex);

/** Set the InternalIndex to break on via it's NetHandle */
extern "C" IRISCORE_API void SetIrisDebugInternalNetRefIndexViaNetHandle(FNetRefHandle RefHandle);

/** Set the InternalIndex to break on via an object pointer*/
extern "C" IRISCORE_API void SetIrisDebugInternalNetRefIndexViaObject(UObject* Instance);

/** Set the RPC Name to break on */
extern "C" IRISCORE_API void SetIrisDebugRPCName(const ANSICHAR* NameBuffer);

}; // end of UE::Net::IrisDebugHelper

#endif

