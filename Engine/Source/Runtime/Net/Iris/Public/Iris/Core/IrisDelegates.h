// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"


class UReplicationSystem;

namespace UE::Net
{

DECLARE_MULTICAST_DELEGATE_OneParam(FIrisCriticalErrorDetected, UReplicationSystem*);

class FIrisDelegates
{
public:
	IRISCORE_API static FIrisCriticalErrorDetected& GetCriticalErrorDetectedDelegate();
};

	
	

} // end namespace UE::Net

