// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Set.h"

class FName;

namespace UE::Net
{

/**
 * Utility class to determine whether some message coupled to a hash hasn't been logged yet. 
 * Example usage would be for example if you only want to log something once per class. 
 */
class FIrisLogOnceTracker
{
public:
	/** Returns true if the Hash has not been encountered in a call to ShouldLog before. */
	IRISCORE_API bool ShouldLog(uint32 Hash) const;

	/** Returns true if the hash of the Name has not been encountered in a call to ShouldLog before. */
	IRISCORE_API bool ShouldLog(const FName& Name) const;

private:
	mutable TSet<uint32> LoggedHashes;
};

}
