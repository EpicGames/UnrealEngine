// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"

LLM_DECLARE_TAG_API(Iris, IRISCORE_API);
LLM_DECLARE_TAG_API(IrisState, IRISCORE_API);
LLM_DECLARE_TAG_API(IrisInitialization, IRISCORE_API);
LLM_DECLARE_TAG_API(IrisConnection, IRISCORE_API);
LLM_DECLARE_TAG_API(NetToken, IRISCORE_API);
LLM_DECLARE_TAG_API(NetTokenStructState, IRISCORE_API);