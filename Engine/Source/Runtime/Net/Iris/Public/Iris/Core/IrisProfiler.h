// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ProfilingDebugging/CsvProfiler.h"

#ifndef IRIS_PROFILER_ENABLE
#	if (UE_BUILD_SHIPPING)
#		define IRIS_PROFILER_ENABLE 0
#	else
#		define IRIS_PROFILER_ENABLE 1
#	endif
#endif

// When true this adds dynamic protocol names in profile captures. The downside is a noticeable cpu cost overhead but only while cpu trace recording is occurring.
#ifndef UE_IRIS_PROFILER_ENABLE_PROTOCOL_NAMES
#	define UE_IRIS_PROFILER_ENABLE_PROTOCOL_NAMES !UE_BUILD_SHIPPING
#endif

// When true this adds low-level cpu trace captures of operations in Iris. Adds a little cpu overhead but only while cpu trace recording is occurring.
#ifndef UE_IRIS_PROFILER_ENABLE_VERBOSE
#	if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#		define UE_IRIS_PROFILER_ENABLE_VERBOSE 0
#	else
#		define UE_IRIS_PROFILER_ENABLE_VERBOSE 1
#	endif
#endif


//#define IRIS_USE_SUPERLUMINAL

#if IRIS_PROFILER_ENABLE
#	ifdef IRIS_USE_SUPERLUMINAL
#		include "c:/Program Files/Superluminal/Performance/API/include/Superluminal/PerformanceAPI.h"
#		include "HAL/PreprocessorHelpers.h"
#		pragma comment (lib, "c:/Program Files/Superluminal/Performance/API/lib/x64/PerformanceAPI_MD.lib")
#		define IRIS_PROFILER_SCOPE(X) PERFORMANCEAPI_INSTRUMENT(UE_STRINGIZE(X))
#		define IRIS_PROFILER_SCOPE_CONDITIONAL(X,Cond) PERFORMANCEAPI_INSTRUMENT(UE_STRINGIZE(X))
#		define IRIS_PROFILER_SCOPE_STR(X) PERFORMANCEAPI_INSTRUMENT_DATA(UE_JOIN(IrisProfilerScope, __LINE__), X)
#		define IRIS_PROFILER_SCOPE_STR_CONDITIONAL(X, Cond) PERFORMANCEAPI_INSTRUMENT_DATA(UE_JOIN(IrisProfilerScope, __LINE__), X)
#		define IRIS_PROFILER_SCOPE_TEXT(X) PERFORMANCEAPI_INSTRUMENT_DATA(UE_JOIN(IrisProfilerScope, __LINE__), X)
#		define IRIS_PROFILER_SCOPE_TEXT_CONDITIONAL(X, Cond) PERFORMANCEAPI_INSTRUMENT_DATA(UE_JOIN(IrisProfilerScope, __LINE__), X)
#	else
#		include "ProfilingDebugging/CpuProfilerTrace.h"
#		define IRIS_PROFILER_SCOPE(X) TRACE_CPUPROFILER_EVENT_SCOPE(X)
#		define IRIS_PROFILER_SCOPE_CONDITIONAL(X,Cond) TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(X, Cond)
#		define IRIS_PROFILER_SCOPE_STR(X) TRACE_CPUPROFILER_EVENT_SCOPE_STR(X)
#		define IRIS_PROFILER_SCOPE_STR_CONDITIONAL(X, Cond) TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL(X, Cond)
#		define IRIS_PROFILER_SCOPE_TEXT(X) TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(X)
#		define IRIS_PROFILER_SCOPE_TEXT_CONDITIONAL(X, Cond) TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(X, Cond)
#	endif
#else
#	define PERFORMANCEAPI_ENABLED 0
#	define IRIS_PROFILER_SCOPE(X)
#	define IRIS_PROFILER_SCOPE_CONDITIONAL(X, Cond)
#	define IRIS_PROFILER_SCOPE_STR(X)
#	define IRIS_PROFILER_SCOPE_STR_CONDITIONAL(X, Cond)
#	define IRIS_PROFILER_SCOPE_TEXT(X)
#	define IRIS_PROFILER_SCOPE_TEXT_CONDITIONAL(X, Cond)
#endif

#if UE_IRIS_PROFILER_ENABLE_PROTOCOL_NAMES
#	define IRIS_PROFILER_PROTOCOL_NAME(X) IRIS_PROFILER_SCOPE_TEXT(X)
#	define IRIS_PROFILER_PROTOCOL_NAME_CONDITIONAL(X, Cond) IRIS_PROFILER_SCOPE_TEXT_CONDITIONAL(X, Cond)
#else
#	define IRIS_PROFILER_PROTOCOL_NAME(X)
#	define IRIS_PROFILER_PROTOCOL_NAME_CONDITIONAL(X, Cond)
#endif

#if UE_IRIS_PROFILER_ENABLE_VERBOSE
#	define IRIS_PROFILER_SCOPE_VERBOSE(X) IRIS_PROFILER_SCOPE(X);
#	define IRIS_PROFILER_SCOPE_VERBOSE_CONDITIONAL(X, Cond) IRIS_PROFILER_SCOPE_CONDITIONAL(X, Cond);
#else
#	define IRIS_PROFILER_SCOPE_VERBOSE(X)
#	define IRIS_PROFILER_SCOPE_VERBOSE_CONDITIONAL(X, Cond)
#endif

#ifndef IRIS_CLIENT_PROFILER_ENABLE
#	define IRIS_CLIENT_PROFILER_ENABLE (!WITH_SERVER_CODE && CSV_PROFILER_STATS)
#endif

namespace UE::Net
{

#if IRIS_CLIENT_PROFILER_ENABLE

class FClientProfiler
{
public:

	/** Record profiler events. */
	static void RecordObjectCreate(FName ObjectName, bool bIsSubObject);
	static void RecordRepNotify(FName RepNotifyName);
	static void RecordRPC(FName RPCName);

	static void RecordBlockedReplication(const TCHAR* BlockedObject, int32 NumBlockedAssets, float BlockedTime);

	/** Return true if capturing events. */
	static bool IsCapturing();
};

#endif

}