// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ChunkedArray.h"

namespace UE::Net
{

// Configure what happens when new chunks are allocated.
enum class EInitMemory : uint8
{
	Zero,		// Memory will be filled with zeros.
	Constructor	// Memory will be initialized by the elements constructor.
};

/**
 * A variation of TChunkArray that is optimized for use in the Iris networking system. The initial chunks
 * created during construction are placed in a contiguous block of memory to promote locality of reference.
 */
template<typename InElementType, uint32 ElementsPerChunk = 100, typename AllocatorType = FDefaultAllocator>
class TNetChunkedArray : public TChunkedArray<InElementType, sizeof(InElementType)* ElementsPerChunk, AllocatorType>
{
public:
	
	using Super = TChunkedArray<InElementType, sizeof(InElementType) * ElementsPerChunk, AllocatorType>;

	TNetChunkedArray(int32 InNumElements = 0, EInitMemory InitMemory = EInitMemory::Constructor)
	{
		this->NumElements = InNumElements;

		// Compute the number of chunks needed.
		const int32 NumChunks = (this->NumElements + this->NumElementsPerChunk - 1) / this->NumElementsPerChunk;

		// Initial blocks should come from a single block of memory.
		this->Chunks.Empty(NumChunks);
		typename Super::FChunk* StartChunks = new typename Super::FChunk[NumChunks];
		for (int32 ChunkIndex = 0; ChunkIndex < NumChunks; ChunkIndex++)
		{
			typename Super::FChunk* CurrentChunk = (StartChunks + ChunkIndex);
			if (InitMemory == EInitMemory::Zero)
			{
				FMemory::Memset<typename Super::FChunk>(*CurrentChunk, 0);
			}

			this->Chunks.Add(CurrentChunk);
		}

		NumPreAllocatedChunks = NumChunks;
	}

	TNetChunkedArray(const TNetChunkedArray& OtherChunkedArray)
	{
		CopyIncludingPreAllocatedChunks(OtherChunkedArray);
	}

	TNetChunkedArray& operator=(const TNetChunkedArray& OtherChunkedArray)
	{
		CopyIncludingPreAllocatedChunks(OtherChunkedArray);
		return *this;
	}

	TNetChunkedArray(TNetChunkedArray&& OtherChunkedArray)
	{
		MoveIncludingPreAllocatedChunks(OtherChunkedArray);
	}

	TNetChunkedArray& operator=(TNetChunkedArray&& OtherChunkedArray)
	{
		if (this != &OtherChunkedArray)
		{
			MoveIncludingPreAllocatedChunks(OtherChunkedArray);
		}
		return *this;
	}

	~TNetChunkedArray()
	{
		InvalidatePreAllocatedChunks();
	}

	int32 NumChunks() const
	{
		return this->Chunks.Num();
	}

	/** 
	 * Return the maximum number of elements the array can hold before having to add another chunk.
	 */
	int32 Capacity() const
	{
		return this->Chunks.Num() * this->NumElementsPerChunk;
	}

	void Empty(int32 Slack = 0)
	{
		checkf(false, TEXT("This function is not supported"));
	}

	void Reset(int32 NewSize = 0)
	{
		checkf(false, TEXT("This function is not supported"));
	}

	/** 
	 * Add elements to the array so that an index can be successfully addressed, leaving 
	 * any new element's memory unitialized or initialized by the element's constructor.
	 * 
	 * @param Index The index that must be addressable.
	 */
	void AddToIndexUninitialized(int32 Index)
	{
		if (Index >= this->NumElements)
		{
			int32 NewElementCount = (Index - this->NumElements) + 1;
			this->Add(NewElementCount);
		}
	}

	/**
	 * Add elements to the array so that an index can be successfully addressed, zeroing
	 * out the memory for each new element.
	 *
	 * @param Index The index that must be addressable.
	 */
	void AddToIndexZeroed(int32 Index)
	{
		const int32 OldChunkCount = this->Chunks.Num();
		AddToIndexUninitialized(Index);
		for (int32 ChunkIndex = OldChunkCount; ChunkIndex < this->Chunks.Num(); ChunkIndex++)
		{
			FMemory::Memset<typename Super::FChunk>(*this->Chunks.GetData()[ChunkIndex], 0);
		}
	}

protected:
	/* The number of preallocated chunks. */
	int32 NumPreAllocatedChunks = 0;

private:
	/**
	 * Invalidate the pre-allocated chunk pointers and free the associated block of memory.
	 *
	 * This function must be called before TChunkedArray frees memory (e.g. the constructor) because
	 * it assumes that each chunk is a seperate allocation and is unaware of the single block
	 * of memory used by the pre-allocated chunks.
	 */
	void InvalidatePreAllocatedChunks()
	{
		// Invalid pre-allocated chunks from the chunks array.
		typename Super::FChunk* FirstChunk = nullptr;
		for (int32 ChunkIndex = 0; ChunkIndex < NumPreAllocatedChunks; ChunkIndex++)
		{
			FirstChunk = (FirstChunk == nullptr) ? this->Chunks.GetData()[ChunkIndex] : FirstChunk;
			this->Chunks.GetData()[ChunkIndex] = nullptr;
		}

		// The first chunk points to the beginning of the block of memory used by the pre-allocated chunks.
		delete[] FirstChunk;

		NumPreAllocatedChunks = 0;
	}

	/**
	 * Copy the contents of another TNetChunkedArray into this instance, ensuring that any existing chunks
	 * (pre-allocated and otherwise) are deallocated.
	 */
	void CopyIncludingPreAllocatedChunks(const TNetChunkedArray& ChunkedArray)
	{
		// Free the memory for any existing pre-allocated chunks.
		InvalidatePreAllocatedChunks();

		this->NumElements = ChunkedArray.NumElements;
		this->NumPreAllocatedChunks = ChunkedArray.NumPreAllocatedChunks;

		// Compute the number of chunks to copy and prepare the chunked array.
		const int32 NumChunks = ChunkedArray.Chunks.Num();
		
		this->Chunks.Empty(NumChunks);

		// Copy the pre-allocated chunks.
		typename Super::FChunk* PreAllocatedChunks = new typename Super::FChunk[this->NumPreAllocatedChunks];
		for (int32 ChunkIndex = 0; ChunkIndex < this->NumPreAllocatedChunks; ChunkIndex++)
		{
			typename Super::FChunk* CurrentChunk = (PreAllocatedChunks + ChunkIndex);
			
			*CurrentChunk = *ChunkedArray.Chunks.GetData()[ChunkIndex];

			this->Chunks.Add(CurrentChunk);
		}

		// Copy any remaining chunks.
		for (int32 ChunkIndex = this->NumPreAllocatedChunks; ChunkIndex < NumChunks; ChunkIndex++)
		{
			const typename Super::FChunk* CurrentChunk = ChunkedArray.Chunks.GetData()[ChunkIndex];

			this->Chunks.Add(new typename Super::FChunk(*CurrentChunk));
		}
	}

	void MoveIncludingPreAllocatedChunks(TNetChunkedArray& ChunkedArray)
	{
		this->Chunks = (typename Super::ChunksType&&)ChunkedArray.Chunks;
		this->NumElements = ChunkedArray.NumElements;
		this->NumPreAllocatedChunks = ChunkedArray.NumPreAllocatedChunks;
		
		ChunkedArray.NumElements = 0;
		ChunkedArray.NumPreAllocatedChunks = 0;
	}
};

}
