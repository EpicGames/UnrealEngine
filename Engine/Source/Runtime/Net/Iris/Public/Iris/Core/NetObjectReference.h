// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Net/Core/NetToken/NetToken.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/IsPODType.h"

class UReplicationSystem;
class UObject;

namespace UE::Net::Private
{
	class FObjectReferenceCache;
}

namespace UE::Net
{

/** Traits for a NetObjectReference. */
enum class ENetObjectReferenceTraits : uint32
{
	None = 0U,
	/**
	 * Whether the reference can be exported or not.
	 * @note This flag is only guaranteed to be set correctly for references obtained from ObjectReferenceCache.
	 */
	CanBeExported = 1U,
};
ENUM_CLASS_FLAGS(ENetObjectReferenceTraits);

/**
 * A reference to a network addressable object.
 */
class FNetObjectReference
{
public:
	inline FNetObjectReference() {}

	/** Returns whether the reference points to a valid object. */
	inline bool IsValid() const { return RefHandle.IsValid() || PathToken.IsValid(); }
	/** Returns whether this reference points to the same network addressable objects as some other reference. */
	inline bool operator==(const FNetObjectReference& Other) const { return RefHandle == Other.GetRefHandle() && PathToken == Other.PathToken; }
	/** Returns whether this reference doesn't point to the same network addressable objects as some other reference. */
	inline bool operator!=(const FNetObjectReference& Other) const { return !(*this == Other); }
		
	/** Returns the NetRefHandle part of the object reference. */
	FNetRefHandle GetRefHandle() const { return RefHandle; }
	/** Returns the NetToken part of the object reference. */
	FNetToken GetPathToken() const { return PathToken; }
	/** Returns whether the object reference can be exported. */
	bool CanBeExported() const { return EnumHasAnyFlags(Traits, ENetObjectReferenceTraits::CanBeExported); }

	/** Returns a human readable string representing the object reference, for debugging purposes. */
	inline FString ToString() const
	{
		if (PathToken.IsValid())
		{
			if (RefHandle.IsValid())
			{
				FString Result(PathToken.ToString());
				Result.Appendf(TEXT(" : %s"), ToCStr(RefHandle.ToString()));
				return Result;			
			}
			else
			{
				return PathToken.ToString();
			}
		}
		else
		{
			return RefHandle.ToString();
		}
	}

private:
	friend class Private::FObjectReferenceCache;

	inline FNetObjectReference(FNetRefHandle InHandle, FNetToken InPathToken, ENetObjectReferenceTraits InTraits)
		: RefHandle(InHandle)
		, PathToken(InPathToken)
		, Traits(InTraits)
	{
	}

	inline explicit FNetObjectReference(FNetRefHandle Handle)
	: FNetObjectReference(Handle, FNetToken(), ENetObjectReferenceTraits::None)
	{
	}

	FNetRefHandle RefHandle;
	FNetToken PathToken;
	ENetObjectReferenceTraits Traits = ENetObjectReferenceTraits::None;
};

/** Representation of an object dependency, e.g. when used in UReplicationBridge::GetInitialDependencies(). */
struct FNetDependencyInfo
{
	explicit FNetDependencyInfo(const FNetObjectReference& InObjectRef)
	: ObjectRef(InObjectRef)
	{}

	FNetObjectReference ObjectRef;
};

/** Return type for various methods that resolves NetObjectReferences. */
enum class ENetObjectReferenceResolveResult : uint32
{
	/** There are no unresolved references. */
	None = 0U,
	/** There are references that were unresolvable at this time. */
	HasUnresolvedReferences = 1 << 0U,
	/**
	 * There are references that were unresolvable at this time and that are required before we can continue further processing, such as calling an RPC.
	 * This could be due to an asset not being loaded yet for example.
	 */
	HasUnresolvedMustBeMappedReferences = HasUnresolvedReferences << 1U,
};
ENUM_CLASS_FLAGS(ENetObjectReferenceResolveResult);

}

template <> struct TIsPODType<UE::Net::FNetObjectReference> { enum { Value = true }; };
