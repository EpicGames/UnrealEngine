// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "DataStream.generated.h"

class UDataStreamManager;

namespace UE::Net
{
	class FNetSerializationContext;
	enum class EPacketDeliveryStatus : uint8;

	namespace Private
	{
		class FNetExports;
	}


	enum class EDataStreamWriteMode : unsigned
	{
		// Allowed to write all data, this is the default WriteMode
		Full,

		// Only write data that should be sent after PostTickDispatch
		PostTickDispatch,

		// Only write debug data that should not count against bandwidth limits
		DebugData,
	};
}

/**
 * Base struct for data stream records which are returned with WriteData calls and provided to ProcessPacketDeliveryStatus calls.
 * It's up to each DataStream implementation to inherit, if needed, and store relevant information regarding what was written
 * in the packet so that when ProcessPacketDeliveryStatus is called the DataStream can act on it appropriately depending on
 * whether the packet was delivered or lost. The DataStream is responsible both for allocating and freeing its own records.
 */
struct FDataStreamRecord
{
};

/**
 * Enum used to control whether a DataStream is allowed to write data or not.
 * As the DataStreamManager needs to know this the behavior is controlled there.
 * @see UDataStreamManager::GetSendStatus, UDataStreamManager::SetSendStatus
 */
UENUM()
enum class EDataStreamSendStatus : uint8
{
	Pause = 0,
	Send,
};

/**
 * DataStream is an interface that facilitates implementing the replication of custom data, 
 * such as bulky data or data with special delivery guarantees.
 */
UCLASS(abstract, MinimalAPI, transient)
class UDataStream : public UObject
{
	GENERATED_BODY()

public:
	enum class EWriteResult
	{
		// If NoData is returned then ReadData will not be called on the receiving end.
		NoData,
		// Everything was sent or this stream don't want to send more this frame even if there's more bandwidth.
		Ok, 
		// We have more data to write and can continue to write more if we get another call to write
		HasMoreData,
	};

	enum class EUpdateType : uint8
	{
		// Update originating from ReplicationSystem::PreSendUpdate
		PreSendUpdate = 0,

		// Update originating from the end of the main Network tick
		PostTickFlush = 1,
	};

	enum class EDataStreamState : uint8
	{
		// Stream is invalid
		Invalid = 0,
		// We should send open/init to other side
		PendingCreate, 
		// We are waiting for confirmation that remote have accepted the stream
		WaitOnCreateConfirmation,
		// Stream is open and we will process incoming data.
		Open,
		// We are closing, but still considerd open until flushed
		PendingClose,
		// We have send a close request and is waiting for confirmation before invalidating stream
		WaitOnCloseConfirmation,

		Count
	};

	struct FInitParameters
	{
	public:
		FInitParameters() = default;
		FInitParameters(UDataStreamManager* InDataStreamManager, const FInitParameters& InParams)
		{
			*this = InParams;
			DataStreamManager = InDataStreamManager;
		}

		UE::Net::Private::FNetExports* NetExports = nullptr;
		FName Name;
		uint32 ReplicationSystemId = 0U;
		uint32 ConnectionId = 0U;
		uint32 PacketWindowSize = 0U;

	private:
		// We only want this to be accessible from UDataStream base
		UDataStreamManager* DataStreamManager = nullptr;
		friend UDataStream;
	};

	struct FBeginWriteParameters
	{
		UE::Net::EDataStreamWriteMode WriteMode = UE::Net::EDataStreamWriteMode::Full;

		// Default to sending 1 packet per write. If 0 = unlimited packets
		uint32 MaxPackets = 1U;
	};

	struct FUpdateParameters
	{
		EUpdateType UpdateType;
	};

public:
	IRISCORE_API virtual ~UDataStream();

	/** Called before any other calls are made. */
	IRISCORE_API virtual void Init(const FInitParameters& Params);

	/** Called when a created stream is destroyed. */
	IRISCORE_API virtual void Deinit();

	/** 
	 * Called to drive required updates during the ReplicationSystem update calls.
 	 */
	IRISCORE_API virtual void Update(const FUpdateParameters& Params);

	/**
	 * Called before any calls to potential WriteData, if it returns EWriteData::NoData no other calls will be made.
	 * The purpose of the method is to enable a DataStream to setup data that can persist over multiple calls to WriteData if bandwidth allows.
	*/
	IRISCORE_API virtual EWriteResult BeginWrite(const FBeginWriteParameters& Params);

	/**
	 * Serialize data to a bitstream and optionally store record of what was serialized to a custom FDataStreamRecord.
	 * The FDataStreamRecord allow streams to implement custom delivery guarantees as they see fit by using the stored
	 * information when ProcessPacketDeliveryStatus is called. For each WriteData call returning something other than NoData
	 * a corresponding call to ProcessPacketDeliveryStatus will be made.
	 * The UDataStream owns the FDataStreamRecord, but there will always be a call to ProcessPacketDeliveryStatus passing
	 * the original OutRecord so that it can be deleted when all packets have been ACKed/NAKed or when the owning connection is closed.
	 * @param Context The FNetSerializationContext which has accesssors for the bitstream to write to among other things.
	 * @param OutRecord Set the data stream specific record to OutRecord so that it can be passed in a future ProcessPacketDeliveryStatus call.
	 *        ProcessPacketDeliveryStatus will be called with the record in the same order as WriteData was called.
	 * @return Whether there was data written or not and if the stream has more data to write if bandwidth settings allow it.
	 */
	virtual EWriteResult WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord) PURE_VIRTUAL(WriteData, return EWriteResult::NoData;);

	/**
	 * Called after the final call to WriteData this frame, allowing the DataStream to cleanup data setup during BeginWrite.
	 */
	IRISCORE_API virtual void EndWrite();

	/**
	 * Deserialize data that was written with WriteData.
	 * @param Context The FNetSerializationContext which has accessors for the bitstream to read from among other things. 
	 */
	virtual void ReadData(UE::Net::FNetSerializationContext& Context) PURE_VIRTUAL(ReadData,);

	/**
	 * For each packet into which we have written data we are guaranteed to get a call to ProcessPacketDeliveryStatus when
	 * it's known whether the packet was delivered or not.
	 * @param Status Whether the packet was delivered or not or if the record should simply be discarded due to closing a connection.
	 * @param Record The record which was set by this stream during a WriteData call.
	 */
	virtual void ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record) PURE_VIRTUAL(ProcessPacketDeliveryStatus,);

	/**
	 * @return true if the stream has no pending reliable data for which it is waiting on an acknowledgement.
	 */
	virtual bool HasAcknowledgedAllReliableData() const PURE_VIRTUAL(HasAcknowledgedAllReliableData, return true;);

	/** 
	 * @return true if the stream indicates that is has debug data to write.
	 */
	IRISCORE_API virtual bool HasDebugDataToSend() const;

	/**
	 * Get name of DataStream
	 */
	FName GetDataStreamName() const
	{
		return DataStreamInitParameters.Name;
	}

	/**
 	 * Initiate close of DataStream. Note: This only applies to DataStreams that are flagged with bDynamicCreate in the DataStreamDefinition
	 * @param Name The name of the DataStream. Names of valid DataStream are configured in UDataStreamDefinitions.
	 * @see UDataStreamDefinitions
	*/
	IRISCORE_API void RequestClose();

	/** Get the current state of the DataStream. */
	IRISCORE_API const UDataStream::EDataStreamState GetState() const;

protected:
	/** Access init parameters. */
	const UDataStream::FInitParameters& GetInitParameters() const
	{
		return DataStreamInitParameters;
	}

private:
	FInitParameters DataStreamInitParameters;
};

const TCHAR* LexToString(const UDataStream::EDataStreamState State);
