// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "DataStream.h"
#include "Templates/PimplPtr.h"
#include "DataStreamManager.generated.h"

enum class EDataStreamSendStatus : uint8;
class UDataStream;

enum class ECreateDataStreamResult : uint8
{
	// DataStream was successfully created.
	Success,
	// A DataStream with that name is already created.
	Error_Duplicate,
	// There's no DataStreamDefinition for the requested DataStream.
	Error_MissingDefinition,
	// There's something wrong with the DataStreamDefinition for the requested DataStream.
	Error_InvalidDefinition,
	// There's a fixed limit on how many unique data streams can be created.
	Error_TooManyStreams,
};

namespace UE::Net::Private
{
	class FNetExports;
}

/**
 * The DataStreamManager contains all active DataStreams that may serialize data.
 * Calls to the DataStream interface functions will be forwarded to active streams.
 * Which streams will be automatically created or allowed to be manually created
 * need to be configured via UDataStreamDefinitions.
 */
UCLASS(transient, MinimalApi)
class UDataStreamManager : public UDataStream
{
	GENERATED_BODY()

public:
	IRISCORE_API virtual ~UDataStreamManager();

	// UDataStream interface

	/** Initializes the manager. No data stream can be created by the manager before this. */
	virtual IRISCORE_API void Init(const FInitParameters& InitParams) override;

	/** Prepare for destruction. Data streams will get ProcessPacketDeliveryStatus for outstanding packets and then marked as garbage. */
	IRISCORE_API void Deinit();

	IRISCORE_API virtual void Update(const FUpdateParameters& Params) override;

	/** Call BeginWrite on all active data streams. */
	IRISCORE_API virtual EWriteResult BeginWrite(const FBeginWriteParameters& Params) override;

	/** Call WriteData on all active data streams. */
	IRISCORE_API virtual EWriteResult WriteData(UE::Net::FNetSerializationContext& context, FDataStreamRecord const*& OutRecord) override;

	/** Call EndWrite on all active data streams. */
	IRISCORE_API virtual void EndWrite() override;

	/** When a packet is received call ReadData on all data streams that wrote something. */
	IRISCORE_API virtual void ReadData(UE::Net::FNetSerializationContext& context) override;

	/** Called for all data streams that wrote to a packet whose delivery status is now known. */
	IRISCORE_API virtual void ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record) override;

	/** Returns true if calling HasAcknowledgedAllReliableData on all data streams also returns true. */
	IRISCORE_API virtual bool HasAcknowledgedAllReliableData() const override;

	/** Returns true if HasDebugDataToSend returns true for any data stream. */
	IRISCORE_API virtual bool HasDebugDataToSend() const override;

	// DataStreamManager specifics

	/**
	 * Returns true of the stream of with the specified name is present in the UDataStreamDefintions
	*/
	IRISCORE_API static bool IsKnownStreamDefinition(const FName StreamName);

	/**
	 * Creates a DataStream that has been configured via UDataStreamDefinitions.
	 * @param StreamName The data stream name as configured in a FDataStreamDefinition.
	 *        Each stream needs a unique name, but there can be multiple streams of the same class
	 *        as long as each data stream name is unique.
	 * @return Success if the stream was successfully created or an error code if it was not.
	 * @see UDataStreamDefinitions
	 * @note Calls need to be synchronized between the sending and receiving side.
	 */
	IRISCORE_API ECreateDataStreamResult CreateStream(const FName StreamName);

	/**
	 * Gets the data stream with a given name.
	 * @param StreamName The name of the data stream as configured in UDataStreamDefinitions.
	 * @return A pointer to a const data stream if it exists, nullptr if not.
	 */
	IRISCORE_API const UDataStream* GetStream(const FName StreamName) const;

	/**
	* Gets the data stream with a given name.
	* @param StreamName The name of the data stream as configured in UDataStreamDefinitions.
	* @return A pointer to the data stream if it exists, nullptr if not.
	*/
	IRISCORE_API UDataStream* GetStream(const FName StreamName);

	/**
	* Request that the dynamic data stream with the given name will be closed after handshake with remote peer. 
	* NOTE: This will not do anything for streams not marked as bDynamicCreate
	* @param StreamName The name of the data stream as configured in UDataStreamDefinitions.
	*/
	IRISCORE_API void CloseStream(const FName StreamName);

	/** Get the current state of a DataStream Returns Invalid if the stream isn't created. */
	IRISCORE_API EDataStreamState GetStreamState(const FName StreamName) const;

	/** Set the send status of an already created data stream. */
	IRISCORE_API void SetSendStatus(const FName StreamName, EDataStreamSendStatus Status);

	/** Get the send status of an already created data stream. Returns Pause if the stream isn't created. */
	IRISCORE_API EDataStreamSendStatus GetSendStatus(const FName StreamName) const;

	UE::Net::Private::FNetExports& GetNetExports();

private:
	UDataStreamManager();

	static void AddReferencedObjects(UObject* Object, FReferenceCollector& Collector);

private:
	class FImpl;
	TPimplPtr<FImpl> Impl;
};
