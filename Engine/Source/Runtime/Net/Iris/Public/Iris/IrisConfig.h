// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "Net/Core/Connection/NetEnums.h"
#include "ProfilingDebugging/CsvProfilerConfig.h"

namespace UE::Net
{

/** Returns if the preferred replication system should be Iris. */
IRISCORE_API bool ShouldUseIrisReplication();

/** Set if the preferred replication system should be Iris or not. */
IRISCORE_API void SetUseIrisReplication(bool EnableIrisReplication);

/** Returns what replication sytem was set to be used by the cmdline. Returns Default when the command line was not set. */
IRISCORE_API EReplicationSystem GetUseIrisReplicationCmdlineValue();

/** Returns if we are scheduling creation dependencies before dependent objects */
UE_EXPERIMENTAL(5.7, "Experimental and subject for removal.")
IRISCORE_API bool IsSchedulingCreationDependenciesFirst();

}

/* NetBitStreamReader/Writer validation support */
#ifndef UE_NETBITSTREAMWRITER_VALIDATE
#define UE_NETBITSTREAMWRITER_VALIDATE !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

#ifndef UE_NETBITSTREAMREADER_VALIDATE
#define UE_NETBITSTREAMREADER_VALIDATE !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

/** CSV stats. */
#ifndef UE_NET_IRIS_CSV_STATS
#	define UE_NET_IRIS_CSV_STATS CSV_PROFILER_STATS
#endif

/** Enables code that detects non-thread safe access to network data */
#ifndef UE_NET_THREAD_SAFETY_CHECK
#	define UE_NET_THREAD_SAFETY_CHECK DO_CHECK
#endif

/** Enables code to simulate async loading stalls on specific objects on the client */
#ifndef UE_NET_ASYNCLOADING_DEBUG
#	define UE_NET_ASYNCLOADING_DEBUG !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

// Controls whether we compile in thread safety mechanisms into Iris. Will add a small amount of overhead
// due to mutex locking, but this is required when running Iris phases in parallel
#ifndef UE_SUPPORT_PARALLEL_IRIS
#define UE_SUPPORT_PARALLEL_IRIS 0
#endif

/** Enables thread safety logic inside UE_IRIS_PARALLEL_EXPR brackets - required when running Iris in parallel */
#if UE_SUPPORT_PARALLEL_IRIS
#define UE_IRIS_PARALLEL_EXPR(expr) expr
#else
#define UE_IRIS_PARALLEL_EXPR(expr) (void)0
#endif

/** Debug. */
#ifndef UE_NET_IRIS_ALLOW_DEBUG_DATA
#	define UE_NET_IRIS_ALLOW_DEBUG_DATA !(UE_BUILD_SHIPPING)
#endif
