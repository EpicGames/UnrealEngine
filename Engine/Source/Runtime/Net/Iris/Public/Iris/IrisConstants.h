// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Net
{

constexpr uint32 InvalidReplicationSystemId = 0xFFFFFFFFU;
constexpr uint32 InvalidConnectionId = 0;

}
