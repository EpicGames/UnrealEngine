// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Map.h"
#include "UObject/NameTypes.h"
#include "Misc/AssertionMacros.h"

#include <type_traits>


namespace UE::Net
{

/**
 * Class used to store a single analytics value.
 * Only supports integers or floating points for now.
 */
struct FNetMetric
{
public:
	
	enum class EDataType
	{
		None,
		Unsigned,
		Signed,
		Double,
	};

public:

	FNetMetric()
		: Double(0.0)
		, DataType(EDataType::None)
	{}

	template<typename T>
	FNetMetric(T InValue)
	{
		static_assert(std::is_integral_v<T> || std::is_floating_point_v<T>, "Only integers and floats are supported");

		if constexpr (std::is_integral_v<T> && std::is_signed_v<T>)
		{
			DataType = EDataType::Signed;
			Signed = InValue;
		}
		else if constexpr (std::is_integral_v<T> && std::is_unsigned_v<T>)
		{
			DataType = EDataType::Unsigned;
			Unsigned = InValue;
		}
		else if constexpr (std::is_floating_point_v<T>)
		{
			DataType = EDataType::Double;
			Double = InValue;
		}
	}

	template<typename T>
	void Set(T InValue)
	{
		static_assert(std::is_integral_v<T> || std::is_floating_point_v<T>, "Only integers and floats are supported");
		*this = FNetMetric(InValue);
	}

	EDataType GetDataType() const 
	{ 
		return DataType; 
	}

	int32 GetSigned() const
	{
		check(DataType == EDataType::Signed);
		return Signed;
	}

	uint32 GetUnsigned() const
	{
		check(DataType == EDataType::Unsigned);
		return Unsigned;
	}

	double GetDouble() const
	{
		check(DataType == EDataType::Double);
		return Double;
	}

private:

	union
	{
		uint32 Unsigned;
		int32 Signed;
		double Double;
	};

	EDataType DataType;
};

/**
* Collects network metrics and keeps track of their name.
*/
struct FNetMetrics
{
public:

	void EmplaceMetric(FName InName, FNetMetric&& InMetric)		{ Metrics.Emplace(InName, MoveTemp(InMetric)); }
	void AddMetric(FName InName, const FNetMetric& InMetric)	{ Metrics.Add(InName, InMetric); }

	const TMap<FName, FNetMetric>& GetMetrics() const { return Metrics; }

private:

	TMap<FName, FNetMetric> Metrics;
};

} // end namespace UE::Net