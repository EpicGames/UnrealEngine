// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Net
{

/** Enum used to report packet notifications to DataStreams. */
enum class EPacketDeliveryStatus : uint8
{
	/** The packet was delivered. */
	Delivered,
	/** The packet was lost or ignored by the recipient due to out of order delivery for example. */
	Lost,
	/** Free any resource related to this packet, such as a DataStreamRecord. Typically used when closing connections and similar scenarios. */
	Discard,
};

}
