// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Net/Serialization/FastArraySerializer.h"
#include "Iris/ReplicationState/ReplicationStateFwd.h"
#include "IrisFastArraySerializer.generated.h"

// Forward  declarations
namespace UE::Net::Private
{

struct FIrisFastArraySerializerPrivateAccessor;

}

/**
 * Specialization of FFastArraySerializer in order to add state tracking support for Iris
 * Current usage is to inherit from this struct instead of FFastArraySerializer, backwards compatible with existing system as it simply forwards calls to MarkDirty/MarkItemDirty
 * This class could be named FFastArrayReplicationState, but kept the FIrisFastArraySerializer to match old naming for the time being
 *
 * NOTE: IrisFastArraySerializer, does not support having local not replicated items in the array.
 * If ShouldWriteFastArrayItem is overridden and filters out items, an ensure will be triggered. 
 * Logic should still work, but extra elements will be replicated and filtered out on receiving end.
 */
USTRUCT(Experimental)
struct FIrisFastArraySerializer : public FFastArraySerializer
{
	// At the moment as we have no way to specify this per derived type, currently we reserve a fixed range of bits used for the changemask, the first bit is used for the array itself
	enum { IrisFastArrayChangeMaskBits = 63U };
	enum { IrisFastArrayChangeMaskBitOffset = 1U };
	enum { IrisFastArrayPropertyBitIndex = 0U };	

	GENERATED_BODY()
	IRISCORE_API FIrisFastArraySerializer();

	~FIrisFastArraySerializer() = default;

	/** Will not copy replication state header */
	IRISCORE_API FIrisFastArraySerializer(const FIrisFastArraySerializer& Other);

	/** We must make sure that we do not copy replication state header and must update dirtiness if bound */
	IRISCORE_API FIrisFastArraySerializer& operator=(const FIrisFastArraySerializer& Other);

	/** Will not copy replication state header */
	IRISCORE_API FIrisFastArraySerializer(const FIrisFastArraySerializer&& Other);

	/** We must make sure that we do not move replication state header and must update dirtiness if bound */
	IRISCORE_API FIrisFastArraySerializer& operator=(FIrisFastArraySerializer&& Other);
	
	/** Override MarkItemDirty in order to mark object as dirty in the DirtyNetObjectTracker */
	void MarkItemDirty(FFastArraySerializerItem & Item)
	{ 
		FFastArraySerializer::MarkItemDirty(Item);
		if (ReplicationStateHeader.IsBound())
		{
			// Mark array dirty for Iris so that is it copied
			InternalMarkArrayAsDirty();
		}
	}
	
	/** Override MarkArrayDirty in order to mark object as dirty in the DirtyNetObjectTracker */
	void MarkArrayDirty()
	{
		FFastArraySerializer::MarkArrayDirty();
		if (ReplicationStateHeader.IsBound())
		{
			// Mark array dirty for Iris
			InternalMarkArrayAsDirty();
		}	
	}

	friend UE::Net::Private::FIrisFastArraySerializerPrivateAccessor;

private:
	enum : unsigned
	{
		MemberChangeMaskStorageIndex = 0U,
		ConditionalChangeMaskStorageIndex = 2U,
	};

	// Mark item at index as changed, and set the array as dirty
	IRISCORE_API void InitChangeMask();
	IRISCORE_API void InternalMarkItemChanged(int32 ItemIdx);
	IRISCORE_API void InternalMarkAllItemsChanged();
	IRISCORE_API void InternalMarkArrayAsDirty();

	// Header for dirty state tracking needs to be just before ChangeMaskStorage. See GetReplicationStateHeader for more info.
	UE::Net::FReplicationStateHeader ReplicationStateHeader;

	// Storage for changemask, this is currently hardcoded
	UPROPERTY(Transient, NotReplicated)
	uint32 ChangeMaskStorage[4];
};

