// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Iris/ReplicationState/IrisFastArraySerializer.h"

namespace UE::Net 
{
	class FNetBitArray;
	class FNetBitArrayView;
}

namespace UE::Net	
{

namespace Private 
{

/*
 * Internal access of FastArraySerializer, for internal use only
*/
struct FIrisFastArraySerializerPrivateAccessor
{
	IRISCORE_API static FNetBitArrayView GetChangeMask(FIrisFastArraySerializer& Array);
	IRISCORE_API static FNetBitArrayView GetConditionalChangeMask(FIrisFastArraySerializer& Array);
	static FReplicationStateHeader& GetReplicationStateHeader(FIrisFastArraySerializer& Array) { return Array.ReplicationStateHeader; }

	/*
	 * Mark array as dirty and notify DirtyObjectTracker that the object is dirty if it has not been done before
	 */
	IRISCORE_API static void MarkArrayDirty(FIrisFastArraySerializer& Array);

	/*
	 * Mark array and all bits in changemask as dirty and notify DirtyObjectTracker that the object is dirty if it has not been done before
	 */
	IRISCORE_API static void MarkAllArrayItemsDirty(FIrisFastArraySerializer& Array, uint32 StartingIndex = 0U);

	/*
	 * Mark array dirty and mark changemask bit used for array Item as dirty and notify DirtyObjectTracker that the object is dirty if it has not been done before
     * Currently the same changemask bit might be used to indicate dirtiness for multiple array items
	 */
	IRISCORE_API static void MarkArrayItemDirty(FIrisFastArraySerializer& Array, int32 Index);
};

} // end namespace Private

/**
 * Experimental support for more explicit interface to edit FastArrays which can be used to avoid polling
 * The idea is that the interface would implement a subset of the Array interface and be used instead of directly modifying the array
 */
template <typename FastArrayType>
class TIrisFastArrayEditor
{
public:
	typedef typename FastArrayType::ItemArrayType FastArrayItemArrayType;
	typedef typename FastArrayItemArrayType::ElementType FastArrayItemType;

	using FIrisFastArraySerializerPrivateAccessor = Private::FIrisFastArraySerializerPrivateAccessor;

	TIrisFastArrayEditor(FastArrayType& InFastArray) : FastArray(InFastArray) {}

	/**
	 * Forwards MarkItemDirty call to FastArray and if the FastArray is bound it will also update DirtyState tracking
	 */
	void MarkItemDirty(FastArrayItemType& Item);
	
	/**
	 * Forwards MarkArrayDirty call to FastArray and if the FastArray is bound it will also update DirtyState tracking
	 */
	void MarkArrayDirty() { FastArray.MarkArrayDirty(); };

	/**
	 * Local add which will add item to array without dirtying it
	 */
	void AddLocal(const FastArrayItemType& ItemEntry) { FastArray.GetItemArray().Add(ItemEntry); }

	/**
	 * Local edit which will modify local item without dirtying it
	 */
	FastArrayItemType& EditLocal(int32 ItemIdx) { return FastArray.GetItemArray()[ItemIdx]; }

	/**
	 * Add Item to array and call MarkItemDirty
	 */
	void Add(const FastArrayItemType& ItemEntry);

	/**
	 * Edit Item in array and call MarkItemDirty
	 */
	FastArrayItemType& Edit(int32 ItemIdx);

	/**
	 * Mutable access to Item, will call MarkItemDirty
	 */
	FastArrayItemType& operator[](int32 ItemIdx) { return Edit(ItemIdx); }

	/**
	 * Remove item at the specified index, will forward call to MarkArrayDirty, if bound will mark all potentially moved items as dirty
	 */
	void Remove(int32 ItemIdx);

	/**
	 * Remove item at the specified index, will only mark the affected item dirty
	 */
	void RemoveAtSwap(int32 ItemIdx);

	int32 Num() const { return FastArray.GetItemArray().Num(); }

	/**
	 * Empty array and call MarkArrayDirty
	 */
	void Empty();

private:
	friend FastArrayType;
	FastArrayType& FastArray;
};

template <typename FastArrayType>
void TIrisFastArrayEditor<FastArrayType>::MarkItemDirty(FastArrayItemType& Item)
{
	const FReplicationStateHeader& Header = FIrisFastArraySerializerPrivateAccessor::GetReplicationStateHeader(FastArray);
	if (!Header.IsBound())
	{
		FastArray.MarkItemDirty(Item);
		return;
	}

	const FastArrayItemArrayType& Array = FastArray.GetItemArray();
	const uint64 Index64 = &Item - Array.GetData();
	if (ensure((IntFitsIn<int32, uint64>(Index64))))
	{
		const int32 Index = static_cast<int32>(Index64);
		if (Array.IsValidIndex(Index))
		{
			// If this is a new element make sure it is at the end, otherwise we must dirty array
			if ((Item.ReplicationID == INDEX_NONE) && (Index != (Array.Num() - 1)))
			{
				// Must mark all items that might have been shifted dirty, including the new one, this is to mimic behavior of current FastArraySerializer
				for (FastArrayItemType& ArrayItem : MakeArrayView(&Item, Array.Num() - Index))
				{
					const bool bIsWritingOnClient = false;
					if (FastArray.template ShouldWriteFastArrayItem<FastArrayItemType, FastArrayType>(ArrayItem, bIsWritingOnClient))
					{
						if (ArrayItem.ReplicationID == INDEX_NONE)
						{
							FastArray.MarkItemDirty(ArrayItem);
						}

						const uint64 ItemIndex64 = &ArrayItem - Array.GetData();
						if (ensure((IntFitsIn<int32, uint64>(ItemIndex64))))
						{
							// Mark item dirty in changemask
							FIrisFastArraySerializerPrivateAccessor::MarkArrayItemDirty(FastArray, static_cast<int32>(ItemIndex64));
						}
					}
				}
			}
			else
			{
				// This is in order to execute old logic
				FastArray.MarkItemDirty(Item);
				// Mark item dirty in changemask
				FIrisFastArraySerializerPrivateAccessor::MarkArrayItemDirty(FastArray, Index);
			}
		}
	}
};
	

// The idea is to provide an interface that is modeled after what we can do with a TArray but with dirty tracking per member
// This way we can update changemask directly and does not need to poll for changes
template <typename FastArrayType>
void TIrisFastArrayEditor<FastArrayType>::Add(const FastArrayItemType& ItemEntry)
{
	FastArrayItemArrayType& ItemArray = FastArray.GetItemArray();
	int32 Idx = ItemArray.Add(ItemEntry);
	MarkItemDirty(ItemArray.GetData()[Idx]);
}

template <typename FastArrayType>
typename TIrisFastArrayEditor<FastArrayType>::FastArrayItemType& TIrisFastArrayEditor<FastArrayType>::Edit(int32 ItemIdx)
{
	FastArrayItemType& Item = FastArray.GetItemArray()[ItemIdx];
	MarkItemDirty(Item);
	return Item;
}

template <typename FastArrayType>
void TIrisFastArrayEditor<FastArrayType>::Remove(int32 ItemIdx)
{
	FastArrayItemArrayType& ItemArray = FastArray.GetItemArray();

	if (ItemIdx >= 0 && ItemArray.Num() > ItemIdx)
	{
		// If the Item is at the end we only need to mark the array as dirty
		ItemArray.RemoveAt(ItemIdx);
		MarkArrayDirty();

		if (ItemIdx < ItemArray.Num())
		{
			const FReplicationStateHeader& Header = FIrisFastArraySerializerPrivateAccessor::GetReplicationStateHeader(FastArray);
			if (Header.IsBound())
			{
				FIrisFastArraySerializerPrivateAccessor::MarkAllArrayItemsDirty(FastArray, ItemIdx);
			}
		}
	}
}

template <typename FastArrayType>
void TIrisFastArrayEditor<FastArrayType>::RemoveAtSwap(int32 ItemIdx)
{
	FastArrayItemArrayType& ItemArray = FastArray.GetItemArray();

	if (ItemIdx >= 0 && ItemArray.Num() > ItemIdx)
	{
		ItemArray.RemoveAtSwap(ItemIdx);

		// We just need to dirty the modified item
		if (ItemArray.Num() != 0)
		{
			MarkItemDirty(ItemIdx);
		}
		else
		{
			MarkArrayDirty();
		}
	}
}

template <typename FastArrayType>
void TIrisFastArrayEditor<FastArrayType>::Empty()
{
	FastArray.GetItemArray().Empty();
	MarkArrayDirty();
}

} // end namespace UE::Net



