// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/PreprocessorHelpers.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

namespace UE::Net
{

class FFragmentRegistrationContext;
class FReplicationFragment;
struct FReplicationStateDescriptor;
enum class EReplicationStateTraits : uint32;
typedef FReplicationFragment* (*CreateAndRegisterReplicationFragmentFunc)(UObject*, const FReplicationStateDescriptor*, FFragmentRegistrationContext&);

}

namespace UE::Net
{

/**
 * This function should be called in the following way to take advantage of SFINAE and perform compile-time
 * error checking for declared types:
 *     static_assert(IsDeclaredType((struct MyStruct*)Ptr), "Error message..");
 *
 * If MyStruct is not declared, the cast will pass a void* pointer to IsDeclaredType(). Otherwise, if MyStruct
 * is declared then the cast will pass a MyStruct* pointer to IsDeclaredType().
 */
template<typename StructName>
constexpr auto IsDeclaredType(StructName*) -> decltype(sizeof(StructName))
{
	return true;
}

constexpr auto IsDeclaredType(void*)
{
	return false;
}

/**
 * Currently we require each supported type to register FPropertyNetSerializerInfo 
 * It provides information on what NetSerializer to use for which property and how to build the required NetSerializer config which is used when we build the dynamic descriptor
 * It is possible to register multiple FPropertyNetSerializerInfo for the same PropertyType-class as long as the IsSupportedFunction only matches a single Property, i.e. bool/nativebool enums of different sizes
 */
struct FPropertyNetSerializerInfo
{
	virtual const FFieldClass* GetPropertyTypeClass() const { return nullptr; }
	virtual bool IsSupported(const FProperty* Property) const { return true; }
	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const { return nullptr; }
	virtual bool CanUseDefaultConfig(const FProperty* Property) const { return true; }
	virtual const FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const { return nullptr; }
	/** Custom replication fragments are currently only supported by structs with a custom NetSerializer. See UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_WITH_CUSTOM_FRAGMENT_INFO. */
	virtual CreateAndRegisterReplicationFragmentFunc GetCreateAndRegisterReplicationFragmentFunction() const { return nullptr; }

	bool IsSupportedStruct(FName InStructName) const
	{
		return StructName == InStructName;
	}

protected:
	// Used by named struct serializers.
	FName StructName;
};

/**
 *  This is a simple static registry used by the ReplicationStateDescriptorBuilder when building descriptors using properties
 */
class FPropertyNetSerializerInfoRegistry
{
public:
	/** Register FPropertyNetSerializerInfo in the registry */
	IRISCORE_API static void Register(const FPropertyNetSerializerInfo* Info);
	IRISCORE_API static void Unregister(const FPropertyNetSerializerInfo* Info);

	/** Reset the registry */
	IRISCORE_API static void Reset();

	/** Sort entries in registry on property type */
	IRISCORE_API static void Freeze();

	/** Find the FPropertyNetSerializerInfo for the provided property */
	IRISCORE_API static const FPropertyNetSerializerInfo* FindSerializerInfo(const FProperty* Property);

	/** Find StructSerializerInfo by name for non property based serialization */
	IRISCORE_API static const FPropertyNetSerializerInfo* FindStructSerializerInfo(const FName Name);

	/**
	 * Get NopNetSerializerInfo. For when you want to use the NopNetSerializer. An example
	 * could be that you want to give a system access to some meta data that should not be
	 * replicated.
	 * 
	 * @see FNopNetSerializerConfig
	 */
	IRISCORE_API static const FPropertyNetSerializerInfo* GetNopNetSerializerInfo();

private:
	typedef TTuple<const FFieldClass*, const FPropertyNetSerializerInfo*> FRegistryEntry;
	typedef TArray<FRegistryEntry> FNetSerializerInfoRegistry;

	static bool bRegistryIsDirty;
	static FNetSerializerInfoRegistry Registry; 
};

/** 
 * Some helpers to register default infos for properties
*/

// Helper to implement simple default PropertyNetSerializerInfo for primitive types
template <typename T, typename ConfigType = FNetSerializerConfig>
struct TSimplePropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	const FNetSerializer& Serializer;

	TSimplePropertyNetSerializerInfo(const FNetSerializer& InSerializer) : Serializer(InSerializer) {}
	virtual const FFieldClass* GetPropertyTypeClass() const override { return T::StaticClass(); }
	virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override { return &Serializer; }
	virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override { return new (NetSerializerConfigBuffer) ConfigType; }
};

// Helper to implement simple default PropertyNetSerializerInfo for structs
struct FNamedStructPropertyNetSerializerInfo : public TSimplePropertyNetSerializerInfo<FStructProperty>
{
	typedef TSimplePropertyNetSerializerInfo<FStructProperty> Super;

	FNamedStructPropertyNetSerializerInfo(const FName InPropertyFName, const FNetSerializer& InSerializer)
	: Super(InSerializer)
	{
		StructName = InPropertyFName;
	}

	virtual bool IsSupported(const FProperty* Property) const override
	{	
		const FStructProperty* StructProp = CastFieldChecked<const FStructProperty>(Property);
		return IsSupportedStruct(StructProp->Struct->GetFName());
	};

	virtual CreateAndRegisterReplicationFragmentFunc GetCreateAndRegisterReplicationFragmentFunction() const
	{
		return CreateAndRegisterReplicationFragmentFunction;
	}

	void SetCreateAndRegisterReplicationFragmentFunction(CreateAndRegisterReplicationFragmentFunc InCreateAndRegisterReplicationFragmentFunction)
	{
		CreateAndRegisterReplicationFragmentFunction = InCreateAndRegisterReplicationFragmentFunction;
	}

private:
	CreateAndRegisterReplicationFragmentFunc CreateAndRegisterReplicationFragmentFunction = nullptr;
};

/** These are options for the LastResortPropertyNetSerializer that are user controllable by passing an instance to a registration macro, for example UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(MyStruct, {.MaxQuantizedSizeBits = 32767}) */
struct FLastResortPropertyNetSerializerUserOptions
{
	/** If the struct needs a custom fragment due to for example not being able to dequantize to a temporary state. */
	CreateAndRegisterReplicationFragmentFunc CreateAndRegisterReplicationFragmentFunction = nullptr;
	/** How large the quantized state may be in bits. Only needed to be set if the requirements are very high. Leave as 0 to use whatever the default is (65535).  */
	uint32 MaxQuantizedSizeBits = 0;
	/** Whether to exclude the serialization from being part of a replication state's default state hash. This should be done only if the property may differ on the sending and receiving side. */
	bool bExcludeFromDefaultStateHash = false;
};

struct FLastResortPropertyNetSerializerInfo : public FPropertyNetSerializerInfo
{
	FLastResortPropertyNetSerializerInfo() = default;
	IRISCORE_API explicit FLastResortPropertyNetSerializerInfo(const FLastResortPropertyNetSerializerUserOptions& UserOptions);

	IRISCORE_API virtual const FFieldClass* GetPropertyTypeClass() const override;
	IRISCORE_API virtual bool IsSupported(const FProperty* Property) const override;
	IRISCORE_API virtual const FNetSerializer* GetNetSerializer(const FProperty* Property) const override final;
	IRISCORE_API virtual bool CanUseDefaultConfig(const FProperty* Property) const override final;
	IRISCORE_API virtual FNetSerializerConfig* BuildNetSerializerConfig(void* NetSerializerConfigBuffer, const FProperty* Property) const override final;

	virtual CreateAndRegisterReplicationFragmentFunc GetCreateAndRegisterReplicationFragmentFunction() const override final
	{
		return CreateAndRegisterReplicationFragmentFunction;
	}

protected:
	CreateAndRegisterReplicationFragmentFunc CreateAndRegisterReplicationFragmentFunction = nullptr;

	// Maximum size of quantized state in bits. 0 means use the default value (65535)
	uint32 MaxQuantizedSizeBits = 0;

	// If set this class should not be included in the default state hash.
	bool bExcludeFromDefaultStateHash = false;
};

struct FNamedStructLastResortPropertyNetSerializerInfo final : public FLastResortPropertyNetSerializerInfo
{
	explicit FNamedStructLastResortPropertyNetSerializerInfo(const FName InPropertyFName)
	: FLastResortPropertyNetSerializerInfo()
	{
		StructName = InPropertyFName;
	}

	FNamedStructLastResortPropertyNetSerializerInfo(const FName InPropertyFName, const FLastResortPropertyNetSerializerUserOptions& UserOptions)
	: FLastResortPropertyNetSerializerInfo(UserOptions)
	{
		StructName = InPropertyFName;
	}

	IRISCORE_API virtual const FFieldClass* GetPropertyTypeClass() const override;
	IRISCORE_API virtual bool IsSupported(const FProperty* Property) const override;
};

// Issue fatal error if matching trait found in UsedReplicationStateTraits is not set for the Serializer.
void IRISCORE_API ValidateForwardingNetSerializerTraits(const FNetSerializer* Serializer, EReplicationStateTraits UsedReplicationStateTraits);

}

// Produce a compiler error if the name does not correspond to a declared UE type.
#define UE_NET_IS_DECLARED_TYPE(Name) \
	static_assert( \
	UE::Net::IsDeclaredType((struct F##Name*)nullptr) || \
	UE::Net::IsDeclaredType((struct U##Name*)nullptr) \
	, "The UE type name '" #Name "' cannot be found. Make sure you have removed the 'F' or 'U' prefix from the type name.");

// Only needed if we want to export PropertyNetSerializerInfo, this goes in the header if we need to export it
#define UE_NET_DECLARE_NETSERIALIZER_INFO(NetSerializerInfo) \
const UE::Net::FPropertyNetSerializerInfo& GetPropertyNetSerializerInfo_##NetSerializerInfo();

// Implement FPropertyNetSerializerInfo from struct, this goes in the cpp file
#define UE_NET_IMPLEMENT_NETSERIALIZER_INFO(NetSerializerInfo) \
const UE::Net::FPropertyNetSerializerInfo& GetPropertyNetSerializerInfo_##NetSerializerInfo() { static NetSerializerInfo StaticInstance; return StaticInstance; };

// Implement simple FPropertyNetSerializerInfo binding a SerializerType to the property
#define UE_NET_IMPLEMENT_SIMPLE_NETSERIALIZER_INFO(PropertyType, SerializerName) \
const UE::Net::FPropertyNetSerializerInfo& GetPropertyNetSerializerInfo_##PropertyType() { static UE::Net::TSimplePropertyNetSerializerInfo<PropertyType> StaticInstance(UE_NET_GET_SERIALIZER(SerializerName)); return StaticInstance; };

// Implement simple FPropertyNetSerializerInfo for struct types with custom serializers
#define UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(Name, SerializerName) \
const UE::Net::FPropertyNetSerializerInfo& GetPropertyNetSerializerInfo_##Name() { static UE::Net::FNamedStructPropertyNetSerializerInfo StaticInstance(Name, UE_NET_GET_SERIALIZER(SerializerName)); return StaticInstance; };

/**
 * Force a struct to use a custom serializer and custom fragment. Use of custom fragments is highly discouraged as serialization order is modified, ending up after normal class properties.
 * It probably requires additional memory allocations to create the fragment as well. Only use a last resort when all other options have been exhausted.
 * It should never be required to use custom fragments except for very rare backward compatibility purposes. Ask the experts first to try to find a better solution.
 */
#define UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_WITH_CUSTOM_FRAGMENT_INFO(Name, SerializerName, CreateAndRegisterReplicationFragmentFunction) \
const UE::Net::FPropertyNetSerializerInfo& GetPropertyNetSerializerInfo_##Name() \
{ \
	static UE::Net::FNamedStructPropertyNetSerializerInfo StaticInstance(Name, UE_NET_GET_SERIALIZER(SerializerName)); \
	StaticInstance.SetCreateAndRegisterReplicationFragmentFunction(CreateAndRegisterReplicationFragmentFunction); \
	return StaticInstance; \
};


// Register 
#define UE_NET_REGISTER_NETSERIALIZER_INFO(Name) \
UE::Net::FPropertyNetSerializerInfoRegistry::Register(&GetPropertyNetSerializerInfo_##Name());

// Unregister
#if !IS_MONOLITHIC
#define UE_NET_UNREGISTER_NETSERIALIZER_INFO(Name) \
UE::Net::FPropertyNetSerializerInfoRegistry::Unregister(&GetPropertyNetSerializerInfo_##Name());
#else
#define UE_NET_UNREGISTER_NETSERIALIZER_INFO(...) 
#endif

// Implement minimal required delegates for a NetSerializer
#define UE_NET_IMPLEMENT_NETSERIALIZER_REGISTRY_DELEGATES(Name) \
struct F##Name##NetSerializerRegistryDelegates : protected UE::Net::FNetSerializerRegistryDelegates \
{ \
	~F##Name##NetSerializerRegistryDelegates() { UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_##Name); } \
	virtual void OnPreFreezeNetSerializerRegistry() override { UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_##Name); } \
}; \
static F##Name##NetSerializerRegistryDelegates Name##NetSerializerRegistryDelegates;

// Utility that can be used to forward serialization of a Struct to a specific NetSerializer
#define UE_NET_IMPLEMENT_FORWARDING_NETSERIALIZER_AND_REGISTRY_DELEGATES(Name, SerializerName) \
	static const FName PropertyNetSerializerRegistry_NAME_##Name( UE_STRINGIZE(Name) ); \
	UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_##Name, SerializerName); \
	UE_NET_IMPLEMENT_NETSERIALIZER_REGISTRY_DELEGATES(Name)

// Implement FPropertyNetSerializerInfo for cases where LastResortPropertyNetSerializer is needed for struct with custom serialization. You need to pass an FName as StructName. Optionally pass FLastResortPropertyNetSerializerUserOptions.
#define UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_INFO(StructName, ... /* FLastResortPropertyNetSerializerUserOptions instance */) \
const UE::Net::FPropertyNetSerializerInfo& GetPropertyNetSerializerInfo_##StructName() \
{ \
	static UE::Net::FNamedStructLastResortPropertyNetSerializerInfo StaticInstance(StructName __VA_OPT__(,) __VA_ARGS__); \
	return StaticInstance; \
}

// Implement FPropertyNetSerializerInfo for cases where LastResortPropertyNetSerializer is needed for struct with custom serialization, with quantized state size override.
#define UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_INFO_WITH_SIZE_OVERRIDE(StructName, MaxBits) \
	UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_INFO(StructName, {.MaxQuantizedSizeBits = MaxBits})

/**
 * Utility that can be used to forward serialization of a Struct to a last resort net serializer. Optionally pass FLastResortPropertyNetSerializerUserOptions.
 * Example: UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(MyStruct, {.MaxQuantizedSizeBits = 32767, .CreateAndRegisterReplicationFragmentFunction = CreateAndRegisterMyStructReplicationFragment})
 */
#define UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(Name, ... /* FLastResortPropertyNetSerializerUserOptions instance */) \
	UE_NET_IS_DECLARED_TYPE(Name); \
	static const FName PropertyNetSerializerRegistry_NAME_##Name( UE_STRINGIZE(Name) ); \
	UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_##Name __VA_OPT__(,) __VA_ARGS__) \
	UE_NET_IMPLEMENT_NETSERIALIZER_REGISTRY_DELEGATES(Name)

// Utility that can be used to forward serialization of a Struct to a last resort net serializer with a maximum quantized size
#define UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES_WITH_SIZE_OVERRIDE(Name, MaxBits) \
	UE_NET_IS_DECLARED_TYPE(Name); \
	static const FName PropertyNetSerializerRegistry_NAME_##Name( UE_STRINGIZE(Name) ); \
	UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_INFO_WITH_SIZE_OVERRIDE(PropertyNetSerializerRegistry_NAME_##Name, MaxBits); \
	UE_NET_IMPLEMENT_NETSERIALIZER_REGISTRY_DELEGATES(Name)
