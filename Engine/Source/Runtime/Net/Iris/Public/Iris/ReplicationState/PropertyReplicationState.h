// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/RefCounting.h"

namespace UE::Net
{
	struct FReplicationStateDescriptor;
}

namespace UE::Net
{

 /** 
 * ReplicationState created at runtime using a descriptor built from existing reflection data
 * StateBuffer contains storage for all properties described by the descriptor,
 * When polling data from the source object properties we will compare the value with what we have stored in the statebuffer
 * and mark the member as dirty.if the value differs
 *
 * Polling is quite expensive but we only do it once per update of the replication system
 */
class FPropertyReplicationState
{
public:
	/** Construct a new state, StateBuffer will be allocated and constructed according to data in the descriptor */
	IRISCORE_API explicit FPropertyReplicationState(const FReplicationStateDescriptor* Descriptor);

	/** Construct a new state, using already constructed state in InStateBuffer */
	IRISCORE_API FPropertyReplicationState(const FReplicationStateDescriptor* Descriptor, uint8* InStateBuffer);

	IRISCORE_API ~FPropertyReplicationState();

	/** Copy constructor, will not copy internal data */
	IRISCORE_API FPropertyReplicationState(const FPropertyReplicationState& Other);

	/** 
	 * Assignment operator, it is debatable if this should be provided or not.
	 * if we are copying to a bound state we will perform a Set which will do a full property compare to properly update dirtiness
	 * if we are copying to a loose state we will just to a copy of all properties contained in the state, changemask will also be copied
	*/
	IRISCORE_API FPropertyReplicationState& operator=(const FPropertyReplicationState& Other);

	/** Set from Other this will update dirtiness by comparing all properties */
	IRISCORE_API void Set(const FPropertyReplicationState& Other);
	
	/** Either StateBuffer is initialized and owned by this instance, or we have been injected with a state from the network system
		A default constructed state is not valid
	*/
	IRISCORE_API bool IsValid() const;

	//
	// Note: There is little to none error checking in the methods below, they are mostly intended to be used from internal code and tests
	// Expected usage for this type of ReplicationState is through the polling system
	//

	/** Set the value at the provided Index, 
		The UProperty is looked up from the descriptor using the index, if the value differs the statemask is updated
		Mostly intended for test code
		Normal use of this class is through the poll layer
		TODO: It would be nice if we could provide some validation on property types
	*/
	IRISCORE_API void SetPropertyValue(uint32 Index, const void* SrcValue);

	/**
	 * Retrieve the value at the provided Index by writing it to DstValue. 
	 * The property is looked up from the descriptor using the index.
	 * Mostly intended for test code. Normal use of this class is through the poll layer.
	 */
	IRISCORE_API void GetPropertyValue(uint32 Index, void* DstValue) const;

	/** Is the property at the given index dirty, for properties with multiple bits in the statemask this will return true if any of those bits are set */
	IRISCORE_API bool IsDirty(uint32 Index) const;

	/** Explicitly mark the property at the given Index as dirty, for properties with multiple bits in the statemask this will mark all bits dirty */
	IRISCORE_API void MarkDirty(uint32 Index);

	/** $IRIS TODO: Move Poll/Push/CallRepNotifies out of PropertyReplicationState as loose functions */

	/**
	 * Poll src data from properties in SrcData, where SrcData is a UClass/UStruct containing properties.
	 * Compare and update representation in DstStateBuffer and update ChangeMask. This is currently used to track and update dirtiness for the properties.
	 * @return True if any state is dirty, false if not.
	 * @todo This requires some thought as the property offset we get in all are expressed from the base, but we do generate descriptors for all parts of the inheritance chain.
	 * Do we carry around the base pointer or should we recalculate the offsets when we build the descriptor
	*/
	IRISCORE_API bool PollPropertyReplicationState(const void* RESTRICT SrcData);

	struct FPollParameters
	{
		bool bEnableVerboseProfiling = false;
		bool bForceRefreshObjectReferences = false;
	};

	/**
	 * Partial poll src data from properties in SrcData based on the state of the inlined per member pollmask
	 * non-pushbased properties are always polled, 
	 * Compare and update representation in DstStateBuffer and update ChangeMask. This is currently used to track and update dirtiness for the properties.
	 * @SrcData is a UClass/UStruct containing properties.
	 * @return True if any state is dirty, false if not.
	 * @todo This requires some thought as the property offset we get in all are expressed from the base, but we do generate descriptors for all parts of the inheritance chain.
	 * Do we carry around the base pointer or should we recalculate the offsets when we build the descriptor
	*/
	IRISCORE_API bool PollPushBasedPropertyReplicationState(const void* RESTRICT SrcData, const FPollParameters& PollParameters);

	/** Certain rep notifies requires the current state to be stored before we overwrite it, this method will copy property values from src data, , where SrcData is a UClass/UStruct containing properties
	 * if the property is marked as dirty in NewStateToBeApplied
	 */
	IRISCORE_API bool StoreCurrentPropertyReplicationStateForRepNotifies(const void* RESTRICT SrcData, const FPropertyReplicationState* NewStateToBeApplied);

	/** Push received state data to properties in DstData buffer. Note: DstData is a UClass/UStruct containing properties. Updates representation in DstStateBuffer and mark destination properties as dirty if updated. */
	IRISCORE_API void PushPropertyReplicationState(const UObject* Owner, void* RESTRICT DstData, bool bPushAll = false) const;


	/**
	 * Poll data from object referencing properties in SrcData, Note: SrcData is a UObject containing properties.
	 * Behaves like PollPropertyReplicationState when it comes to dirtiness and such.
	 * @return True if any state is dirty, false if not.
	 */
	IRISCORE_API bool PollObjectReferences(const void* RESTRICT SrcData);

	/**
	 * Copy dirty properties from the other state including changemask
	 */
	IRISCORE_API void CopyDirtyProperties(const FPropertyReplicationState& Other);

	struct FCallRepNotifiesParameters
	{
		// Previous state if requested
		const FPropertyReplicationState* PreviousState = nullptr;

		// This is an init state
		bool bIsInit = false;

		// Only call repnotify if value differs from local value
		bool bOnlyCallIfDiffersFromLocal = false;
	};

	/** Invoke repnotifies for all dirty members */
	IRISCORE_API void CallRepNotifies(void* RESTRICT DstData, const FCallRepNotifiesParameters& Params) const;
	inline void CallRepNotifies(void* RESTRICT DstData, const FPropertyReplicationState* PreviousState, bool bIsInit) const;

	/** Debug output state to FString */
	IRISCORE_API FString ToString(bool bIncludeAll = true) const;

	/** Debug output state to StringBuilder */
	IRISCORE_API const TCHAR* ToString(FStringBuilderBase& StringBuilder, bool bIncludeAll = true) const;

	const FReplicationStateDescriptor* GetReplicationStateDescriptor() const { return ReplicationStateDescriptor; }
	uint8* GetStateBuffer() const { return StateBuffer; }

protected:
	/** Explicitly mark an array at the given Index as dirty. It will not mark element changemask bits as dirty. */
	IRISCORE_API void MarkArrayDirty(uint32 Index);

public:
	// These methods are for internal use only.

	enum : unsigned
	{
		// How many bits to use to track dirtiness for individual elements.
		TArrayElementChangeMaskBits = 63U,
		// The changemask bit into the TArray changemask info that indicates the TArray is dirty.
		TArrayPropertyChangeMaskBitIndex = 0U,
		// At which offset in the TArray changemask element dirtiness begins.
		TArrayElementChangeMaskBitOffset = 1U,
	};

	/**
	 * If state has custom conditionals then this function will check whether the condition is enabled or not.
	 * If state doesn't have custom conditionals it will return true.
	 */
	IRISCORE_API bool IsCustomConditionEnabled(uint32 Index) const;

	/**
	 * Returns true if state is marked dirty for polling
	 */
	IRISCORE_API bool IsDirtyForPolling() const;

private:
	// Returns true if member should be polled, returns true if memberpollmask is dirty or if changemask is dirty.
	bool IsDirtyForPolling(uint32 Index) const;

	// Returns true if changemask or header of the state is marked dirty
	bool IsDirty() const;
	bool IsInitState() const;
	void ConstructStateInternal();
	void DestructStateInternal();
	void InjectState(const FReplicationStateDescriptor* Descriptor, uint8* InStateBuffer);
	void PollPropertyValue(uint32 Index, const void* SrcValue);
	void PushPropertyValue(uint32 Index, void* DstValue) const;

	TRefCountPtr<const FReplicationStateDescriptor> ReplicationStateDescriptor;
	uint8* StateBuffer;
	uint32 bOwnState : 1;
};

void FPropertyReplicationState::CallRepNotifies(void* RESTRICT DstData, const FPropertyReplicationState* PreviousState, bool bIsInit) const
{
	FCallRepNotifiesParameters Params; 

	Params.PreviousState = PreviousState;
	Params.bIsInit = bIsInit;
	CallRepNotifies(DstData, Params);
}

}
