// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/TypeHash.h"
#include <atomic>

struct FNetSerializerConfig;
class FProperty;
class FString;
namespace UE::Net
{
	struct FNetDebugName;
	struct FNetSerializer;
	typedef uint64 FRepTag;
	struct FReplicationStateDescriptor;
	class FFragmentRegistrationContext;

	class FReplicationFragment;
	typedef FReplicationFragment* (*CreateAndRegisterReplicationFragmentFunc)(UObject* Owner, const FReplicationStateDescriptor* Descriptor, FFragmentRegistrationContext& Context);
}

namespace UE::Net
{

// MemberDescriptors - Offset to where to find member data in both external and internal representation
struct FReplicationStateMemberDescriptor
{
	uint32 ExternalMemberOffset;		// this is our offset in the external state	
	uint32 InternalMemberOffset;		// this is the internal offset
};

// MemberSerializer - What serializer and config to use for the given member
struct FReplicationStateMemberSerializerDescriptor
{
	const FNetSerializer* Serializer;
	const FNetSerializerConfig* SerializerConfig;
};

enum class EReplicationStateMemberTraits : uint16
{
	None = 0U,

	HasDynamicState = 1U << 0U,
	HasObjectReference = HasDynamicState << 1U,
	HasConnectionSpecificSerialization = HasObjectReference << 1U,
	HasRepNotifyAlways = HasConnectionSpecificSerialization << 1U,
	UseSerializerIsEqual = HasRepNotifyAlways << 1U,
	HasPushBasedDirtiness = UseSerializerIsEqual << 1U,
};
ENUM_CLASS_FLAGS(EReplicationStateMemberTraits);

struct FReplicationStateMemberTraitsDescriptor
{
	EReplicationStateMemberTraits Traits;
};

struct FReplicationStateMemberTagDescriptor
{
	FRepTag Tag;
	// Which member in the replication state
	uint16 MemberIndex;
	// When we propagate tags from a struct in a replication state we need to know how to get additional tag information. ~0 means invalid.
	uint16 InnerTagIndex;
};

struct FReplicationStateMemberFunctionDescriptor
{
	const UFunction* Function;
	const FReplicationStateDescriptor* Descriptor;
};

// MemberChangeMaskDescriptor - Data needed for tracking of dirty members
struct FReplicationStateMemberChangeMaskDescriptor
{
	uint16 BitOffset;					// Offset in the change mask where we store the bits for the member
	uint16 BitCount;					// BitCount, In most cases we have one dirty bit per member but we will support assigning more bits for specific member in order to support dirty member tracking for arrays etc.
};

struct FNetReferenceInfo
{
	enum EResolveType : uint8
	{
		Invalid = 0U,					// Invalid, used when encoding references contained in dynamic memory, we identify them by setting the ResolveType to Invalid
		ResolveOnClient,				// This reference should be resolved on the client - Default behavior
		MustExistOnClient,				// This reference must be acknowledged by client before server replicates object with reference
		ResolveOnlyWhenRecvd,			// This reference will only be resolved if it exists when we receive the data, if it is unresolvable client will set it to nullptr and not try to resolve it again until it gets replicated the next time
	};

	FNetReferenceInfo() : ResolveType(Invalid), Padding(0) {}
	explicit FNetReferenceInfo(FNetReferenceInfo::EResolveType InResolveType) : ResolveType(InResolveType), Padding(0) {}

	EResolveType ResolveType;
	uint8 Padding;
};

// We store information about all members that store references to other objects in order to allow us to quickly iterate over all references
struct FReplicationStateMemberReferenceDescriptor
{
	uint32 Offset;				// If data is in an dynamic array this contains offset to ArrayMember, if ref is in the StateBuffer the offset is to the reference itself

	FNetReferenceInfo Info;		// Actual info about the reference

	uint16 MemberIndex;			// Member index for this reference
	uint16 InnerReferenceIndex;	// if set to something else than ~0 this means that this is a nested reference
};

// Per member debug descriptor, not strictly needed as long as we have the property list
struct FReplicationStateMemberDebugDescriptor
{
	const FNetDebugName* DebugName;
};

struct FReplicationStateIdentifier
{
	uint64 Value; // Currently this is a CityHash of the name of the state
	uint64 DefaultStateHash; // Currently this is a CityHash of the serialized default state

	// We currently do not include the DefaultStateHash when comparing FReplicationStateIdentifier as we might make it optional
	bool operator==(const FReplicationStateIdentifier& Other)const { return Value == Other.Value; }
	bool operator<(const FReplicationStateIdentifier& Other)const { return Value < Other.Value; }
	bool operator!=(const FReplicationStateIdentifier& Other)const { return Value != Other.Value; }
};

inline uint32 GetTypeHash(const FReplicationStateIdentifier& Identifier)
{
	return ::GetTypeHash(Identifier.Value);
}

struct FReplicationStateMemberLifetimeConditionDescriptor
{
	// This is a more compact storage form of ELifetimeCondition
	int8 Condition;
};

/**
 *  To be able to enable/disable custom conditional properties we need a fast way to go from RepIndex to MemberIndex.
 */
struct FReplicationStateMemberRepIndexToMemberIndexDescriptor
{
	enum : uint16
	{
		InvalidEntry = 65535U,
	};

	// MemberIndex whose property RepIndex matches the index of this entry. ~0 if this entry is invalid.
	uint16 MemberIndex;
};

// Additional data required to call RepNotifies etc
struct FReplicationStateMemberPropertyDescriptor
{
	const UFunction* RepNotifyFunction;
	uint16 ArrayIndex;
};

//
// Behavior / Feature Traits 
//
enum class EReplicationStateTraits : uint32
{
	None								= 0U,
	InitOnly							= 1U,
	// LifetimeConditionals is backward compatibility with EReplicationCondition.
	HasLifetimeConditionals				= InitOnly << 1U,
	HasObjectReference					= HasLifetimeConditionals << 1U,
	NeedsRefCount						= HasObjectReference << 1U,
	HasRepNotifies						= NeedsRefCount << 1U,
	KeepPreviousState					= HasRepNotifies << 1U,
	HasDynamicState						= KeepPreviousState << 1U,
	IsSourceTriviallyConstructible		= HasDynamicState << 1U,
	IsSourceTriviallyDestructible		= IsSourceTriviallyConstructible << 1U,
	AllMembersAreReplicated				= IsSourceTriviallyDestructible << 1U,
	IsFastArrayReplicationState			= AllMembersAreReplicated << 1U,
	IsNativeFastArrayReplicationState	= IsFastArrayReplicationState << 1U,
	HasConnectionSpecificSerialization	= IsNativeFastArrayReplicationState << 1U,
	HasPushBasedDirtiness				= HasConnectionSpecificSerialization << 1U,
	HasFullPushBasedDirtiness			= HasPushBasedDirtiness << 1U,
	// Whether delta compression is supported or not
	SupportsDeltaCompression			= HasFullPushBasedDirtiness << 1U,
	UseSerializerIsEqual				= SupportsDeltaCompression << 1U,
	// Whether this is a descriptor for a struct derived from something with a NetSerializer, but doesn't have a custom NetSerializer itself.
	IsDerivedFromStructWithCustomSerializer	= UseSerializerIsEqual << 1U,
	IsStructWithCustomSerializer = IsDerivedFromStructWithCustomSerializer << 1U,
};
ENUM_CLASS_FLAGS(EReplicationStateTraits);

// Required functions to construct and destruct external state in the provided buffer
typedef void (*ConstructReplicationStateFunc)(uint8* Buffer, const FReplicationStateDescriptor* Descriptor);
typedef void (*DestructReplicationStateFunc)(uint8* Buffer, const FReplicationStateDescriptor* Descriptor);

// A ReplicationState is our replication primitive, all members of a ReplicationState has the same high level conditional,
// i.e. connection level, IsInit, and has the same owner
// filtering on the block level might be Initial, Connection, Owner 
// Within a ReplicationState we do we also support per member conditionals
struct FReplicationStateDescriptor
{
	// RefCounting required for runtime created descriptors
	IRISCORE_API void AddRef() const;
	IRISCORE_API void Release() const;

	bool IsInitState() const { return EnumHasAnyFlags(Traits, EReplicationStateTraits::InitOnly); }
	bool HasObjectReference() const { return EnumHasAnyFlags(Traits, EReplicationStateTraits::HasObjectReference); }

	uint32 GetChangeMaskOffset() const { return ChangeMasksExternalOffset; }
	uint32 GetConditionalChangeMaskOffset() const { return ChangeMasksExternalOffset + 4U*((ChangeMaskBitCount + 31)/32); }
	uint32 GetMemberPollMaskOffset() const { return GetConditionalChangeMaskOffset() + (EnumHasAnyFlags(Traits, EReplicationStateTraits::HasLifetimeConditionals) ? (4U*((ChangeMaskBitCount + 31)/32)) : 0U); }

	const FReplicationStateMemberDescriptor* MemberDescriptors;
	const FReplicationStateMemberChangeMaskDescriptor* MemberChangeMaskDescriptors;
	const FReplicationStateMemberSerializerDescriptor* MemberSerializerDescriptors;
	const FReplicationStateMemberTraitsDescriptor* MemberTraitsDescriptors;
	const FReplicationStateMemberFunctionDescriptor* MemberFunctionDescriptors;
	const FReplicationStateMemberTagDescriptor* MemberTagDescriptors;

	const FReplicationStateMemberReferenceDescriptor* MemberReferenceDescriptors;

	// This should possibly be moved to its own external descriptor as we do not want to rely on UProperties if we can.
	// Currently we need this since we do not know anything about the external types.
	const FProperty** MemberProperties;

	// Additional data associated with applying state data for properties
	// We keep this in a separate array as we might only need this if we can receive data
	const FReplicationStateMemberPropertyDescriptor* MemberPropertyDescriptors;

	// Non-null if trait HasLifetimeConditionals is set.
	const FReplicationStateMemberLifetimeConditionDescriptor* MemberLifetimeConditionDescriptors;

	// Non-null if RepIndexCount > 0
	const FReplicationStateMemberRepIndexToMemberIndexDescriptor* MemberRepIndexToMemberIndexDescriptors;

	// Non-null for derived struct descriptors.
	const UScriptStruct* BaseStruct;

	// Optional debug info
	const FNetDebugName* DebugName;
	const FReplicationStateMemberDebugDescriptor* MemberDebugDescriptors;

	uint32 ExternalSize;		// Size of the external representation including alignment
	uint32 InternalSize;		// Size of the internal representation including alignment
	uint16 ExternalAlignment;
	uint16 InternalAlignment;

	uint16 MemberCount;
	uint16 FunctionCount;
		
	uint16 TagCount;
	uint16 ObjectReferenceCount;

	// How many RepIndex to MemberIndex entries there are.
	uint16 RepIndexCount;

	// How many bits do we need for our tracking of dirty changes
	uint16 ChangeMaskBitCount;

	// This is the offset to where we store data for ChangeMask, ConditionalChangeMask and MemberPollMask in the external state.
	// Retrieve offset with helper methods, GetChangeMaskOffset(), GetConditionalChangeMaskOffset(), GetMemberPollMaskOffset()
	uint32 ChangeMasksExternalOffset;

	// We need to assign a unique key that is stable between server and client (name hash of class + state type for now)
	FReplicationStateIdentifier DescriptorIdentifier;

	// Function to construct external state representation in a preallocated buffer
	ConstructReplicationStateFunc ConstructReplicationState;

	// Function to destruct external state representation
	DestructReplicationStateFunc DestructReplicationState;

	// Function used to construct custom replication fragments
	CreateAndRegisterReplicationFragmentFunc CreateAndRegisterReplicationFragmentFunction;

	EReplicationStateTraits Traits;

	mutable std::atomic<int32> RefCount;

	// Pointer to default state buffer, must be explicitly destroyed if set since it might contain dynamic data
	const uint8* DefaultStateBuffer;
};

IRISCORE_API void DescribeReplicationDescriptor(FString& OutString, const FReplicationStateDescriptor* Descriptor);

}

