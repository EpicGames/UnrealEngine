// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Templates/RefCounting.h"
#include "UObject/CoreNet.h"

class IConsoleVariable;
class UReplicationSystem;
namespace UE::Net::Private
{
	class FReplicationStateDescriptorRegistry;
}

namespace UE::Net
{

class FReplicationStateDescriptorBuilder
{
public:
	struct FParameters
	{
		IRISCORE_API FParameters();

		Private::FReplicationStateDescriptorRegistry* DescriptorRegistry;
		UReplicationSystem* ReplicationSystem = nullptr;

		// The template to build the state source from
		const UObject* DefaultStateSource = nullptr;

		 // Include super class when building descriptor.
		bool IncludeSuper : 1 = true;
		// Get the lifetime properties for conditionals if applicable. This applies only to classes.
		bool GetLifeTimeProperties : 1 = true;
		// If EnableFastArrayHandling is true and the struct inherits from FFastArraySerializer then special logic will be applied which allows it to be bound to a FastArrayReplicationFragment.
		bool EnableFastArrayHandling : 1 = true;
		// If AllowFastArrayWithExtraReplicatedProperties is set to true, we will allow building descriptors for fastarrays with more than a single property.
		bool AllowFastArrayWithExtraReplicatedProperties : 1 = false;
		// In SkipCheckForCustomNetSerializer is true the descriptor will be built using the underlying representation even if a CustomNetSerializer is registered for the struct
		bool SkipCheckForCustomNetSerializerForStruct : 1 = false;
		 // If SinglePropertyIndex != -1 we will create a state that only includes the specified property.
		int32 SinglePropertyIndex = -1;
	};

	typedef TArray<TRefCountPtr<const FReplicationStateDescriptor>> FResult;

	/**
		Create the descriptors from an optional template or the class CDO.

		Done by processing reflection information at runtime for existing replicated properties.

		Since we want to allow ReplicationStates to be shared as much as possible between different connection we do split them up based on
		feature requirements, filtering, conditionals etc. We also keep Init states separate.

		If a descriptor registry is provided, the functions below will search the registry before creating a new ReplicationStateDescriptor.
		Any created ReplicationStateDescriptor will be registered in the provided registry.

		Returns number of states created for the provided class.
	*/
	IRISCORE_API static SIZE_T CreateDescriptorsForClass(FResult& OutCreatedDescriptors, UClass* InClass, const FParameters& Parameters = FParameters());

	/**
		Create ReplicationStateDescriptor for a struct

		If a descriptor registry is provided, the functions below will search the registry before creating a new ReplicationStateDescriptor.
		Any new created ReplicationStateDescriptor will be registered in the provided registry.
	*/
	IRISCORE_API static TRefCountPtr<const FReplicationStateDescriptor> CreateDescriptorForStruct(const UStruct* InStruct, const FParameters& Parameters = FParameters());

	/**
	 * Create ReplicationStateDescriptor for a function
	 *
	 * If a descriptor registry is provided, the function below will search the registry before creating a new ReplicationStateDescriptor.
	 * Any new created ReplicationStateDescriptor will be registered in the provided registry.
	 */
	IRISCORE_API static TRefCountPtr<const FReplicationStateDescriptor> CreateDescriptorForFunction(const UFunction* Function, const FParameters& Parameters = FParameters());
};

}
