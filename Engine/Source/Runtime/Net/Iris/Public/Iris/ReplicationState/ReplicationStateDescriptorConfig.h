// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "UObject/ObjectMacros.h"
#include "ReplicationStateDescriptorConfig.generated.h"

USTRUCT()
struct FReplicationStateDescriptorClassPushModelConfig
{
	GENERATED_BODY()
	
	/** Short class name, e.g. PlayerController. */
	UPROPERTY()
	FName ClassName;
};

USTRUCT()
struct FSupportsStructNetSerializerConfig
{
	GENERATED_BODY()

	/** Struct name. */
	UPROPERTY()
	FName StructName;

	/** If the named struct works with the default Iris StructNetSerializer. */
	UPROPERTY()
	bool bCanUseStructNetSerializer = true;
};

UCLASS(transient, config=Engine)
class UReplicationStateDescriptorConfig : public UObject
{
	GENERATED_BODY()

public:
	IRISCORE_API TConstArrayView<FSupportsStructNetSerializerConfig> GetSupportsStructNetSerializerList() const;
	
	IRISCORE_API TConstArrayView<FReplicationStateDescriptorClassPushModelConfig> GetEnsureFullyPushModelClassNames() const;

	inline bool EnsureAllClassesAreFullyPushModel() const;

protected:
	UReplicationStateDescriptorConfig();

private:
	/**
	 * Structs that works using the default struct NetSerializer when running iris replication even though they implement a custom NetSerialize or NetDeltaSerialize method.
	 */
	UPROPERTY(Config)
	TArray<FSupportsStructNetSerializerConfig> SupportsStructNetSerializerList;

	/** Which classes should ensure they are fully push model. */
	UPROPERTY(Config)
	TArray<FReplicationStateDescriptorClassPushModelConfig> EnsureFullyPushModelClassNames;

	/** If you want to be alerted of all classes not being fully push model. */
	UPROPERTY(Config)
	bool bEnsureAllClassesAreFullyPushModel = false;
};

bool UReplicationStateDescriptorConfig::EnsureAllClassesAreFullyPushModel() const
{
	return bEnsureAllClassesAreFullyPushModel;
}
