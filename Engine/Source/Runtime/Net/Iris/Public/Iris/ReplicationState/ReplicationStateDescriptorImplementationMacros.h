// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/Serialization/NetSerializers.h"

//
// $IRIS: TEMPORARY TO HELP WITH FAKE GENERATING REPLICATIONSTATES
// define macros to declare and implement the fake Generated State
// The idea is to generate this automatically for native iris replication
//

namespace UE::Net::Private
{

struct FInternalTypeInfo
{
	uint32 Size;
	uint32 Alignment;
};

constexpr uint32 GetInternalMemberOffset(const FReplicationStateMemberDescriptor* MemberDescriptors, const FInternalTypeInfo* InternalTypeInfo, uint32 MemberIndex)
{
	return MemberIndex == 0 ? 0u : Align(MemberDescriptors[MemberIndex - 1].InternalMemberOffset + InternalTypeInfo[MemberIndex - 1].Size, InternalTypeInfo[MemberIndex].Alignment);
}

template <typename T>
constexpr uint32 GetInternalStateSize(const T& MemberDescriptors, const FInternalTypeInfo* InternalTypeInfo)
{
	const uint32 Count = UE_ARRAY_COUNT(MemberDescriptors);

	if (Count == 0)
	{
		return 0;
	}

	return MemberDescriptors[Count - 1].InternalMemberOffset + InternalTypeInfo[Count - 1].Size;
}

template <typename T>
constexpr uint16 GetInternalStateAlignment(const T& MemberDescriptors, const FInternalTypeInfo* InternalTypeInfo)
{
	const uint32 Count = UE_ARRAY_COUNT(MemberDescriptors);

	uint16 Alignment = 1;

	for (uint32 It = 0; It < Count; ++It)
	{
		Alignment = FPlatformMath::Max<uint16>(Alignment, static_cast<uint16>(InternalTypeInfo[It].Alignment));
	}

	return Alignment;
}

template <typename T>
constexpr uint32 GetMemberChangeMaskSize(const T& MemberChangeMaskDescriptors)
{
	constexpr uint32 Count = UE_ARRAY_COUNT(MemberChangeMaskDescriptors);
	return Count > 0u ? MemberChangeMaskDescriptors[Count - 1].BitOffset + MemberChangeMaskDescriptors[Count - 1].BitCount : 0u;
}

}

//
// IMPLEMENTATION MACROS
//

// Used to declare an temporary array with serializer, config size and alignment for the internal types used by the specified serializes
// This is used by our fake generated states to make the declarations more readable.
#define IRIS_BEGIN_INTERNAL_TYPE_INFO(StateName) static const UE::Net::Private::FInternalTypeInfo StateName ## TypeInfoData[] = {
#define IRIS_INTERNAL_TYPE_INFO(SerializerName) { UE_NET_GET_SERIALIZER_INTERNAL_TYPE_SIZE(SerializerName), UE_NET_GET_SERIALIZER_INTERNAL_TYPE_ALIGNMENT(SerializerName) },
#define IRIS_END_INTERNAL_TYPE_INFO() };

// Used to declare the array of member serializer descriptors
#define IRIS_BEGIN_SERIALIZER_DESCRIPTOR(StateName) const UE::Net::FReplicationStateMemberSerializerDescriptor StateName::sReplicationStateDescriptorMemberSerializerDescriptors[] = {
#define IRIS_SERIALIZER_DESCRIPTOR(SerializerName, ConfigPointer) { &UE_NET_GET_SERIALIZER(SerializerName), ConfigPointer ? ConfigPointer : UE_NET_GET_SERIALIZER_DEFAULT_CONFIG(SerializerName)},
#define IRIS_END_SERIALIZER_DESCRIPTOR() };

// Used to declare the array of member traits descriptors
#define IRIS_BEGIN_TRAITS_DESCRIPTOR(StateName) const UE::Net::FReplicationStateMemberTraitsDescriptor StateName::sReplicationStateDescriptorMemberTraitsDescriptors[] = {
#define IRIS_TRAITS_DESCRIPTOR(Traits) { Traits },
#define IRIS_END_TRAITS_DESCRIPTOR() };

// Used to declare the optional array of member tag descriptors. Since tags are optional we need to create a fake one to prevent a zero-sized array.
#define IRIS_BEGIN_TAG_DESCRIPTOR(StateName) const UE::Net::FReplicationStateMemberTagDescriptor StateName::sReplicationStateDescriptorMemberTagDescriptors[] = {
#define IRIS_TAG_DESCRIPTOR(Tag, MemberIndex) { Tag, MemberIndex, uint16(~0) },
#define IRIS_END_TAG_DESCRIPTOR() { UE::Net::FRepTag(0) /* UE::Net::GetInvalidRepTag() */, uint16(~0), uint16(~0) } };

// Used to declare the optional array of member function descriptors. Since functions are optional we need to create a fake one to prevent a zero-sized array.
#define IRIS_BEGIN_FUNCTION_DESCRIPTOR(StateName) const UE::Net::FReplicationStateMemberFunctionDescriptor StateName::sReplicationStateDescriptorMemberFunctionDescriptors[] = {
#define IRIS_FUNCTION_DESCRIPTOR(Function, Descriptor) { Function, Descriptor },
#define IRIS_END_FUNCTION_DESCRIPTOR() { {} } };

// Used to declare the optional array of member reference descriptors. Since references are optional we need to create a fake one to prevent a zero-sized array.
#define IRIS_BEGIN_REFERENCE_DESCRIPTOR(StateName) const UE::Net::FReplicationStateMemberReferenceDescriptor StateName::sReplicationStateDescriptorMemberReferenceDescriptors[] = {
#define IRIS_END_REFERENCE_DESCRIPTOR() { {} } };

// Declare an entry in the ReplicationStateMemberDescriptor array, requires the temporary TypeInfoData array to be declared
#define IRIS_BEGIN_MEMBER_DESCRIPTOR(StateName) const UE::Net::FReplicationStateMemberDescriptor StateName::sReplicationStateDescriptorMemberDescriptors[] = {
#define IRIS_MEMBER_DESCRIPTOR(StateName, MemberName, MemberIndex) { offsetof(StateName, MemberName), UE::Net::Private::GetInternalMemberOffset(sReplicationStateDescriptorMemberDescriptors, StateName ## TypeInfoData, MemberIndex) },
#define IRIS_END_MEMBER_DESCRIPTOR() };

// Used to declare the mandatory array of member debug descriptors
#define IRIS_BEGIN_MEMBER_DEBUG_DESCRIPTOR(StateName) const UE::Net::FReplicationStateMemberDebugDescriptor StateName::sReplicationStateDescriptorMemberDebugDescriptors[] = {
#define IRIS_MEMBER_DEBUG_DESCRIPTOR(StateName, MemberDebugName) { UE::Net::CreatePersistentNetDebugName(TEXT(#MemberDebugName)), },
#define IRIS_END_MEMBER_DEBUG_DESCRIPTOR() };

// Implement the required construct and destruct functions
#define IRIS_IMPLEMENT_CONSTRUCT_AND_DESTRUCT(StateName) \
void Construct##StateName(uint8* StateBuffer, const UE::Net::FReplicationStateDescriptor* Descriptor) { new (StateBuffer) StateName(); } \
void Destruct##StateName(uint8* StateBuffer, const UE::Net::FReplicationStateDescriptor* Descriptor) { StateName* State = reinterpret_cast<StateName*>(StateBuffer); State->~StateName(); }

// Implement the ReplicationStateDescriptor for the faked state

#define IRIS_IMPLEMENT_REPLICATIONSTATEDESCRIPTOR_WITH_TRAITS(StateName, Traits) \
const UE::Net::FReplicationStateDescriptor StateName::sReplicationStateDescriptor = \
{ \
	&sReplicationStateDescriptorMemberDescriptors[0], \
	&sReplicationStateChangeMaskDescriptors[0], \
	&sReplicationStateDescriptorMemberSerializerDescriptors[0], \
	&sReplicationStateDescriptorMemberTraitsDescriptors[0], \
	(UE_ARRAY_COUNT(sReplicationStateDescriptorMemberFunctionDescriptors) > 1 ? &sReplicationStateDescriptorMemberFunctionDescriptors[0] : static_cast<const UE::Net::FReplicationStateMemberFunctionDescriptor*>(nullptr)), \
	(UE_ARRAY_COUNT(sReplicationStateDescriptorMemberTagDescriptors) > 1 ? &sReplicationStateDescriptorMemberTagDescriptors[0] : static_cast<const UE::Net::FReplicationStateMemberTagDescriptor*>(nullptr)), \
	(UE_ARRAY_COUNT(sReplicationStateDescriptorMemberReferenceDescriptors) > 1 ? &sReplicationStateDescriptorMemberReferenceDescriptors[0] : static_cast<const UE::Net::FReplicationStateMemberReferenceDescriptor*>(nullptr)), \
	static_cast<const FProperty**>(nullptr), /* MemberProperties */ \
	static_cast<const UE::Net::FReplicationStateMemberPropertyDescriptor*>(nullptr), \
	static_cast<const UE::Net::FReplicationStateMemberLifetimeConditionDescriptor*>(nullptr), \
	static_cast<const UE::Net::FReplicationStateMemberRepIndexToMemberIndexDescriptor*>(nullptr), \
	static_cast<const UScriptStruct*>(nullptr), \
	UE::Net::CreatePersistentNetDebugName(TEXT(#StateName)), \
	&sReplicationStateDescriptorMemberDebugDescriptors[0],\
	sizeof(StateName), \
	UE::Net::Private::GetInternalStateSize(sReplicationStateDescriptorMemberDescriptors, StateName ## TypeInfoData), \
	alignof(StateName),	\
	UE::Net::Private::GetInternalStateAlignment(sReplicationStateDescriptorMemberDescriptors, StateName ## TypeInfoData), \
	static_cast<uint16>(UE_ARRAY_COUNT(sReplicationStateDescriptorMemberDescriptors)), /* MemberCount */ \
	static_cast<uint16>(UE_ARRAY_COUNT(sReplicationStateDescriptorMemberFunctionDescriptors) - 1U), /* FunctionCount */ \
	static_cast<uint16>(UE_ARRAY_COUNT(sReplicationStateDescriptorMemberTagDescriptors) - 1U), /* TagCount */ \
	static_cast<uint16>(UE_ARRAY_COUNT(sReplicationStateDescriptorMemberReferenceDescriptors) - 1U), /* ObjectReferenceCount */ \
	static_cast<uint16>(0U), /* RepIndexCount */ \
	static_cast<uint16>(UE::Net::Private::GetMemberChangeMaskSize(sReplicationStateChangeMaskDescriptors)), /* ChangeMaskBitCount */ \
	offsetof(StateName, ChangeMask), \
	[](){return UE::Net::FReplicationStateIdentifier({ CityHash64(#StateName, strlen(#StateName))});}(), \
	Construct##StateName, \
	Destruct##StateName, \
	static_cast<UE::Net::CreateAndRegisterReplicationFragmentFunc>(nullptr), \
	Traits, \
	{}, \
};

#define IRIS_IMPLEMENT_REPLICATIONSTATEDESCRIPTOR(StateName) IRIS_IMPLEMENT_REPLICATIONSTATEDESCRIPTOR_WITH_TRAITS(StateName, UE::Net::EReplicationStateTraits::None)
