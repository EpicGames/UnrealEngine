// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"


//
// $IRIS: TEMPORARY TO HELP WITH FAKE GENERATING REPLICATIONSTATES
// define macros to declare and implement the fake Generated State
// This will be generate by UHT 
//

//
// Helper functions
//

namespace UE::Net::Private
{

template <typename T>
static constexpr uint32 CalculateWordCountForChangeMask(const T& ChangeMaskDescriptors)
{
	const uint32 Index = UE_ARRAY_COUNT(ChangeMaskDescriptors);
	return Index == 0u ? 0u : UE::Net::FNetBitArrayView::CalculateRequiredWordCount(ChangeMaskDescriptors[Index - 1u].BitOffset + ChangeMaskDescriptors[Index - 1u].BitCount);
}

}

//
// DECLARATION MACROS
//
#define IRIS_DECLARE_COMMON() \
private: \
	static const UE::Net::FReplicationStateDescriptor sReplicationStateDescriptor; \
	static const UE::Net::FReplicationStateMemberDescriptor sReplicationStateDescriptorMemberDescriptors[]; \
	static const UE::Net::FReplicationStateMemberSerializerDescriptor sReplicationStateDescriptorMemberSerializerDescriptors[]; \
	static const UE::Net::FReplicationStateMemberTraitsDescriptor sReplicationStateDescriptorMemberTraitsDescriptors[]; \
	static const UE::Net::FReplicationStateMemberFunctionDescriptor  sReplicationStateDescriptorMemberFunctionDescriptors[]; \
	static const UE::Net::FReplicationStateMemberTagDescriptor  sReplicationStateDescriptorMemberTagDescriptors[]; \
	static const UE::Net::FReplicationStateMemberReferenceDescriptor  sReplicationStateDescriptorMemberReferenceDescriptors[]; \
	static const UE::Net::FReplicationStateMemberDebugDescriptor sReplicationStateDescriptorMemberDebugDescriptors[]; \
	UE::Net::FReplicationStateHeader InternalReplicationState; \
	UE::Net::FNetBitArrayView::StorageWordType ChangeMask[UE::Net::Private::CalculateWordCountForChangeMask(sReplicationStateChangeMaskDescriptors)] = { }; \
\
	/* helpers, should do special methods for this with no safety at all since this will be generated. Single bitcase should be as fast as possible */ \
	void SetDirty(const UE::Net::FReplicationStateMemberChangeMaskDescriptor& Bits) { UE::Net::FNetBitArrayView Mask(&ChangeMask[0], sReplicationStateDescriptor.ChangeMaskBitCount); UE::Net::Private::MarkDirty(InternalReplicationState, Mask, Bits);} \
	bool IsDirty(const UE::Net::FReplicationStateMemberChangeMaskDescriptor& Bits) const { const UE::Net::FNetBitArrayView Mask(const_cast<UE::Net::FNetBitArrayView::StorageWordType*>(&ChangeMask[0]), sReplicationStateDescriptor.ChangeMaskBitCount); return Mask.GetBit(Bits.BitOffset); } \
public: \
	static const UE::Net::FReplicationStateDescriptor* GetReplicationStateDescriptor() { return &sReplicationStateDescriptor; }

#define IRIS_ACCESS_BY_VALUE(MemberName, MemberType, MemberIndex) \
void Set##MemberName(MemberType Value) { if (MemberName != Value) { SetDirty(sReplicationStateChangeMaskDescriptors[MemberIndex]); } MemberName = Value; } \
MemberType Get##MemberName() const { return MemberName; } \
bool Is##MemberName##Dirty() const { return IsDirty(sReplicationStateChangeMaskDescriptors[MemberIndex]); }

#define IRIS_ACCESS_BY_REFERENCE(MemberName, MemberType, MemberIndex) \
void Set##MemberName(const MemberType& Value) { if (MemberName != Value) { SetDirty(sReplicationStateChangeMaskDescriptors[MemberIndex]); } MemberName = Value; } \
const MemberType& Get##MemberName() const { return MemberName; } \
bool Is##MemberName##Dirty() const { return IsDirty(sReplicationStateChangeMaskDescriptors[MemberIndex]); }
