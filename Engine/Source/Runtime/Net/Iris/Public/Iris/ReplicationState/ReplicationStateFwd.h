// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Net/Core/NetHandle/NetHandle.h"
#include "Iris/Core/BitTwiddling.h"

namespace UE::Net
{

namespace Private
{
	struct FReplicationStateHeaderAccessor;
}

/** 
 * A ReplicationState always contains a ReplicationStateHeader which we use to bind replication states for dirty tracking
 */
struct FReplicationStateHeader
{
	FReplicationStateHeader() : NetHandleId(0), bInitStateIsDirty(0), bStateIsDirty(0) {}

	/** Returns true if the state is bound to the dirty tracking system */
	bool IsBound() const { return NetHandleId != 0; }

private:
	friend Private::FReplicationStateHeaderAccessor;

	// All replication states that are bound by an instance protocol is assigned a NetHandle for dirty state tracking
	uint64 NetHandleId : FNetHandle::IdBits;
	// Init state doesn't use changemasks, instead we have a reserved bit here
	uint64 bInitStateIsDirty : 1;
	// Track whether any state is dirty.
	uint64 bStateIsDirty : 1;
};

namespace Private
{

/** 
 * Internal access, should only be used by internal code
 */
struct FReplicationStateHeaderAccessor
{
	static uint64 GetNetHandleId(const FReplicationStateHeader& Header) { return Header.NetHandleId; }
	static bool GetIsInitStateDirty(const FReplicationStateHeader& Header) { return Header.bInitStateIsDirty; }
	static bool GetIsStateDirty(const FReplicationStateHeader& Header) { return Header.bStateIsDirty; }

	static void MarkInitStateDirty(FReplicationStateHeader& Header) { Header.bInitStateIsDirty = true; }
	static void MarkStateDirty(FReplicationStateHeader& Header) { Header.bStateIsDirty = true; }

	/** Clears both state and init state dirtiness. */
	static void ClearAllStateIsDirty(FReplicationStateHeader& Header)
	{ 
		Header.bInitStateIsDirty = false;
		Header.bStateIsDirty = false;
	}

	static void SetNetHandleId(FReplicationStateHeader& Header, FNetHandle NetHandle);
};

inline void FReplicationStateHeaderAccessor::SetNetHandleId(FReplicationStateHeader& Header, FNetHandle NetHandle)
{
	const uint64 Id = NetHandle.GetId();

	static constexpr uint64 LargestId = UE::Net::LargestBitValueForBits(FNetHandle::IdBits);
	ensureMsgf(static_cast<uint64>(Id) < LargestId, TEXT("Setting a NetHandle identifier %llu that cannot be packed into %lld bits (largest value of %llu)."), Id, FNetHandle::IdBits, LargestId);

	Header.NetHandleId = Id;
}

}

}
