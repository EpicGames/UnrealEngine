// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/NetHandle/NetHandleManager.h"
#include "Net/Core/DirtyNetObjectTracker/GlobalDirtyNetObjectTracker.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/ReplicationState/ReplicationStateFwd.h"

namespace UE::Net
{
	/** Mark the Replication State Header as dirty so it can be copied in the CopyDirtyState pass. */
	inline void MarkNetObjectStateHeaderDirty(FReplicationStateHeader& Header)
	{
		Private::FReplicationStateHeaderAccessor::MarkStateDirty(Header);
	}

} // end namespace UE::Net

namespace UE::Net::Private
{

/**
 * Get FReplicationStateHeader from a ReplicationState
 */
inline UE::Net::FReplicationStateHeader& GetReplicationStateHeader(void* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	// $IRIS: $TODO: Currently we do not store the offset to the internal replication state
	const SIZE_T ReplicationStateHeaderOffset = (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::IsNativeFastArrayReplicationState) ? SIZE_T(Descriptor->ChangeMasksExternalOffset - sizeof(FReplicationStateHeader)) : SIZE_T(0));
	return *reinterpret_cast<UE::Net::FReplicationStateHeader*>(reinterpret_cast<uint8*>(StateBuffer) + ReplicationStateHeaderOffset);
}

inline bool IsReplicationStateBound(void* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	return GetReplicationStateHeader(StateBuffer, Descriptor).IsBound();
}

/**
 * Get MemberChangeMask
 */
inline FNetBitArrayView GetMemberChangeMask(uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	return FNetBitArrayView((FNetBitArrayView::StorageWordType*)(StateBuffer + Descriptor->GetChangeMaskOffset()), Descriptor->ChangeMaskBitCount, FNetBitArrayView::NoResetNoValidate);
}

/**
 * Get MemberConditionalChangeMask. Valid if state has lifetime conditionals.
 */
inline FNetBitArrayView GetMemberConditionalChangeMask(uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	return FNetBitArrayView((FNetBitArrayView::StorageWordType*)(StateBuffer + Descriptor->GetConditionalChangeMaskOffset()), Descriptor->ChangeMaskBitCount, FNetBitArrayView::NoResetNoValidate);
}

/**
 * Get MemberPollMask. Valid if state has members
 */
inline FNetBitArrayView GetMemberPollMask(uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	return FNetBitArrayView((FNetBitArrayView::StorageWordType*)(StateBuffer + Descriptor->GetMemberPollMaskOffset()), Descriptor->MemberCount, FNetBitArrayView::NoResetNoValidate);
}

/**
 * Reset MemberPollMask
 */
inline void ResetMemberPollMask(uint8* StateBuffer, const FReplicationStateDescriptor* Descriptor)
{
	FNetBitArrayView((FNetBitArrayView::StorageWordType*)(StateBuffer + Descriptor->GetMemberPollMaskOffset()), Descriptor->MemberCount, FNetBitArrayView::ResetOnInit);
}

/**
 * Mark specific member dirty, if this is the first bit marked as dirty in the local MemberChangeMask and the state is bound, mark owning object as dirty as well.
 * Note that all bits described by the ChangeMaskInfo will be dirtied by the call. 
 */
inline void MarkDirty(UE::Net::FReplicationStateHeader& InternalState, FNetBitArrayView& MemberChangeMask, const FReplicationStateMemberChangeMaskDescriptor& ChangeMaskInfo)
{
	// If this state is bound to a replicated object, notify the replication system that we have data to copy
	// Note that we only check the first bit of the changemask, as this is used to indicate whether the property is dirty or not
	// some properties might use additional bits but will only be treated as dirty if the parent bit is set.
	if (InternalState.IsBound() && !MemberChangeMask.GetBit(ChangeMaskInfo.BitOffset))
	{
		FReplicationStateHeaderAccessor::MarkStateDirty(InternalState);
		MarkNetObjectStateHeaderDirty(InternalState);
	}

	MemberChangeMask.SetBits(ChangeMaskInfo.BitOffset, ChangeMaskInfo.BitCount);
}

}
