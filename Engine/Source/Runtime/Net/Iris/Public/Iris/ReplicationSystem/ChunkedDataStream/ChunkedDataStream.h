// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/DataStream/DataStream.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"
#include "ChunkedDataStream.generated.h"

namespace UE::Net
{
	struct FIrisPackageMapExports;
	namespace Private
	{
		class FChunkedDataWriter;
		class FChunkedDataReader;
	}
}

/** Scope used to setup PackageMap owned by ChunkedDataStream to write and capture exports */
class FChunkedDataStreamExportWriteScope
{
public:
	IRISCORE_API FChunkedDataStreamExportWriteScope(UChunkedDataStream* DataStream);
	IRISCORE_API UIrisObjectReferencePackageMap* GetPackageMap();

private:
	UE::Net::FIrisObjectReferencePackageMapWriteScope WriteScope;
};

/** Scope used to setup PackageMap owned by ChunkedDataStream for reading captured exports */
class FChunkedDataStreamExportReadScope
{
public:
	IRISCORE_API FChunkedDataStreamExportReadScope(UChunkedDataStream* DataStream);
	IRISCORE_API UIrisObjectReferencePackageMap* GetPackageMap();

private:
	UE::Net::FIrisObjectReferencePackageMapReadScope ReadScope;
};

/** 
 * ChunkedDataStream
 * Experimental DataStream used to split and carry large payloads with potential exports
 */
UCLASS(MinimalAPI)
class UChunkedDataStream : public UDataStream
{
	GENERATED_BODY()

public:
	enum class EDispatchResult : uint8
	{
		Ok,
		WaitingForMustBeMappedReferences,
		NothingToDispatch
	};

	/**
	 * Enqueue Payload for sending,
	 * Object References written to the payload by using the PackageMap associated with the DataStream will be appended to the payload.
	 * @param Payload SharedPtr to Payload to send, The DataStream will hold a shared reference until the transfer is complete.
	 * @return false if SendBuffer is full
	 */
	IRISCORE_API bool EnqueuePayload(const TSharedPtr<TArray64<uint8>>& Payload);

	/**
	 * Dispatch received Payload
	 * Object References can be read from the payload through the PackageMap associated with the DataStream.
	 * @param DispatchDataChunkFunction Reference to TFunction to process Payload.
	 */
	IRISCORE_API EDispatchResult DispatchReceivedPayload(TFunctionRef<void(TConstArrayView64<uint8>)> DispatchPayloadFunction);

	/**
	* Dispatch all received Payloads
	* Object References can be read from the payload through the PackageMap associated with the DataStream.
	* @param DispatchDataChunkFunction Reference to TFunction to process Payload.
	*/
	IRISCORE_API EDispatchResult DispatchReceivedPayloads(TFunctionRef<void(TConstArrayView64<uint8>)> DispatchPayloadFunction);

	/** Get the number of received payloads that are ready for dispatch */
	IRISCORE_API uint32 GetNumReceivedPayloadsPendingDispatch() const;

	/** Get UIrisObjectReferencePackageMap associated with the DataStream */
	IRISCORE_API UIrisObjectReferencePackageMap* GetPackageMap();
	IRISCORE_API const UIrisObjectReferencePackageMap* GetPackageMap() const;

	/** Get number of payload bytes that is yet to be acknowledged */
	IRISCORE_API uint32 GetQueuedByteCount() const;

	/** 
	 * Set the maximum number of undispatched payload bytes we can have on the receiing side. 
	 * @param MaxUndispatchedPayloadBytes Stream will be set in error state and close if we have received too much data without dispatching it
	 */
	IRISCORE_API void SetMaxUndispatchedPayloadBytes(uint32 MaxUndispatchedPayloadBytes);

	/** 
	* Set the maximum number of enqueued payload bytes we can have on the sending side. 
	* @param MaxEnqueuedPayloadBytes Stream will be set in error state and close if we have received too much data without dispatching it
	*/
	IRISCORE_API void SetMaxEnqueuedPayloadBytes(uint32 MaxEnqueuedPayloadBytes);

	/** 
	* Returns true if the stream is in an error state and should be closed.
	*/
	IRISCORE_API bool HasError() const;

protected:

	// UDataStream interface
	IRISCORE_API virtual void Init(const UDataStream::FInitParameters& Params) override;
	IRISCORE_API virtual void Deinit() override;

	IRISCORE_API virtual EWriteResult BeginWrite(const FBeginWriteParameters& Params) override;
	IRISCORE_API virtual EWriteResult WriteData(UE::Net::FNetSerializationContext& Context, FDataStreamRecord const*& OutRecord) override;

	IRISCORE_API virtual void ReadData(UE::Net::FNetSerializationContext& Context) override;
	IRISCORE_API virtual void ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus Status, FDataStreamRecord const* Record) override;
	IRISCORE_API virtual bool HasAcknowledgedAllReliableData() const override;

private:
	friend FChunkedDataStreamExportWriteScope;
	friend FChunkedDataStreamExportReadScope;

	UE::Net::Private::FChunkedDataWriter* ChunkedWriter = nullptr;
	UE::Net::Private::FChunkedDataReader* ChunkedReader = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<UIrisObjectReferencePackageMap> PackageMap = nullptr;
};
