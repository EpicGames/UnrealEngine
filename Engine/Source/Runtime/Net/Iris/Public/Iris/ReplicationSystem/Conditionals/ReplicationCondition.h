// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::Net
{

/** Replication conditions that can be set on replicated objects. */
enum class EReplicationCondition : uint32
{
	/**
	 * Similar to its ENetRole counterpart RoleAutonomous can only be set for a single connection. All other connections are assumed to have a "Simulated" object.
	 * @see UReplicationSystem::SetReplicationConditionConnectionFilter
	 */
	RoleAutonomous,
	/**
	 * ReplicatePhysics affects whether properties with a physics condition are replicated or not.
	 * It also affects which members of FReplicatedMovement are replicated. The condition is assumed to be false by default.
	 * @see UReplicationSystem::SetReplicationCondition
	 */ 
	ReplicatePhysics,
};

}
