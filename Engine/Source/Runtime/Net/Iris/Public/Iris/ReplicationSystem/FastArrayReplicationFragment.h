// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/ReplicationSystem/Private/FastArrayReplicationFragmentInternal.h" // IWYU pragma: export
#include "Iris/ReplicationState/IrisFastArraySerializer.h"
#include "Iris/ReplicationState/Private/IrisFastArraySerializerInternal.h" // IWYU pragma: export
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationState/ReplicationStateUtil.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"

#include "Net/Core/NetBitArray.h"

#include "Templates/UnrealTemplate.h"

namespace UE::Net
{

/**
 * TFastArrayReplicationFragment - Binds a typed FastArray to a FReplicationfragment
 * Used to support FFastArray-based serialization with no required code modifications
 * Backed by a PropertyReplicationState which means that we will have to poll source data for dirtiness,
 * in the case of FastArrays this involves comparing the replication key of the array and its items.
 */
template <typename FastArrayItemType, typename FastArrayType>
class TFastArrayReplicationFragment : public Private::FFastArrayReplicationFragmentBase
{
public:
	typedef TArray<FastArrayItemType> ItemArrayType;
	TFastArrayReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor, bool bValidateDescriptor = true);

protected:
	enum EAllowAdditionalPropertiesType { AllowAdditionalProperties };
	TFastArrayReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor, const EAllowAdditionalPropertiesType) : TFastArrayReplicationFragment(InTraits, InOwner, InDescriptor, false) {}

	// For the select few cases where we allow additional properties, this is a helper to deal with applying them directly from quantized state
	void ApplyReplicatedStateForExtraProperties(FReplicationStateApplyContext& Context) const;

	// FReplicationFragment
	virtual void ApplyReplicatedState(FReplicationStateApplyContext& Context) const override;
	virtual void CallRepNotifies(FReplicationStateApplyContext& Context) override;
	virtual bool PollReplicatedState(EReplicationFragmentPollFlags PollOption) override;
	virtual void ReplicatedStateToString(FStringBuilderBase& StringBuilder, FReplicationStateApplyContext& Context, EReplicationStateToStringFlags Flags) const override;

protected:
	// Poll entire FastFrray
	bool PollAllState(bool bForceFullCompare = false);

	// Returns true if the FastArray is dirty
	bool IsDirty() const;

	// Returns true if the FastArray should be polled
	bool IsDirtyForPolling() const;

	// Mark FastArray dirty
	void MarkDirty();

	// Get FastArraySerializer from owner
	inline FastArrayType* GetFastArraySerializerFromOwner() const;

	// Get FastArraySerializer from our cached ReplicationState
	inline FastArrayType* GetFastArraySerializerFromReplicationState() const;

	// Get FastArraySerialzier from received state
	inline FastArrayType* GetFastArraySerializerFromApplyContext(FReplicationStateApplyContext& Context) const;

	// TODO: Can be removed when we have implemented explicit code to traverse quantized data directly
	TUniquePtr<FastArrayType> AccumulatedReceivedState;
};

/**
 * TNativeFastArrayReplicationFragment - Binds a typed FastArray to a FReplicationfragments
 * Used to support FFastArray-based serialization with some minor code modifications
 * The FastArray must be changed to inherit from IrisFastArraySerializer instead of FFastArraySerializer which will inject a
 * ReplicationStateHeader and a fixed size changemask, in it most basic form this allows us to not keep a full copy the fast array to detect dirtiness 
 * but instead only store the ReplicationID and ReplicationKeys. We can also provide an alternative interface for editing the FastArrays which allows us 
 * update dirtiness directly and skip the poll step completely.
 */
template <typename FastArrayItemType, typename FastArrayType, typename PollingPolicyType = FastArrayPollingPolicies::FNoPollingPolicy>
class TNativeFastArrayReplicationFragment final : public Private::FNativeFastArrayReplicationFragmentBase
{
public:
	typedef TArray<FastArrayItemType> ItemArrayType;

	TNativeFastArrayReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor);
	void Register(FFragmentRegistrationContext& Fragments, EReplicationFragmentTraits Traits = EReplicationFragmentTraits::None);

protected:
	// FReplicationFragment implementation
	virtual void ApplyReplicatedState(FReplicationStateApplyContext& Context) const override;
	virtual void CallRepNotifies(FReplicationStateApplyContext& Context) override;
	virtual bool PollReplicatedState(EReplicationFragmentPollFlags PollOption) override;
	virtual void ReplicatedStateToString(FStringBuilderBase& StringBuilder, FReplicationStateApplyContext& Context, EReplicationStateToStringFlags Flags) const override;

protected:
	bool PollAllState();

	bool IsDirty() const;

	// Get FastArraySerializer from owner
	inline FastArrayType* GetFastArraySerializerFromOwner() const;

private:
	PollingPolicyType PollingPolicy;
};

/**
 * TFastArrayReplicationFragment implementation
 */

template <typename FastArrayItemType, typename FastArrayType>
TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::TFastArrayReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor, bool bValidateDescriptor) : FFastArrayReplicationFragmentBase(InTraits, InOwner, InDescriptor, bValidateDescriptor)
{
	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReceive))
	{
		AccumulatedReceivedState = MakeUnique<FastArrayType>();
	}
}

template <typename FastArrayItemType, typename FastArrayType>
void TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::ApplyReplicatedState(FReplicationStateApplyContext& Context) const
{
	// Get the wrapped FastArraySerializer and array
	FastArrayType* DstArraySerializer = GetFastArraySerializerFromOwner();
	ItemArrayType* DstWrappedArray = reinterpret_cast<ItemArrayType*>(reinterpret_cast<uint8*>(DstArraySerializer) + WrappedArrayOffsetRelativeFastArraySerializerProperty);

	// For now we maintain a dequantized representation of the array into which we accumulate new data.
	// TODO: change to only maintain the map of indices
	InternalPartialDequantizeFastArray(Context, reinterpret_cast<uint8*>(AccumulatedReceivedState.Get()), Context.StateBufferData.RawStateBuffer, GetFastArrayPropertyStructDescriptor());

	// Intentionally not const as we allow the src state to be modified
	FastArrayType* SrcArraySerializer = AccumulatedReceivedState.Get();
	const ItemArrayType* SrcWrappedArray = reinterpret_cast<const ItemArrayType*>(reinterpret_cast<uint8*>(SrcArraySerializer) + WrappedArrayOffsetRelativeFastArraySerializerProperty);	

	// Apply state to target FastArray and issue callbacks
	Private::FFastArrayReplicationFragmentHelper::ApplyReplicatedState(DstArraySerializer, DstWrappedArray, SrcArraySerializer, SrcWrappedArray, GetArrayElementDescriptor(), Context);
}

template <typename FastArrayItemType, typename FastArrayType>
void TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::ApplyReplicatedStateForExtraProperties(FReplicationStateApplyContext& Context) const
{
	// Dequantize additional properties directly to DstArraySerialzier
	InternalDequantizeExtraProperties(*Context.NetSerializationContext, reinterpret_cast<uint8*>(GetFastArraySerializerFromOwner()), Context.StateBufferData.RawStateBuffer, GetFastArrayPropertyStructDescriptor());
}

template <typename FastArrayItemType, typename FastArrayType>
FastArrayType* TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::GetFastArraySerializerFromOwner() const
{
	return reinterpret_cast<FastArrayType*>(reinterpret_cast<uint8*>(Owner) + ReplicationStateDescriptor->MemberProperties[0]->GetOffset_ForGC());
}

template <typename FastArrayItemType, typename FastArrayType>
inline FastArrayType* TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::GetFastArraySerializerFromApplyContext(FReplicationStateApplyContext& Context) const
{
	return reinterpret_cast<FastArrayType*>(Context.StateBufferData.ExternalStateBuffer + ReplicationStateDescriptor->MemberDescriptors[0].ExternalMemberOffset);	
}

template <typename FastArrayItemType, typename FastArrayType>
FastArrayType* TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::GetFastArraySerializerFromReplicationState() const
{
	checkSlow(ReplicationState);
	return ReplicationState ? reinterpret_cast<FastArrayType*>(ReplicationState->GetStateBuffer() + ReplicationStateDescriptor->MemberDescriptors[0].ExternalMemberOffset) : nullptr;
}

template <typename FastArrayItemType, typename FastArrayType>
bool TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::PollReplicatedState(EReplicationFragmentPollFlags PollOption)
{
	// If the ForceObjectReferences flag is set we cannot early out and must always refresh cached data
	if (EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::ForceRefreshCachedObjectReferencesAfterGC))
	{
		constexpr bool bForceFullCompare = true;
		return PollAllState(bForceFullCompare);
	}

	// We can early out if we are pushbased and not dirty for polling
	const bool bPoll = EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollAllState) ||
		(EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollDirtyState) && (!EnumHasAnyFlags(EReplicationFragmentTraits::HasPushBasedDirtiness, Traits) || IsDirtyForPolling()));

	if (bPoll)
	{
		constexpr bool bForceFullCompare = false;
		return PollAllState(bForceFullCompare);
	}
	else
	{
		return IsDirty();
	}
}

template <typename FastArrayItemType, typename FastArrayType>
bool TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::PollAllState(bool bForceFullCompare)
{
	// Lookup source data, we need the actual FastArraySerializer and the Array it is wrapping
	FastArrayType* SrcArraySerializer = GetFastArraySerializerFromOwner();
	ItemArrayType* SrcWrappedArray = reinterpret_cast<ItemArrayType*>(reinterpret_cast<uint8*>(SrcArraySerializer) + WrappedArrayOffsetRelativeFastArraySerializerProperty);
	
	IRIS_PROFILER_PROTOCOL_NAME(ReplicationStateDescriptor->DebugName->Name);

	// Lookup destination data
	FastArrayType* DstArraySerializer = GetFastArraySerializerFromReplicationState();
	ItemArrayType* DstWrappedArray = reinterpret_cast<ItemArrayType*>(reinterpret_cast<uint8*>(DstArraySerializer) + WrappedArrayOffsetRelativeFastArraySerializerProperty);

	// Check if we can early out
	if (!bForceFullCompare && SrcArraySerializer->ArrayReplicationKey == DstArraySerializer->ArrayReplicationKey)
	{
		return IsDirty();
	}

	// First we must resize target to match size of source data
	bool bMarkArrayDirty = false;
	const uint32 ElementCount = SrcWrappedArray->Num();
	const uint32 OldDstElementCount = DstWrappedArray->Num();
	if (DstWrappedArray->Num() != ElementCount)
	{
		DstWrappedArray->SetNum(ElementCount);
	}

	const FReplicationStateDescriptor* ArrayElementDescriptor = GetArrayElementDescriptor();
	const FReplicationStateMemberDescriptor* MemberDescriptors = ArrayElementDescriptor->MemberDescriptors;

	// We currently use a simple modulo scheme for bits in the changemask
	// A single bit might represent several entries in the array which all will be considered dirty, it is up to the serializer to handle this
	// The first bit is used by the owning property we need to offset by one and deduct one from the usable bits
	FNetBitArrayView MemberChangeMask = UE::Net::Private::GetMemberChangeMask(ReplicationState->GetStateBuffer(), ReplicationStateDescriptor);
	
	const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = ReplicationStateDescriptor->MemberChangeMaskDescriptors[0];
	const uint32 ChangeMaskBitOffset = MemberChangeMaskDescriptor.BitOffset + FIrisFastArraySerializer::IrisFastArrayChangeMaskBitOffset;
	const uint32 ChangeMaskBitCount = MemberChangeMaskDescriptor.BitCount - FIrisFastArraySerializer::IrisFastArrayChangeMaskBitOffset;

	FastArrayItemType* DstItems = DstWrappedArray->GetData();
	FastArrayItemType* SrcItems = SrcWrappedArray->GetData();

	// We keep a separate count as we do not care about items that should not be replicated.
	int32 DstItemCount = 0;
	{
#if WITH_PUSH_MODEL
		// Disable push model by temporarily setting the FastArray's RepIndex to none.
		// This prevents the array from adding itself to the global dirty list via MarkItemDirty while we are polling it.
		TGuardValue DisablePushModel(SrcArraySerializer->RepIndex, (int32)INDEX_NONE);
#endif

		// Iterate over array entries and copy the statedata using internal data if it has changed
		for (int32 ElementIt = 0, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			const int32 DstElementIt = DstItemCount;
			FastArrayItemType& SrcItem = SrcItems[ElementIt];
			FastArrayItemType& DstItem = DstItems[DstElementIt];

			const bool bIsWritingOnClient = false;
			if (SrcArraySerializer->template ShouldWriteFastArrayItem<FastArrayItemType, FastArrayType>(SrcItem, bIsWritingOnClient))
			{
				if (SrcItem.ReplicationID == INDEX_NONE)
				{
					SrcArraySerializer->MarkItemDirty(SrcItem);
				}

				const bool bReplicationKeyChanged = SrcItem.ReplicationKey != DstItem.ReplicationKey || SrcItem.ReplicationID != DstItem.ReplicationID;
				if (bReplicationKeyChanged || (bForceFullCompare && !InternalCompareArrayElement(ArrayElementDescriptor, &DstItem, &SrcItem)))
				{
					InternalCopyArrayElement(ArrayElementDescriptor, &DstItem, &SrcItem);
					DstItem.ReplicationKey = SrcItem.ReplicationKey;

					// Mark element as dirty and mark array as dirty as well.
					if (ChangeMaskBitCount)
					{
						MemberChangeMask.SetBit((DstElementIt % ChangeMaskBitCount) + ChangeMaskBitOffset);
					}
					bMarkArrayDirty = true;
				}
				++DstItemCount;
			}
		}
	}

	// Set actual num replicated items.
	if (DstWrappedArray->Num() != DstItemCount)
	{
		// Set actual num, but keeping the allocated size of the source array
		DstWrappedArray->SetNum(DstItemCount, EAllowShrinking::No);
	}

	// Mark dirty if the new filtered size differs from our previous state.
	bMarkArrayDirty = bMarkArrayDirty || (OldDstElementCount != DstItemCount);

	// We update this after the poll since every call to MarkItem() dirty will Increase the ArrayReplicationKey
	DstArraySerializer->ArrayReplicationKey = SrcArraySerializer->ArrayReplicationKey;

	// Mark the NetObject as dirty
	if (bMarkArrayDirty && ReplicationState->IsCustomConditionEnabled(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex))
	{
		MemberChangeMask.SetBit(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex);
		MarkNetObjectStateHeaderDirty(UE::Net::Private::GetReplicationStateHeader(ReplicationState->GetStateBuffer(), ReplicationStateDescriptor));
	}

	return IsDirty();
}

template <typename FastArrayItemType, typename FastArrayType>
void TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::ReplicatedStateToString(FStringBuilderBase& StringBuilder, FReplicationStateApplyContext& Context, EReplicationStateToStringFlags Flags) const
{
	// Temporary
	FastArrayType ReceivedState;

	// Dequantize into temporary array, using partial dequantize based on changemask
	InternalDequantizeFastArray(*Context.NetSerializationContext, reinterpret_cast<uint8*>(&ReceivedState), Context.StateBufferData.RawStateBuffer, GetFastArrayPropertyStructDescriptor());

	// Output state to string
	ToString(StringBuilder, reinterpret_cast<uint8*>(&ReceivedState), GetFastArrayPropertyStructDescriptor());
}

template <typename FastArrayItemType, typename FastArrayType>
bool TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::IsDirty() const
{
	FReplicationStateHeader& ReplicationStateHeader = Private::GetReplicationStateHeader(ReplicationState->GetStateBuffer(), ReplicationStateDescriptor);
	return Private::FReplicationStateHeaderAccessor::GetIsStateDirty(ReplicationStateHeader);
}

template <typename FastArrayItemType, typename FastArrayType>
bool TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::IsDirtyForPolling() const
{
	FReplicationStateHeader& ReplicationStateHeader = Private::GetReplicationStateHeader(ReplicationState->GetStateBuffer(), ReplicationStateDescriptor);
	bool bIsStateDirty = Private::FReplicationStateHeaderAccessor::GetIsStateDirty(ReplicationStateHeader);
	return bIsStateDirty || Private::GetMemberPollMask(ReplicationState->GetStateBuffer(), ReplicationStateDescriptor).IsAnyBitSet();
}

template <typename FastArrayItemType, typename FastArrayType>
void TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::MarkDirty()
{
	// Mark the NetObject as dirty
	if (ReplicationState->IsCustomConditionEnabled(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex))
	{
		FNetBitArrayView MemberChangeMask = UE::Net::Private::GetMemberChangeMask(ReplicationState->GetStateBuffer(), ReplicationStateDescriptor);
		MemberChangeMask.SetBit(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex);
		MarkNetObjectStateHeaderDirty(UE::Net::Private::GetReplicationStateHeader(ReplicationState->GetStateBuffer(), ReplicationStateDescriptor));
	}
}

template <typename FastArrayItemType, typename FastArrayType>
void TFastArrayReplicationFragment<FastArrayItemType, FastArrayType>::CallRepNotifies(FReplicationStateApplyContext& Context)
{
	const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;

	// If we get here we are either the init state or dirty
	if (const UFunction* RepNotifyFunction = Descriptor->MemberPropertyDescriptors[0].RepNotifyFunction)
	{
		// if this is the init state, we compare against default and early out if initial state does not differ from default (empty)
		if (Context.bIsInit)
		{
			FastArrayType ReceivedState;
			FastArrayType DefaultState;
			InternalDequantizeFastArray(*Context.NetSerializationContext, reinterpret_cast<uint8*>(&ReceivedState), Context.StateBufferData.RawStateBuffer, GetFastArrayPropertyStructDescriptor());
			InternalDequantizeExtraProperties(*Context.NetSerializationContext, reinterpret_cast<uint8*>(&ReceivedState), Context.StateBufferData.RawStateBuffer, GetFastArrayPropertyStructDescriptor());			
			if (Descriptor->MemberProperties[0]->Identical(&ReceivedState, &DefaultState))
			{
				return;
			}
		}

		Owner->ProcessEvent(const_cast<UFunction*>(RepNotifyFunction), nullptr);
	}
}

/*
 * TNativeFastArrayReplicationFragment implementation
 */
template <typename FastArrayItemType, typename FastArrayType, typename PollingPolicyType>
TNativeFastArrayReplicationFragment<FastArrayItemType, FastArrayType, PollingPolicyType>::TNativeFastArrayReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor)
: FNativeFastArrayReplicationFragmentBase(InTraits, InOwner, InDescriptor)
{
	if (EnumHasAnyFlags(InTraits, EReplicationFragmentTraits::CanReplicate) && (PollingPolicy.GetPollingState() != nullptr))
	{
		// For PropertyReplicationStates we need to poll properties from our owner in order to detect state changes.
		Traits |= EReplicationFragmentTraits::NeedsPoll;
	}
}

template <typename FastArrayItemType, typename FastArrayType, typename PollingPolicyType>
void TNativeFastArrayReplicationFragment<FastArrayItemType, FastArrayType, PollingPolicyType>::Register(FFragmentRegistrationContext& Context, EReplicationFragmentTraits InTraits)
{
	Traits |= InTraits;
	Context.RegisterReplicationFragment(this, ReplicationStateDescriptor.GetReference(), reinterpret_cast<uint8*>(GetFastArraySerializerFromOwner()));
}

template <typename FastArrayItemType, typename FastArrayType, typename PollingPolicyType>
void TNativeFastArrayReplicationFragment<FastArrayItemType, FastArrayType, PollingPolicyType>::CallRepNotifies(FReplicationStateApplyContext& Context)
{
	const FReplicationStateDescriptor* Descriptor = ReplicationStateDescriptor;

	// if we get here we are either the init state or dirty
	if (const UFunction* RepNotifyFunction = Descriptor->MemberPropertyDescriptors[0].RepNotifyFunction)
	{
		// if this is the init state, we compare against default and early out if initial state does not differ from default (empty)
		if (Context.bIsInit)
		{
			FastArrayType ReceivedState;
			FastArrayType DefaultState;
			InternalDequantizeFastArray(*Context.NetSerializationContext, reinterpret_cast<uint8*>(&ReceivedState), Context.StateBufferData.RawStateBuffer, GetFastArrayPropertyStructDescriptor());
			if (Descriptor->MemberProperties[0]->Identical(&ReceivedState, &DefaultState))
			{
				return;
			}
		}

		Owner->ProcessEvent(const_cast<UFunction*>(RepNotifyFunction), nullptr);
	}
}

template <typename FastArrayItemType, typename FastArrayType, typename PollingPolicyType>
bool TNativeFastArrayReplicationFragment<FastArrayItemType, FastArrayType, PollingPolicyType>::PollReplicatedState(EReplicationFragmentPollFlags PollOption)
{
	// We ignore object references polling. Since the source state will have references cleaned up they will be valid once any affected item is dirtied.
	if (EnumHasAnyFlags(PollOption, EReplicationFragmentPollFlags::PollAllState | EReplicationFragmentPollFlags::PollDirtyState))
	{
		if (!EnumHasAnyFlags(EReplicationFragmentTraits::HasPushBasedDirtiness, Traits) || IsDirty())
		{
			return PollAllState();
		}
	}

	return IsDirty();
}

template <typename FastArrayItemType, typename FastArrayType, typename PollingPolicyType>
bool TNativeFastArrayReplicationFragment<FastArrayItemType, FastArrayType, PollingPolicyType>::PollAllState()
{
	using FPollingState = FastArrayPollingPolicies::FPollingState;
	if (FPollingState* PollingState = PollingPolicy.GetPollingState())
	{
		// Get the source FastArraySerializer and array
		FastArrayType* SrcArraySerializer = GetFastArraySerializerFromOwner();
		ItemArrayType* SrcWrappedArray = reinterpret_cast<ItemArrayType*>(reinterpret_cast<uint8*>(SrcArraySerializer) + WrappedArrayOffsetRelativeFastArraySerializerProperty);

		// Check if we can early out
		if (SrcArraySerializer->ArrayReplicationKey == PollingState->ArrayReplicationKey)
		{
			return IsDirty();
		}

		// First we must resize target to match size of source data
		bool bMarkArrayDirty = false;
		const uint32 ElementCount = SrcWrappedArray->Num();
		if (PollingState->ItemPollData.Num() != ElementCount)
		{
			PollingState->ItemPollData.SetNum(ElementCount);
			bMarkArrayDirty = true;
		}

		FNetBitArrayView MemberChangeMask = UE::Net::Private::FIrisFastArraySerializerPrivateAccessor::GetChangeMask(*SrcArraySerializer);

		// We currently use a simple modulo scheme for bits in the changemask
		// A single bit might represent several entries in the array which all will be considered dirty, it is up to the serializer to handle this
		// The first bit is used by the owning property we need to offset by one and deduct one from the usable bits
		const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = ReplicationStateDescriptor->MemberChangeMaskDescriptors[0];
		const uint32 ChangeMaskBitOffset = MemberChangeMaskDescriptor.BitOffset + FIrisFastArraySerializer::IrisFastArrayChangeMaskBitOffset;
		const uint32 ChangeMaskBitCount = MemberChangeMaskDescriptor.BitCount - FIrisFastArraySerializer::IrisFastArrayChangeMaskBitOffset;

		FPollingState::FEntry* DstItems = PollingState->ItemPollData.GetData();
		FastArrayItemType* SrcItems = SrcWrappedArray->GetData();

		// Iterate over array entries and copy the statedata using internal data if it has changed
		for (uint32 ElementIt = 0, ElementEndIt = ElementCount; ElementIt < ElementEndIt; ++ElementIt)
		{
			FastArrayItemType& SrcItem = SrcItems[ElementIt];
			FPollingState::FEntry& DstItem = DstItems[ElementIt];

			const bool bIsWritingOnClient = false;
			if (SrcArraySerializer->template ShouldWriteFastArrayItem<FastArrayItemType, FastArrayType>(SrcItem, bIsWritingOnClient))
			{
				if (SrcItem.ReplicationID == INDEX_NONE)
				{
					SrcArraySerializer->MarkItemDirty(SrcItem);
				}

				if (SrcItem.ReplicationKey != DstItem.ReplicationKey || SrcItem.ReplicationID != DstItem.ReplicationID)
				{
					DstItem.ReplicationKey = SrcItem.ReplicationKey;
					DstItem.ReplicationID = SrcItem.ReplicationID;

					// Mark element as dirty and mark array as dirty as well.
					if (ChangeMaskBitCount)
					{
						MemberChangeMask.SetBit((ElementIt % ChangeMaskBitCount) + ChangeMaskBitOffset);
					}
					bMarkArrayDirty = true;
				}
			}
			else
			{
				ensureMsgf(false, TEXT("Native IrisFastArraySerializer does not support local non-replicated items, use FastArraySerializer intead if this is required."));

				// Even if the native variant do not really support not replicated entries we still mark the item dirty and update the stored data to keep logic working.
				if (SrcItem.ReplicationKey != DstItem.ReplicationKey || SrcItem.ReplicationID != DstItem.ReplicationID)
				{
					DstItem.ReplicationKey = SrcItem.ReplicationKey;
					DstItem.ReplicationID = SrcItem.ReplicationID;

					// Mark element as dirty and mark array as dirty as well.
					if (ChangeMaskBitCount)
					{
						MemberChangeMask.SetBit((ElementIt % ChangeMaskBitCount) + ChangeMaskBitOffset);
					}
					bMarkArrayDirty = true;
				}
			}
		}

		// We update this after the poll since every call to MarkItem() dirty will Increase the ArrayReplicationKey
		PollingState->ArrayReplicationKey = SrcArraySerializer->ArrayReplicationKey;

		if (bMarkArrayDirty)
		{
			const bool bHasCustomConditionals = EnumHasAnyFlags(ReplicationStateDescriptor->Traits, EReplicationStateTraits::HasLifetimeConditionals);
			if (!bHasCustomConditionals || UE::Net::Private::FIrisFastArraySerializerPrivateAccessor::GetConditionalChangeMask(*SrcArraySerializer).GetBit(FIrisFastArraySerializer::IrisFastArrayPropertyBitIndex))
			{
				UE::Net::Private::FIrisFastArraySerializerPrivateAccessor::MarkArrayDirty(*SrcArraySerializer);
			}
		}
	}

	return IsDirty();
}

template <typename FastArrayItemType, typename FastArrayType, typename PollingPolicyType>
bool TNativeFastArrayReplicationFragment<FastArrayItemType, FastArrayType, PollingPolicyType>::IsDirty() const
{
	FastArrayType* SrcArraySerializer = GetFastArraySerializerFromOwner();
	const FReplicationStateHeader& ReplicationStateHeader = Private::FIrisFastArraySerializerPrivateAccessor::GetReplicationStateHeader(*SrcArraySerializer);
	return Private::FReplicationStateHeaderAccessor::GetIsStateDirty(ReplicationStateHeader);
}

template <typename FastArrayItemType, typename FastArrayType, typename PollingPolicyType>
void TNativeFastArrayReplicationFragment<FastArrayItemType, FastArrayType, PollingPolicyType>::ApplyReplicatedState(FReplicationStateApplyContext& Context) const
{
	// As we must preserve local data and generate proper callbacks we must dequantize into a temporary state
	// We could do a selective operation and keep an targetstate around and only do incremental updates of this state
	FastArrayType ReceivedState;

	// Dequantize into temporary array
	InternalDequantizeFastArray(*Context.NetSerializationContext, reinterpret_cast<uint8*>(&ReceivedState), Context.StateBufferData.RawStateBuffer, GetFastArrayPropertyStructDescriptor());

	// Get the wrapped FastArraySerializer and array
	FastArrayType* DstArraySerializer = reinterpret_cast<FastArrayType*>(reinterpret_cast<uint8*>(Owner) + ReplicationStateDescriptor->MemberProperties[0]->GetOffset_ForGC());
	ItemArrayType* DstWrappedArray = reinterpret_cast<ItemArrayType*>((uint8*)(DstArraySerializer) + WrappedArrayOffsetRelativeFastArraySerializerProperty);

	// Intentionally not const as we allow the src state to be modified
	FastArrayType* SrcArraySerializer = &ReceivedState;
	const ItemArrayType* SrcWrappedArray = reinterpret_cast<const ItemArrayType*>(reinterpret_cast<uint8*>(&ReceivedState) + WrappedArrayOffsetRelativeFastArraySerializerProperty);	

	// Apply state and issue callbacks etc
	Private::FFastArrayReplicationFragmentHelper::ApplyReplicatedState(DstArraySerializer, DstWrappedArray, SrcArraySerializer, SrcWrappedArray, GetArrayElementDescriptor(), Context);
}

template <typename FastArrayItemType, typename FastArrayType, typename PollingPolicyType>
void TNativeFastArrayReplicationFragment<FastArrayItemType, FastArrayType, PollingPolicyType>::ReplicatedStateToString(FStringBuilderBase& StringBuilder, FReplicationStateApplyContext& Context, EReplicationStateToStringFlags Flags) const
{
	// Create temporary
	FastArrayType ReceivedState;

	// Dequantize into temporary array
	InternalDequantizeFastArray(*Context.NetSerializationContext, reinterpret_cast<uint8*>(&ReceivedState), Context.StateBufferData.RawStateBuffer, GetFastArrayPropertyStructDescriptor());

	// Output state to string
	ToString(StringBuilder, reinterpret_cast<uint8*>(&ReceivedState), GetFastArrayPropertyStructDescriptor());
}

template <typename FastArrayItemType, typename FastArrayType, typename PollingPolicyType>
FastArrayType* TNativeFastArrayReplicationFragment<FastArrayItemType, FastArrayType, PollingPolicyType>::GetFastArraySerializerFromOwner() const
{
	return reinterpret_cast<FastArrayType*>(reinterpret_cast<uint8*>(Owner) + ReplicationStateDescriptor->MemberProperties[0]->GetOffset_ForGC());
}

namespace Private {

template <typename FastArrayType>
FReplicationFragment* CreateAndRegisterFragment(UObject* Owner, const FReplicationStateDescriptor* Descriptor, FFragmentRegistrationContext& Context)
{
	using namespace UE::Net;
	static_assert(TFastArrayTypeHelper<FastArrayType>::HasValidFastArrayItemType(), "Invalid FastArrayItemType detected. Make sure that FastArraySerializer has a single replicated property that is a dynamic array of the expected type");

	if constexpr (TIsDerivedFrom<FastArrayType, FIrisFastArraySerializer>::IsDerived)
	{
		if (EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::IsNativeFastArrayReplicationState))
		{
			typedef TNativeFastArrayReplicationFragment<typename TFastArrayTypeHelper<FastArrayType>::FastArrayItemType, FastArrayType, FastArrayPollingPolicies::FNeedPollingPolicy> FFragmentType;
			if (FFragmentType* Fragment = new FFragmentType(Context.GetFragmentTraits(), Owner, Descriptor))
			{
				Fragment->Register(Context, EReplicationFragmentTraits::DeleteWithInstanceProtocol);
				return Fragment;
			}
			return nullptr;
		}
	}

	typedef TFastArrayReplicationFragment<typename TFastArrayTypeHelper<FastArrayType>::FastArrayItemType, FastArrayType> FFragmentType;
	if (FFragmentType* Fragment = new FFragmentType(Context.GetFragmentTraits(), Owner, Descriptor))
	{
		Fragment->Register(Context, EReplicationFragmentTraits::DeleteWithInstanceProtocol);
		return Fragment;
	}
	return nullptr;
}

}} // End of namespaces


