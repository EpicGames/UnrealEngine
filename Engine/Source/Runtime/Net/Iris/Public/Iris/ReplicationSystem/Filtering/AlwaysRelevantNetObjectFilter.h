// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "AlwaysRelevantNetObjectFilter.generated.h"

UCLASS(transient, MinimalAPI)
class UAlwaysRelevantNetObjectFilterConfig final : public UNetObjectFilterConfig
{
	GENERATED_BODY()
};

UCLASS()
class UAlwaysRelevantNetObjectFilter final : public UNetObjectFilter
{
	GENERATED_BODY()

protected:
	// UNetObjectFilter interface
	IRISCORE_API virtual void OnInit(const FNetObjectFilterInitParams&) override;
	IRISCORE_API virtual void OnDeinit() override;
	IRISCORE_API virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectFilterAddObjectParams&) override;
	IRISCORE_API virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectFilteringInfo&) override;
	IRISCORE_API virtual void UpdateObjects(FNetObjectFilterUpdateParams&) override;
	IRISCORE_API virtual void Filter(FNetObjectFilteringParams&) override;
	IRISCORE_API virtual void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex) override;
};
