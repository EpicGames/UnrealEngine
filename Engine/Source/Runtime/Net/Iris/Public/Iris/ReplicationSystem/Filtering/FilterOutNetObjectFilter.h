// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "FilterOutNetObjectFilter.generated.h"

UCLASS(transient, MinimalAPI)
class UFilterOutNetObjectFilterConfig final : public UNetObjectFilterConfig
{
	GENERATED_BODY()
};

/**
 * Objects added to this filter will never be relevant automatically.
 * In order to become relevant to a connection they need to be part of higher-priority relevancy settings.
 * Ex: @see UReplicationSystem::AddInclusionFilterGroup, UObjectReplicationBridge::AddDependentObject, etc.
 */
UCLASS()
class UFilterOutNetObjectFilter final : public UNetObjectFilter
{
	GENERATED_BODY()

protected:
	// UNetObjectFilter interface
	virtual void OnInit(const FNetObjectFilterInitParams&) override {}
	virtual void OnDeinit() override {}
	virtual void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex) override {}
	virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectFilterAddObjectParams&) override { return true; }
	virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectFilteringInfo&) override {}

	IRISCORE_API virtual void Filter(FNetObjectFilteringParams&) override;
};
