// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Net/Core/NetBitArray.h"
#include "UObject/StrongObjectPtr.h"
#include "NetObjectConnectionFilter.generated.h"

namespace UE::Net::Private
{
	class FNetRefHandleManager;
}

UCLASS(transient, MinimalAPI)
class UNetObjectConnectionFilterConfig : public UNetObjectFilterConfig
{
	GENERATED_BODY()

public:
	/** The maximum amount of objects that may be added to the filter. It's not designed to handle massive amounts- static connection filtering via the ReplicationSystem API is preferred. */
	UPROPERTY(Config)
	uint16 MaxObjectCount = 4096;
};

/**
 * The NetObjectConnectionFilter is a dynamic pre-poll filter that supports per connection filtering. It should be seen as an alternative to the ReplicationSystem SetConnectionFilter API for use cases where
 * for example the object in question can be a dependent object. Dependent objects will override dynamic filtering and only dynamic filtering.
 */
UCLASS(transient, MinimalAPI) 
class UNetObjectConnectionFilter : public UNetObjectFilter
{
	GENERATED_BODY()

public:
	IRISCORE_API void SetReplicateToConnection(UE::Net::FNetRefHandle RefHandle, uint32 ConnectionId, UE::Net::ENetFilterStatus FilterStatus);

protected:
	struct FFilteringInfo : public FNetObjectFilteringInfo
	{
		void SetLocalObjectIndex(uint16 Index) { Data[0] = Index; }
		uint16 GetLocalObjectIndex() const { return Data[0]; }
	};

	struct FPerConnectionInfo
	{
		UE::Net::FNetBitArray ReplicationEnabledObjects;
	};

	// UNetObjectFilter interface
	IRISCORE_API virtual void OnInit(const FNetObjectFilterInitParams&) override;
	IRISCORE_API virtual void OnDeinit() override;
	virtual void OnMaxInternalNetRefIndexIncreased(uint32 NewMaxInternalIndex) override {}
	IRISCORE_API virtual void AddConnection(uint32 ConnectionId) override;
	IRISCORE_API virtual void RemoveConnection(uint32 ConnectionId) override;
	IRISCORE_API virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectFilterAddObjectParams&) override;
	IRISCORE_API virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectFilteringInfo&) override;
	IRISCORE_API virtual void PreFilter(FNetObjectPreFilteringParams&) override;
	IRISCORE_API virtual void Filter(FNetObjectFilteringParams&) override;

	TStrongObjectPtr<UNetObjectConnectionFilterConfig> Config;
	TArray<UE::Net::FInternalNetRefIndex> LocalToNetRefIndex;
	TArray<FPerConnectionInfo> PerConnectionInfos;
	UE::Net::FNetBitArray UsedLocalInfoIndices;

	UE::Net::FInternalNetRefIndex MaxInternalIndex = 0;
	bool bObjectRemoved = false;
};
