// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/NetBitArray.h"
#include "Iris/Core/NetChunkedArray.h"
#include "Iris/ReplicationSystem/InternalNetRefIndexManager.h"
#include "Iris/ReplicationSystem/ReplicationView.h"
#include "UObject/ObjectMacros.h"
#include "NetObjectFilter.generated.h"

struct FNetObjectFilteringInfo;

class UReplicationSystem;

namespace UE::Net
{
	typedef uint32 FNetObjectFilterHandle;

	struct FReplicationInstanceProtocol;
	struct FReplicationProtocol;

	class FNetRefHandle;

	namespace Private
	{
		class FNetRefHandleManager;
	}
}

namespace UE::Net
{

constexpr FNetObjectFilterHandle InvalidNetObjectFilterHandle = FNetObjectFilterHandle(0);
constexpr FNetObjectFilterHandle ToOwnerFilterHandle = FNetObjectFilterHandle(1);
/** ConnectionFilterHandle is for internal use only. */
constexpr FNetObjectFilterHandle ConnectionFilterHandle = FNetObjectFilterHandle(2);

/** Used to control whether an object is allowed to be replicated or not. */
enum class ENetFilterStatus : uint32
{
	/** Do not allow replication. */
	Disallow,
	/** Allow replication. */
	Allow,
};

IRISCORE_API const TCHAR* LexToString(ENetFilterStatus Status);

} // end namespace UE::Net

/**
 * Parameters passed to UNetObjectFilter::Filter.
 */
struct FNetObjectFilteringParams
{
	/**
	 * The contents of OutAllowedObjects is undefined when passed to Filter(). The filter is responsible
	 * for setting and clearing bits for objects that have this filter set, which is provided in the
	 * FilteredObjects member. It's safe to set or clear all bits in the bitarray as the callee will
	 * only care about bits which the filter is responsible for.
	 */
	UE::Net::FNetBitArrayView OutAllowedObjects;

	/** FilteringInfos for all objects. Index using the set bit indices in FilteredObjects. */
	TArrayView<const FNetObjectFilteringInfo> FilteringInfos;

	/** State buffers for all objects. Index using the set bit indices in FilteredObjects. */
	const UE::Net::TNetChunkedArray<uint8*>* StateBuffers = nullptr;

	/** ID of the connection that the filtering applies to. */
	uint32 ConnectionId = 0;

	/** The view associated with the connection and its sub-connections that objects are filtered for. */
	UE::Net::FReplicationView View;

	/** List of objects that have been filtered out by groups for the ConnectionId */
	const UE::Net::FNetBitArrayView GroupFilteredOutObjects;

	/** Gives access to some Iris internal bitarrays exposing status flags for each netobject */
	const UE::Net::FInternalNetRefIndexManager& NetRefIndexManager;
};

/**
 * Parameters passed to UNetObjectFilter::PreFilter.
 */
struct FNetObjectPreFilteringParams
{
	// The IDs of all valid connections.
	UE::Net::FNetBitArrayView ValidConnections;

	/** FilteringInfos for all objects. Index using the set bit indices in FilteredObjects. */
	TArrayView<const FNetObjectFilteringInfo> FilteringInfos;

	/** Gives access to some Iris internal bitarrays exposing status flags for each netobject */
	const UE::Net::FInternalNetRefIndexManager& NetRefIndexManager;
};

/**
 * Parameters passed to UNetObjectFilter::PostFilter.
 */
struct FNetObjectPostFilteringParams
{
};

/**
 * Filter specific data stored per object, such as offsets to tags.
 * The data is initialized to zero by default.
 */
struct alignas(8) FNetObjectFilteringInfo
{
	uint16 Data[4];
};

enum class ENetFilterTraits : uint8
{
	None = 0x00,
	/** Set this trait for NetFilters that filter according to the WorldLocation of it's objects. */
	Spatial = 0x01,
	/** Set this trait so that UpdateObjects will be called on your NetFilter. Default is to not call the virtual */
	NeedsUpdate = 0x02,
};
ENUM_CLASS_FLAGS(ENetFilterTraits);

/**
 * Base class for filter specific configuration.
 * @see FNetObjectFilterDefinition
 */
UCLASS(Transient, MinimalAPI, config=Engine)
class UNetObjectFilterConfig : public UObject
{
	GENERATED_BODY()

public:
};

/** Parameters passed to the filter's Init() call. */
struct FNetObjectFilterInitParams
{
public:
	TObjectPtr<UReplicationSystem> ReplicationSystem = nullptr;
	/** Optional config as set in the FNetObjectFilterDefinition. */
	UNetObjectFilterConfig* Config = nullptr;
	/** The maximum number of replicated objects in the system. */
	uint32 AbsoluteMaxNetObjectCount = 0;
	/** The current maximum replicated objects referenced by an index (may grow at runtime). */
	UE::Net::FInternalNetRefIndex CurrentMaxInternalIndex = 0;
	/** The maximum number of connections in the system. */
	uint32 MaxConnectionCount = 0;
};

struct FNetObjectFilterAddObjectParams
{
	/** The info is zeroed before the AddObject() call. Fill in with filter specifics, like offsets to tags. */
	FNetObjectFilteringInfo& OutInfo;

	/** Name of a specialized configuration profile. When none, the default settings are expected. */
	FName ProfileName;

	/** The FReplicationInstanceProtocol which describes the source state data. */
	const UE::Net::FReplicationInstanceProtocol* InstanceProtocol;

	/** The FReplicationProtocol which describes the internal state data. */
	const UE::Net::FReplicationProtocol* Protocol;

	/** Gives access to some Iris internal bitarrays exposing status flags for each netobject */
	const UE::Net::FInternalNetRefIndexManager& NetRefIndexManager;

	/**
	 * One can retrieve relevant information from the object state buffer using the FReplicationProtocol.
	 * Note that this is the internal network representation of the data which is stored in quantized form.
	 * NetSerializers can dequantize the data to the original source data form.
	 */
	const uint8* StateBuffer;
};

/** Parameters passed to the filter's UpdateObjects() call. */
struct FNetObjectFilterUpdateParams
{
	//TODO: This should be a TConstArrayView
	/** Indices of the updated objects. */
	const UE::Net::FInternalNetRefIndex* ObjectIndices = nullptr;
	/** The number of objects that have been updated. */
	uint32 ObjectCount = 0;

	/** Infos for all objects. Index using ObjectIndices[0..ObjectCount-1]. */
	TArrayView<FNetObjectFilteringInfo> FilteringInfos;
};

UCLASS(Abstract, MinimalAPI)
class UNetObjectFilter : public UObject
{
	GENERATED_BODY()

public:
	IRISCORE_API void Init(const FNetObjectFilterInitParams& Params);
	IRISCORE_API void Deinit();

	IRISCORE_API void MaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex MaxInternalIndex, TArrayView<FNetObjectFilteringInfo> NewFilterInfoView);
	
	/** A new connection has been added. An opportunity for the filter to allocate per connection info. */
	IRISCORE_API virtual void AddConnection(uint32 ConnectionId);

	/** A new connection has been removed. An opportunity for the filter to deallocate per connection info. */
	IRISCORE_API virtual void RemoveConnection(uint32 ConnectionId);

	/** A new object want to use this filter. Opportunity to cache some information for it. The info struct passed has been zeroed. Must be overriden. */
	virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectFilterAddObjectParams&) PURE_VIRTUAL(AddObject, return false;)

	/** An object no longer wants to use this filter. */
	virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectFilteringInfo&) PURE_VIRTUAL(RemoveObject,)

	/** A set of objects using this filter are dirty since the last update. An opportunity for the filter to update cached data. */
	IRISCORE_API virtual void UpdateObjects(FNetObjectFilterUpdateParams&);

	/**
	 * If there are any connections being replicated and there's a chance Filter() will be called then PreFilter()
	 * will be called exactly once before all calls to Filter().
	 */
	IRISCORE_API virtual void PreFilter(FNetObjectPreFilteringParams&);

	/** Filter a batch of objects. There may be multiple calls to this function even for the same connection. Must be overriden. */
	IRISCORE_API virtual void Filter(FNetObjectFilteringParams&);

	/**
	 * If PreFilter() was called then PostFilter() will be called exactly once after all Filter() calls.
	 */
	IRISCORE_API virtual void PostFilter(FNetObjectPostFilteringParams&);

	/** Returns all the filter's traits. */
	ENetFilterTraits GetFilterTraits() const { return FilterTraits; }

	/** Tells if the filter was assigned a specific trait */
	bool HasFilterTrait(ENetFilterTraits FilterTrait) const { return EnumHasAnyFlags(FilterTraits, FilterTrait); }

	/** The list of objects that are filtered by this filter */
	UE::Net::FNetBitArrayView GetFilteredObjects() { return MakeNetBitArrayView(FilteredObjects); }


	struct FDebugInfoParams
	{
		FName FilterName;

		TArrayView<const FNetObjectFilteringInfo> FilteringInfos;

		/** ID of the connection that the filtering applies to. */
		uint32 ConnectionId = 0;

		/** The view associated with the connection and its sub-connections that objects are filtered for. */
		UE::Net::FReplicationView View;
	};
	virtual FString PrintDebugInfoForObject(const FDebugInfoParams& Params, uint32 ObjectIndex) const { return Params.FilterName.ToString(); };

protected:
	IRISCORE_API UNetObjectFilter();

	/** Called right after constructor for enabled filters. Must be overriden. */
	virtual void OnInit(const FNetObjectFilterInitParams&) PURE_VIRTUAL(OnInit, );

	/** Called when the replication system is shutting down. Use this to remove references to other systems */
	virtual void OnDeinit() PURE_VIRTUAL(OnDeinit);

	/** Called when the maximum InternalNetRefIndex increased and we need to realloc our lists */
	virtual void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex) PURE_VIRTUAL(OnMaxInternalNetRefIndexIncreased);

	/* Returns the filtering info for this object if it's handled by this filter, nullptr otherwise. */
	IRISCORE_API FNetObjectFilteringInfo* GetFilteringInfo(UE::Net::FInternalNetRefIndex ObjectIndex);

	/* Returns the internal net ref index for the given NetRefHandle */
	IRISCORE_API UE::Net::FInternalNetRefIndex GetObjectIndex(UE::Net::FNetRefHandle NetRefHandle) const;

	/* Returns true if the object is assigned to be filtered by this filter.*/
	inline bool IsObjectFiltered(UE::Net::FInternalNetRefIndex ObjectIndex) const;

	/** Adds traits. */
	inline void AddFilterTraits(ENetFilterTraits Traits);

	/** Sets the traits specified by TraitsMask to Traits. */
	inline void SetFilterTraits(ENetFilterTraits Traits, ENetFilterTraits TraitsMask);

	/** Returns the public facade over Iris-internal data. */
	inline UE::Net::FInternalNetRefIndexManager GetNetRefIndexManager() const;

	inline const UE::Net::FNetBitArrayView GetFilteredObjects() const;

private:
	ENetFilterTraits FilterTraits = ENetFilterTraits::None;

	/** The indices of the objects that have this filter set. The indices of set bits correspond to the object indices. */
	UE::Net::FNetBitArray FilteredObjects;

	TArrayView<FNetObjectFilteringInfo> FilteringInfos;

	const UE::Net::Private::FNetRefHandleManager* NetRefHandleManager = nullptr;
};

inline bool UNetObjectFilter::IsObjectFiltered(UE::Net::FInternalNetRefIndex ObjectIndex) const
{
	return ObjectIndex < FilteredObjects.GetNumBits() && FilteredObjects.IsBitSet(ObjectIndex);
}

inline void UNetObjectFilter::AddFilterTraits(ENetFilterTraits Traits)
{
	FilterTraits |= Traits;
}

inline void UNetObjectFilter::SetFilterTraits(ENetFilterTraits Traits, ENetFilterTraits TraitsMask)
{
	const ENetFilterTraits NewFilterTraits = (FilterTraits & ~TraitsMask) | (Traits & TraitsMask);
	FilterTraits = NewFilterTraits;
}

inline UE::Net::FInternalNetRefIndexManager UNetObjectFilter::GetNetRefIndexManager() const
{
	checkSlow(NetRefHandleManager != nullptr);
	return UE::Net::FInternalNetRefIndexManager(*NetRefHandleManager);
}

const UE::Net::FNetBitArrayView UNetObjectFilter::GetFilteredObjects() const
{
	return UE::Net::MakeNetBitArrayView(FilteredObjects);
}
