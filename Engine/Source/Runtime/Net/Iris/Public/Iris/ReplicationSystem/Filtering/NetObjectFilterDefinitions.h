// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "UObject/ObjectMacros.h"
#include "NetObjectFilterDefinitions.generated.h"

USTRUCT()
struct FNetObjectFilterDefinition
{
	GENERATED_BODY()

	/** Filter identifier. Used to uniquely identify a filter. */
	UPROPERTY(EditAnywhere, Category="Filter")
	FName FilterName;

	/**
	 * UClass name, specified by its fully qualified path, used to create the UNetObjectFilter. You can have multiple instances
	 * of the same filter as long as their FilterNames are unique.
	 */
	UPROPERTY(EditAnywhere, Category="Filter")
	FName ClassName;		

	/**
	 * Optional UClass name, specified by its fully qualified path, used to create the UNetObjectFilterConfig. The class default instance
	 * will be passed at filter initialization time. If you want multiple instances of the same
	 * filter then use subclassing to create unique filter configs.
	 */
	UPROPERTY(EditAnywhere, Category="Filter")
	FName ConfigClassName;		
};

/** Configurable filter definitions. Valid filter definitions are auto-created by the filter system. */
UCLASS(transient, config=Engine)
class UNetObjectFilterDefinitions final : public UObject
{
	GENERATED_BODY()

public:
	/** Returns the filter definitions exactly as configured. May contain invalid definitions, such as ClassNames not inheriting from UNetObjectFilter. */
	IRISCORE_API TConstArrayView<FNetObjectFilterDefinition> GetFilterDefinitions() const;

private:
	UPROPERTY(Config)
	TArray<FNetObjectFilterDefinition> NetObjectFilterDefinitions;
};
