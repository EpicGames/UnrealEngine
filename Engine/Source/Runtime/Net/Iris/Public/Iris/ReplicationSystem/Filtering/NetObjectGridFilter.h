// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ChunkedArray.h"

#include "Iris/IrisConfig.h"
#include "Iris/ReplicationSystem/InternalNetRefIndexManager.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"

#include "Net/Core/NetBitArray.h"

#include "Math/Vector.h"

#include "UObject/StrongObjectPtr.h"

#include "NetObjectGridFilter.generated.h"

namespace UE::Net
{
	class FNetCullDistanceOverrides;
	class FWorldLocations;
	struct FRepTagFindInfo;
}

/**
 * Specialized template that configures unique properties.
 * Useful when you need to specialize a behavior per class or object type
 */
USTRUCT()
struct FNetObjectGridFilterProfile
{
	GENERATED_BODY()

	/** The config name used to map to this profile */
	UPROPERTY()
	FName FilterProfileName;

	/** Number of frames we keep the object relevant until it is officially culled out.*/
	UPROPERTY()
	uint16 FrameCountBeforeCulling = 4;

	bool operator==(FName Key) const { return FilterProfileName == Key; }
};

/**
 * Common settings used to configure how the GridFilter behaves
 */
UCLASS(transient, config=Engine, MinimalAPI)
class UNetObjectGridFilterConfig : public UNetObjectFilterConfig
{
	GENERATED_BODY()

public:
	/** 
	 * How many frames a previous grid cell should continue to be considered relevant. To avoid culling issues when player borders cells. 
	 * Only used when bUseExactCullDistance is false.
	 */
	UPROPERTY(Config)
	uint32 ViewPosRelevancyFrameCount = 2;

	UPROPERTY(Config)
	uint16 DefaultFrameCountBeforeCulling = 4;

	UPROPERTY(Config)
	float CellSizeX = 20000.0f;

	UPROPERTY(Config)
	float CellSizeY = 20000.0f;

	/** Objects without a NetCullDistanceSquared property will assume to have this value but squared unless there's a cull distance override. */
	UPROPERTY(Config)
	float DefaultCullDistance = 15000.0f;

	/** 
	 * If true: use the exact distance between an object and the viewer to determine if the object is relevant or should be culled out.
	 * When false: consider all objects within a grid cell to be relevant when a viewer is located within the cell. This can extend the relevant distance of objects beyond their cull distance. 
	 */
	UPROPERTY(Config)
	bool bUseExactCullDistance = true;

	/** Map of specialized configuration profiles */
	UPROPERTY(Config)
	TArray<FNetObjectGridFilterProfile> FilterProfiles;
};

UCLASS(abstract)
class UNetObjectGridFilter : public UNetObjectFilter
{
	GENERATED_BODY()

protected:
	// UNetObjectFilter interface
	IRISCORE_API virtual void OnInit(const FNetObjectFilterInitParams&) override;
	IRISCORE_API virtual void OnDeinit() override;
	IRISCORE_API virtual void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex) override;
	IRISCORE_API virtual void AddConnection(uint32 ConnectionId) override;
	IRISCORE_API virtual void RemoveConnection(uint32 ConnectionId) override;
	IRISCORE_API virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectNetIndex, FNetObjectFilterAddObjectParams&) override;
	IRISCORE_API virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectNetIndex, const FNetObjectFilteringInfo&) override;
	IRISCORE_API virtual void PreFilter(FNetObjectPreFilteringParams&) override;
	IRISCORE_API virtual void Filter(FNetObjectFilteringParams&) override;
	IRISCORE_API virtual void PostFilter(FNetObjectPostFilteringParams&) override;
	IRISCORE_API virtual FString PrintDebugInfoForObject(const FDebugInfoParams& Params, UE::Net::FInternalNetRefIndex ObjectNetIndex) const override;

protected:
	struct FObjectLocationInfo : public FNetObjectFilteringInfo
	{
		/**
		* Data mapping:
		* uint16 Data[0] = Not used
		* uint16 Data[1] = Not used
		* uint16 Data[2] = FPerObjectInfo index (low bytes)
		* uint16 Data[3] = FPerObjectInfo index (high bytes)
		*/
		void SetInfoIndex(uint32 Index)	{ Data[2] = Index & 65535U; Data[3] = Index >> 16U; }
		uint32 GetInfoIndex() const		{ return (uint32(Data[3]) << 16U) | uint32(Data[2]); }
	};

	IRISCORE_API bool AddCellInfoForObject(const FObjectLocationInfo& ObjectInfo, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol);
	IRISCORE_API void RemoveCellInfoForObject(const FObjectLocationInfo& ObjectInfo);
	IRISCORE_API void UpdateCellInfoForObject(const FObjectLocationInfo& ObjectInfo, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol);

protected:

	enum : unsigned
	{
		ObjectInfosChunkSize = 64 * 1024,
	};

	struct FCellBox
	{
		int32 MinX = 0;
		int32 MaxX = -1;
		int32 MinY = 0;
		int32 MaxY = -1;

		bool operator==(const FCellBox&) const;
		bool operator!=(const FCellBox&) const;
	};

	// We can't fit all info we need in 4x16bits.
	struct FPerObjectInfo
	{
		FVector Position = FVector::ZeroVector;
		FCellBox CellBox;
		UE::Net::FInternalNetRefIndex ObjectIndex = UE::Net::InvalidInternalNetRefIndex;
		uint16 FrameCountBeforeCulling = 0U;
		//unused 16bits of padding

		float GetCullDistance() const
		{
			return CullDistance;
		}

		float GetCullDistanceSq() const
		{
			return CullDistanceSq;
		}

		void SetCullDistance(float Distance)
		{
			CullDistance = Distance;
			CullDistanceSq = Distance * Distance;
		}

		void SetCullDistanceSq(float DistanceSq)
		{
			CullDistance = FPlatformMath::Sqrt(DistanceSq);
			CullDistanceSq = DistanceSq;
		}

	private:
		float CullDistance = 0.0f;
		float CullDistanceSq = 0.0f;
	};

	/** Sets the current position of the object based on how we access it's given location. */
	virtual void UpdateObjectInfo(FPerObjectInfo& PerObjectInfo, const FObjectLocationInfo& ObjectLocationInfo, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol) {}

	/** Build the data needed to properly filter this object. */
	virtual bool BuildObjectInfo(UE::Net::FInternalNetRefIndex ObjectNetIndex, FNetObjectFilterAddObjectParams& Params) { return false; }

	/** Callback when an object is removed from the grid. Useful to cleanup data tied to the object. */
	virtual void OnObjectRemoved(UE::Net::FInternalNetRefIndex ObjectNetIndex) {}

private:

	struct FCellCoord
	{
		int32 X;
		int32 Y;

	private:
		friend bool operator==(const FCellCoord& A, const FCellCoord& B)
		{
			return (A.X == B.X) & (A.Y == B.Y);
		}

		friend uint32 GetTypeHash(const FCellCoord& Coords)
		{
			return ::GetTypeHash((uint64(uint32(Coords.X)) << 32U) | uint64(uint32(Coords.Y)));
		}
	};

	struct FCellObjects
	{
		TSet<UE::Net::FInternalNetRefIndex> ObjectIndices;
	};

	struct FCellAndTimestamp
	{
		FCellCoord Cell;
		uint32 Timestamp;

	private:
		friend bool operator==(const FCellAndTimestamp& A, const FCellAndTimestamp& B)
		{
			return (A.Cell == B.Cell) & (A.Timestamp == B.Timestamp);
		}
	};

	struct FPerConnectionInfo
	{
		// We don't expect a lot of view positions from a single connection
		TArray<FCellAndTimestamp, TInlineAllocator<32>> RecentCells;
		
		// Objects that have been recently visible to the connection and the frame countdown.
		TMap<UE::Net::FInternalNetRefIndex, uint16> RecentObjectFrameCount;
	};

	/** Aggregator for stats */
	struct FNetGridFilterStats
	{
		FNetGridFilterStats() = default;

		void Reset() { *this = FNetGridFilterStats(); }
		
		/** GridFilter stats */
		uint64 CullTestingTimeInCycles = 0;
		uint32 CullTestedObjects = 0;
	};

private:
	uint32 AllocObjectInfo();
	void FreeObjectInfo(uint32 Index);

	void UpdatePositionAndCullDistance(const FObjectLocationInfo& ObjectLocationInfo, FPerObjectInfo& PerObjectInfo, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol);
	void CalculateCellBox(const FPerObjectInfo& PerObjectInfo, FCellBox& OutCellBox);
	void CalculateCellCoord(FCellCoord& OutCoord, const FVector& Pos);

	uint16 GetFrameCountBeforeCulling(FName ProfileName) const;

	static bool AreCellsDisjoint(const FCellBox& A, const FCellBox& B);
	static bool DoesCellContainCoord(const FCellBox& Cell, const FCellCoord& Coord);

	TStrongObjectPtr<UNetObjectGridFilterConfig> Config;
	TArray<FPerConnectionInfo> PerConnectionInfos;
	TChunkedArray<FPerObjectInfo, ObjectInfosChunkSize> ObjectInfos;
	UE::Net::FNetBitArray AssignedObjectInfoIndices;

#if UE_NET_IRIS_CSV_STATS
	FNetGridFilterStats Stats;
#endif

	TMap<FCellCoord, FCellObjects> Cells;
	uint32 FrameIndex = 0;

	struct FDebugUpdateCellInfo
	{
		FCellCoord Coord;
		FCellBox NewCellBox;
		FCellBox PrevCellBox;
		SIZE_T CellsSize;
		FVector ObjectPosition;
		float ObjectCullDistance;
	};

	// Cached state from UpdateCellInfoForObject().
	FDebugUpdateCellInfo DebugUpdateCellInfo;

	/** World boundaries (min and max position). */
	FVector MinWorldPos;
	FVector MaxWorldPos;
	float MaxNetCullDistance = 0.f;
};

/**
 * Filter for replicated objects that have a WorldLocation reference (e.g. Actors).
 * 
 * This filter is more efficient since it's run before Polling and culls out objects that are not relevant to any connection.
 */
UCLASS(transient, MinimalAPI)
class UNetObjectGridWorldLocFilter : public UNetObjectGridFilter
{
	GENERATED_BODY()

protected:
	IRISCORE_API virtual void OnInit(const FNetObjectFilterInitParams&) override;
	IRISCORE_API virtual void OnDeinit() override;
	IRISCORE_API virtual void PreFilter(FNetObjectPreFilteringParams&) override;
	IRISCORE_API virtual void UpdateObjectInfo(FPerObjectInfo& PerObjectInfo, const UNetObjectGridFilter::FObjectLocationInfo& ObjectLocationInfo, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol) override;
	IRISCORE_API virtual bool BuildObjectInfo(uint32 ObjectIndex, FNetObjectFilterAddObjectParams& Params) override;

private:

	const UE::Net::FWorldLocations* WorldLocations = nullptr;
};


//
inline bool UNetObjectGridFilter::FCellBox::operator==(const UNetObjectGridFilter::FCellBox& Other) const
{
	return (MinX == Other.MinX) & (MinY == Other.MinY) & (MaxX == Other.MaxX) & (MaxY == Other.MaxY);
}

inline bool UNetObjectGridFilter::FCellBox::operator!=(const UNetObjectGridFilter::FCellBox& Other) const
{
	return (MinX != Other.MinX) | (MinY != Other.MinY) | (MaxX != Other.MaxX) | (MaxY != Other.MaxY);
}
