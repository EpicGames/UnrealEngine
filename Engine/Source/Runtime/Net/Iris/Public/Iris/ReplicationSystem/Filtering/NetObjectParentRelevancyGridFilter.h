// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ChunkedArray.h"
#include "Containers/ContainerAllocationPolicies.h"

#include "Iris/ReplicationSystem/Filtering/NetObjectGridFilter.h"

#include "Net/Core/NetBitArray.h"

#include "NetObjectParentRelevancyGridFilter.generated.h"

/**
 * Configuration for UNetObjectParentRelevancyGridFilter.
 */
UCLASS(transient, config=Engine, MinimalAPI)
class UNetObjectParentRelevancyGridFilterConfig : public UNetObjectGridFilterConfig
{
	GENERATED_BODY()

public:
	/** When > 0 the filter will ensure and stop evaluating parents once it hits a chain of parents that is considered too deep. */
	UPROPERTY(Config)
	uint32 MaxParentChainDepth = 0U;
};

/**
 * Grid filter that forces relevant the creation dependency parents of the objects it manages. Those parents must also be managed by this filter to be promoted.
 */
UCLASS(transient, MinimalAPI)
class UNetObjectParentRelevancyGridFilter : public UNetObjectGridWorldLocFilter
{
	GENERATED_BODY()

protected:
	IRISCORE_API virtual void OnInit(const FNetObjectFilterInitParams& Params) override;
	IRISCORE_API virtual void OnDeinit() override;
	IRISCORE_API virtual void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex) override;
	IRISCORE_API virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectNetIndex, FNetObjectFilterAddObjectParams& Params) override;
	IRISCORE_API virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectNetIndex, const FNetObjectFilteringInfo& Info) override;
	IRISCORE_API virtual void PreFilter(FNetObjectPreFilteringParams& Params) override;
	IRISCORE_API virtual void Filter(FNetObjectFilteringParams& Params) override;
	IRISCORE_API virtual FString PrintDebugInfoForObject(const FDebugInfoParams& Params, UE::Net::FInternalNetRefIndex ObjectNetIndex) const override;

private:
	/**
	 * Per-object parent chain. The first NumImmediateParents entries are the immediate creation-dependency parents; the rest are the transitive ancestors.
	 * A linear chain means that every object in the chain only had one or zero parent. When true we can skip evaluating the chain once we hit a visited parent.
	 */
	struct FParentChain
	{
		TArray<UE::Net::FInternalNetRefIndex> ParentNetIndexes;
		uint8 NumImmediateParents = 0;
		bool bIsLinearChain:1 = false;
	};

	// Walks the creation-dependency graph of the replicated object and fills OutChain.
	void BuildParentChain(UE::Net::FInternalNetRefIndex ChildNetIndex, FParentChain& OutChain, const UE::Net::FInternalNetRefIndexManager& NetRefIndexManager, const UE::Net::FNetBitArrayView ObjectsInFilter) const;

	enum : unsigned
	{
		ParentChainsChunkSize = 64 * 1024,
	};
	/** Per-object parent chain, indexed by FObjectLocationInfo's info index. */
	TChunkedArray<FParentChain, ParentChainsChunkSize> ParentChains;

	/** Bitarray used for temporary operations in PreFilter() and Filter(). Should never be accessed outside those functions */
	UE::Net::FNetBitArray ScratchList;

	int32 MaxParentChainDepth = 0;
};
