// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Math/UnrealMathUtility.h"
#include "ReplicationFilteringConfig.generated.h"

USTRUCT()
struct FObjectScopeHysteresisProfile
{
	GENERATED_BODY()

public:
	bool operator==(FName Name) const
	{
		return FilterProfileName == Name;
	}

	/** The config name used to map to this profile */
	UPROPERTY()
	FName FilterProfileName;

	/** The number of frames to keep the object in scope after it has been filtered out by dynamic filtering. */
	UPROPERTY()
	uint8 HysteresisFrameCount = 0;
};

/**
 * Object scope hysteresis support. Keep dynamically filtered out objects around for a specified amount of frames. 
 * Configure behavior via hysteresis profiles that determine the frame timeout per class.
 * The filter config for a specific class can then mention the hysteresis profile in order to get the appropriate behavior. 
 *
 * Example:
 * [/Script/IrisCore.ReplicationFilteringConfig]
 * bEnableObjectScopeHysteresis=true
 * DefaultHysteresisFrameCount=4
 * HysteresisUpdateConnectionThrottling=4
 * !HysteresisProfiles=ClearArray
 * +FilterProfiles=(FilterProfileName=PawnFilterProfile, HysteresisFrameCount=30)
 * 
 * [/Script/ IrisCore.ObjectReplicationBridgeConfig]
 * +FilterConfigs=(ClassName=/Script/Engine.Pawn, DynamicFilterName=Spatial, FilterProfile=PawnFilterProfile)
 */

UCLASS(transient, config = Engine)
class UReplicationFilteringConfig final : public UObject
{
	GENERATED_BODY()

public:
	bool IsObjectScopeHysteresisEnabled() const;
	uint8 GetDefaultHysteresisFrameCount() const;
	uint8 GetHysteresisUpdateConnectionThrottling() const;

	const TArray<FObjectScopeHysteresisProfile>& GetHysteresisProfiles() const;

private:
	UPROPERTY(Config)
	/** If enabled a dynamically filtered out object will not be considered out of scope for a particular number of frames. */
	bool bEnableObjectScopeHysteresis = true;

	UPROPERTY(Config)
	/** How many frames a dynamically filtered out object should still be considered in scope by default. Can be overridden with HysteresisClassConfigs. */ 
	uint8 DefaultHysteresisFrameCount = 0;

	UPROPERTY(Config)
	/**
	 * Update every Nth connection each frame. If 1 then every connection will be updated every frame, if 2 then half of the connections will be updated per frame and so on.
	 * Keep this number low. The value will be clamped to 128. Due to the nature of the throttling objects may linger for N-1 extra frames before considered out of scope.
	 */
	uint8 HysteresisUpdateConnectionThrottling = 1;

	/** Specialized configuration profiles */
	UPROPERTY(Config)
	TArray<FObjectScopeHysteresisProfile> HysteresisProfiles;
};

inline bool UReplicationFilteringConfig::IsObjectScopeHysteresisEnabled() const
{
	return bEnableObjectScopeHysteresis;
}

inline uint8 UReplicationFilteringConfig::GetDefaultHysteresisFrameCount() const
{
	return DefaultHysteresisFrameCount;
}

inline uint8 UReplicationFilteringConfig::GetHysteresisUpdateConnectionThrottling() const
{
	return FMath::Clamp<uint8>(HysteresisUpdateConnectionThrottling, 1U, 128U);
}

inline const TArray<FObjectScopeHysteresisProfile>& UReplicationFilteringConfig::GetHysteresisProfiles() const
{
	return HysteresisProfiles;
}
