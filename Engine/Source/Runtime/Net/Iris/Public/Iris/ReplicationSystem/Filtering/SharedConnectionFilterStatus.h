// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Iris/IrisConstants.h"
#include "Iris/ReplicationSystem/ReplicationView.h" // for UE_IRIS_INLINE_VIEWS_PER_CONNECTION
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Net/Core/Connection/ConnectionHandle.h"
#include "Containers/Map.h"
#include "Containers/Set.h"

namespace UE::Net
{

/**
 * Keeps track of the filter status for a connection and its child connections. All FConnectionHandles must have the same ParentConnection. As soon as the filter status is set for a FConnectionHandle all subsequent operations must have the same FConnectionHandle.
 * The default state is that the filter status is Disallowed. If there's at least one connection that has set the filter status to Allowed then will the filter status for the group be Allowed.
 */
class FSharedConnectionFilterStatus
{
public:
	/** Returns true if the filter status could be successfully set for the ConnectionHandle. A return value of false indicates either the connection handle is invalid or its ParentConnectionId doesn't match prior calls to SetFilterStatus. */
	IRISCORE_API bool SetFilterStatus(FConnectionHandle ConnectionHandle, ENetFilterStatus FilterStatus);
	
	/** Returns Disallowed if no connections have set the filter status to Allowed and returns Allowed otherwise. */
	ENetFilterStatus GetFilterStatus() const;
	
	/** Removes the filter status for ConnectionHandle. If it's a parent connection it will act as if all child connections were removed too and allow the instance to be used with a different ParentConnectionId. */
	IRISCORE_API void RemoveConnection(FConnectionHandle ConnectionHandle);

	/** Returns the parent connection ID the group operates on or InvalidConnectionId if no valid connection handles ever set filter status. */
	uint32 GetParentConnectionId() const;

private:
	// Only keep track of connections which allow replication as the replication status is Disallow by default.
	TSet<uint32, DefaultKeyFuncs<uint32>, TInlineSetAllocator<UE_IRIS_INLINE_VIEWS_PER_CONNECTION>> AllowConnections;
	uint32 ParentConnectionId = InvalidConnectionId;
};

/** Keeps track of the filter status for multiple connections and their child connections. A FConnectionHandleFilterGroup is stored per unique ParentConnectionId calling SetFilterStatus. */
class FSharedConnectionFilterStatusCollection
{
public:
	IRISCORE_API void SetFilterStatus(FConnectionHandle ConnectionHandle, ENetFilterStatus FilterStatus);

	/** Returns Disallowed if no connections with the supplied ParentConnectionId have set the filter status to Allowed. If at least one connection with the ParentConnectionId allows replication then Allowed is returned. */
	IRISCORE_API ENetFilterStatus GetFilterStatus(uint32 ParentConnectionId) const;

	/** Remove a connection from filter status records. If it's a parent connection the corresponding FConnectionHandleFilterGroup will be removed altogether. */
	IRISCORE_API void RemoveConnection(FConnectionHandle ConnectionHandle);

private:
	FSharedConnectionFilterStatus* FindSharedConnectionFilterStatus(uint32 ParentConnectionId);
	const FSharedConnectionFilterStatus* FindSharedConnectionFilterStatus(uint32 ParentConnectionId) const;
	FSharedConnectionFilterStatus& FindOrAddSharedConnectionFilterStatus(uint32 ParentConnectionId);

	TMap<uint32, FSharedConnectionFilterStatus> ParentToFilterStatus;
};

inline ENetFilterStatus FSharedConnectionFilterStatus::GetFilterStatus() const
{
	return AllowConnections.IsEmpty() ? ENetFilterStatus::Disallow : ENetFilterStatus::Allow;
}

inline uint32 FSharedConnectionFilterStatus::GetParentConnectionId() const
{
	return ParentConnectionId;
}

}
