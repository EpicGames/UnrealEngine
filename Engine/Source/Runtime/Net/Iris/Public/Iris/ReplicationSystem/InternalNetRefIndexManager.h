// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "HAL/Platform.h"

namespace UE::Net
{
	class FNetBitArrayView;
}

namespace UE::Net::Private
{
	class FNetRefHandleManager;
}

namespace UE::Net
{
	/** Internal type to visualize indexes mapping a NetRefHandle to their position in internal NetBitArrays. */
	typedef uint32 FInternalNetRefIndex;

	/** The invalid index is always the first index since it will never be assigned. */
	constexpr uint32 InvalidInternalNetRefIndex = 0U;
}

namespace UE::Net
{

/** Returns true if the NetIndex is not the default invalid value */
static bool IsValidInternalNetRefIndex(const FInternalNetRefIndex NetIndex)
{
	return NetIndex != InvalidInternalNetRefIndex;
}

/**
 * This class is a facade over the NetRefHandleManager to allow public access to some internal Iris netbitarrays.
 */
class FInternalNetRefIndexManager
{
public:

	FInternalNetRefIndexManager() = delete;
	FInternalNetRefIndexManager(const UE::Net::Private::FNetRefHandleManager& InNetRefHandleManager);

	/**
	 * Returns the creation-dependency parents of the replicated object.
	 * Empty view if the object has none.
	 */
	IRISCORE_API TConstArrayView<FInternalNetRefIndex> GetCreationDependencies(FInternalNetRefIndex ObjectIndex) const;

	/**
	 * Returns the creation-dependent children of the replicated object.
	 * Empty view if the object has none.
	 */
	IRISCORE_API TConstArrayView<FInternalNetRefIndex> GetCreationDependents(FInternalNetRefIndex ParentIndex) const;

	/** Returns the list of replicated objects that have creation dependencies. */
	IRISCORE_API const FNetBitArrayView GetObjectsWithCreationDependencies() const;

	/** Returns the list of replicated objects whose creation dependencies were mutated this frame. Cleared at end of PreSendUpdate. */
	IRISCORE_API const FNetBitArrayView GetObjectsWithDirtyCreationDependencies() const;

	/** Returns the readable debug info identifying the replicated object. Useful for log, ensure, etc. */
	IRISCORE_API FString PrintObjectFromIndex(FInternalNetRefIndex ObjectIndex) const;

private:

	const UE::Net::Private::FNetRefHandleManager& NetRefHandleManager;
};

} // end namespace UE::Net