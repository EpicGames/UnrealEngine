// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"

class UClass;
class UObject;

#ifndef UE_NET_ENABLE_IRISCREATIONFLOWLOG
#	define UE_NET_ENABLE_IRISCREATIONFLOWLOG 0
#endif // UE_NET_ENABLE_IRISCREATIONFLOWLOG

#if UE_NET_ENABLE_IRISCREATIONFLOWLOG
	/** bIsRootObject is evaluated first and short-circuits Condition */
#	define UE_CLOGF_IRISCREATIONFLOW(bIsRootObject, Condition, Verbosity, Format, ...) UE_CLOGF((bIsRootObject) && (Condition), LogIrisCreationFlow, Verbosity, Format, ##__VA_ARGS__)
#else
#	define UE_CLOGF_IRISCREATIONFLOW(...)
#endif // UE_NET_ENABLE_IRISCREATIONFLOWLOG

#if UE_NET_ENABLE_IRISCREATIONFLOWLOG

IRISCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogIrisCreationFlow, Log, All);

namespace UE::Net::CreationFlowLog
{
	/** Returns true if Class or any of its parents matches the configured filter list. */
	IRISCORE_API bool IsTracedClass(const UClass* Class);

	/** Returns true if Instance's Class is traced for TargetConnectionId */
	IRISCORE_API bool ShouldEmitForConnection(const UObject* Instance, uint32 OwningConnectionId, uint32 TargetConnectionId);

	/** Drop all cached match results so future lookups re-resolve against the current config. */
	IRISCORE_API void ClearCache();

	/** Register the PostGarbageCollect prune hook. Called from FIrisCoreModule::StartupModule. */
	IRISCORE_API void Init();

	/** Unregister the PostGarbageCollect prune hook. Called from FIrisCoreModule::ShutdownModule. */
	IRISCORE_API void Shutdown();

} // namespace UE::Net::CreationFlowLog

#endif // UE_NET_ENABLE_IRISCREATIONFLOWLOG
