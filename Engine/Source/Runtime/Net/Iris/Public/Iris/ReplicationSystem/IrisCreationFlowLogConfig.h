// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TopLevelAssetPath.h"

#include "IrisCreationFlowLogConfig.generated.h"

USTRUCT()
struct FCreationFlowLogClassConfig
{
	GENERATED_BODY()

	/** Fully qualified class path (e.g. /Script/Engine.PlayerState) */
	UPROPERTY()
	FTopLevelAssetPath ClassPath;

	/** Whether to include subclasses of ClassPath */
	UPROPERTY()
	bool bIncludeSubclasses = true;

	/** If true, only trace this class when replicated to its owning connection. */
	UPROPERTY()
	bool bTraceOwningConnectionOnly = true;
};

UCLASS(transient, config=Engine)
class UIrisCreationFlowLogConfig final : public UObject
{
	GENERATED_BODY()

public:
	IRISCORE_API static const UIrisCreationFlowLogConfig* GetConfig();
	TConstArrayView<FCreationFlowLogClassConfig> GetClassFilters() const;

	virtual void PostReloadConfig(class FProperty* PropertyToLoad) override;

private:
	/** Classes whose replication should emit LogIrisCreationFlow events. */
	UPROPERTY(Config)
	TArray<FCreationFlowLogClassConfig> ClassFilters;
};
