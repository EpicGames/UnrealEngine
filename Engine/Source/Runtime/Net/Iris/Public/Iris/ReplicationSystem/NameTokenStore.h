// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Iris/IrisConfig.h"
#include "Iris/ReplicationSystem/NetTokenStore.h"

namespace UE::Net
{

class FNameTokenStore : public FNetTokenDataStore
{
	UE_NONCOPYABLE(FNameTokenStore);
public:
	IRISCORE_API explicit FNameTokenStore(FNetTokenStore& TokenStore);

	// Create a NetToken for the provided name
	IRISCORE_API FNetToken GetOrCreateToken(FName Name);

	// Resolve NetToken, to resolve remote tokens RemoteTokenStoreState must be valid
	IRISCORE_API FName ResolveToken(FNetToken Token, const FNetTokenStoreState* RemoteTokenStoreState = nullptr) const;	

	static FName GetTokenStoreName()
	{ 
		return NameTokenStoreName;
	}

protected:
	// Serialize data for a token, note there is not validation in this function
	IRISCORE_API virtual void WriteTokenData(FNetSerializationContext& Context, FNetTokenStoreKey TokenStoreKey) const override;
	IRISCORE_API virtual void WriteTokenData(FArchive& Archive, FNetTokenStoreKey TokenStoreKey, UPackageMap* Map = nullptr) const override;

	// Read data for a token, returns a valid StoreKey if successful read
	IRISCORE_API virtual FNetTokenStoreKey ReadTokenData(FNetSerializationContext& Context, const FNetToken& NetToken) override;
	IRISCORE_API virtual FNetTokenStoreKey ReadTokenData(FArchive& Archive, const FNetToken& NetToken, UPackageMap* Map = nullptr) override;

	// Create and store data for Token
	FNetTokenStoreKey GetOrCreateTokenStoreKey(FName Name);

private:

	inline static FName NameTokenStoreName = "NameTokenStore";

#if UE_SUPPORT_PARALLEL_IRIS
	//Used to protect parallel access to FNameToKey and StoredFNames. Can be called within a FScopedNetTokenStoreAccess scope, but code that already locks this mutex should not lock TokenStoreAccess afterwards (avoid lock order inversion)
	mutable FTransactionallySafeRWLock NameTokenStoreMutex;
#endif // UE_SUPPORT_PARALLEL_IRIS

	TMap<FName, FNetTokenStoreKey> FNameToKey;
	TArray<FName> StoredFNames;
};

}
