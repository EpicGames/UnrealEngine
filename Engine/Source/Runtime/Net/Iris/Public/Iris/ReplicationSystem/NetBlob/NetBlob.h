// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "HAL/UnrealMemory.h"
#include "Containers/ArrayView.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"

namespace UE::Net
{
	class FNetBlob;
	class FNetSerializationContext;
	struct FReplicationStateDescriptor;
	class FNetReferenceCollector;
	namespace Private
	{
		class FNetBlobManager;
	}
}

namespace UE::Net
{

/** Flags that can be set at blob creation time. */
enum class ENetBlobFlags : uint32
{
	None = 0,

	/** The blob should be delivered reliably in order with respect to other reliable blobs. Implies Ordered. */
	Reliable = 1U << 0U,

	/** Used for FRawDataNetBlob derived classes to avoid duplicate serialization when splitting large blob. */
	RawDataNetBlob = Reliable << 1U,

	/** Used to indicate that this blob have ObjectReferences or NetTokens that might have to be exported. */
	HasExports = RawDataNetBlob << 1U,

	/** The blob should respect delivery order with respect to other Ordered blobs, including Reliable ones. Unreliable ordered blobs will only be sent once. */
	Ordered = HasExports << 1U,
};
ENUM_CLASS_FLAGS(ENetBlobFlags);

typedef uint32 FNetBlobType;

/** Denotes an invalid NetBlob type. */
constexpr FNetBlobType InvalidNetBlobType = ~FNetBlobType(0);

/**
 * Information typically passed to a FNetBlob at creation time.
 * Flags will not be serialized unless the FNetBlob derived class chooses to do so.
 */
struct FNetBlobCreationInfo
{
	FNetBlobType Type = InvalidNetBlobType;
	ENetBlobFlags Flags = ENetBlobFlags::None;
};

class FNetBlob
{
public:
	struct FQuantizedBlobState
	{
		enum class EMemoryAllocationFlags : uint32
		{
			None = 0,
			// If the state is constructed as protectable one can call Protect() to detect memory stomps.
			Protectable = 1,
		};

		FQuantizedBlobState() = default;
		FQuantizedBlobState(const FQuantizedBlobState&) = delete;
		FQuantizedBlobState(FQuantizedBlobState&&);
		IRISCORE_API FQuantizedBlobState(uint32 Size, uint32 Alignment, EMemoryAllocationFlags MemoryAllocationFlags = EMemoryAllocationFlags::None);
		IRISCORE_API ~FQuantizedBlobState();

		FQuantizedBlobState& operator=(const FQuantizedBlobState&) = delete;
		FQuantizedBlobState& operator=(FQuantizedBlobState&&);

		uint8* GetStateBuffer() { return StateBuffer; }
		const uint8* GetStateBuffer() const { return StateBuffer; }

		/** Protects the state buffer such that a page fault will occur if something tries to modify its contents. The state must have been constructed with EMemoryAllocationFlags::Protectable. */
		IRISCORE_API void Protect();

		/** Allows the state buffer to be modified again if it was protected. */
		IRISCORE_API void Unprotect();

	private:
		uint8* StateBuffer = nullptr;
		EMemoryAllocationFlags MemoryAllocationFlags = EMemoryAllocationFlags::None;
		uint32 AllocationSize = 0;
	};

	/** Construct a NetBlob with reference count zero. */
	IRISCORE_API FNetBlob(const FNetBlobCreationInfo&);

	/** Set the blob state. Use when there's a descriptor to avoid having to override serialization functions. */
	IRISCORE_API void SetState(const TRefCountPtr<const FReplicationStateDescriptor>& BlobDescriptor, FQuantizedBlobState&& QuantizedBlobState);

	/** Returns the FNetBlobCreationInfo. */
	const FNetBlobCreationInfo& GetCreationInfo() const { return CreationInfo; }

	/* Whether the blob is reliably sent or not. */
	bool IsReliable() const { return EnumHasAnyFlags(GetCreationInfo().Flags, ENetBlobFlags::Reliable); }

	/** Returns the FReplicationStateDescriptor if there is one. It's recommended to use a descriptor instead of overriding the serialization methods. */
	const FReplicationStateDescriptor* GetReplicationStateDescriptor() const { return BlobDescriptor.GetReference(); }

	/** Serializes the necessary parts of the creation info so that the blob can be recreated on the receiving side. */
	IRISCORE_API static void SerializeCreationInfo(FNetSerializationContext& Context, const FNetBlobCreationInfo& CreationInfo);

	/** Deserializes the necessary parts of the creation info so that the correct blob can be created. */
	IRISCORE_API static void DeserializeCreationInfo(FNetSerializationContext& Context, FNetBlobCreationInfo& OutCreationInfo);

	/** Serialize the blob together with/targeting a specific object knowing the NetHandle has already been serialized. */
	IRISCORE_API virtual void SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const;

	/** Deserialize a blob that was serialized with SerializeWithObject. */
	IRISCORE_API virtual void DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle);

	/** Serialize the blob. */
	IRISCORE_API virtual void Serialize(FNetSerializationContext& Context) const;

	/** Deserialize a blob that was serialized with Serialize. */
	IRISCORE_API virtual void Deserialize(FNetSerializationContext& Context);

	/** Collect object references from quantized data */
	IRISCORE_API void CollectObjectReferences(FNetSerializationContext& Context, FNetReferenceCollector& Collector) const;

	/** Adds a reference. A blob is created with reference count zero. */
	void AddRef() const
	{ 
		UE_AUTORTFM_OPEN
		{
			++RefCount;
		};
		UE_AUTORTFM_ONABORT(this)
		{
			--RefCount;
		};
	}

	/** Removes a reference. When the reference count reaches zero the blob will be deleted. A blob is created with reference count zero. */
	IRISCORE_API void Release() const;

	/** Returns the reference count.  A blob is created with reference count zero. */
	int32 GetRefCount() const
	{
		int32 ReturnVal = 0;
		UE_AUTORTFM_OPEN
		{
			ReturnVal = RefCount;
		};
		return ReturnVal;
	}

	/* Returns true if the blob has additional exports to add. */
	bool HasExports() const { return EnumHasAnyFlags(GetCreationInfo().Flags, ENetBlobFlags::HasExports); }

	/** Retrieve the object references that need to be exported. */
	TArrayView<const FNetObjectReference> CallGetNetObjectReferenceExports() const { return GetNetObjectReferenceExports(); };

	/** Retrieve NetToken that needs to be exported. */
	TArrayView<const FNetToken> CallGetNetTokenExports() const { return GetNetTokenExports(); };

protected:
	FNetBlob(const FNetBlob&) = delete;
	FNetBlob& operator=(const FNetBlob&) = delete;

	/** The destructor will free dynamic state, if present, in the state buffer and remove a reference to the descriptor. */
	IRISCORE_API virtual ~FNetBlob();

	/** Override to return the object references that need to be exported. */
	virtual TArrayView<const FNetObjectReference> GetNetObjectReferenceExports() const;

	/** Override to return NetTokenExports that need to be exported, mostly relevant for pre-serialized blobs. */
	virtual TArrayView<const FNetToken> GetNetTokenExports() const;

	/** Serializes the state if there's a valid descriptor and state buffer. */
	IRISCORE_API void SerializeBlob(FNetSerializationContext& Context) const;

	/** Deserializes the state if there's a valid descriptor and state buffer. */
	IRISCORE_API void DeserializeBlob(FNetSerializationContext& Context);

protected:
	/** The CreationInfo that was passed to the constructor. */
	FNetBlobCreationInfo CreationInfo;

	/** The state descriptor. */
	TRefCountPtr<const FReplicationStateDescriptor> BlobDescriptor;

	/** The state buffer that holds the data described by the descriptor. */
	FQuantizedBlobState QuantizedBlobState;

private:
	mutable std::atomic<int32> RefCount;
};

/**
 * FNetObjectAttachment serves as a base class for NetBlobs targeting a specific object.
 * @see UReplicationSystem::QueueNetObjectAttachment
 */
class FNetObjectAttachment : public FNetBlob
{
public:
	IRISCORE_API FNetObjectAttachment(const FNetBlobCreationInfo&);
	const FNetObjectReference& GetNetObjectReference() const { return NetObjectReference; }
	const FNetObjectReference& GetTargetObjectReference() const { return TargetObjectReference; }

protected:
	virtual ~FNetObjectAttachment();

	/** Serializes the owner and subobject reference. */
	IRISCORE_API void SerializeObjectReference(FNetSerializationContext& Context) const;

	/** Serializes the owner and subobject reference. */
	IRISCORE_API void DeserializeObjectReference(FNetSerializationContext& Context);

	/** Serializes only the subobject reference and assumes the passed NetHandle is the owner. */
	IRISCORE_API void SerializeSubObjectReference(FNetSerializationContext& Context, FNetRefHandle RefHandle) const;

	/** Deserializes a subobject reference that was serialized using SerializeSubObjectReference with the same NetHandle. */
	IRISCORE_API void DeserializeSubObjectReference(FNetSerializationContext& Context, FNetRefHandle RefHandle);

	/** Set the owner and subobject references. */
	void SetNetObjectReference(const FNetObjectReference& InQueueOwnerReference, const FNetObjectReference& InTargetObjectReference);

protected:
	friend class Private::FNetBlobManager;

	/** The owner reference. */
	FNetObjectReference NetObjectReference;

	/** The subobject reference. */
	FNetObjectReference TargetObjectReference;
};

inline FNetBlob::FQuantizedBlobState::FQuantizedBlobState(FNetBlob::FQuantizedBlobState&& Other)
{
	StateBuffer = Other.StateBuffer;
	MemoryAllocationFlags = Other.MemoryAllocationFlags;
	AllocationSize = Other.AllocationSize;

	Other.StateBuffer = nullptr;
	Other.MemoryAllocationFlags = EMemoryAllocationFlags::None;
	Other.AllocationSize = 0;
}

inline FNetBlob::FQuantizedBlobState& FNetBlob::FQuantizedBlobState::operator=(FNetBlob::FQuantizedBlobState&& Other)
{
	if (StateBuffer)
	{
		Unprotect();
		FMemory::Free(StateBuffer);
	}

	StateBuffer = Other.StateBuffer;
	MemoryAllocationFlags = Other.MemoryAllocationFlags;
	AllocationSize = Other.AllocationSize;

	Other.StateBuffer = nullptr;
	Other.MemoryAllocationFlags = EMemoryAllocationFlags::None;
	Other.AllocationSize = 0;
	return *this;
}

inline void FNetObjectAttachment::SetNetObjectReference(const FNetObjectReference& InQueueOwnerReference, const FNetObjectReference& InTargetObjectReference)
{
	NetObjectReference = InQueueOwnerReference;
	if (InQueueOwnerReference != InTargetObjectReference)
	{
		TargetObjectReference = InTargetObjectReference;
	}
}

}
