// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Iris/ReplicationSystem/NetBlob/PartialNetBlob.h"
#include "Iris/Serialization/NetBitStreamWriter.h"

class USequentialPartialNetBlobHandlerConfig;

namespace UE::Net
{

struct FNetBlobAssemblerInitParams
{
	const USequentialPartialNetBlobHandlerConfig* PartialNetBlobHandlerConfig = nullptr;
};

/**
 * Utility class to reassemble split blobs.
 * Partial blobs need to be added in order and represent the same blob until it's assembled or an error occurs.
 */
class FNetBlobAssembler
{
public:
	IRISCORE_API FNetBlobAssembler();

	/** Initialize the NetBlobAssembler. */
	IRISCORE_API void Init(const FNetBlobAssemblerInitParams& InitParams);

	/**
	 * Add the next expected part of the split blob. If it's the first part of a blob the effects of prior calls to this function are discarded.
	 * @param Context FNetSerializationContext for error reporting. Call HasError() on it afterwards to check whether something went wrong.
	 * @param NetHandle The NetHandle needs to remain constant for every call until the original blob has been assembled or the first part of a new blob is added.
	 */
	IRISCORE_API void AddPartialNetBlob(FNetSerializationContext& Context, FNetRefHandle RefHandle, const TRefCountPtr<FPartialNetBlob>& PartialNetBlob);

	/** Returns true if all parts of the split blob have been added and is ready to be assembled. */
	bool IsReadyToAssemble() const { return bIsReadyToAssemble; }

	/** Returns true if the sequence order is broken. */
	bool IsSequenceBroken() const { return bIsBrokenSequence; }

	/**
	 * Assemble all parts of the split blob.
	 * @param Context FNetSerializationContext for error reporting. Call HasError() on it afterwards to check whether something went wrong.
	 * @return The assembled blob.
	 * @note IsReadyToAssemble() must return true before calling this function.
	 * @note After this function has been called it may not be called again until a new blob is ready to be assembled.
	 */
	IRISCORE_API TRefCountPtr<FNetBlob> Assemble(FNetSerializationContext& Context);

private:
	FNetBlobCreationInfo NetBlobCreationInfo;
	TArray<uint32> Payload;
	FNetBitStreamWriter BitWriter;
	FNetRefHandle RefHandle;
	const USequentialPartialNetBlobHandlerConfig* PartialNetBlobHandlerConfig = nullptr;
	uint32 NextPartIndex = 0;
	// Initialized to an arbitrary number. If we receive the first part of a sequence it doesn't matter. If the next received blob isn't the first part it's a broken sequence.
	uint32 NextSequenceSumber = 0U;
	// The sequence number of the last part expected to be received before being ready to assemble.
	uint32 LastPartSequenceNumber = ~0U;
	uint32 FirstPayloadBitCount = 0;
	bool bIsReadyToAssemble = false;
	bool bIsBrokenSequence = false;
	bool bIsProcessingReliable = false;
};

}
