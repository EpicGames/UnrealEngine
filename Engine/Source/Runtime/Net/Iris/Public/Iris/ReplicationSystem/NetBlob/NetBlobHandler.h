// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "Templates/RefCounting.h"
#include "NetBlobHandler.generated.h"

namespace UE::Net::Private
{
	class FNetBlobHandlerManager;
}

namespace UE::Net
{

IRISCORE_API extern const FName GNetError_UnsupportedNetBlob;

}

/** Interface for being able to receive a NetBlob and forward it to the appropriate UNetBlobHandler. */
class INetBlobReceiver
{
protected:
	using FNetBlobCreationInfo = UE::Net::FNetBlobCreationInfo;
	using FNetBlob = UE::Net::FNetBlob;

public:
	virtual TRefCountPtr<UE::Net::FNetBlob> CreateNetBlob(const FNetBlobCreationInfo&) const = 0;
	virtual void OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>&) = 0;

};

/**
 * A UNetBlobHandler is responsible for creating and processing a single type of NetBlob.
 * If the handler should be able to receive blobs it needs to be configured in
 * UNetBlobHandlerDefinitions and registered to the UReplicationSystem on both the
 * sending and receiving side.
 * @see UReplicationSystem::RegisterNetBlobHandler
 * @note Certain handlers such as NetRPCHandler, PartialNetObjectAttachmentHandler and NetObjectBlobHandler will be registered automatically.
 */
UCLASS(transient, MinimalApi, Abstract)
class UNetBlobHandler : public UObject, public INetBlobReceiver
{
	GENERATED_BODY()

public:
	virtual ~UNetBlobHandler();

	/** Create a blob that the handler can process. Forwards to the virtual CreateNetBlob. */
	TRefCountPtr<FNetBlob> CreateNetBlob(UE::Net::ENetBlobFlags Flags) const;

	/** Get the net blob type. The blob type is determined at runtime and can differ from run to run. */
	UE::Net::FNetBlobType GetNetBlobType() const { return NetBlobType; }

	/** Called when a connection is added. For handler specific connection handling. */
	IRISCORE_API virtual void AddConnection(uint32 ConnectionId);

	/** Called when a connection is removed. For handler specific connection handling. */
	IRISCORE_API virtual void RemoveConnection(uint32 ConnectionId);

protected:
	UNetBlobHandler();	

private:
	// INetBlobReceiver

	/** Override to create the NetBlob. This will be called when receiving a NetBlob of the type this handler is responsible for. */
	virtual TRefCountPtr<FNetBlob> CreateNetBlob(const FNetBlobCreationInfo&) const override PURE_VIRTUAL(CreateNetBlob, return nullptr;);

	/** Override to process the NetBlob when it's received. */
	virtual void OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>&) override PURE_VIRTUAL(OnNetBlobReceived,);

private:
	friend class UE::Net::Private::FNetBlobHandlerManager;

	/** The type is assigned by the FNetBlobHandlerManager when the handler is created. */
	UE::Net::FNetBlobType NetBlobType;
};

inline TRefCountPtr<UE::Net::FNetBlob> UNetBlobHandler::CreateNetBlob(UE::Net::ENetBlobFlags Flags) const
{
	FNetBlobCreationInfo CreationInfo;
	CreationInfo.Flags = Flags;
	CreationInfo.Type = GetNetBlobType();
	return CreateNetBlob(CreationInfo);
}
