// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "Iris/ReplicationSystem/NetBlob/RawDataNetBlob.h"
#include "Iris/Core/NetObjectReference.h"
#include "Net/Core/Trace/NetDebugName.h"

namespace UE::Net
{

class FPartialNetBlob final : public FNetBlob
{
public:
	struct FSplitParams
	{
		uint32 MaxPartBitCount;
		uint32 MaxPartCount;
		FNetObjectReference NetObjectReference;
		FNetDebugName DebugName;
		bool bSerializeWithObject;
	};

	// Split a NetBlob into multiple PartialNetBlobs. The blob will be split even if the original one didn't need it.
	IRISCORE_API static bool SplitNetBlob(FNetSerializationContext& Context, const FNetBlobCreationInfo& CreationInfo, const FSplitParams& SplitParams, const TRefCountPtr<FNetBlob>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs);

	// Split a RawDataNetBlob into multiple PartialNetBlobs. The blob will be split even if the original one didn't need it.
	IRISCORE_API static bool SplitNetBlob(const FNetBlobCreationInfo& CreationInfo, const FSplitParams& SplitParams, const TRefCountPtr<FRawDataNetBlob>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs);

public:
	enum class ESequenceFlags : uint32
	{
		None = 0,
		IsFirstPart = 1U,
	};
	FRIEND_ENUM_CLASS_FLAGS(ESequenceFlags);

	IRISCORE_API FPartialNetBlob(const FNetBlobCreationInfo& CreationInfo);

	// This will only return legible data for the first part on the receiving end.
	uint32 GetPartCount() const { return (IsFirstPart() ? PartCount : 0U); }

	bool IsFirstPart() const { return EnumHasAnyFlags(SequenceFlags, ESequenceFlags::IsFirstPart); }

	uint32 GetSequenceNumber() const { return SequenceNumber; }

	uint32 GetPayloadBitCount() const { return PayloadBitCount; }
	const uint32* GetPayload() const { return Payload.GetData(); }

	const FNetBlobCreationInfo& GetOriginalCreationInfo() const { return OriginalCreationInfo; }

	void SetDebugName(const FNetDebugName& InDebugName) { DebugName = InDebugName; }

private:

	virtual TArrayView<const FNetObjectReference> GetNetObjectReferenceExports() const override final;
	virtual TArrayView<const FNetToken> GetNetTokenExports() const override final;

	virtual void SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const override;
	virtual void DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) override;

	virtual void Serialize(FNetSerializationContext& Context) const override;
	virtual void Deserialize(FNetSerializationContext& Context) override;

	void InternalSerialize(FNetSerializationContext& Context) const;
	void InternalDeserialize(FNetSerializationContext& Context);

	void InternalSerializeBlob(FNetSerializationContext& Context) const;
	void InternalDeserializeBlob(FNetSerializationContext& Context);

	struct FPayloadSplitParams
	{
		FNetDebugName DebugName;

		FNetBlobCreationInfo CreationInfo;
		FNetBlobCreationInfo OriginalCreationInfo;
		FNetBlob* OriginalBlob;
		TConstArrayView<FNetObjectReference> ObjectReferencesPendingExport;
		TConstArrayView<FNetToken> NetTokensPendingExport;
		const uint32* Payload;
		uint32 PayloadBitCount;
		uint32 PartBitCount;
	};
	static void SplitPayload(const FPayloadSplitParams& SplitParams, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs);

	// Used by the first part to be able to reconstruct the original message.
	FNetBlobCreationInfo OriginalCreationInfo;
	TRefCountPtr<FNetBlob> OriginalBlob;

	ESequenceFlags SequenceFlags = ESequenceFlags::None;
	uint32 SequenceNumber = 0;
	// PartCount is only valid if it's the first part.
	uint16 PartCount = 0;
	uint16 PayloadBitCount = 0;
	// Use uint32 for guaranteed FNetBitStreamReader/Writer compatibility
	TArray<uint32> Payload;
	TArray<FNetObjectReference, TInlineAllocator<2>> ObjectReferenceExportsArray;
	TArray<FNetToken, TInlineAllocator<4>> NetTokenExportsArray;

	FNetDebugName DebugName;
};

}
