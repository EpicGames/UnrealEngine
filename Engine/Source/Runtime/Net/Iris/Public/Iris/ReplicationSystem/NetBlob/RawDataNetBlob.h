// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "Containers/ArrayView.h"

namespace UE::Net
{

/**
 * Helper class for stateless data, such as when arbitrary data has been serialized to a bitstream.
 * The serialization will simply serialize the raw data regardless of whether a NetRefHandle is provided or not.
 * Things like splitting and assembling have optimized code paths for this type of blob.
 * You can inherit from this blob type but you cannot override the serialization functions.
 * @note Sending huge blobs that require splitting and assembling is strongly discouraged. 
 */
class FRawDataNetBlob : public FNetBlob
{
public:
	IRISCORE_API FRawDataNetBlob(const FNetBlobCreationInfo&);

	/**
	 * Set the raw data via moving an array.
	 * @param RawData The array to be moved.
	 * @param RawDataBitCount The number of bits that should be serialized.
	 *        If RawDataBitCount is not a multiple of 32 then the (RawDataBitCount % 32)
	 *        least significant bits of the last uint32 are serialized.
	 */
	IRISCORE_API void SetRawData(TArray<uint32>&& RawData, uint32 RawDataBitCount);

	/**
	 * Set the raw data. The data is copied.
	 * @param RawData The data to be copied.
	 * @param RawDataBitCount The number of bits that should be serialized.
	 *        If RawDataBitCount is not a multiple of 32 then the (RawDataBitCount % 32)
	 *        least significant bits of the last uint32 are serialized.
	 */
	IRISCORE_API void SetRawData(const TArrayView<const uint32> RawData, uint32 RawDataBitCount);

	/** Returns the raw data. */
	TArrayView<const uint32> GetRawData() const { return MakeArrayView(RawData.GetData(), RawData.Num()); }

	/** Returns the number of valid bits in the raw data. */
	uint32 GetRawDataBitCount() const { return RawDataBitCount; }
	
protected:
	/** Serializes the raw data. */
	IRISCORE_API void InternalSerialize(FNetSerializationContext& Context) const;

	/** Deserializes the raw data. */
	IRISCORE_API void InternalDeserialize(FNetSerializationContext& Context);

private:
	virtual void SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const override final;
	virtual void DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) override final;

	virtual void Serialize(FNetSerializationContext& Context) const override final;
	virtual void Deserialize(FNetSerializationContext& Context) override final;

	TArray<uint32> RawData;
	uint32 RawDataBitCount;
};

}
