// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/PacketControl/PacketNotification.h"
#include "Templates/RefCounting.h"

namespace UE::Net
{
	class FNetBlob;
	class FNetSerializationContext;
	struct FReplicationStateDescriptor;
	namespace Private
	{
		class FNetBlobHandlerManager;
	}
}

namespace UE::Net::Private
{

/** Helper class to deliver blobs reliably in order. */
class FReliableNetBlobQueue
{
public:
	// A single ReplicationRecord supports up to four disjoint sequences to be serialized in a packet. This allows for resends of dropped data while potentially also writing new blobs for the first time.
	struct FReplicationRecord
	{
		struct FSequence
		{
			enum : unsigned
			{
				NumberBitCount = 10U,
				CountBitCount = 6U,
			};

			FSequence() = default;
			explicit FSequence(uint16 Value)
			{
				Number = Value & ((1U << NumberBitCount) - 1U);
				Count = Value >> NumberBitCount;
			}

			uint16 ToUint16() const
			{
				return static_cast<uint16>((Count << NumberBitCount) | Number);
			}

			uint16 Number : NumberBitCount = 0;
			uint16 Count : CountBitCount = 0;
		};

		FReplicationRecord() = default;
		explicit FReplicationRecord(uint64 Value)
		{
			for (unsigned Index : {3U, 2U, 1U, 0U})
			{
				Sequences[Index] = FSequence(Value & 0xFFFFU);
				Value >>= 16U;
			}
		}

		uint64 ToUint64() const
		{
			uint64 Value = 0;
			for (unsigned Index : {0U, 1U, 2U, 3U})
			{
				Value = (Value << 16U) | Sequences[Index].ToUint16();
			}

			return Value;
		}

		bool IsValid() const { return (Sequences[0].Count | Sequences[1].Count | Sequences[2].Count | Sequences[3].Count) != 0U; }

		FSequence Sequences[4];
	};

	/** This represents a ReplicationRecord where nothing was serialized. */

	/** How many blobs can be sent before an ACK/NAK is required to continue sending. Changing this might require changing FReplicationRecord too. */
	static constexpr uint32 MaxUnackedBlobCount = 1024U;

	FReliableNetBlobQueue();
	~FReliableNetBlobQueue();

	/** Returns the number of blobs that have not yet been sent. */
	uint32 GetUnsentBlobCount() const { return UnsentBlobCount; }

	/** Returns true if there are unsent blobs. There may still be unacked blobs even if there are no unsent ones. */
	bool HasUnsentBlobs() const { return GetUnsentBlobCount() > 0; }

	/** Returns whether all blobs have been sent and acknowledged as received. */
	bool IsAllSentAndAcked() const { return FirstSeq == LastSeq && GetUnsentBlobCount() == 0; }

	/** Returns whether the send window is full or not. */
	bool IsSendWindowFull() const;

	/** Returns true if it's safe to destroy this queue. */
	IRISCORE_API bool IsSafeToDestroy() const;

	/** Put a blob to be sent in the queue. Returns true if the blob was successfully queued and false if the queue was full. */
	IRISCORE_API bool Enqueue(const TRefCountPtr<FNetBlob>& Blob);

	/** On the receiving end this will return a pointer to the next blob that can be processed. */
	IRISCORE_API const TRefCountPtr<FNetBlob>* Peek();

	/** On the receiving end this will remove the next blob to be processed from the queue. Call after processing the blob returned from Peek(). */
	IRISCORE_API void Pop();

	/** On the receiving end this will move all received unreliable NetBlobs to the array and release them from the queue. This breaks the ordering guarantees provided by using Peek and Pop. Reliable NetBlobs are unaffected by this operation. */
	IRISCORE_API void DequeueUnreliable(TArray<TRefCountPtr<FNetBlob>>& Unreliable);

	/** Count all unreliable blobs */
	IRISCORE_API SIZE_T GetUnreliableCount() const;

	/** On sending side this will drop all unreliable in the queue to avoid creating a backlog of unsent unreliable blobs */
	IRISCORE_API void DropUnreliable();

	/**
	 * Serializes as many blobs as possible using their respective SerializeWithObject() method. It is assumed the NetRefHandle will be 
	 * reconstructed somehow on the receiving end and passed to DeserializeWithObject().
	 * This provides an opportunity for FNetObjectAttachments, such as FNetRPCs, to avoid serializing the same NetRefHandle redundantly.
	 * @param Context A FNetSerializationContext.
	 * @param RefHandle The handle for the blobs' target object.
	 * @param OutRecord The record to pass to CommitReplicationRecord() if a packet containing the serialized data was sent.
	 * @return The number of blobs that were serialized.
	 */
	IRISCORE_API uint32 SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle, FReplicationRecord& OutRecord);

	/**
	 * Deserializes blobs with object using their respective DeserializeWithObject() method.
	 * @return The number of blobs that were deserialized.
	 * @see SerializeWithObject
	 */
	IRISCORE_API uint32 DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle);

	/**
	 * Serializes as many blobs as possible using their respective Serialize() method.
	 * @param Context A FNetSerializationContext.
	 * @param OutRecord The record to pass to CommitReplicationRecord() if a packet containing the serialized data was sent.
	 * @return The number of blobs that were serialized.
	 */
	IRISCORE_API uint32 Serialize(FNetSerializationContext& Context, FReplicationRecord& OutRecord);

	/**
	 * Deserializes blobs with object using their respective Deserialize() method.
	 * @return The number of blobs that were deserialized.
	 * @see Serialize
	 */
	IRISCORE_API uint32 Deserialize(FNetSerializationContext& Context);

	/**
	 * Call after a packet containing serialized data was sent.
	 * @see SerializeWithObject, Serialize
	 */
	IRISCORE_API void CommitReplicationRecord(const FReplicationRecord& Record);

	/**
	 * For each packet for which CommitReplicationRecord() was called ProcessPacketDeliveryStatus() needs
	 * to be called in the same order when it's known whether the packet was delivered or not.
	 * @param Status Whether the packet was delivered or not or if the record should simply be discarded due to closing a connection.
	 * @param Record The record that was obtained via a Serialize/SerializeWithObject call and passed to CommitReplicationRecord.
	 * @see CommitReplicationRecord
	 */
	IRISCORE_API void ProcessPacketDeliveryStatus(EPacketDeliveryStatus Status, const FReplicationRecord& Record);

private:
	enum Constants : uint32
	{
		IndexBitCount = 10U,
		MaxWriteSequenceCount = UE_ARRAY_COUNT(FReplicationRecord::Sequences),
		MaxSequenceLength = (1U << FReplicationRecord::FSequence::CountBitCount) - 1U,
	};

	uint32 SequenceToIndex(uint32 Seq) const;

	bool IsSequenceAcked(uint32 Seq) const;
	bool IsIndexAcked(uint32 Index) const;
	void SetIndexIsAcked(uint32 Index);
	void SetSequenceIsAcked(uint32 Index);
	void ClearIndexIsAcked(uint32 Index);

	bool IsSequenceSent(uint32 Seq) const;
	bool IsIndexSent(uint32 Index) const;
	void SetIndexIsSent(uint32 Index);
	void SetSequenceIsSent(uint32 Index);
	void ClearSequenceIsSent(uint32 Seq);
	void ClearIndexIsSent(uint32 Seq);

	bool IsValidReceiveSequence(uint32 Seq) const;

	uint32 SerializeInternal(FNetSerializationContext& Context, FNetRefHandle RefHandle, FReliableNetBlobQueue::FReplicationRecord& OutRecord, const bool bSerializeWithObject);
	uint32 DeserializeInternal(FNetSerializationContext& Context, FNetRefHandle RefHandle, const bool bSerializeWithObject);

	void OnPacketDelivered(const FReplicationRecord& Record);
	void OnPacketDropped(const FReplicationRecord& Record);

	void PopInOrderAckedBlobs();

	TRefCountPtr<FNetBlob> NetBlobs[MaxUnackedBlobCount];
	uint32 Sent[(MaxUnackedBlobCount + 31)/32];
	uint32 Acked[(MaxUnackedBlobCount + 31)/32];
	uint32 FirstSeq;
	uint32 LastSeq;
	uint32 UnsentBlobCount;
};

//
inline bool FReliableNetBlobQueue::IsSendWindowFull() const
{
	return (LastSeq - FirstSeq) >= MaxUnackedBlobCount;
}

inline uint32 FReliableNetBlobQueue::SequenceToIndex(uint32 Seq) const
{ 
	return Seq % MaxUnackedBlobCount;
}

inline bool FReliableNetBlobQueue::IsIndexAcked(uint32 Index) const
{
	return (Acked[Index >> 5U] & (1U << (Index & 31U))) != 0U;
}

inline bool FReliableNetBlobQueue::IsSequenceAcked(uint32 Seq) const
{
	return IsIndexAcked(SequenceToIndex(Seq));
}

inline void FReliableNetBlobQueue::SetIndexIsAcked(uint32 Index)
{
	Acked[Index >> 5U] |= (1U << (Index & 31U));
}

inline void FReliableNetBlobQueue::SetSequenceIsAcked(uint32 Seq)
{
	return SetIndexIsAcked(SequenceToIndex(Seq));
}

inline void FReliableNetBlobQueue::ClearIndexIsAcked(uint32 Index)
{
	Acked[Index >> 5U] &= ~(1U << (Index & 31U));
}

inline bool FReliableNetBlobQueue::IsIndexSent(uint32 Index) const
{
	return (Sent[Index >> 5U] & (1U << (Index & 31U))) != 0U;
}

inline bool FReliableNetBlobQueue::IsSequenceSent(uint32 Seq) const
{
	return IsIndexSent(SequenceToIndex(Seq));
}

inline void FReliableNetBlobQueue::SetIndexIsSent(uint32 Index)
{
	Sent[Index >> 5U] |= (1U << (Index & 31U));
}

inline void FReliableNetBlobQueue::SetSequenceIsSent(uint32 Seq)
{
	return SetIndexIsSent(SequenceToIndex(Seq));
}

inline void FReliableNetBlobQueue::ClearIndexIsSent(uint32 Index)
{
	Sent[Index >> 5U] &= ~(1U << (Index & 31U));
}

inline void FReliableNetBlobQueue::ClearSequenceIsSent(uint32 Seq)
{
	return ClearIndexIsSent(SequenceToIndex(Seq));
}

inline bool FReliableNetBlobQueue::IsValidReceiveSequence(uint32 Seq) const
{
	return (Seq >= FirstSeq) & (Seq - FirstSeq < MaxUnackedBlobCount); // Must be less than MaxUnackedBlobCount as LastSeq == Seq + 1
}

}
