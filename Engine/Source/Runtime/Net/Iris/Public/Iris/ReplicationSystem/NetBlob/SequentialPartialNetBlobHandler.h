// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Iris/ReplicationSystem/NetBlob/NetBlobHandler.h"
#include "SequentialPartialNetBlobHandler.generated.h"

class FNetBlobHandlerManager;
class UReplicationSystem;
class USequentialPartialNetBlobHandlerConfig;
namespace UE::Net
{
	class FNetObjectReference;
}

struct FSequentialPartialNetBlobHandlerInitParams
{
	UReplicationSystem* ReplicationSystem;

	const USequentialPartialNetBlobHandlerConfig* Config;
};

UCLASS(Config=Engine, MinimalAPI)
class USequentialPartialNetBlobHandlerConfig : public UObject
{
	GENERATED_BODY()

public:
	uint32 GetMaxPartBitCount() const { return MaxPartBitCount; }
	uint32 GetMaxPartCount() const { return MaxPartCount; }
	uint64 GetTotalMaxPayloadBitCount() const { return GetMaxPartBitCount()*uint64(GetMaxPartCount()); }

protected:
	/** How many bits a PartialNetBlob payload can hold at most. Cannot exceed 65535, but anything near the max packet size is discouraged as it is unlikely to fit. Keep it a power of two. */
	UPROPERTY(Config)
	uint32 MaxPartBitCount = 128*8;

	/** How many parts a NetBlob can be split into at most. If more parts are required the splitting will fail. Cannot exceed 65535. */
	UPROPERTY(Config)
	uint32 MaxPartCount = 4096;
};

UCLASS(abstract, MinimalApi, transient)
class USequentialPartialNetBlobHandler : public UNetBlobHandler
{
	GENERATED_BODY()

public:
	/** Unconditionally splits a NetBlob into a sequence of PartialNetBlobs which are small in size. Calls FNetBlob::Serialize(). */
	IRISCORE_API bool SplitNetBlob(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs, const UE::Net::FNetDebugName* InDebugName = nullptr) const;

	/** Unconditionally splits a NetBlob into a sequence of PartialNetBlobs which are small in size. Calls FNetBlob::SerializeWithObject(). */
	IRISCORE_API bool SplitNetBlob(UE::Net::FNetSerializationContext& Context, const UE::Net::FNetObjectReference& NetObjectReference, const TRefCountPtr<FNetBlob>& Blob, TArray<TRefCountPtr<FNetBlob>>& OutPartialBlobs, const UE::Net::FNetDebugName* InDebugName = nullptr) const;

protected:
	IRISCORE_API USequentialPartialNetBlobHandler();

	IRISCORE_API void Init(const FSequentialPartialNetBlobHandlerInitParams& InitParams);

	const USequentialPartialNetBlobHandlerConfig* GetConfig() const { return Config; }

	// Convenience
	UReplicationSystem* ReplicationSystem;

private:
#if WITH_AUTOMATION_WORKER
	friend class UMockSequentialPartialNetBlobHandler;
#endif

	// UNetBlobHandler API. Not exposed to subclasses.
	virtual TRefCountPtr<FNetBlob> CreateNetBlob(const FNetBlobCreationInfo&) const override;

	/* A call will result in an error. Either override or use an external NetBlobAssembler instead and forward the final assembled blob to the appropriate handler. */
	virtual void OnNetBlobReceived(UE::Net::FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& NetBlob) override;

	const USequentialPartialNetBlobHandlerConfig* Config;
};
