// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "Iris/ReplicationSystem/NetBlob/NetBlob.h"
#include "Net/Core/NetToken/NetToken.h"

namespace UE::Net
{

/**
 * A ShrinkWrapNetBlob/NetObjectAttachment is typically used on the sending side for data with multiple destinations.
 * In that case the contents of the original blob can be serialized, once, to a buffer and then wrapped
 * in an instance of this class. The serialization of the buffer is likely faster than the original
 * serialization as no particular logic needs to be performed and serializing a buffer is an optimized path.
 * @note If tracing is enabled the OriginalBlob will be serialized instead of the already serialized buffer.
 *       This is for debugging purposes.
 * @note Deserialization will always be performed by the original blob type.
 */
class FShrinkWrapNetBlob final : public FNetBlob
{
public:
	IRISCORE_API FShrinkWrapNetBlob(FNetSerializationContext& Context, const TRefCountPtr<FNetBlob>& OriginalBlob, TArray<uint32>&& Payload, uint32 PayloadBitCount);

private:
	virtual TArrayView<const FNetObjectReference> GetNetObjectReferenceExports() const override final;
	virtual TArrayView<const FNetToken> GetNetTokenExports() const override final;
	virtual void SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const override final;
	virtual void DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) override final;

	virtual void Serialize(FNetSerializationContext& Context) const override final;
	virtual void Deserialize(FNetSerializationContext& Context) override final;

	void InternalSerialize(FNetSerializationContext& Context) const;

	TRefCountPtr<FNetBlob> OriginalBlob;
	TArray<FNetToken, TInlineAllocator<4>> NetTokenExportsArray;
	TArray<uint32> SerializedBlob;
	uint32 SerializedBlobBitCount;
};

class FShrinkWrapNetObjectAttachment final : public FNetBlob
{
public:
	IRISCORE_API FShrinkWrapNetObjectAttachment(FNetSerializationContext& Context, const TRefCountPtr<FNetObjectAttachment>& OriginalBlob, TArray<uint32>&& Payload, uint32 PayloadBitCount);

private:
	virtual TArrayView<const FNetObjectReference> GetNetObjectReferenceExports() const override final;
	virtual TArrayView<const FNetToken> GetNetTokenExports() const override final;
	virtual void SerializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) const override final;
	virtual void DeserializeWithObject(FNetSerializationContext& Context, FNetRefHandle RefHandle) override final;

	virtual void Serialize(FNetSerializationContext& Context) const override final;
	virtual void Deserialize(FNetSerializationContext& Context) override final;

	void InternalSerialize(FNetSerializationContext& Context) const;

	TRefCountPtr<FNetObjectAttachment> OriginalBlob;
	TArray<FNetToken, TInlineAllocator<4>> NetTokenExportsArray;
	TArray<uint32> SerializedBlob;
	uint32 SerializedBlobBitCount;
};

}
