// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"
#include "Iris/ReplicationSystem/ReplicationBridgeTypes.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"

#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"

#include "NetObjectFactory.generated.h"

class UObjectReplicationBridge;
class UNetObjectFactory;

namespace UE::Net
{
	class FNetSerializationContext;

	struct FNetObjectResolveContext;

	typedef uint32 FReplicationProtocolIdentifier;
}

namespace UE::Net
{

enum class EDetachReason : uint32
{
	/** StopReplication called without a specific reason */
	Stopped = 0U,

	/** Object is still replicating but is no longer relevant to this connection */
	NoLongerRelevant,

	/** Object is being torn off permanently */
	TornOff,

	/** Special entry used in FSubObjectDetachContext when the root object stays relevant. */
	RootObjectStillRelevant,

	/** Special info sent for static (loaded) objects that are destroyed on the server. Even if never replicated they can ask the RepSystem to synchronize their destruction on clients. */
	StaticDestroyed,

	/** Put new entries above this one*/
	MAX,
};

/** Contextual information passed to an header so it can serialize/deserialize itself  */
struct FCreationHeaderContext
{
	/** The handle of the replicated object represented by the header */
	FNetRefHandle Handle;
	/** The bridge responsible for the replicated object */
	UObjectReplicationBridge* Bridge;
	/** The factory that allocated the header */
	UNetObjectFactory* Factory;
	/** Access to the bitstream reader or writer */
	FNetSerializationContext& Serialization;

	FCreationHeaderContext(FNetRefHandle InHandle, UObjectReplicationBridge* InBridge, UNetObjectFactory* InFactory, FNetSerializationContext& InSerialization) : Handle(InHandle), Bridge(InBridge), Factory(InFactory), Serialization(InSerialization) {}
};

/*
 * Class holding the raw information allowing any client from retrieving or allocating a replicated UObject instance.
 * Can also implement the serialization of it's data into a bitstream
 */ 
class FNetObjectCreationHeader
{
public:

	virtual ~FNetObjectCreationHeader() = default;

	void SetProtocolId(uint32 InId) { ProtocolIdentifier = InId; }
	void SetFactoryId(FNetObjectFactoryId InId) { FactoryId = InId; }

	FReplicationProtocolIdentifier GetProtocolId() const { return ProtocolIdentifier; }
	FNetObjectFactoryId GetNetFactoryId() const { return FactoryId; }

	/** Transform the header information into a readable format */
	virtual FString ToString() const { return TEXT("NotImplemented"); }

private:

	FReplicationProtocolIdentifier ProtocolIdentifier = 0;
	FNetObjectFactoryId FactoryId = InvalidNetObjectFactoryId;
};

enum class ERootObjectReplicationParamsReason : uint32
{
	StartReplication,
	StartReplayRecording,
};

struct FRootObjectReplicationParamsContext
{
	/** Object params are being requested for. */
	UObject* Object = nullptr;

	/** The reason for FillRootObjectReplicationParams being called. */
	ERootObjectReplicationParamsReason Reason = ERootObjectReplicationParamsReason::StartReplication;
};

struct FRootObjectReplicationParams
{
	/** *Actor only* When true the actor will receive a PreReplication callback just before it gets polled. */
	bool bNeedsPreUpdate = false;

	/** When true the object has a dynamic world location and we should ask the factory to update its current location every time it is polled. */
	bool bNeedsWorldLocationUpdate = false;

	/** Whether the object is dormant or not */
	bool bIsDormant = false;

	/** Ask the class config for a dynamic filter assigned to this class or one of it's parent class. Default is true. */
	bool bUseClassConfigDynamicFilter = true;

	/** When enabled we ignore the class config for this object and instead use the one specified by ExplicitDynamicFilter */
	bool bUseExplicitDynamicFilter = false;

	/** Set to true to prevent the replication system from reusing a NetRefHandle previously assigned to the object. Useful if the object had stopped replicating and that old representation is considered to be destroyed. */
	bool bForceNewHandle = false;

	/**
	* The name of the dynamic filter to use for this object (instead of asking the class config).
	* Can be none so that no dynamic filter is assigned to the object.
	* Only used when bUseExplicitDynamicFilter is true.
	*/
	FName ExplicitDynamicFilterName;

	/**
	* If StaticPriority is > 0 the ReplicationSystem will use that as priority when scheduling objects.
	* If it's <= 0.0f one will look for a world location support and then use the default spatial prioritizer.
	*/
	float StaticPriority = 0.0f;

	/**
	* How often per second the object should be polled for dirtiness, including calling the InstancePreUpdate function.
	* When set to zero it will be polled every frame.
	*/
	float PollFrequency = 0.0f;
};

} // end namespace UE::Net

/**
 * The class is responsible for creating the header representing specific replicated object types.
 * Also responsible for instantiating the UObject from a replicated header.
 */
UCLASS(MinimalAPI, transient, abstract)
class UNetObjectFactory : public UObject
{
	GENERATED_BODY()

public:

	void Init(UE::Net::FNetObjectFactoryId InId, UObjectReplicationBridge* InBridge);
	void Deinit();
	void PostReceiveUpdate();

	struct FInstantiateResult;
	struct FInstantiateContext;
	struct FPostInstantiationContext;
	struct FPostInitContext;
	struct FDestroyedContext;
	struct FDetachContext;
	struct FSubObjectDetachContext;
	struct FWorldInfoContext;
	struct FWorldInfoData;

	/**
	 * Creates the header containing all information required to instantiate a remote version of the object represented by the handle.
	 * @param Handle The handle of the object represented by the header
	 * @param ProtocolId The protocol id to add to the header
	 * @return Return a valid header filled with the information that will be sent to remote connections
	 */
	TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateHeader(UE::Net::FNetRefHandle Handle, UE::Net::FReplicationProtocolIdentifier ProtocolId);

	/** 
	 * Serializes a valid header so it can be replicated to remote connections 
	 * @param Handle The handle of the object represented by the header
	 * @param Header The filled header that will be serialized
	 * @return Returns true if the serialization was a success
	 */
	bool WriteHeader(UE::Net::FNetRefHandle Handle, UE::Net::FNetSerializationContext& Serialization, const UE::Net::FNetObjectCreationHeader* Header);

	/** 
	 * Deserialize the header data received and return a valid header
	 * @param Handle The handle of the object represented by the header
	 * @param Serialization Gives access to the bitstream reader
	 * @return Return a valid header filled with the information that represents the remote object
	 */
	TUniquePtr<UE::Net::FNetObjectCreationHeader> ReadHeader(UE::Net::FNetRefHandle Handle, UE::Net::FNetSerializationContext& Serialization);

	/**
	* Create or bind a replicated object from the received creation header.
	* @param Context Gives you access to useful info on the object to instantiate
	* @param Header The filled header information to use to spawn the object
	* @return The instantiated object if successful and relevant flags for the bridge to act on
	*/
	virtual FInstantiateResult InstantiateReplicatedObjectFromHeader(const FInstantiateContext& Context, const UE::Net::FNetObjectCreationHeader* Header) PURE_VIRTUAL(UNetObjectFactory::InstantiateReplicatedObjectFromHeader, return FInstantiateResult(););

	/**
	 * Optional callback triggered at the end of the instantiation process and before any replicated properties were applied.
	 * Useful to apply any additional data included with the header or to signal the new object to other systems.
	 * @param Context Gives you access to the new object and the header it was created from.
	 */
	virtual void PostInstantiation(const FPostInstantiationContext& Context) {}

	/**
	 * Optional callback triggered after we applied the initial replicated properties to the instantiated object.
	 * From here the remote object is ready to be used by the game engine.
	 * @param Context Gives you access to the new object.
	 */
	virtual void PostInit(const FPostInitContext& Context) {}

	UE_DEPRECATED(5.8, "FDestroyedContext struct has been deprecated and replaced with the FDetachContext")
	virtual void DetachedFromReplication(const FDestroyedContext& Context) {}

	/**
	 * Callback triggered when a replicated object is no longer replicated on the client.
	 * This is where a factory would destroy dynamic objects, reset stable objects or put objects back in a pool.
	 * Note that this callback is called on remotes (clients) only.
	 * @param Context Gives access to the instance that is detached along with details on why the object is no longer replicated
	 * @param SubObjectContext Optional structure passed only when the detached object is a subobject. Gives access to the root object of the subobject.
	 */
	virtual void DetachedFromReplication(const FDetachContext& Context, const TOptional<FSubObjectDetachContext>& SubObjectContext) PURE_VIRTUAL(UNetObjectFactory::DetachedFromReplication,);

	/** 
	 * Optional callback triggered when a root object managed by this factory gets assigned a subobject. 
	 * The subobject factory must have set EReplicationBridgeCreateNetRefHandleResultFlags::ShouldCallSubObjectCreatedFromReplication in it's results. The default factories usually set the flag only for dynamic (not loaded) subobjects.
	 * This callback is called on remotes only. 
	 * At the time of the callback the RootObject will have the latest replicated properties set, but the subobject will only be default constructed and won't be assigned the replicated properties received alongside the creation request. 
	 * @param RootObject The root object that owns the subobject
	 * @param SubObjectCreated The subobject that was just instantiated
	*/
	virtual void SubObjectCreatedFromReplication(UE::Net::FNetRefHandle RootObject, UE::Net::FNetRefHandle SubObjectCreated) {}

	UE_DEPRECATED(5.8, "FDestroyedContext struct has been deprecated and replaced with the FDetachContext")
	virtual void SubObjectDetachedFromReplication(const FDestroyedContext& Context) {}

	/**
	 * Optional callback triggered when a subobject will be detached from the replication system and potentially destroyed
	 * by it's factory. Both static and dynamic subobjects will be passed to this function.
	 * Note that this callback is called on remotes (clients) only.
	 * @param Context Gives access to the subobject that is no longer replicated.
	 * @param SubObjectContext Gives access to the rootobject of the detached subobject
	 */
	virtual void SubObjectDetachedFromReplication(const FDetachContext& Context, const FSubObjectDetachContext& SubObjectContext) {}

	/**
	 * Fetch world information about a replicated object so it can be updated in the network engine.
	 * Only called for root objects.
	 * @param Context Gives access to the replicated object and which specific world information needs to be updated.
	 * @return The object's world info, or NullOpt if there is none.
	 */
	virtual TOptional<FWorldInfoData> GetWorldInfo(const FWorldInfoContext& Context) const PURE_VIRTUAL(UNetObjectFactory::GetWorldInfo, return NullOpt;);

	UE_DEPRECATED(5.8, "This method is no longer supported.")
	/** Return the poll frequency of a root object managed by this factory. */
	virtual float GetPollFrequency(UE::Net::FNetRefHandle RootObjectHandle, UObject* RootObjectInstance) { return 100.0f; }

	virtual void FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams) PURE_VIRTUAL(UNetObjectFactory::FillRootObjectReplicationParams);

public:

	/** Result of the instantiate request */
	struct FInstantiateResult
	{
		/** The instantiated object represented by the header */
		UObject* Instance = nullptr;
		/** The template used to instantiate the object with. Only set this when the template is different from the object's archetype. */
		UObject* Template = nullptr;
		/** Flags to pass back to the bridge */
		EReplicationBridgeCreateNetRefHandleResultFlags Flags = EReplicationBridgeCreateNetRefHandleResultFlags::None;
		/** String containing reason if instantiate request fails */
		FString FailureDiagnosticMessage;
	};

	/** Contextual information to use during instantiation */
	struct FInstantiateContext
	{
		/** The handle tied to the replicated object to instantiate */
		UE::Net::FNetRefHandle Handle;

		const UE::Net::FNetObjectResolveContext& ResolveContext;

		/** The handle of the object's root object (only if instantiating a subobject) */
		UE::Net::FNetRefHandle RootObjectOfSubObject;

		FInstantiateContext(UE::Net::FNetRefHandle InHandle, const UE::Net::FNetObjectResolveContext& InResolveContext, UE::Net::FNetRefHandle InRootObjectHandle) : Handle(InHandle), ResolveContext(InResolveContext), RootObjectOfSubObject(InRootObjectHandle) {}

		/** Tells if we instantiating a root object or a subobject. */
		bool IsRootObject() const { return !RootObjectOfSubObject.IsValid(); }
		bool IsSubObject() const { return !IsRootObject(); }
	};

	/** Contextual information to use in the PostInstantiation callback */
	struct FPostInstantiationContext
	{
		/** The object instantiated */
		UObject* Instance = nullptr;
		/** The header representing the replicated object */
		const UE::Net::FNetObjectCreationHeader* Header = nullptr;
		/** The connection that owns the replicated object */
		uint32 ConnectionId = 0;
	};

	/** Contextual information to use in the PostInit callback */
	struct FPostInitContext
	{
		/** The object instantiated */
		UObject* Instance = nullptr;
		/** The handle of the object */
		UE::Net::FNetRefHandle Handle;
	};

	/** Contextual information for the destroy callbacks */
	struct UE_DEPRECATED(5.8, "Replaced by the FDetachContext structure") FDestroyedContext
	{
		/** The object about to be destroyed */
		UObject* DestroyedInstance = nullptr;

		/** Optional pointer to the root object when the destroyed object is a subobject */
		UObject* RootObject = nullptr;

		/** The handle of the object destroyed */
		UE::Net::FNetRefHandle DestroyedObjectHandle;

		/** Optional handle of the root object when the destroyed object is a subobject */
		UE::Net::FNetRefHandle RootObjectHandle;

		EReplicationBridgeDestroyInstanceReason DestroyReason = EReplicationBridgeDestroyInstanceReason::DoNotDestroy;
		EReplicationBridgeDestroyInstanceFlags DestroyFlags = EReplicationBridgeDestroyInstanceFlags::None;
	};
	
	/** Contextual information about an object that is no longer replicated to a client */
	struct FDetachContext
	{
		/** The object that is no longer replicated. Can be null if the object was asked to stop replicating locally before the server authority sent a detach request.  */
		UObject* DetachedInstance = nullptr;

		/** The handle of the object */
		UE::Net::FNetRefHandle DetachedObjectHandle;

		/** The context that caused the object to be detached for this client. */
		UE::Net::EDetachReason Reason = UE::Net::EDetachReason::Stopped;
	};

	/** Contextual information about the RootObject of a detached subobject. */
	struct FSubObjectDetachContext
	{
		/** The root object instance of the detached subobject. */
		UObject* RootObject = nullptr;

		/** The root object handle of the detached subobject. */
		UE::Net::FNetRefHandle RootObjectHandle;

		/** The context in which the root object was detached. Will be set to RootObjectStillRelevant when only the subobject is no longer replicating. */
		UE::Net::EDetachReason RootObjectDetachReason = UE::Net::EDetachReason::Stopped;
	};

	/** Details which info needs to be updated in GetWorldInfo */
	enum class EWorldInfoRequested : uint32
	{
		None = 0x0000,
		Location = 0x0001,
		CullDistance = 0x0002,
		All = Location | CullDistance,
	};

	/** Context when asking the factory for information on a specific object */
	struct FWorldInfoContext
	{
		/** The object instance we are requesting information about */
		UObject* Instance = nullptr;

		/** The handle of the object */
		UE::Net::FNetRefHandle Handle;

		/** Specify which info is requested to be updated. */
		EWorldInfoRequested InfoRequested = EWorldInfoRequested::All;
	};

	/** The world data the factory needs to fill about a given object. */
	struct FWorldInfoData
	{
		/** The current location of the object in the world. */
		FVector WorldLocation = FVector::ZeroVector;

		/** The network cull distance of the object. */
		float CullDistance = 0.0f;
	};

protected:

	/** Called when the netfactory is created */
	virtual void OnInit() {}

	/** Called before the netfactory will be destroyed */
	virtual void OnDeinit() {}

	/** Called after we finished processing all incoming packets */
	virtual void OnPostReceiveUpdate() {}

	/**
	 * Create the correct header type for a given replicated object and fill the header with the information representing it.
	 * @param Handle The net handle of the replicated object.
	 * @return A valid creation header if successful. 
	 */
	virtual TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateAndFillHeader(UE::Net::FNetRefHandle Handle) PURE_VIRTUAL(UNetObjectFactory::CreateAndFillHeader, return nullptr;);

	/**
	* Serialize the header into the bitstream
	* @param Context Gives access to the bitstream writer and other useful information.
	* @param Header A valid and filled header to serialize
	* @return Return true if the header serialization was successful.
	*/
	virtual bool SerializeHeader(const UE::Net::FCreationHeaderContext& Context, const UE::Net::FNetObjectCreationHeader* Header) PURE_VIRTUAL(UNetObjectFactory::SerializeHeader, return false;);

	/**
	* Create a new header and deserialize it's data from the incoming bitstream
	* @param Serialization Gives you access to the bit reader
	* @return Return a valid and filled header if successful.
	*/
	virtual TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateAndDeserializeHeader(const UE::Net::FCreationHeaderContext& Context) PURE_VIRTUAL(UNetObjectFactory::CreateAndDeserializeHeader, return nullptr;);

protected:

	UObjectReplicationBridge* Bridge = nullptr;
	UE::Net::FNetObjectFactoryId FactoryId = UE::Net::InvalidNetObjectFactoryId;

};

ENUM_CLASS_FLAGS(UNetObjectFactory::EWorldInfoRequested);

IRISCORE_API const TCHAR* LexToString(UE::Net::EDetachReason Reason);
