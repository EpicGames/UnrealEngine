// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Iris/Core/BitTwiddling.h"

// Define this to increase the maximum amount of NetObjectFactories that can exist
#ifndef UE_IRIS_MAX_NETOBJECT_FACTORIES
	#define UE_IRIS_MAX_NETOBJECT_FACTORIES 16
#endif

namespace UE::Net
{

typedef uint8 FNetObjectFactoryId;
enum { InvalidNetObjectFactoryId = MAX_uint8 };

static_assert((uint32)UE_IRIS_MAX_NETOBJECT_FACTORIES < (uint32)InvalidNetObjectFactoryId, "We currently don't support more then 255 unique factories");


/**
 * Keeps track of Iris NetObjectFactory templates
 * NetObjectFactories must all be registered before any Iris ReplicationSystem is created.
 */
class FNetObjectFactoryRegistry
{
public:

	/** Register a UNetObjectFactory class and associate with a specific name. */
	IRISCORE_API static void RegisterFactory(UClass* FactoryClass, FName FactoryName);

	/** Unregister the UNetObjectFactory class associated with the name */
	IRISCORE_API static void UnregisterFactory(FName FactoryName);

	/** Find the FNetFactoryID that was assigned to name on registration */
	IRISCORE_API static FNetObjectFactoryId GetFactoryIdFromName(FName FactoryName);

	IRISCORE_API static bool IsValidFactoryId(FNetObjectFactoryId Id);

	/** The engine sets this false after it created an Iris replication system since its now illegal to register new factories */
	IRISCORE_API static void SetFactoryRegistrationAllowed(bool bAllowed);

	/** Limit how many factories can be registered */
	static constexpr uint32 MaxFactories = UE_IRIS_MAX_NETOBJECT_FACTORIES;

	/** The amount of bits to serialize FNetFactoryIds with  */
	static consteval uint32 GetMaxBits() { return UE::Net::GetBitsNeeded(MaxFactories-1); }

	struct FFactoryData
	{
		/** Name associated with this factory class */
		FName Name;
		/** FactoryId assigned to this factory */
		FNetObjectFactoryId Id = InvalidNetObjectFactoryId;
		/** Class representing a concrete UNetObjectFactory */
		TWeakObjectPtr<UClass> NetFactoryClass;
	};

	/** The registered factories ready to be instantiated by the replication bridge */
	static const TConstArrayView<FFactoryData> GetRegisteredFactories() { return MakeConstArrayView(NetFactories.GetData(), NetFactories.Num()); }

private:
	
	FNetObjectFactoryRegistry() = delete;
	~FNetObjectFactoryRegistry() = delete;

private:

	static TArray<FFactoryData, TFixedAllocator<MaxFactories>> NetFactories;
};

} // end namespace UE::Net