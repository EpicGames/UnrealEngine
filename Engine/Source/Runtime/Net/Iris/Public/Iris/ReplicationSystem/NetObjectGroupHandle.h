// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Iris/IrisConfig.h"
#include "Containers/StringFwd.h"
#include "Templates/TypeHash.h"

// Forward declarations
class FString;

namespace UE::Net::Private
{
	class FNetObjectGroups;
}

namespace UE::Net
{

class FNetObjectGroupHandle
{
public:
	typedef uint32 FGroupIndexType;

	enum { GroupIndexBits = 24U };
	enum { EpochBits = 8U };
	enum { EpochMask = (1U << EpochBits) - 1U };
	enum { MaxGroupIndexCount = 1U << GroupIndexBits };

	/** Reserved group indices */
	enum : FGroupIndexType
	{
		InvalidNetObjectGroupIndex = 0,
		NotReplicatedNetObjectGroupIndex,
		AsyncStopReplicationNetObjectGroupIndex,
		NetGroupOwnerNetObjectGroupIndex,
		NetGroupReplayNetObjectGroupIndex
	};

	inline static FNetObjectGroupHandle GetInvalid() {return FNetObjectGroupHandle();}

	FNetObjectGroupHandle() : Value(0u) {}

	// $IRIS TODO: IsValid could be considered a shortcut to Group->IsValidGroup but it's abolutely not the same thing. Rename this to IsInitialized() to remove the confusion.
	/** Returns true if the handle is valid, note this does not mean that the group is valid */
	inline bool IsValid() const { return Value != 0u; }

	/** Returns the GroupIndex of the group associated with the handle */
	inline FGroupIndexType GetGroupIndex() const { return FGroupIndexType(Index); }

	/** Returns the unique id for this group */
	uint32 GetUniqueId() const { return UniqueId; }

	/** Returns true if the provided GroupIndex is a reserved NetObjectsGroupIndex */
	static bool IsReservedNetObjectGroupIndex(FGroupIndexType GroupIndex) { return GroupIndex >= NotReplicatedNetObjectGroupIndex && GroupIndex <= NetGroupReplayNetObjectGroupIndex; }

	/** Returns true if this is a reserved NetObjectsGroups */
	bool IsReservedNetObjectGroup() const { return FNetObjectGroupHandle::IsReservedNetObjectGroupIndex(Index); }

	/** Special group, NetHandles assigned to this group will be filtered out for all connections */
	bool IsNotReplicatedNetObjectGroup() const { return Index == NotReplicatedNetObjectGroupIndex; }

	/** Special group, NetHandles assigned to this group will be filtered out for all connections used for async end replication */
	bool IsAsyncStopReplicationNetObjectGroup() const { return Index == AsyncStopReplicationNetObjectGroupIndex; }

	/** Special group, SubObjects assigned to this group will replicate to owner of RootParent */
	bool IsNetGroupOwnerNetObjectGroup() const { return Index == NetGroupOwnerNetObjectGroupIndex; }

	/** Special group, SubObjects assigned to this group will replicate if replay netconditions is met  */
	bool IsNetGroupReplayNetObjectGroup() const { return Index == NetGroupReplayNetObjectGroupIndex; }

	uint64 GetRawValue() const
	{
		return Value;
	}
	
private:
	friend UE::Net::Private::FNetObjectGroups;

	FNetObjectGroupHandle(FGroupIndexType IndexIn, FGroupIndexType EpochIn, uint32 InUniqueId)
	{ 
		if (IndexIn == InvalidNetObjectGroupIndex)
		{
			*this = FNetObjectGroupHandle();
		}
		else
		{
			Index = (uint32)IndexIn;
			Epoch = (uint32)EpochIn;
			UniqueId = InUniqueId;
		}
	}

	union 
	{
		uint64 Value;
		struct
		{
			uint32 Index : GroupIndexBits;
			uint32 Epoch : EpochBits;
			uint32 UniqueId;
		};
	};

	friend inline bool operator==(const FNetObjectGroupHandle& Lhs, const FNetObjectGroupHandle& Rhs) { return Lhs.Value == Rhs.Value; }
	friend inline bool operator!=(const FNetObjectGroupHandle& Lhs, const FNetObjectGroupHandle& Rhs) { return Lhs.Value != Rhs.Value; }
};

static_assert(sizeof(FNetObjectGroupHandle) == sizeof(uint64), "FNetObjectGroupHandle must be of size 64bits.");

inline uint32 GetTypeHash(const FNetObjectGroupHandle Handle)
{
	return ::GetTypeHash(Handle.GetRawValue());
}
}
