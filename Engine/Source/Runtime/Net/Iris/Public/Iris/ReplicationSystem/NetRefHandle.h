// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Iris/IrisConfig.h"
#include "Iris/Core/BitTwiddling.h"
#include "Containers/StringFwd.h"
#include "Templates/TypeHash.h"

#ifndef UE_NET_IRIS_NETREFHANDLE_SERIAL_SIZE
	#define UE_NET_IRIS_NETREFHANDLE_SERIAL_SIZE 53
#endif

// Forward declarations
class FString;

namespace UE::Net::Private
{
	class FNetRefHandleManager;
}

namespace UE::Net
{

/**
 * FNetRefHandle
 */
class FNetRefHandle
{
public:
	
	inline static FNetRefHandle GetInvalid() { return FNetRefHandle(); }

private:
	static constexpr uint64 InvalidValue = 0;
	static constexpr uint64 StaticBits = 1;
	static constexpr uint64 SerialBits = UE_NET_IRIS_NETREFHANDLE_SERIAL_SIZE;
	static constexpr uint64 IdBits = (StaticBits + SerialBits);
	static constexpr uint64 ReplicationSystemIdBits = 10;

	static_assert(SerialBits > 0, "The number of bits representing the FNetRefHandle serial must be greater than zero");
	static_assert(IdBits <= 64, "The number of bits representing the FNetRefHandle id cannot be greater than 64-bits");

public:

	static constexpr uint64 MaxReplicationSystemId = UE::Net::LargestBitValueForBits(ReplicationSystemIdBits);
	static constexpr uint64 MaxReplicationSystemCount = MaxReplicationSystemId + 1;
	static constexpr uint64 MaxId = UE::Net::LargestBitValueForBits(IdBits);
	static constexpr uint64 MaxSerial = UE::Net::LargestBitValueForBits(SerialBits);

	uint64 GetId() const { return (Fields.Serial << 1) | (Fields.Static & 1); }
	uint64 GetSerial() const { return Fields.Serial; }
	uint32 GetReplicationSystemId() const { check(Fields.ReplicationSystemId != 0); return (uint32)(Fields.ReplicationSystemId - 1); }
	bool IsValid() const { return Fields.Serial != InvalidValue; }

	/** Does the handle know which ReplicationSystem it is related to. */
	bool IsCompleteHandle() const { return Fields.Serial != InvalidValue && Fields.ReplicationSystemId != 0U; }

	/** Static handles have ODD Id's */
	bool IsStatic() const { return Fields.Static; }

	/** Dynamic handles have EVEN Id's */
	bool IsDynamic() const { return IsValid() && !IsStatic(); }

	bool operator==(const FNetRefHandle& Other)const { return GetId() == Other.GetId(); }
	bool operator<(const FNetRefHandle& Other)const { return GetId() < Other.GetId(); }
	bool operator!=(const FNetRefHandle& Other)const { return GetId() != Other.GetId(); }

	IRISCORE_API FString ToString() const;
	IRISCORE_API FString ToCompactString() const;

	static bool FullCompare(FNetRefHandle A, FNetRefHandle B) 
	{
		return A.Fields.Static == B.Fields.Static &&
			A.Fields.Serial == B.Fields.Serial &&
			A.Fields.ReplicationSystemId == B.Fields.ReplicationSystemId;
	}
	
private:
	friend uint32 GetTypeHash(const FNetRefHandle& Handle);
	friend Private::FNetRefHandleManager;

	struct
	{
		uint64 Static : StaticBits = InvalidValue;
		uint64 Serial : SerialBits = InvalidValue;
		uint64 ReplicationSystemId : ReplicationSystemIdBits = InvalidValue;
	} Fields;
};

inline uint32 GetTypeHash(const FNetRefHandle& Handle)
{
	return ::GetTypeHash(Handle.GetId());
}

inline uint64 GetObjectIdForNetTrace(const FNetRefHandle& Handle)
{
	return Handle.GetId();
}

}

IRISCORE_API FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const UE::Net::FNetRefHandle& NetHandle);
IRISCORE_API FAnsiStringBuilderBase& operator<<(FAnsiStringBuilderBase& Builder, const UE::Net::FNetRefHandle& NetHandle);
IRISCORE_API FArchive& operator<<(FArchive& Ar, UE::Net::FNetRefHandle& RefHandle);
