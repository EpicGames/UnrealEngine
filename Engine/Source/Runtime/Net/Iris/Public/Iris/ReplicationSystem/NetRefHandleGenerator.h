// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "Net/Core/Connection/NetEnums.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"
#include "NetRefHandleGenerator.generated.h"

namespace UE::Net
{
	struct IRISCORE_API FNetRefHandleGeneratorBindParams
	{
		/**
		 * If set, this delegate will override the delegate in FReplicationSystemParams::ObjectToNetRefHandleSerial.
		 */
		FObjectToNetRefHandleSerial* ObjectToNetRefHandleSerial = nullptr;

		/**
		 * If set, this delegate will override the delegate in FReplicationSystemParams::NetRefHandleSerialToObject.
		 */
		FNetRefHandleSerialToObject* NetRefHandleSerialToObject = nullptr;
	};
}

UCLASS(Abstract, Transient)
class IRISCORE_API UNetRefHandleGenerator : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UNetRefHandleGenerator() = default;

	/** 
	 * This function is used to bind bespoke functions to the delegates in Params.
	 */
	virtual void Bind(UE::Net::FNetRefHandleGeneratorBindParams& Params) PURE_VIRTUAL(UNetRefHandleGenerator::Bind, );
};

/** 
 * Generate a FNetRefHandle serial using an object's corresponding FRemoteObjectId.
 *
 * This generator requires UE_WITH_REMOTE_OBJECT_HANDLE to be defined.
 */
UCLASS()
class IRISCORE_API URemoteObjectNetRefHandleGenerator : public UNetRefHandleGenerator
{
	GENERATED_BODY()

public:
	virtual void Bind(UE::Net::FNetRefHandleGeneratorBindParams& Params) override;
};

/** 
 * Generate a FNetRefHandle serial for use in a multi-server environment that isn't using UE_WITH_REMOTE_OBJECT_HANDLE.
 *
 * The FNetRefHandle serial will reserve bits for a server id, and the remaining bits of the serial
 * will be used for an auto-incrementing serial number.
 */ 
UCLASS(Config=Engine, DefaultConfig)
class IRISCORE_API UMultiServerNetRefHandleGenerator : public UNetRefHandleGenerator
{
	GENERATED_BODY()

public:
	virtual void Bind(UE::Net::FNetRefHandleGeneratorBindParams& Params) override;

private:
	uint64 NextSerial = 1;

	UPROPERTY(Config)
	uint64 NumServerIdBits = 4;

	UPROPERTY(Config)
	uint32 ServerId = 0;

	UPROPERTY(Config)
	uint32 InitialSerial = 1;
};
