// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Iris/IrisConfig.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Net/Core/NetToken/NetToken.h"
#include "UObject/NameTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Templates/Tuple.h"

#include "NetTokenStore.generated.h"

/** 
* The idea with NetTokens is to allow export of "stable"-pieces of data such as string and names by replacing them with a NetToken during quantization/serialization.
* The data associated with a NetToken is then exported separately from the data. As soon as the exported data has been acknowledged the data will only be serialized using the NetToken.
* 
* As both Client and Server can communicate NetTokens, each side can end up assigning different tokens that will differs from each other.
* Here is a high-level overview of the algorithm which works slightly differently based on if using Iris replication or the old replication system.
*
* The sending side looks-up or creates a NetToken for the Data being serialized. Servers will mark assigned NetTokens as authoritative, while clients generate a temporary NetToken.
* When a network bunch/batch that contains NetToken is being sent, there is a per-connection look-up to see if we need to append and serialize exports or not, if the data associated with the token
* has been acknowledged the token will not be exported again.
*
* On the receiving side, imported exports are always guaranteed to have been processed before we attempt to read received data containing NetTokens which allows
* the receiving side to resolve the NetToken to get the actual data.
*
* The implementation details differs a bit depending on if we are using iris replication or old style replication.
*
* A current example that us used by both systems is GameplayTags, For Iris: See GameplayTagNetSerializer.cpp, for the old replication system: See: GameplayTagContainer.cpp.
*
*/

/* NetBitArray validation support. */
#ifndef UE_NET_VALIDATE_NETTOKENTYPE
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#	define UE_NET_VALIDATE_NETTOKENTYPE 0
#else
#	define UE_NET_VALIDATE_NETTOKENTYPE 1
#endif
#endif


class UNetTokenDataStream;
namespace UE::Net
{
	class FNetTokenDataStore;
	class FNetTokenStore;
	class FNetTokenStoreState;
	class FNetSerializationContext;

	namespace Private
	{
		class FNetExportContext;
	}
}

USTRUCT()
struct FNetTokenStoreTypeIdPair
{
	GENERATED_BODY()
	UPROPERTY()
	FString StoreTypeName;
	UPROPERTY()
	uint32 TypeID = 0;
	bool operator<(const FNetTokenStoreTypeIdPair& Other) const
	{
		return TypeID < Other.TypeID;
	}
};
	
UCLASS(Transient, MinimalAPI, config=Engine)
class UNetTokenTypeIdConfig : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Config)
	TArray<FNetTokenStoreTypeIdPair> ReservedTypeIds;

	uint32 GetTypeID(const FString& TypeName) const;
	uint32 GetTypeID(const FName& TypeName) const
	{
		return GetTypeID(TypeName.ToString());
	}
	
protected:
	bool ReservedTypeIdsAppearValid() const;
};

namespace UE::Net
{
/**
 * NetTokenDataStore
 * Implemented per type to store and serialize data associated with a NetToken.
*/
class FNetTokenDataStore
{
public:
	class FNetTokenStoreKey
	{
	public:
		enum { InvalidKeyIndex = 0U };

		FNetTokenStoreKey()
		: KeyIndex(InvalidKeyIndex)
		{
		}

		explicit FNetTokenStoreKey(uint32 InKeyIndex)
		: KeyIndex(InKeyIndex)
		{
		}

		uint32 GetKeyIndex() const
		{ 
			return KeyIndex;
		}

		bool IsValid() const
		{ 
			return KeyIndex != InvalidKeyIndex;
		}

		bool operator==(const FNetTokenStoreKey& Other) const = default;

	private:
		uint32 KeyIndex = InvalidKeyIndex;
	};

	IRISCORE_API virtual ~FNetTokenDataStore();

	// Serialization methods for NetTokens that does not include the TypeId as this is known by the NetTokenDataStore.

	// Write NetToken
	IRISCORE_API void WriteNetToken(UE::Net::FNetSerializationContext& Context, FNetToken Token) const;

	// Read NetToken
	IRISCORE_API FNetToken ReadNetToken(UE::Net::FNetSerializationContext& Context);

	// Write NetToken
	IRISCORE_API void WriteNetToken(FArchive& Ar, FNetToken Token);

	// Read NetToken
	IRISCORE_API FNetToken ReadNetToken(FArchive& Ar);

protected:

	explicit IRISCORE_API FNetTokenDataStore(FNetTokenStore& InTokenStore);

	virtual void WriteTokenData(FNetSerializationContext& Context, FNetTokenStoreKey Key) const = 0;
	virtual FNetTokenDataStore::FNetTokenStoreKey ReadTokenData(FNetSerializationContext& Context, const FNetToken& NetToken) = 0;

	//Do not export things through the UPackageMap. This will break things.
	virtual void WriteTokenData(FArchive& Ar, FNetTokenStoreKey Key, UPackageMap* Map = nullptr) const = 0;
	virtual FNetTokenDataStore::FNetTokenStoreKey ReadTokenData(FArchive& Ar, const FNetToken& NetToken, UPackageMap* Map = nullptr) = 0;
	
	IRISCORE_API FNetTokenDataStore::FNetTokenStoreKey GetTokenKey(FNetToken Token, const FNetTokenStoreState& TokenStoreState) const;

	inline FNetToken::FTypeId GetTypeId() const
	{
		return TypeId;
	}

	// Create new NetToken
	IRISCORE_API FNetToken CreateAndStoreTokenForKey(FNetTokenStoreKey Key);
	IRISCORE_API void StoreTokenForKey(FNetTokenStoreKey Key, FNetToken NetToken);
	IRISCORE_API FNetToken GetNetTokenFromKey(FNetTokenStoreKey) const;

	// Allocate next TokenStoreKey
	FNetTokenDataStore::FNetTokenStoreKey GetNextNetTokenStoreKey();

protected:
	// Maps from FNetTokenStoreKey (index) to NetToken, this typically is the locally assigned NetToken, but can be overridden if we receive a token from the authority.
	TArray<FNetToken> StoredTokens;
	FNetTokenStore& TokenStore;

private:
	friend FNetTokenStore;
	FNetToken::FTypeId TypeId;
};

/**
* A helper RAII class to allow thread-safe access to NetTokenStore
* When making calls to the FNetTokenStore in a parallel phase, first create FScopedNetTokenStoreAccess to
* ensure that the access is threadsafe during the lifetime of the FScopedNetTokenStoreAccess object.
*/
class FScopedNetTokenStoreAccess
{
public:
	IRISCORE_API explicit FScopedNetTokenStoreAccess(const FNetTokenStore* InTokenStore);

	IRISCORE_API ~FScopedNetTokenStoreAccess();

private:
	const FNetTokenStore* TokenStore = nullptr;
};

/**
 * FNetTokenStore
 * Main system for using NetTokensm currently owns type specific NetTokenDataStores and per connection state
 * Currently we have a unique instance per NetDriver/ReplicationSystem but it is possible we will share this across game instance.
 */
class FNetTokenStore
{
	UE_NONCOPYABLE(FNetTokenStore);
public:
	IRISCORE_API FNetTokenStore();
	IRISCORE_API ~FNetTokenStore();

	/** External configuration variables used to initialize the NetTokenStore */
	struct FInitParams
	{
		FNetToken::ENetTokenAuthority Authority;
		uint32 MaxConnections = 256;
	};
	IRISCORE_API void Init(FInitParams& InitParams);

	/** Returns true if this is the authority, typically the server. */
	bool IsAuthority() const
	{
		return Params.Authority == FNetToken::ENetTokenAuthority::Authority;
	}

	/** A token is local if the authority of the NetTokenStore and the token matches, Invalid tokens are always local. */
	bool IsLocalToken(const FNetToken NetToken) const
	{
		return !NetToken.IsValid() || IsAuthority() == NetToken.IsAssignedByAuthority();
	}

	/** Register DataStore and return true if it was registered. */
	IRISCORE_API bool RegisterDataStore(TUniquePtr<FNetTokenDataStore> DataStore, FName TokenStoreName);

	/** UnRegister  a DataStore and return true if it was registered. */
	IRISCORE_API bool UnRegisterDataStore(FName TokenStoreName);

	/** Get const access to data store by name. */
	IRISCORE_API const FNetTokenDataStore* GetDataStore(FName Name) const;

	/** Get data store by name. */
	IRISCORE_API FNetTokenDataStore* GetDataStore(FName Name);

	/** Create and return data store of specified type, it will be owned by NetTokenStore. */
	template<typename T>
	bool CreateAndRegisterDataStore()
	{
		TUniquePtr<FNetTokenDataStore> NetTokenDataStore(MakeUnique<T>(*this));
		
		return RegisterDataStore(MakeUnique<T>(*this), T::GetTokenStoreName());
	}
	
	/** Unregister a data store of specified type. */
	template<typename T>
	bool DeleteAndUnRegisterDataStore()
	{
		return UnRegisterDataStore(T::GetTokenStoreName());
	}

	/** Return data store of specified type. */
	template<typename T>
	T* GetDataStore()
	{
		return static_cast<T*>(GetDataStore(T::GetTokenStoreName()));
	}

	/** Return data store of specified type. */
	template<typename T>
	const T* GetDataStore() const
	{
		return static_cast<const T*>(GetDataStore(T::GetTokenStoreName()));
	}
	
	// FNetTokenStoreState maps from NetTokenIndex -> NetTokenStoreKey (Index)
	// Remote and local NetTokens use separate NetTokensStore states.

	/** Get const access to the local NetTokenStoreState */
	const FNetTokenStoreState* GetLocalNetTokenStoreState() const
	{
		return LocalNetTokenStoreState.Get();
	}

	/** Get access to the local NetTokenStoreState */
	FNetTokenStoreState* GetLocalNetTokenStoreState()
	{
		return LocalNetTokenStoreState.Get();
	}

	/** Init RemoteNetTokenStoreState for given ConnectionId, if it already exists it will be reset. */
	IRISCORE_API void InitRemoteNetTokenStoreState(uint32 ConnectionId);

	/** Get const access to RemoteNetTokenStoreState for given ConnectionId. */
	IRISCORE_API const FNetTokenStoreState* GetRemoteNetTokenStoreState(uint32 ConnectionId) const;

	/** Get RemoteNetTokenStoreState for given ConnectionId. */
	IRISCORE_API FNetTokenStoreState* GetRemoteNetTokenStoreState(uint32 ConnectionId);

	/** Write data associated with the NetToken. */
	IRISCORE_API void WriteTokenData(FNetSerializationContext& Context, const FNetToken NetToken) const;

	/** Read data associated with the NetToken. */
	IRISCORE_API void ReadTokenData(FNetSerializationContext& Context, const FNetToken NetToken, FNetTokenStoreState& RemoteNetTokenStoreState);

	/** Write data associated with the NetToken. Do not export anything via the UPackageMap. */
	IRISCORE_API void WriteTokenData(FArchive& Ar, const FNetToken NetToken, UPackageMap* Map = nullptr) const;
	
	/** Read data associated with the NetToken. Do not export anything via the UPackageMap. */
	IRISCORE_API void ReadTokenData(FArchive& Ar, const FNetToken NetToken, FNetTokenStoreState& RemoteNetTokenStoreState, UPackageMap* Map = nullptr);

	/** Conditionally write NetTokenData unless already exported. */
	IRISCORE_API void ConditionalWriteNetTokenData(FNetSerializationContext& Context, Private::FNetExportContext* ExportContext, const FNetToken NetToken) const;

	/** Conditionally read NetTokenData if exported. */
	IRISCORE_API void ConditionalReadNetTokenData(FNetSerializationContext& Context, const FNetToken NetToken);

	/** Utility methods, consolidate with other changes to NetTokenStore as next step. */
	IRISCORE_API static void AppendExport(FNetSerializationContext&, FNetToken NetToken);

	/** 
	 * Convenience method to Write a NetToken without writing type bits
	 * In development builds it will verify that the token type matches the StoreType, otherwise it will skip the lookup of DataStore.
	 */
	template <typename T>
	void WriteNetTokenWithKnownType(FNetSerializationContext& Context, FNetToken NetToken)
	{
#if UE_NET_VALIDATE_NETTOKENTYPE
		// We only verify this in dev builds as it should trap most programmer errors.
		if (NetToken.IsValid())
		{
			const T* DataStore = GetDataStore<T>();
			if (!DataStore || (DataStore->GetTypeId() != NetToken.GetTypeId()))
			{
				UE_LOGF(LogNetToken, Error, "Tried to write NetToken %ls using invalid NetTokenStore %ls", *NetToken.ToString(), *(T::GetTokenStoreName().ToString()));
				// Just to get some attention to the log.
				ensure(DataStore && (DataStore->GetTypeId() == NetToken.GetTypeId()));
			}
		}
#endif
		const bool bWriteTypeId = false;
		FNetTokenStore::InternalWriteNetToken(Context, NetToken, bWriteTypeId);
	}

	/** 
	 * Convenience method to read a NetToken without reading type bits.
	*/
	template <typename T>
	FNetToken ReadNetTokenWithKnownType(FNetSerializationContext& Context)
	{
		FNetToken NetToken;
		if (T* DataStore = GetDataStore<T>())
		{
			NetToken = DataStore->ReadNetToken(Context);
		}
		else
		{
			UE_LOGF(LogNetToken, Error, "ReadNetTokenWithKnownType Tried to read NetToken using invalid NetTokenStore %ls", *(T::GetTokenStoreName().ToString()));
			// Just to get some attention to the log.
			ensure(DataStore);
		}

		return NetToken;
	}

	/** Write NetToken including TypeId */
	void WriteNetToken(FNetSerializationContext& Context, FNetToken NetToken) const
	{
		const bool bWriteTypeId = true;
		FNetTokenStore::InternalWriteNetToken(Context, NetToken, bWriteTypeId);
	}

	/** Read NetToken including TypeId */
	FNetToken ReadNetToken(FNetSerializationContext& Context) const
	{
		return InternalReadNetToken(Context, FNetToken::InvalidTokenTypeId);
	}

	/** Write NetToken including TypeId */
	void WriteNetToken(FArchive& Ar, FNetToken NetToken) const
	{
		const bool bWriteTypeId = true;
		FNetTokenStore::InternalWriteNetToken(Ar, NetToken, bWriteTypeId);
	}

	/** Read NetToken including TypeId */
	FNetToken ReadNetToken(FArchive& Ar) const
	{
		return InternalReadNetToken(Ar, FNetToken::InvalidTokenTypeId);
	}

#if UE_SUPPORT_PARALLEL_IRIS
protected:
	friend FScopedNetTokenStoreAccess;

	//Coarse grain mutex which is used to protect access of all NetTokenStore, NetTokenStoreState and NetTokenDataStore. 
	mutable FTransactionallySafeCriticalSection NetTokenStoreCS;
#endif

private:

	friend UNetTokenDataStream;
	friend FNetTokenDataStore;

	using FNetTokenStoreKey = FNetTokenDataStore::FNetTokenStoreKey;

	// Internal method to write a NetToken, if bWriteTokenType is true it will write the TypeId as well.
	IRISCORE_API static void InternalWriteNetToken(UE::Net::FNetSerializationContext& Context, FNetToken Token, bool bWriteTokenType);
	IRISCORE_API static void InternalWriteNetToken(FArchive& Ar, FNetToken Token, bool bWriteTokenType);

	// Internal method to read a NetToken, if the TokenTypeId is valid it will be used instead of reading it from the bitstream
	IRISCORE_API static FNetToken InternalReadNetToken(UE::Net::FNetSerializationContext& Context, FNetToken::FTypeId TokenTypeId);
	IRISCORE_API static FNetToken InternalReadNetToken(FArchive& Ar, FNetToken::FTypeId TokenTypeId);

	bool ValidateAndStoreNetTokenData(FNetTokenDataStore& DataStore, FNetTokenStoreState& RemoteNetTokenStoreState, const FNetToken NetToken, const FNetTokenStoreKey StoreKey);
	
	TArray<FNetToken> GetAllNetTokens() const;

	static FNetToken MakeNetToken(uint32 TypeId, uint32 Index, FNetToken::ENetTokenAuthority Authority)
	{ 
		if ((TypeId < FNetToken::MaxTypeIdCount) && (Index < FNetToken::MaxNetTokenCount))
		{
			return FNetToken(TypeId, Index, Authority);
		}
		
		return FNetToken();
	}

	TUniquePtr<FNetTokenStoreState> LocalNetTokenStoreState;
	TArray<TUniquePtr<FNetTokenStoreState>> RemoteNetTokenStoreStates;
	TArray<TTuple<FName, TUniquePtr<FNetTokenDataStore>>> TokenDataStores;
	FInitParams Params;
};

inline FNetTokenDataStore::FNetTokenStoreKey FNetTokenDataStore::GetNextNetTokenStoreKey()
{
	UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(&TokenStore));
	uint32 NextTokenStoreKeyIndex = (uint32)StoredTokens.Num();
	if (ensure(NextTokenStoreKeyIndex < FNetToken::MaxNetTokenCount))
	{
		FNetTokenStoreKey TokenKey(NextTokenStoreKeyIndex);

		// We fill in the actual key later
		StoredTokens.Add(FNetToken());

		return TokenKey;
	}

	return FNetTokenStoreKey();
}
}
