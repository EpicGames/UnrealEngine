// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"

namespace UE::Net
{

template<typename T>
struct TNetTokenStructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{ 
	enum
	{ 
		WithNetSerializer = true,
		WithIdenticalViaEquality = true,
		WithNetSharedSerialization = true,
	};
};

};
	
// Optional, declares a default Native NetSerializer StructOpts and IRIS NetSerializer methods that serializes the struct as a NetToken by default.
#define UE_NET_DECLARE_NAMED_NETTOKEN_STRUCT_SERIALIZERS(NAME, API) \
	template<> struct TStructOpsTypeTraits<F##NAME> : public UE::Net::TNetTokenStructOpsTypeTraits<F##NAME> {}; \
	namespace UE::Net \
	{ \
		UE_NET_DECLARE_SERIALIZER(F##NAME##NetSerializer, API);\
	}

// Optional, declares a default native NetSerializer method, GetTokenStoreName method and Equality operators (for equality via identity) for a NetToken Struct type
#define UE_NET_NETTOKEN_GENERATED_BODY(NAME, API) \
	inline static FName TokenStoreName = TEXT( UE_STRINGIZE(F##NAME) );\
	static FName GetTokenStoreName() \
	{ \
		return TokenStoreName; \
	} \
	API bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess); \
	API bool operator==(const F##NAME& Other) const; \
	API bool operator!=(const F##NAME& Other) const; 

// Optional, implements a default NetSerializer method and Native IRIS NetSerializer methods that works for NetToken types by default
#define UE_NET_IMPLEMENT_NAMED_NETTOKEN_STRUCT_SERIALIZERS(NAME) \
	bool F##NAME::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) \
	{ \
		bOutSuccess = bOutSuccess && UE::Net::TStructNetTokenDataStoreHelper<F##NAME>::NetSerializeAndExportToken(Ar, Map, *static_cast<F##NAME*>(this)); \
		return true; \
	} \
	bool F##NAME::operator==(const F##NAME& Other) const \
	{ \
		return GetUniqueKey() == Other.GetUniqueKey(); \
	}; \
	bool F##NAME::operator!=(const F##NAME& Other) const \
	{ \
		return GetUniqueKey() != Other.GetUniqueKey(); \
	} \
	namespace UE::Net { \
		struct F##NAME##NetSerializer : public TStructAsNetTokenNetSerializerImpl<F##NAME> { \
			static const uint32 Version = 0; \
			static inline const ConfigType DefaultConfig = ConfigType(); \
		};\
		UE_NET_IMPLEMENT_SERIALIZER(F##NAME##NetSerializer); \
		UE_NET_IMPLEMENT_FORWARDING_NETSERIALIZER_AND_REGISTRY_DELEGATES(NAME, F##NAME##NetSerializer); \
	}