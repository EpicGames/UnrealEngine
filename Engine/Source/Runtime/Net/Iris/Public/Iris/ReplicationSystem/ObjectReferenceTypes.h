// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Core/BitTwiddling.h"

#include "ObjectReferenceTypes.generated.h"

// From UObjectGlobals.h
typedef int32 TAsyncLoadPriority;

UENUM()
enum class EIrisAsyncLoadingPriority : uint8
{
	/** The default loading priority used for all types. Generally corresponds to 0 */
	Default,
	/** A loading priority setting for important classes. */
	High,
	/** A loading priority setting for critical classes. */
	VeryHigh,
	/** Maximum possible value to the calculate minimum bits needed to serialize the enum */
	Max = VeryHigh,
	/** Don't use this in the config. Used in code to tell if a value was read and set. */
	Invalid,
};

static uint32 GetIrisAsyncLoadingPriorityBits()
{
	return UE::Net::GetBitsNeeded(EIrisAsyncLoadingPriority::Max);
}

TAsyncLoadPriority ConvertAsyncLoadingPriority(EIrisAsyncLoadingPriority IrisPriority);

static const TCHAR* LexToString(EIrisAsyncLoadingPriority Priority)
{
	switch(Priority)
	{
		case EIrisAsyncLoadingPriority::Default:
		{
			return TEXT("Default");
		} break;

		case EIrisAsyncLoadingPriority::High:
		{
			return TEXT("High");
		} break;

		case EIrisAsyncLoadingPriority::VeryHigh:
		{
			return TEXT("VeryHigh");
		} break;

		case EIrisAsyncLoadingPriority::Invalid:
		{
			return TEXT("Invalid");
		} break;

		default:
		{
			ensure(false);
			return TEXT("Missing");
		} break;
	}
}