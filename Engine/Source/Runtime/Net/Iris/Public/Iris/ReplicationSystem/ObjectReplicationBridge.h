// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "Iris/ReplicationSystem/ReplicationBridge.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"
#include "Iris/ReplicationSystem/NetObjectFactory.h"
#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"
#include "Iris/Serialization/NetErrorContext.h"

#include "Containers/ArrayView.h"
#include "Containers/Set.h"

#include "Delegates/IDelegateInstance.h"

#include "Misc/EnumClassFlags.h"

#include "Net/Core/NetBitArray.h"
#include "Net/Core/NetHandle/NetHandle.h"

#include "ObjectReplicationBridge.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogIrisFilterConfig, Log, All);

class UProxyBackendNetDriver;

namespace UE::Net
{
	enum class ENetRefHandleError : uint32;
	enum class EReplicationFragmentTraits : uint32;

	struct FNetObjectResolveContext;

	typedef uint32 FNetObjectFilterHandle;
	typedef uint32 FNetObjectPrioritizerHandle;
	typedef uint32 FReplicationProtocolIdentifier;

	class FNetObjectReference;
	class FWorldLocations;

	namespace Private
	{
		enum class EInternalDetachReason : uint8;

		class FObjectPollFrequencyLimiter;
		class FObjectPoller;
		class FReplicationSystemImpl;
		class FBridgeSerialization;
	}

	class FReplicationSystemProxyTestFixture;

	using FOnRootObjectPostInit = TMulticastDelegate<void(FNetRefHandle RootObjectHandle, UObject* RootObject, FNetObjectFactoryId FactoryId)>;
	using FOnRootObjectDetached = TMulticastDelegate<void(FNetRefHandle RootObjectHandle, UObject* RootObject, EDetachReason DetachReason)>;
	using FOnSubObjectPostInit = TMulticastDelegate<void(FNetRefHandle SubObjectHandle, UObject* SubObject, FNetRefHandle RootObjectHandle, UObject* RootObject, FNetObjectFactoryId FactoryId)>;
	using FOnSubObjectDetached = TMulticastDelegate<void(FNetRefHandle SubObjectHandle, UObject* SubObject, FNetRefHandle RootObjectHandle, UObject* RootObject, EDetachReason DetachReason)>;
}

namespace UE::Net::Private
{

/** Info representing destroyed static objects */
struct FStaticDestructionInfo
{
	/** The reference to the static object that is considered destroyed */
	UE::Net::FNetObjectReference StaticRef;
	
	/** The container the object was a part of */
	UE::Net::FNetObjectGroupHandle ContainerGroupHandle;
	
	/** The NetFactory assigned to this object */
	UE::Net::FNetObjectFactoryId NetFactoryId = UE::Net::InvalidNetObjectFactoryId;

	/** The repindex assigned to this destruction info */
	UE::Net::FInternalNetRefIndex InternalReplicationIndex;
};

/** 
 * Delegates that are triggered as part of root object and sub-object lifecycles in the replication bridge.
 *
 * These delegates are currently used to enable a proxy server to start and stop replicating objects in the frontend
 * replication system when they are attached or detached from a backend replication system.
 */
class FObjectReplicationBridgeDelegates
{
private:

	/** (Client only) After a root object has been attached and initialized. */
	IRISCORE_API static FOnRootObjectPostInit::RegistrationType& GetRootObjectPostInitDelegate(UObjectReplicationBridge* Bridge);
	/** (Client only) Before a root object is detached; any sub-objects that are part of this root will also be detached before the root. */
	IRISCORE_API static FOnRootObjectDetached::RegistrationType& GetRootObjectDetachedDelegate(UObjectReplicationBridge* Bridge);
	/** (Client only) After a sub-object has been attached and initialized. */
	IRISCORE_API static FOnSubObjectPostInit::RegistrationType& GetSubObjectPostInitDelegate(UObjectReplicationBridge* Bridge);
	/** (Client only) Before a sub-object is detached, either on it's own or before it's parent root object is detached. */
	IRISCORE_API static FOnSubObjectDetached::RegistrationType& GetSubObjectDetachedDelegate(UObjectReplicationBridge* Bridge);

	friend class ::UProxyBackendNetDriver;
	friend class UE::Net::FReplicationSystemProxyTestFixture;

};

} // end namespace UE::Net::Private

struct FObjectReplicationBridgeInstantiateResult
{
	UObject* Object = nullptr;
	EReplicationBridgeCreateNetRefHandleResultFlags Flags = EReplicationBridgeCreateNetRefHandleResultFlags::None;
};

/*
* Partial implementation of ReplicationBridge that can be used as a foundation for 
* implementing support for replicating objects derived from UObject
*/
UCLASS(Transient, MinimalApi)
class UObjectReplicationBridge : public UReplicationBridge
{
	GENERATED_BODY()

public:

	using EGetRefHandleFlags = UE::Net::EGetRefHandleFlags;

	using FRootObjectReplicationParams = UE::Net::FRootObjectReplicationParams;
	struct FSubObjectReplicationParams;

	IRISCORE_API UObjectReplicationBridge();

	/** Get the Object from a replicated handle, if the handle is invalid or not is a replicated handle the function will return nullptr */
	IRISCORE_API UObject* GetReplicatedObject(FNetRefHandle Handle) const;

	/** Get NetRefHandle from a replicated UObject. */
	IRISCORE_API FNetRefHandle GetReplicatedRefHandle(const UObject* Object, EGetRefHandleFlags GetRefHandleFlags = EGetRefHandleFlags::None) const;

	/** Get NetRefHandle from a NetHandle. */
	IRISCORE_API FNetRefHandle GetReplicatedRefHandle(FNetHandle Handle) const;

	/** Try to resolve UObject from NetObjectReference, this function tries to resolve the object by loading if necessary. */
	IRISCORE_API UObject* ResolveObjectReference(const UE::Net::FNetObjectReference& ObjectRef, const UE::Net::FNetObjectResolveContext& ResolveContext);

	/** Describe the NetObjectReference */
	IRISCORE_API FString DescribeObjectReference(const UE::Net::FNetObjectReference& ObjectRef, const UE::Net::FNetObjectResolveContext& ResolveContext);

	/** Get or create NetObjectReference for object instance. */
	IRISCORE_API UE::Net::FNetObjectReference GetOrCreateObjectReference(const UObject* Instance) const;

	/** Get or create NetObjectReference for object identified by path relative to outer. */
	IRISCORE_API UE::Net::FNetObjectReference GetOrCreateObjectReference(const FString& Path, const UObject* Outer) const;

	/** 
	 * Start replicating a RootObject and return a valid NetRefHandle if successful. 
	 * 
	 * @param Instance The instance that needs to be replicated
	 * @param Params Optional configuration parameters to specify how the root object will be replicated.
	 * @param NetFactoryId NetFactory to use for this object.
	 * @return The NetRefHandle associated to this object if the operation succeeded.
	 */
	UE_DEPRECATED(5.8, "Use version not taking a FRootObjectReplicationParams parameter. The responsibility of filling in these params has moved to UNetObjectFactory.")
	IRISCORE_API FNetRefHandle StartReplicatingRootObject(UObject* Instance, const FRootObjectReplicationParams& Params, UE::Net::FNetObjectFactoryId NetFactoryId);

	/** 
	 * Start replicating a RootObject and return a valid NetRefHandle if successful. 
	 * 
	 * @param Instance The instance that needs to be replicated
	 * @param NetFactoryId NetFactory to use for this object.
	 * @return The NetRefHandle associated to this object if the operation succeeded.
	 */
	IRISCORE_API FNetRefHandle StartReplicatingRootObject(UObject* Instance, UE::Net::FNetObjectFactoryId NetFactoryId);

	/**
	 * Start replicating a SubObject and return a valid NetRefHandle if successful.
	 * 
	 * @param Instance, the object that should be replicated as a SubObject
	 * @param Params, See FSubObjectReplicationParams for details.
	 * @param NetFactoryId NetFactory to use for this SubObject.
	 * @return The NetRefHandle of the subobject if successful.
	 */
	IRISCORE_API FNetRefHandle StartReplicatingSubObject(UObject* Instance, const FSubObjectReplicationParams& Params, UE::Net::FNetObjectFactoryId NetFactoryId);

	/** 
	 * Stop replicating any type of NetObject. @see UReplicationBridge::StopReplicatingNetObject for more details.
	 * @param Instance The instance that won't be replicated anymore
	 * @param EndReplicationFlags Optional settings to modify the function behavior. Defaults to destroy the  instance on remote clients.
	 */
	IRISCORE_API void StopReplicatingNetObject(UObject* Instance, EEndReplicationFlags EndReplicationFlags = EEndReplicationFlags::Destroy);

	/** 
	 * Set NetCondition for a subobject, the condition is used to determine if the SubObject should replicate or not.
	 * @note As the filtering is done at the serialization level it is typically more efficient to use a separate NetObject for connection 
	 * specific data as filtering can then be done at a higher level.
	 */
	IRISCORE_API void SetSubObjectNetCondition(FNetRefHandle SubObjectHandle, ELifetimeCondition Condition);

	/** Returns true if the handle is valid and associated with a root object */
	IRISCORE_API bool IsRootObject(FNetRefHandle RootObjectHandle) const;

	/** 
	 * Returns the root object handle associated with any object. 
	 * If the passed handle is a subobject it will return its root.  
	 * If the passed handle is a rootobject it will return the same handle.
	 */
	IRISCORE_API FNetRefHandle GetRootObjectOfAnyObject(FNetRefHandle NetRefHandle) const;

	/** Get the handle of the root object of any replicated subobject. SubObjectHandle MUST be binded to a subobject */
	IRISCORE_API FNetRefHandle GetRootObjectOfSubObject(FNetRefHandle SubObjectHandle) const;

	/** Returns the associated factory created for this bridge. */
	IRISCORE_API UNetObjectFactory* GetNetFactory(UE::Net::FNetObjectFactoryId FactoryId) const;

	/** Add static destruction info, this is used when stably named objects are destroyed prior to starting replication */
	IRISCORE_API void AddStaticDestructionInfo(const FString& ObjectPath, const UObject* Outer, const UE::Net::FDestructionParameters& Parameters);

	/** Store static destruction info for the replicated object. */
	IRISCORE_API FNetRefHandle StoreDestructionInfo(FNetRefHandle Handle, const UE::Net::FDestructionParameters& Parameters);

	/** Remove static destruction info for the replicated object. */
	IRISCORE_API void RemoveDestructionInfo(FNetRefHandle Handle);
	
	/** PIE package name remapping support. */
	virtual bool RemapPathForPIE(uint32 ConnectionId, FString& Path, bool bReading) const { return false; }

	/** Returns true of the level that the Object belongs to has finished loading. */
	UE_DEPRECATED(5.8, "Call ObjectContainerHasFinishedLoading instead.")
	virtual bool ObjectLevelHasFinishedLoading(UObject* Object) const { return ObjectContainerHasFinishedLoading(Object); }

	/** Returns true of the container that the Object belongs to has finished loading. */
	virtual bool ObjectContainerHasFinishedLoading(UObject* Object) const { return true; }

	/**
	 * Adds a dependent object. A dependent object can replicate separately or if a parent replicates.
	 * Dependent objects cannot be filtered out by dynamic filtering unless the parent is also filtered out.
	 * @note: There is no guarantee that the data will end up in the same packet so it is a very loose form of dependency.
	 */
	IRISCORE_API void AddDependentObject(FNetRefHandle Parent, FNetRefHandle DependentObject, UE::Net::EDependentObjectSchedulingHint SchedulingHint = UE::Net::EDependentObjectSchedulingHint::Default);

	/** Remove dependent object from parent. The dependent object will function as a standard standalone replicated object. */
	IRISCORE_API void RemoveDependentObject(FNetRefHandle Parent, FNetRefHandle DependentObject);

	/** Force the child to only be created after the parent exists on a client. This also forces the child to never be relevant if their parents are not also relevant at the same time. */
	IRISCORE_API void AddCreationDependencyLink(FNetRefHandle Parent, FNetRefHandle Child);

	/** Remove a creation dependency link on the child */
	IRISCORE_API void RemoveCreationDependencyLink(FNetRefHandle Parent, FNetRefHandle Child);

	// Dormancy support

	/** Set whether object should go dormant. If dormancy is enabled any dirty state will be replicated first. */
	IRISCORE_API void SetObjectWantsToBeDormant(FNetRefHandle Handle, bool bWantsToBeDormant);

	/** Returns whether the object wants to be dormant. */
	IRISCORE_API bool GetObjectWantsToBeDormant(FNetRefHandle Handle) const;

	/** Trigger a single poll to refresh the dirty values of a dormant object. */
	IRISCORE_API void NetFlushDormantObject(FNetRefHandle Handle);	

	/** Set poll frequency on root object and its subobjects. They will be polled on the same frame. */
	IRISCORE_API void SetPollFrequency(FNetRefHandle RootHandle, float PollFrequency);

	/** Set the object filter to use for objects of this class and any derived classes without an explicit config. */
	IRISCORE_API void SetClassDynamicFilterConfig(FName ClassPathName, const UE::Net::FNetObjectFilterHandle FilterHandle, FName FilterProfile=NAME_None);
	IRISCORE_API void SetClassDynamicFilterConfig(FName ClassPathName, FName FilterName, FName FilterProfile=NAME_None);

	/** Set the TypeStats to use for specified class and any derived classes without explicit config */
	IRISCORE_API void SetClassTypeStatsConfig(FName ClassPathName, FName TypeStatsName);
	IRISCORE_API void SetClassTypeStatsConfig(const FString& ClassPathName, const FString& TypeStatsName);

	/** Getter for CVar net.Iris.UseVerboseIrisCsvStats which is true if we want per-class iris stats output to CSV */
	IRISCORE_API bool ShouldUseVerboseCsvStats() const;

	/** Return the name of the default spatial filter config. @see UObjectReplicationBridgeConfig */
	FName GetDefaultSpatialFilterName() const { return DefaultSpatialFilterName; }

	/** Return true only if Handle was registered via PreRegisterNewObjectReferenceHandle or PreregisterObjectWithReferenceHandle */
	IRISCORE_API bool IsNetRefHandlePreRegistered(FNetRefHandle Handle);

	/** Get the Object from a handle, even if the object was pre-registered and hasn't replicated yet. */
	IRISCORE_API UObject* GetPreRegisteredObject(FNetRefHandle Handle) const;

	/** Tell the remote connection that we detected a reading error with a specific replicated object */
	virtual void SendErrorWithNetRefHandle(UE::Net::ENetRefHandleError ErrorType, FNetRefHandle RefHandle, uint32 ConnectionId, TConstArrayView<const FNetRefHandle> ExtraNetRefHandle = {}, const UE::Net::FNetErrorContext& ErrorContext = UE::Net::FNetErrorContext()) {}

	/**
	 * Creates a NetRefHandle explicitly for object Instance and flags it as pre-registered in the ObjectReferenceCache.
	 * If supported by an object factory, the pre-registered flag is meant to be serialized in a root object creation header
	 * and when a client receives a dynamic object header with this flag, it will look for an instance already registered by
	 * PreRegisterObjectWithReferenceHandle instead of creating a new instance of the object.
	 * Meant for use by the NetDriver's CreateNetIDForObject function.
	 * @see UNetDriver::CreateNetIDForObject, UObjectReplicationBridge::PreRegisterObjectWithReferenceHandle
	 */
	IRISCORE_API UE::Net::FNetRefHandle PreRegisterNewObjectReferenceHandle(UObject* Instance);

	/**
	 * Directly assigns Handle to object Instance and flags it as pre-registered in the ObjectReferenceCache.
	 * If supported by the object factory, clients receiving an object creation header with the pre-registered flag
	 * will look for existing instances registered by this function instead of creating a new instance and
	 * use the found instance for replication.
	 * Meant for use by the NetDriver's AssignNetIDToObject function.
	 * @see UNetDriver::AssignNetIDToObject, UObjectReplicationBridge::PreRegisterNewObjectReferenceHandle
	 */
	IRISCORE_API void PreRegisterObjectWithReferenceHandle(const UObject* Instance, FNetRefHandle Handle);

	/** Re-initialize config-driven parameters found in ObjectReplicationBridgeConfig. */
	IRISCORE_API void ReloadConfig();

public:


	struct FSubObjectReplicationParams
	{
		/** The root object that the subobject will be attached to */
		FNetRefHandle RootObjectHandle;

		/**
		 * Optional handle to another parent subobject sharing the same Root, if set, the added subobject will replicate as a child to the specified 
		 * parent and be subject to any conditionals applied to the parent. If not set, the RootObjectHandle will be used instead.
		 */
		FNetRefHandle ParentSubObjectHandle;

		/** Optional handle to another subobject, used in combination with InsertionOrder to provide control over replication order for subobjects sharing the same Parent (Root or specified Parent). */
		FNetRefHandle InsertRelativeToSubObjectHandle; 

		/** Specify if child subobjects added to this SubObject should replicate before or after the parent. Note: Root always replicate before SubObjects.
		 * This mostly exists to allow ActorComponents to keep old behavior where they always replicated SubObjects before the component itself.
		 */
		UE::Net::EChildSubObjectsReplicationOrder ChildSubObjectsReplicationOrder = UE::Net::EChildSubObjectsReplicationOrder::AfterParent;

		/** Optional enum to select where in the list this subobject will be inserted */
		UE::Net::ESubObjectInsertionOrder InsertionOrder = UE::Net::ESubObjectInsertionOrder::None;

		/** Set to true to prevent the replication system from reusing a NetRefHandle previously assigned to the object. Useful if the object had stopped replicating and that old representation is considered to be destroyed. */
		bool bForceNewHandle = false;
	};

public:

	/** Print traits to control what gets logged in PrintIrisDebugInfoForNetHandle */
	enum class EPrintDebugInfoTraits : uint32
	{
		Default = 0x0000,
		// Do not print the entire protocol state when this is set 
		NoProtocolState = 0x0001,
		// Also print the debug information of child dependents
		WithDependents = 0x0002,
	};

	// Debug functions that are triggered via console commands declared in ObjectReplicationBridgeDebugging.cpp
	void PrintDynamicFilterClassConfig(uint32 ArgTraits);
	void PrintReplicatedObjects(uint32 ArgTraits) const;
	void PrintAlwaysRelevantObjects(uint32 ArgTraits) const;
	void PrintRelevantObjects(uint32 ArgTraits) const;
	void PrintRelevantObjectsForConnections(const TArray<FString>& Args) const;
	void PrintNetCullDistances(const TArray<FString>& Args) const;
	void PrintPushBasedStatuses() const;

	void PrintDebugInfoForNetRefHandlesAndConnections(const TArray<FNetRefHandle>& NetHandles, const TArray<FString>& Args, EPrintDebugInfoTraits PrintTraits);
	IRISCORE_API void PrintDebugInfoForNetRefHandle(const FNetRefHandle NetHandle, uint32 ConnectionId, EPrintDebugInfoTraits PrintTraits=EPrintDebugInfoTraits::Default) const;

protected:
	IRISCORE_API virtual ~UObjectReplicationBridge();

	// UReplicationBridge

	IRISCORE_API virtual void Initialize(UReplicationSystem* InReplicationSystem) override;
	IRISCORE_API virtual void Deinitialize() override;
	IRISCORE_API virtual void PreSendUpdateSingleHandle(FNetRefHandle RefHandle) override;	
	IRISCORE_API virtual void PreSendUpdate() override;	
	IRISCORE_API virtual void OnStartPreSendUpdate() override;
	IRISCORE_API virtual void OnPostSendUpdate() override;
	IRISCORE_API virtual void OnPostReceiveUpdate() override;
	IRISCORE_API virtual void UpdateInstancesWorldLocation() override;
	IRISCORE_API virtual void PruneStaleObjects() override;	
	IRISCORE_API virtual bool CacheNetRefHandleCreationInfo(FNetRefHandle Handle) override;
	IRISCORE_API virtual void OnProtocolMismatchReported(FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<uint8>& ClientCDOStateBytes = TArray<uint8>()) override;
	IRISCORE_API virtual void OnErrorWithNetRefHandleReported(UE::Net::ENetRefHandleError ErrorType, FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<FNetRefHandle>& ExtraHandles, const FString& DiagnosticMessage) override;
	IRISCORE_API virtual void RemoveDestructionInfosForGroup(UE::Net::FNetObjectGroupHandle GroupHandle) override;
	IRISCORE_API virtual void RemoveDestructionInfo(UE::Net::FInternalNetRefIndex InternalObjectIndex) override;

private:

	void OnErrorBitstreamCorrupted(FNetRefHandle RefHandle, uint32 ConnectionId);
	void OnErrorCannotReplicateObject(FNetRefHandle RefHandle, uint32 ConnectionId, const FString& ClientDiagnosticMessage);
	void OnErrorBlockedByAsyncLoading(FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<FNetRefHandle>& MustBeMappedRefs);
	void OnErrorBlockedByParents(FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<FNetRefHandle>& BlockedParents);
	
protected:
	
	/** Called when we found a divergence between the local and remote protocols when trying to instantiate a remote replicated object. */
	virtual void OnProtocolMismatchDetected(FNetRefHandle ObjectHandle, const TArray<uint8>& CDOStateBytes = TArray<uint8>()) {}

	/** Lookup the UObject associated with the provided Handle. This function will not try to resolve the reference. */
	IRISCORE_API UObject* GetObjectFromReferenceHandle(FNetRefHandle RefHandle) const;

	/** Helper method that calls provided PreUpdateFunction and polls state data for all replicated instances with the NeedsPoll trait. */
	using FInstancePreUpdateFunction = TFunction<void(TArrayView<UObject*>, const UObjectReplicationBridge*)>;

	/** Set the function that we should call before copying state data. */
	IRISCORE_API void SetInstancePreUpdateFunction(FInstancePreUpdateFunction InPreUpdateFunction);
	
	/** Get the function that we should call before copying state data. */
	IRISCORE_API FInstancePreUpdateFunction GetInstancePreUpdateFunction() const;

	// Poll frequency support

	/** Force polling of Object when ObjectToPollWith is polled. */
	IRISCORE_API void SetPollWithObject(FNetRefHandle ObjectToPollWith, FNetRefHandle Object);

	/** Transform a poll frequency (in updates per second) to an equivalent number of frames. */
	UE_DEPRECATED(5.8, "FObjectPollFrequencyLimiter will internally convert poll frequency to its internal representation.")
	IRISCORE_API uint8 ConvertPollFrequencyIntoFrames(float PollFrequency) const;

	/** Re-initialize the poll frequency of all replicated root objects. */
	UE_DEPRECATED(5.8, "FObjectPollFrequencyLimiter will re-initialize poll frequencies when its update frequency is changed. This method is a no-op")
	IRISCORE_API void ReinitPollFrequency();

	/** Re-initialize config-driven parameters found in ObjectReplicationBridgeConfig. */
	IRISCORE_API void LoadConfig();

	/** Force the specified object to update it's cached WorldLocation data. */
	IRISCORE_API void ForceUpdateWorldLocation(FNetRefHandle NetRefHandle, UE::Net::FInternalNetRefIndex InternalObjectIndex);

	/**
	 * Set the function used to determine whether an object should use the default spatial filter
	 * unless the class is configured to use some other filter.
	 */
	IRISCORE_API void SetShouldUseDefaultSpatialFilterFunction(TFunction<bool(const UClass*)>);

	/** Set the function used to determine whether two classes are to be considered equal when it comes to filtering. Used on subclasses. */
	IRISCORE_API void SetShouldSubclassUseSameFilterFunction(TFunction<bool(const UClass* Class, const UClass* Subclass)>);

	/** Finds the final poll frequency for a class and cache it for future lookups. OutPollPeriod will only be modified if this method returns true. */
	IRISCORE_API bool FindOrCachePollFrequency(const UClass* Class, float& OutPollPeriod);

	/**
	 * Find the poll period of a class if it was configured with an override. 
	 * Use this only if all class configs have been properly cached.
	 * OutPollPeriod will only be modified if this method returns true.
	 */
	IRISCORE_API bool GetClassPollFrequency(const UClass* Class, float& OutPollPeriod) const;

	/** Find the configured async loading priority for a given class.  Will return EIrisAsyncLoadingPriority::Default if no class config exist */
	IRISCORE_API EIrisAsyncLoadingPriority GetClassIrisAsyncLoadingPriority(const UClass* Class);

	/** Returns true if the class is considered critical and we force a disconnection if a protocol mismatch prevents instances of this class from replicating. */
	IRISCORE_API bool IsClassCritical(const UClass* Class);

	/** Returns true if the class is replicated by default. This bridge will assume so. */
	IRISCORE_API virtual bool IsClassReplicatedByDefault(const UClass* Class) const;

	/** Returns the most relevant description of the client tied to this connection id. */
	[[nodiscard]] IRISCORE_API virtual FString PrintConnectionInfo(uint32 ConnectionId) const;

	/** Current max tick rate set by the engine */
	float GetMaxTickRate() const { return MaxTickRate; }

	/** Change the max tick rate to match the one from the engine */
	IRISCORE_API void SetMaxTickRate(float InMaxTickRate);

	/**
	 * Parses a list of arguments and returns a list of Connection's that match them
	 * Ex: ConnectionId=1 or ConnectionId=1,5,7
	 */
	IRISCORE_API virtual TArray<uint32> FindConnectionsFromArgs(const TArray<FString>& Args) const;

	/** Retrieves the dynamic filter to set for the given class. Will return an invalid handle if no dynamic filter should be set. */
	IRISCORE_API UE::Net::FNetObjectFilterHandle GetDynamicFilter(const UClass* Class, bool bRequireForceEnabled, FName& OutFilterProfile);

private:

	/** Forcibly poll a single replicated object */
	void ForcePollObject(FNetRefHandle RefHandle);

	/** Build the list of relevant objects who hit their polling period or were flagged ForceNetUpdate. */
	void BuildPollList(UE::Net::FNetBitArrayView ObjectsConsideredForPolling);

	/** Call the user function PreUpdate (aka PreReplication) on objects about to be polled. */
	void PreUpdate(const UE::Net::FNetBitArrayView ObjectsConsideredForPolling);

	/** Find any objects that got set dirty by user code during PreUpdate then lock future modifications to the global dirty list */
	void FinalizeDirtyObjects();

	/** Find any new subobjects created inside PreUpdate and ensure they will be replicated this frame */
	void ReconcileNewSubObjects(UE::Net::FNetBitArrayView ObjectsConsideredForPolling);

	/** Poll all objects in the list and copy the dirty source data into the ReplicationState buffers*/
	void PollAndCopy(const UE::Net::FNetBitArrayView ObjectsConsideredForPolling);

	/** Register instance received from remote, binding RefHandle, instance and protocol together */
	void RegisterRemoteInstance(FNetRefHandle RefHandle, UObject* InstancePtr, const UE::Net::FReplicationProtocol* Protocol, UE::Net::FReplicationInstanceProtocol* InstanceProtocol, uint32 ConnectionId);

	/** 
	 * Called when the authority tells a client that a root object is no longer replicated to this client.
	 * Will remove the root object and all it's subobjects from the replication system locally and ask the factory to deal with the instance of the object.
	 * @param RootObjectHandle The handle of the root object
	 * @param DetachReason The reason why the object is no longer replicated
	 * @param NetFactoryId The netobjectfactory of the detached object
	 */
	void DetachRootObjectFromRemote(FNetRefHandle RootObjectHandle, UE::Net::EDetachReason DetachReason, UE::Net::FNetObjectFactoryId NetFactoryId);

	/**
	 * Called when the authority tells a client that a specific subobject is no longer replicated to this client.
	 * @param SubObjectHandle The handle of the sub object
	 * @param DetachReason The reason why the object is no longer replicated
	 * @param NetFactoryId The netobjectfactory of the detached object
	 */
	void DetachSubObjectFromRemote(FNetRefHandle SubObjectHandle, UE::Net::EDetachReason DetachReason, UE::Net::FNetObjectFactoryId NetFactoryId);

	
	/**
	 * Called when the instance is detached from the protocol on request by the remote. 
	 * @param Handle The handle of the object to destroy or tear off.
	 * @param DestroyReason Reason for destroying the instance. 
	 * @param DestroyFlags Special flags such as whether the instance may be destroyed when reason is TearOff, which may revert to destroying, or Destroy.
	 */
	UE_DEPRECATED(5.8, "Replciated with DetachRootObjectFromRemote and DetachSubObjectFromRemote")
	void DetachInstanceFromRemote(FNetRefHandle Handle, EReplicationBridgeDestroyInstanceReason DestroyReason, EReplicationBridgeDestroyInstanceFlags DestroyFlags,  UE::Net::FNetObjectFactoryId NetFactoryId);

	UE_DEPRECATED(5.8, "Replciated with DetachRootObjectFromRemote and DetachSubObjectFromRemote")
	void DetachSubObjectInstancesFromRemote(UE::Net::FInternalNetRefIndex RootObjectIndex, EReplicationBridgeDestroyInstanceReason DestroyReason, EReplicationBridgeDestroyInstanceFlags DestroyFlags);

	void SetNetPushIdOnInstance(UE::Net::FReplicationInstanceProtocol* InstanceProtocol, FNetHandle NetHandle);

	/** Tries to load the classes used in poll period overrides. */
	void FindClassesInPollPeriodOverrides();

	FNetRefHandle InternalStartReplicatingRootObject(UObject* Instance, const FRootObjectReplicationParams& Params, UE::Net::FNetObjectFactoryId NetFactoryId);

	/** Does all that is necessary to replicate any given UObject. Returns the object's handle if successful  */
	UE::Net::FNetRefHandle StartReplicatingNetObject(UObject* Instance, UE::Net::EReplicationFragmentTraits Traits, UE::Net::FNetObjectFactoryId NetFactoryId, bool bForceNewHandle);

	/** Retrieves the prioritizer to set for the given class. If bRequireForceEnabled the config needs to have bForceEnableOnAllInstances set in order for this method to return the configured prioritizer. Returns an invalid handle if no prioritizer should be set. */
	UE::Net::FNetObjectPrioritizerHandle GetPrioritizer(const UClass* Class, bool bRequireForceEnabled);

	/** Assign the proper dynamic filter to a new object */
	void AssignDynamicFilter(UObject* Instance, const FRootObjectReplicationParams& Params, FNetRefHandle RefHandle);

	/** Returns true if instances of this class should be delta compressed */
	bool ShouldClassBeDeltaCompressed(const UClass* Class);

	/** Set the initial dormancy status of a subobject */
	void SetSubObjectDormancyStatus(FNetRefHandle SubObjectRefHandle, FNetRefHandle OwnerRefHandle);

	/** Marks a spatially filtered object as requiring or not requiring frequent world location updates independent of it having dirty replicated properties */
	void OptionallySetObjectRequiresFrequentWorldLocationUpdate(FNetRefHandle RefHandle, bool bDesiresFrequentWorldLocationUpdate);

	struct FUpdateWorldInfoContext
	{
		UE::Net::FWorldLocations& WorldLocations;
		UE::Net::FInternalNetRefIndex ObjectIndex;
		FNetRefHandle NetRefHandle;
	};
	/** Update the WorldLocationInfo of a specific root object */
	void UpdateRootObjectWorldInfo(const FUpdateWorldInfoContext& Context);

	/** Returns the TypeStatsIndex this class should use */
	int32 GetTypeStatsIndex(const UClass* Class);

	FInstancePreUpdateFunction PreUpdateInstanceFunction;
	
	FName GetConfigClassPathName(const UClass* Class);

	void CreateDestructionProtocol();
	void InitConditionalPropertyDelegates();
	void InitNetObjectFactories();
	void DeinitNetObjectFactories();

	void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex);

private:

	//------------------------------------------------------------------------
	// These private methods are available through the FBridgeSerialization interface
	//------------------------------------------------------------------------

	friend UE::Net::Private::FBridgeSerialization;

	/** Return the destruction info assigned to a replicated object */
	const UE::Net::Private::FStaticDestructionInfo* GetStaticDestructionInfo(UE::Net::FNetRefHandle Handle) const
	{
		return StaticObjectsPendingDestroy.Find(Handle);
	}

	/**
	 * Write data required to instantiate NetObject remotely to bitstream.
	 * @param Context The serialization context parameters.
	 * @param Handle The handle of the object to write creation data for.
	 * @param InternalObjectIndex the internalindex we are currently writing creation info for.
	 */
	bool WriteNetRefHandleCreationInfo(FReplicationBridgeSerializationContext& Context, FNetRefHandle Handle, UE::Net::FInternalNetRefIndex InternalObjectIndex);

	/**
	 * Client only function that executes the EndReplication request by the authority
	 * @param InternalObjectIndex The index of the object to destroy
	 * @param InternalDetachReason Reason the object is no longer replicated
	 */
	void DestroyNetObjectFromRemote(UE::Net::FInternalNetRefIndex InternalObjectIndex, UE::Net::Private::EInternalDetachReason InternalDetachReason);

	void ReadAndExecuteDestructionInfoFromRemote(FReplicationBridgeSerializationContext& Context);

	/** Read data required to instantiate NetObject from bitstream. */
	FReplicationBridgeCreateNetRefHandleResult CreateNetRefHandleFromRemote(FNetRefHandle RootObjectNetHandle, FNetRefHandle WantedNetHandle, FReplicationBridgeSerializationContext& Context);

	/** Invoked after we have applied the initial state for an object.*/
	void PostApplyInitialState(UE::Net::FInternalNetRefIndex InternalObjectIndex);

	/** Invoked right before we apply the state for a new received subobject but after we have applied state on the root object. */
	void SubObjectCreatedFromReplication(UE::Net::FInternalNetRefIndex RootObjectIndex, FNetRefHandle SubObjectCreated);

	/** Gets the bReplicateTransactionally flag from the ReplicationSystem */
	bool IsReplicatingTransactionally() const;

private:

	friend UE::Net::Private::FObjectPoller;
	friend UE::Net::Private::FReplicationSystemImpl;
	
	friend UE::Net::Private::FObjectReplicationBridgeDelegates;

	friend UReplicationSystem;

	TMap<const UClass*, FName> ConfigClassPathNameCache;

	// Prioritization
	struct FClassPrioritizerInfo
	{
		UE::Net::FNetObjectPrioritizerHandle PrioritizerHandle;
		bool bForceEnable = false;
	};
	
	// Filtering
	struct FClassFilterInfo
	{
		UE::Net::FNetObjectFilterHandle FilterHandle;
		FName FilterProfile;
		bool bForceEnable = false;
	};

	// Polling

	/** The maximum tick per second of the engine. Default is to use 30hz */
	float MaxTickRate = 30.0f;

	struct FPollInfo
	{
		float PollFrequency = 0.0f;
		TWeakObjectPtr<const UClass> Class;
	};
	UE::Net::Private::FObjectPollFrequencyLimiter* PollFrequencyLimiter;

	//$IRIS TODO: The poll class config management code should be moved into it's own class. Maybe in a class that handles any type of per-class settings.
	// Class hierarchies with poll period overrides
	TMap<FName, FPollInfo> ClassHierarchyPollPeriodOverrides;
	// Exact classes with poll period overrides
	TMap<FName, FPollInfo> ClassesWithPollPeriodOverride;
	// Exact classes without poll period override
	TSet<FName> ClassesWithoutPollPeriodOverride;

	// Filter mapping
	//$IRIS TODO: Look into improving this class map by balancing runtime speed (implicit addition of new classes) with runtime modifications of base classes.
    //		      Right-now any changes to say APawn's filter will not be read if a APlayerPawn entry was created based on the APawn entry.
	TMap<FName, FClassFilterInfo> ClassesWithDynamicFilter;
	TFunction<bool(const UClass*)> ShouldUseDefaultSpatialFilterFunction;
	TFunction<bool(const UClass*,const UClass*)> ShouldSubclassUseSameFilterFunction;

	// Prioritizer mapping
	TMap<FName, FClassPrioritizerInfo> ClassesWithPrioritizer;

	// Delta compression
	TMap<FName, bool> ClassesWithDeltaCompression;

	// Classes that may force a disconnection when a protocol mismatch is detected.
	TMap<FName, bool> ClassesFlaggedCritical;

	// Classes that have an async loading priority
	TMap<FName, EIrisAsyncLoadingPriority> ClassesIrisAsyncLoadingPriority;

	// Type stats
	TMap<FName, FName> ClassesWithTypeStats;

	// Track the static objects that are destroyed so clients can be told to delete them after loading the container
	TMap<FNetRefHandle, UE::Net::Private::FStaticDestructionInfo> StaticObjectsPendingDestroy;
	TMap<FNetRefHandle, FNetRefHandle> DestroyedStaticObjectToDestructionInfoHandleMap;

	// Objects which has object references and could be affected by garbage collection.
	UE::Net::FNetBitArray ObjectsWithObjectReferences;
	// Objects that needs to update their object references due to garbage collection.
	UE::Net::FNetBitArray GarbageCollectionAffectedObjects;

	FName DefaultSpatialFilterName;
	UE::Net::FNetObjectFilterHandle DefaultSpatialFilterHandle;

	FDelegateHandle OnCustomConditionChangedHandle;
	FDelegateHandle OnDynamicConditionChangedHandle;

	// Pre-built protocol used to send destruction requests for static objects
	const UE::Net::FReplicationProtocol* DestructionInfoProtocol = nullptr;
	
	UPROPERTY()
	TArray<TObjectPtr<UNetObjectFactory>> NetObjectFactories;

	bool bHasPollOverrides = false;
	bool bHasDirtyClassesInPollPeriodOverrides = false;

	/** Set to true when the system does not allow new root objects to be replicated at this moment. Useful when calling into user code and to warn them of illegal operations. */
	bool bBlockStartRootObjectReplication = false;

protected:
	bool bSuppressCreateInstanceFailedEnsure = false;

private:

#if UE_NET_ASYNCLOADING_DEBUG
	friend UE::Net::Private::FAsyncLoadingSimulator;
	/** 
	 * Debug array that forces objects to be stalled, as if async loading external references.
	 * Only enabled when net.Iris.AsyncLoading.ForceObjectsToStall is enabled.
	 */
	TArray<FNetRefHandle> DebugObjectsForcedToStall;
#endif

	TMap<FObjectKey, bool> ArchetypesAlreadyPrinted;

	/** Tells if we already logged a warning about a given object or a class type that had a location that is out of bounds. */
	TSet<FObjectKey> WorldLocationOOBWarnings;

	/** List of class names that will not prevent replication to happen when a protocol mismatch is detected.*/
	TArray<FString> ClassNamesToIgnoreProtocolMismatch;
#if UE_SUPPORT_PARALLEL_IRIS
private:
	/** Protects access to CachedCreationHeaders inside WriteNetRefHandleCreationInfo */
	FCriticalSection WriteNetRefHandleCreationCS;
#endif // UE_SUPPORT_PARALLEL_IRIS

	UE::Net::FOnRootObjectPostInit OnRootObjectPostInit;
	UE::Net::FOnRootObjectDetached OnRootObjectDetached;
	UE::Net::FOnSubObjectPostInit OnSubObjectPostInit;
	UE::Net::FOnSubObjectDetached OnSubObjectDetached;
};

//------------------------------------------------------------------------

ENUM_CLASS_FLAGS(UObjectReplicationBridge::EPrintDebugInfoTraits);

//------------------------------------------------------------------------

namespace UE::Net::Private
{

#if UE_NET_ASYNCLOADING_DEBUG
struct FAsyncLoadingSimulator
{
	static void AddStalledObject(UObjectReplicationBridge* Bridge, FNetRefHandle ReplicatedObject)
	{
		check(Bridge);
		if (ReplicatedObject.IsValid())
		{
			Bridge->DebugObjectsForcedToStall.AddUnique(ReplicatedObject);
		}
	}

	static void RemoveStalledObject(UObjectReplicationBridge* Bridge, FNetRefHandle ReplicatedObject)
	{
		check(Bridge);
		Bridge->DebugObjectsForcedToStall.Remove(ReplicatedObject);
	}

	static void RemoveAllStalledObjects(UObjectReplicationBridge* Bridge)
	{
		check(Bridge);
		Bridge->DebugObjectsForcedToStall.Empty();
	}

	static bool HasStalledObjects(UObjectReplicationBridge* Bridge)
	{
		check(Bridge);
		return !Bridge->DebugObjectsForcedToStall.IsEmpty();
	}

	static bool IsObjectStalled(UObjectReplicationBridge* Bridge, FNetRefHandle Handle)
	{
		check(Bridge);
		return Bridge->DebugObjectsForcedToStall.Contains(Handle);
	}

	static FReplicationBridgeCreateNetRefHandleResult CreateNetRefHandleFromRemote(UObjectReplicationBridge* Bridge, FNetRefHandle RootObjectNetHandle, FNetRefHandle WantedNetHandle, FReplicationBridgeSerializationContext& Context)
	{
		check(Bridge);
		return Bridge->CreateNetRefHandleFromRemote(RootObjectNetHandle, WantedNetHandle, Context);
	}
};
#endif // UE_NET_ASYNCLOADING_DEBUG

/** Helper class to handle private functionnalities needed by Iris serialization classes */
class FBridgeSerialization
{
private:

	friend class FReplicationWriter;
	friend class FReplicationReader;
	friend UNetObjectFactory;

	static bool WriteNetRefHandleCreationInfo(UObjectReplicationBridge* Bridge, FReplicationBridgeSerializationContext& Context, FNetRefHandle Handle, FInternalNetRefIndex InternalObjectIndex)
	{
		return Bridge->WriteNetRefHandleCreationInfo(Context, Handle, InternalObjectIndex);
	}

	static const UE::Net::Private::FStaticDestructionInfo* GetStaticDestructionInfo(UObjectReplicationBridge* Bridge, UE::Net::FNetRefHandle Handle)
	{
		return Bridge->GetStaticDestructionInfo(Handle);
	}

	static void DestroyNetObjectFromRemote(UObjectReplicationBridge* Bridge, FInternalNetRefIndex InternalObjectIndex, EInternalDetachReason InternalDetachReason)
	{
		Bridge->DestroyNetObjectFromRemote(InternalObjectIndex, InternalDetachReason);
	}

	static void ReadAndExecuteDestructionInfoFromRemote(UObjectReplicationBridge* Bridge, FReplicationBridgeSerializationContext& Context)
	{
		Bridge->ReadAndExecuteDestructionInfoFromRemote(Context);
	}

	static FReplicationBridgeCreateNetRefHandleResult CreateNetRefHandleFromRemote(UObjectReplicationBridge* Bridge, FNetRefHandle RootObjectNetHandle, FNetRefHandle WantedNetHandle, FReplicationBridgeSerializationContext& Context)
	{
		return Bridge->CreateNetRefHandleFromRemote(RootObjectNetHandle, WantedNetHandle, Context);
	}

	static void PostApplyInitialState(UObjectReplicationBridge* Bridge,  FInternalNetRefIndex InternalObjectIndex)
	{
		return Bridge->PostApplyInitialState(InternalObjectIndex);
	}

	static void SubObjectCreatedFromReplication(UObjectReplicationBridge* Bridge, UE::Net::FInternalNetRefIndex RootObjectIndex, FNetRefHandle SubObjectCreated)
	{
		return Bridge->SubObjectCreatedFromReplication(RootObjectIndex, SubObjectCreated);
	}

	static bool IsReplicatingTransactionally(const UObjectReplicationBridge* Bridge)
	{
		return Bridge->IsReplicatingTransactionally();
	}
};

} // end namespace UE::Net::Private