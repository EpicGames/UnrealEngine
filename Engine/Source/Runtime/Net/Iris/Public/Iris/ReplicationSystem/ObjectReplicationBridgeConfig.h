// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"

#include "Iris/ReplicationSystem/ObjectReferenceTypes.h"
#include "Iris/ReplicationSystem/NetRefHandleGenerator.h"

#include "ObjectReplicationBridgeConfig.generated.h"

USTRUCT()
struct FObjectReplicationBridgePollConfig
{
	GENERATED_BODY()

	/**
	 * Instances of this class, specified by its fully qualified path, should use the supplied poll frame period to check for modified replicated properties.
	 * Object and Actor are forbidden class names for performance reasons.
	 */
	UPROPERTY()
	FName ClassName;

	/**
	 * How many times per second should we poll for modified replicated properties.
	 * The value will be converted into a frame count based on the current TickRate up to a maximum of 255 frames
	 * This means the slowest poll frequency is 255*MaxTickRate (ex: 8.5secs at 30hz)
	 * If set to 0 it means we poll the object every frame.
	 */
	UPROPERTY()
	float PollFrequency = 0.0f;
	

	/** Whether instances of subclasses should also use this poll period. */
	UPROPERTY()
	bool bIncludeSubclasses = true;
};

USTRUCT()
struct FObjectReplicationBridgeFilterConfig
{
	GENERATED_BODY()

	/** Instances of this class should use the filter supplied. */
	UPROPERTY()
	FName ClassName;

	/** The name of the filter to set on the class instances. */
	UPROPERTY()
	FName DynamicFilterName;

	/** Optional name to a profile that can further specialize the settings within a dynamic filter */
	UPROPERTY()
	FName FilterProfile;

	/** Whether this filter should be used for all instances of this class and subclasses, regardless of bAlwaysRelevant and bOnlyRelevantToOwner settings on instance. */
	UPROPERTY()
	bool bForceEnableOnAllInstances = false;
};

USTRUCT()
struct FObjectReplicationBridgePrioritizerConfig
{
	GENERATED_BODY()

	/** Instances of this class and its subclasses, specified by its fully qualified path, should use the prioritizer supplied. */
	UPROPERTY()
	FName ClassName;

	/** The name of the prioritizer to set on the class instances. "Default" can be used to specify the default spatial prioritizer. */
	UPROPERTY()
	FName PrioritizerName;

	/** Whether this prioritizer should be used for all instances of this class and subclasses, regardless of bAlwaysRelevant and bOnlyRelevantToOwner settings on instance. */
	UPROPERTY()
	bool bForceEnableOnAllInstances = false;
};

USTRUCT()
struct FObjectReplicationBridgeDeltaCompressionConfig
{
	GENERATED_BODY()

	/** Instances of this class or derived from this class should use delta compression */
	UPROPERTY()
	FName ClassName;

	/** Set to true if delta compression should be enabled for instances derived from this class. */
	UPROPERTY()
	bool bEnableDeltaCompression = true;
};

USTRUCT()
struct FObjectReplicatedBridgeCriticalClassConfig
{
	GENERATED_BODY()

	/** Instances of this class or its subclasses will force a client disconnection when it detects a protocol mismatch.*/
	UPROPERTY()
	FName ClassName;

	/** When true we force the client to disconnect when a protocol mismatch prevents it from instantiating replicated objects of this class. */
	UPROPERTY()
	bool bDisconnectOnProtocolMismatch = true;
};

USTRUCT()
struct FObjectReplicationBridgeAsyncLoadingClassConfig
{
	GENERATED_BODY()

	/** Path names of classes that want to configure their async loading priority. */
	UPROPERTY()
	FName ClassName;

	/** The async loading priority to use for assets reference by instances of a class */
	UPROPERTY()
	EIrisAsyncLoadingPriority IrisAsyncLoadingPriority = EIrisAsyncLoadingPriority::Default;
};

USTRUCT()
struct FObjectReplicationBridgeTypeStatsConfig
{
	GENERATED_BODY()

	/** Instances of this class or derived from this class should use delta compression */
	UPROPERTY()
	FName ClassName;

	/** The TypeStatsName this class should use. */
	UPROPERTY()
	FName TypeStatsName;

	/** If set to true this type will be reported even in configs with minimal stats reporting */
	UPROPERTY()
	bool bIncludeInMinimalCSVStats = false;
};

UCLASS(transient, config=Engine)
class UObjectReplicationBridgeConfig : public UObject
{
	GENERATED_BODY()

public:

	IRISCORE_API static const UObjectReplicationBridgeConfig* GetConfig();

	IRISCORE_API TConstArrayView<FObjectReplicationBridgePollConfig> GetPollConfigs() const;
	IRISCORE_API TConstArrayView<FObjectReplicationBridgeFilterConfig> GetFilterConfigs() const;
	IRISCORE_API TConstArrayView<FObjectReplicationBridgePrioritizerConfig> GetPrioritizerConfigs() const;
	IRISCORE_API TConstArrayView<FObjectReplicationBridgeDeltaCompressionConfig> GetDeltaCompressionConfigs() const;
	IRISCORE_API TConstArrayView<FObjectReplicatedBridgeCriticalClassConfig> GetCriticalClassConfigs() const;
	IRISCORE_API TConstArrayView<FObjectReplicationBridgeTypeStatsConfig> GetTypeStatsConfigs() const;
	IRISCORE_API TConstArrayView<FObjectReplicationBridgeAsyncLoadingClassConfig> GetAsyncLoadingClassConfigs() const;

	UClass* GetNetRefHandleGeneratorClass() const { return NetRefHandleGeneratorClass; }

	FName GetDefaultSpatialFilterName() const;
	FName GetRequiredNetDriverChannelClassName() const;

	/** When true any class with a protocol mismatch will force a disconnection. */
	bool AreAllClassesCritical() const { return bAllClassesCritical; }

protected:
	UObjectReplicationBridgeConfig();

private:
	/**
	 * Which classes should override how often they're polled for modified replicated properties.
	 * A config for a class deeper in the class hierarchy has precedence over a more generic class.
	 * By default an Actor's NetUpdateFrequency is used to calculate how often it should be polled.
	 */
	UPROPERTY(Config)
	TArray<FObjectReplicationBridgePollConfig> PollConfigs;

	/**
	 * Which classes should apply a certain filter. Subclasses will inherit the settings unless
	 * they have different relevancy or spatial behavior.
	 */
	UPROPERTY(Config)
	TArray<FObjectReplicationBridgeFilterConfig> FilterConfigs;

	/**
	 * Which classes should apply a certain prioritizer. Subclasses will inherit the settings.
	 * Instances with fixed priorities will ignore any config.
	 */
	UPROPERTY(Config)
	TArray<FObjectReplicationBridgePrioritizerConfig> PrioritizerConfigs;

	/**
	 * Which classes should enable deltacompression. Derived classes will get the same behavior unless overidden
	 */
	UPROPERTY(Config)
	TArray<FObjectReplicationBridgeDeltaCompressionConfig> DeltaCompressionConfigs;

	/** Classes that are considered critical and will force a disconnection when a protocol mismatch is detected. */
	UPROPERTY(Config)
	TArray<FObjectReplicatedBridgeCriticalClassConfig> CriticalClassConfigs;

	/** Set this to true if you want any class with a protocol mismatch to force a disconnection. */
	UPROPERTY(Config)
	bool bAllClassesCritical = false;

	/**
	 * Which classes should collect TypeStats. Derived classes will get the same behavior unless overidden
	 */
	UPROPERTY(Config)
	TArray<FObjectReplicationBridgeTypeStatsConfig> TypeStatsConfigs;

	/**
	 * The name of the filter to apply objects that can have spatial filtering applied.
	 */
	UPROPERTY(Config)
	FName DefaultSpatialFilterName;

	/** The names of classes that want to configure the priority when async loading packages of replicated references. */
	UPROPERTY(Config)
	TArray<FObjectReplicationBridgeAsyncLoadingClassConfig> AsyncLoadingClassConfigs;

	/**
	 * The name of the channel class required for object replication to work.
	 */
	UPROPERTY(Config)
	FName RequiredNetDriverChannelClassName;

	/** Classes that must be instantiated otherwise we need to disconnect from the session entirely.  */
	UPROPERTY(Config)
	TArray<FName> CriticalActorClasses;
	
	/** 
	 * An optional configuration property defines how a NetRefHandle is generated.
	 *
	 * If not set, the NetRefHandle will be generated as an incremental value starting with 1.
	 */
	UPROPERTY(Config)
	TSubclassOf<UNetRefHandleGenerator> NetRefHandleGeneratorClass;
};

inline FName UObjectReplicationBridgeConfig::GetDefaultSpatialFilterName() const
{
	return DefaultSpatialFilterName;
}

inline FName UObjectReplicationBridgeConfig::GetRequiredNetDriverChannelClassName() const
{
	return RequiredNetDriverChannelClassName;
}
