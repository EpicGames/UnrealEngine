// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Prioritization/LocationBasedNetObjectPrioritizer.h"
#include "Net/Core/NetBitArray.h"
#include "Math/VectorRegister.h"
#include "UObject/StrongObjectPtr.h"
#include "FieldOfViewNetObjectPrioritizer.generated.h"

class FMemStackBase;

UCLASS(transient, config=Engine)
class UFieldOfViewNetObjectPrioritizerConfig : public UNetObjectPrioritizerConfig
{
	GENERATED_BODY()

public:
	/* Inner sphere radius that will set InnerSpherePriority on objects in it. */
	UPROPERTY(Config)
	float InnerSphereRadius = 3000.0f;

	/** Priority for objects inside the inner sphere */
	UPROPERTY(Config)
	float InnerSpherePriority = 1.0f;

	/* Outer sphere radius that will set OuterSpherePriority on objects in it. */
	UPROPERTY(Config)
	float OuterSphereRadius = 10000.0f;

	/** Priority for objects inside the outer sphere */
	UPROPERTY(Config)
	float OuterSpherePriority = 0.2f;

	/* The field of view used to form the cone. */
	UPROPERTY(Config)
	float ConeFieldOfViewDegrees = 45.0f;

	/** Give max cone priority up to this length of the cone. */
	UPROPERTY(Config)
	float InnerConeLength = 3000.0f;

	/** Total cone length. */
	UPROPERTY(Config)
	float ConeLength = 20000.0f;

	/** The minimum priority to set for objects within the cone. The priority falls off linearly from InnerConeLength to ConeLength. */
	UPROPERTY(Config)
	float MinConePriority = 0.2f;

	/** The maximum priority to set for objects within the cone, up to InnerConeLength. The priority falls off linearly from InnerConeLength to ConeLength. */
	UPROPERTY(Config)
	float MaxConePriority = 1.0f;

	/** Line of sight length is same as cone length but the width needs to be specified. */
	UPROPERTY(Config)
	float LineOfSightWidth = 200.0f;

	/** Priority for objects in line of sight. */
	UPROPERTY(Config)
	float LineOfSightPriority = 1.0f;
	
	/** Priority outside the field of view cone, spheres and line of sight */
	UPROPERTY(Config)
	float OutsidePriority = 0.1f;
};

/**
 * The FieldOfViewNetObjectPrioritizer prioritizes objects based on a cone, derived from a configured field of view and length. The priorities withing the cone are calculated
 * according to configured cone priorities falling off linearly based on distance to the view location. Additionally there's a inner sphere with a most likely high priority and and larger circle with most likely less priority than the inner circle. 
 * The circle priorities aren't scaled based on distance to view location, they're fixed. Finally there's a line of sight capsule with a configured width with a fixed priority
 * for objects within it. The final result is the maximum priority based on these four geometric shapes. An image visualization is generated by code in TestFieldOfViewNetObjectPrioritizer.cpp.
 */
UCLASS(Transient, MinimalAPI)
class UFieldOfViewNetObjectPrioritizer : public ULocationBasedNetObjectPrioritizer
{
	GENERATED_BODY()

protected:
	// UNetObjectPrioritizer interface
	IRISCORE_API virtual void Init(FNetObjectPrioritizerInitParams& Params) override;
	IRISCORE_API virtual void Deinit() override;
	IRISCORE_API virtual void Prioritize(FNetObjectPrioritizationParams&) override;

protected:
	struct FPriorityCalculationConstants
	{
		// Cone constants
		VectorRegister InnerConeLength;
		VectorRegister ConeLength;
		// OuterConeLength - InnerConeLength
		VectorRegister ConeLengthDiff;
		// 1/ConeLengthDiff
		VectorRegister InvConeLengthDiff;
		// ConeLength/ConeRadius for cone radius calculations at various distances.
		VectorRegister ConeRadiusFactor;
		VectorRegister InnerConePriority;
		VectorRegister OuterConePriority;
		// OuterConePriority - InnerConePriority 
		VectorRegister ConePriorityDiff;

		// Inner and outer sphere constants
		VectorRegister InnerSphereRadiusSqr;
		VectorRegister OuterSphereRadiusSqr;
		VectorRegister InnerSpherePriority;
		VectorRegister OuterSpherePriority;

		// Line of sight constants
		VectorRegister LineOfSightRadiusSqr;
		VectorRegister LineOfSightPriority;

		// Outside priority
		VectorRegister OutsidePriority;
	};

	struct FBatchParams
	{
		FPriorityCalculationConstants PriorityCalculationConstants;
		UE::Net::FReplicationView View;
		uint32 ConnectionId;

		uint32 ObjectCount;
		VectorRegister* Positions;
		/** 16-byte aligned pointer to priorities. */
		float* Priorities;
	};

protected:
	void PrepareBatch(FBatchParams& BatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset);
	void PrioritizeBatch(FBatchParams& BatchParams);
	void FinishBatch(const FBatchParams& BatchParams, FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset);
	void SetupBatchParams(FBatchParams& OutBatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 MaxBatchObjectCount, FMemStackBase& Mem);
	void SetupCalculationConstants(FPriorityCalculationConstants& OutConstants);

	TStrongObjectPtr<UFieldOfViewNetObjectPrioritizerConfig> Config;
};
