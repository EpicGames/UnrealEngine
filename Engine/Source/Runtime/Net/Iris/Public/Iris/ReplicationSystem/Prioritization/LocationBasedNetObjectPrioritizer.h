// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizer.h"
#include "Net/Core/NetBitArray.h"
#include "Math/VectorRegister.h"
#include "UObject/StrongObjectPtr.h"
#include "LocationBasedNetObjectPrioritizer.generated.h"

namespace UE::Net
{
	class FWorldLocations;
}

UCLASS(Transient, MinimalAPI, Abstract)
class ULocationBasedNetObjectPrioritizer : public UNetObjectPrioritizer
{
	GENERATED_BODY()

protected:
	// UNetObjectPrioritizer interface
	IRISCORE_API virtual void Init(FNetObjectPrioritizerInitParams& Params) override;
	IRISCORE_API virtual void Deinit() override;
	IRISCORE_API virtual void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex) override;
	IRISCORE_API virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerAddObjectParams& Params) override;
	IRISCORE_API virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectPrioritizationInfo& Info) override;
	IRISCORE_API virtual void UpdateObjects(FNetObjectPrioritizerUpdateParams&) override;

protected:
	IRISCORE_API ULocationBasedNetObjectPrioritizer();

	struct FObjectLocationInfo : public FNetObjectPrioritizationInfo
	{
		bool IsUsingWorldLocations() const { return GetLocationStateIndex() == InvalidStateIndex; }
		bool IsUsingLocationInState() const { return GetLocationStateIndex() != InvalidStateIndex; }

		void SetLocationStateOffset(uint16 Offset) { Data[0] = Offset; }
		uint16 GetLocationStateOffset() const { return Data[0]; }

		void SetLocationStateIndex(uint16 Index) { Data[1] = Index; }
		uint16 GetLocationStateIndex() const { return Data[1]; }

		void SetLocationIndex(uint32 Index) { Data[2] = Index & 65535U; Data[3] = Index >> 16U; }
		uint32 GetLocationIndex() const { return (uint32(Data[3]) << 16U) | uint32(Data[2]); }
	};

	IRISCORE_API VectorRegister GetLocation(const FObjectLocationInfo& Info) const;
	IRISCORE_API void SetLocation(const FObjectLocationInfo& Info, VectorRegister Location);
	IRISCORE_API void UpdateLocation(const UE::Net::FInternalNetRefIndex ObjectIndex, const FObjectLocationInfo& Info, const UE::Net::FReplicationInstanceProtocol* InstanceProtocol);

private:
	enum : unsigned
	{
		LocationsChunkSize = 64*1024,
		InvalidStateIndex = 65535U,
		InvalidStateOffset = 65535U,
	};

	uint32 AllocLocation();
	void FreeLocation(uint32 Index);

	TChunkedArray<VectorRegister, LocationsChunkSize> Locations;
	UE::Net::FNetBitArray AssignedLocationIndices;
	const UE::Net::FWorldLocations* WorldLocations = nullptr;
};
