// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Prioritization/NetObjectPrioritizer.h"
#include "Net/Core/NetBitArray.h"
#include "UObject/StrongObjectPtr.h"
#include "NetObjectCountLimiter.generated.h"

UENUM()
enum class ENetObjectCountLimiterMode : uint32
{
	/**
	 * Each net update the next N, as configured, objects will be allowed to be replicated if they have modified properties.
	 * This means that even if there are many objects that have modified properties none will be sent if the N objects that
	 * are considered this frame haven't been modified.
	 */
	RoundRobin,
	/**
	 * Each net update the N least recently replicated objects with modified properties will be allowed to be replicated.
	 * This can cause an object to be replicated very often if it's modified a lot and nothing else is.
	 */
	Fill,
};

//TODO $IRIS: Document class usage
UCLASS(transient, config=Engine, MinimalAPI)
class UNetObjectCountLimiterConfig : public UNetObjectPrioritizerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	ENetObjectCountLimiterMode Mode = ENetObjectCountLimiterMode::RoundRobin;

	/**
	 * How many objects to be considered for replication each frame.
	 * With 2 at least 1 object that isn't owned by the connection will be considered.
	 * If the prioritizer won't deal with objects that are owned by a specific connection
	 * it's safe to set to 1.
	 */
	UPROPERTY(Config)
	uint32 MaxObjectCount = 2;

	/**
	  * Which priority to set for objects considered for replication.
	  * Priority is accumulated for an object until it's replicated.
	  * 1.0f is the threshold at which the object may be replicated.
	  */
	UPROPERTY(Config)
	float Priority = 1.0f;

	/**
	 * The priority to set for a considered object if it's owned by the connection being prioritized for.
	 */
	UPROPERTY(Config)
	float OwningConnectionPriority = 1.0f;

	/**
	 * Whether objects owned by the connection should always be considered for replication.
	 * If so, such objects won't count against the MaxObjectCount.
	 */
	UPROPERTY(Config)
	bool bEnableOwnedObjectsFastLane = true;
};

UCLASS()
class UNetObjectCountLimiter : public UNetObjectPrioritizer
{
	GENERATED_BODY()

protected:
	// UNetObjectPrioritizer interface
	IRISCORE_API virtual void Init(FNetObjectPrioritizerInitParams&) override;
	IRISCORE_API virtual void Deinit() override;
	virtual void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex) override {}
	IRISCORE_API virtual void AddConnection(uint32 ConnectionId) override;
	IRISCORE_API virtual void RemoveConnection(uint32 ConnectionId) override;
	IRISCORE_API virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerAddObjectParams&) override;
	IRISCORE_API virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectPrioritizationInfo&) override;
	IRISCORE_API virtual void UpdateObjects(FNetObjectPrioritizerUpdateParams&) override;
	IRISCORE_API virtual void PrePrioritize(FNetObjectPrePrioritizationParams&) override;
	IRISCORE_API virtual void Prioritize(FNetObjectPrioritizationParams&) override;

protected:
	struct FObjectInfo : public FNetObjectPrioritizationInfo
	{
		void SetPrioritizerInternalIndex(uint16 Index) { Data[0] = Index; }
		uint16 GetPrioritizerInternalIndex() const { return Data[0]; }

		void SetOwningConnection(uint32 ConnectionId) { Data[1] = static_cast<uint16>(ConnectionId); }
		uint32 GetOwningConnection() const { return static_cast<uint32>(Data[1]); }
	};

	struct FPerConnectionInfo
	{
		// Which frame was this object last considered for replication for this connection
		TArray<uint32> LastConsiderFrames;
	};

	struct FBatchParams
	{
		uint32 ConnectionId;
		uint32 ObjectCount;
		float* Priorities;
	};

protected:
	IRISCORE_API UNetObjectCountLimiter();

	IRISCORE_API uint32 GetLastConsiderFrame(uint32 ConnectionId, uint32 ObjectIndex) const;
	IRISCORE_API void SetLastConsiderFrame(uint32 ConnectionId, uint32 ObjectIndex, uint32 FrameNumber);

private:
	enum : unsigned
	{
		// Keep the ObjectGrowCount fairly low as allocation size = ObjectGrowCount * max number of connections * size for per object info
		ObjectGrowCount = 64U,
	};

	struct FRoundRobinState
	{
		UE::Net::FNetBitArray InternalObjectIndices;
		uint16 NextIndexToConsider = 0;
	};

	struct FFillState
	{
		uint32 LastPrioFrame = 0;
		uint32 LastConnectionId = 0;
	};

	uint16 AllocInternalIndex();
	void FreeInternalIndex(uint16 Index);
	void PrePrioritizeForRoundRobin();
	void PrioritizeForRoundRobin(FNetObjectPrioritizationParams&) const;
	void PrioritizeForFill(FNetObjectPrioritizationParams&);

private:
	TObjectPtr<const UReplicationSystem> ReplicationSystem;
	TStrongObjectPtr<UNetObjectCountLimiterConfig> Config;
	TArray<FPerConnectionInfo> PerConnectionInfos;
	UE::Net::FNetBitArray InternalObjectIndices;
	FRoundRobinState RoundRobinState;
	FFillState FillState;
	uint32 PrioFrame;
	uint32 ReplicationSystemId;
};
