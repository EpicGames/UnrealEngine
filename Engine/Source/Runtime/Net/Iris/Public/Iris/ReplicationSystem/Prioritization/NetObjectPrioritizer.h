// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Core/NetChunkedArray.h"
#include "Iris/ReplicationSystem/ReplicationView.h"
#include "UObject/ObjectMacros.h"
#include "NetObjectPrioritizer.generated.h"

struct FNetObjectPrioritizationInfo;
class UReplicationSystem;

namespace UE::Net
{
	typedef uint32 FNetObjectPrioritizerHandle;
	typedef uint32 FInternalNetRefIndex;

	struct FReplicationInstanceProtocol;
	struct FReplicationProtocol;
	struct FReplicationView;
}

namespace UE::Net
{

/** Used to represent an invalid handle to a prioritizer. */
constexpr FNetObjectPrioritizerHandle InvalidNetObjectPrioritizerHandle = ~FNetObjectPrioritizerHandle(0);

/**
 * The handle of the default prioritizer.
 * The first valid prioritizer definition will assume the role as default spatial prioritizer. All objects with a RepTag_WorldLocation tag 
 * will be added to the default prioritizer. To override the behavior a prioritizer must be set via calls to the ReplicationSystem.
 * @see UNetObjectPrioritizerDefinitions
 */
constexpr FNetObjectPrioritizerHandle DefaultSpatialNetObjectPrioritizerHandle = FNetObjectPrioritizerHandle(0);

}

/**
  * Parameters passed to UNetObjectPrioritizer::Prioritize.
  *
  * The prioritizer should honor the existing priority set for an object and only update the
  * priority if the calculated value is higher than what is already stored.
  * Prioritize() is only allowed to modify Priorities.
  */
struct FNetObjectPrioritizationParams
{
	//TODO: this should be a TConstArrayView
	/** 
	 * The indices for the objects that are being prioritized. Will only contain objects
	 * which have been added to the prioritizer and have dirty properties or are in need of resending.
	 */
	const UE::Net::FInternalNetRefIndex* ObjectIndices;

	/** The number of objects to prioritize. */
	uint32 ObjectCount;

	//TODO: This should be a TArrayView
	/** Priorities for all objects. Index using ObjectIndices[0..ObjectCount-1]. */
	float* Priorities;

	/** PrioritizationInfos for all objects. Index using ObjectIndices[0..ObjectCount-1]. */
	const FNetObjectPrioritizationInfo* PrioritizationInfos;

	/** ID of the connection that objects are prioritized for. */
	uint32 ConnectionId;

	/** The view associated with the connection and its sub-sconnections that objects are prioritized for. */
	UE::Net::FReplicationView View;
};

/** Parameters passed to the prioritizer's PrePrioritize call. */
struct FNetObjectPrePrioritizationParams
{
};

/** Parameters passed to the prioritizer's PostPrioritize call. */
struct FNetObjectPostPrioritizationParams
{
};

/**
 * Prioritizer specific data stored per object, such as offsets to tags.
 * The data is initialized to zero by default.
 */
struct alignas(8) FNetObjectPrioritizationInfo
{
	uint16 Data[4];
};

/**
 * Base class for prioritizer specific configuration.
 * @see FNetObjectPrioritizerDefinition
 */
UCLASS(Transient, MinimalAPI)
class UNetObjectPrioritizerConfig : public UObject
{
	GENERATED_BODY()
};

/** Parameters passed to the prioritizer's Init() call. */
struct FNetObjectPrioritizerInitParams
{
	/** The ReplicationSystem that owns the prioritizer. */
	TObjectPtr<const UReplicationSystem> ReplicationSystem;
	/** Optional config as set in the FNetObjectPrioritizerDefinition. */
	UNetObjectPrioritizerConfig* Config = nullptr;
	/** The maximum number of replicated objects in the system. */
	uint32 AbsoluteMaxNetObjectCount = 0;
	/** The current maximum replicated objects referenced by an index (may grow at runtime). */
	uint32 CurrentMaxInternalIndex = 0;
	/** The maximum number of connections in the system. */
	uint32 MaxConnectionCount = 0;
};

/** Parameters passed to the prioritizer's AddObject() call. */
struct FNetObjectPrioritizerAddObjectParams
{
	/** The info is zeroed before the AddObject() call. Fill in with prioritizer specifics, like offsets to tags. */
	FNetObjectPrioritizationInfo& OutInfo;

	/** The FReplicationInstanceProtocol which describes the source state data. */
	const UE::Net::FReplicationInstanceProtocol* InstanceProtocol;

	/** The FReplicationProtocol which describes the internal state data. */
	const UE::Net::FReplicationProtocol* Protocol;

	/**
	 * One can retrieve relevant information from the object state buffer using the FReplicationProtocol.
	 * Note that this is the internal network representation of the data which is stored in quantized form.
	 * NetSerializers can dequantize the data to the original source data form.
	 */
	const uint8* StateBuffer;
};

/** Parameters passed to the prioritizer's UpdateObjects() call. */
struct FNetObjectPrioritizerUpdateParams
{
	//TODO: this should be a TConstArrayView
	/** Indices of the updated objects. */
	const UE::Net::FInternalNetRefIndex* ObjectIndices;
	/** The number of objects that have been updated. */
	uint32 ObjectCount;

	/** InstanceProtocols for updated objects. Index using 0..ObjectCount-1. */
	UE::Net::FReplicationInstanceProtocol const*const* InstanceProtocols;

	/** State buffers for all objects. Index using ObjectIndices[0..ObjectCount-1]. */
	const UE::Net::TNetChunkedArray<uint8*>* StateBuffers = nullptr;

	/** Infos for all objects. Index using ObjectIndices[0..ObjectCount-1]. */
	FNetObjectPrioritizationInfo* PrioritizationInfos;
};

/**
 * NetObjectPrioritizers are responsible for determining how important it is to replicate an object. Priorities should be at least 0.0f, 
 * meaning no need to replicate. At 1.0f objects are being considered for replication. Priorities are acumulated per object and connection 
 * until it's replicated, at which point the priority is reset to zero. Bandwidth constraints and other factors may cause a highly prioritized 
 * object to still not be replicated to a particular connection a certain frame. There is no mechanism to force an object to be replicated a 
 * certain frame, but the priority is a major factor in the decision.
 */
UCLASS(Abstract, MinimalAPI)
class UNetObjectPrioritizer : public UObject
{
	GENERATED_BODY()

public:
	/** Called once at init time before any other calls to the prioritizer. */
	virtual void Init(FNetObjectPrioritizerInitParams& Params) PURE_VIRTUAL(Init,)

	/** Called when the replication system is shutting down. Use this to remove references to other systems */
	virtual void Deinit() PURE_VIRTUAL(Deinit)

	/** Called when the maximum InternalNetRefIndex increased and we need to realloc our lists */
	virtual void OnMaxInternalNetRefIndexIncreased(UE::Net::FInternalNetRefIndex NewMaxInternalIndex) PURE_VIRTUAL(OnMaxInternalNetRefIndexIncreased);

	/** A new connection has been added. An opportunity for the prioritizer to allocate per connection info. */
	IRISCORE_API virtual void AddConnection(uint32 ConnectionId);

	/** A new connection has been added. An opportunity for the prioritizer to deallocate per connection info. */
	IRISCORE_API virtual void RemoveConnection(uint32 ConnectionId);

	/** A new object want to use this prioritizer. Opportunity to cache some information for it. The info struct passed has been zeroed. */
	virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerAddObjectParams& Params) PURE_VIRTUAL(AddObject, return false;)

	/** An object do no longer want to use this prioritizer. */
	virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectPrioritizationInfo& Info) PURE_VIRTUAL(RemoveObject,)

	/** A set of objects used by this prioritizer have been updated. An opportunity for the prioritizer to update cached data. */
	virtual void UpdateObjects(FNetObjectPrioritizerUpdateParams&) PURE_VIRTUAL(UpdateObjects,)

	/**
	 * If there are any connections being replicated and there's a chance Prioritize() will be called then PrePrioritize()
	 * will be called exactly once before all calls to Prioritize().
	 */
	IRISCORE_API virtual void PrePrioritize(FNetObjectPrePrioritizationParams&);

	/**
	 * Prioritize a batch of objects. There may be multiple calls to this function even for the same connection. 
	 * Stored priorities are expected to use the maximum of the already stored priority and the prioritizer calculated one.
	 * That allows multiple prioritizers to be used on the same object.
	 */
	virtual void Prioritize(FNetObjectPrioritizationParams&) PURE_VIRTUAL(Prioritize,);

	/** If PrePrioritize() was called then PostPrioritize() will be called exactly once after all Prioritize() calls. */
	IRISCORE_API virtual void PostPrioritize(FNetObjectPostPrioritizationParams&);

protected:
	IRISCORE_API UNetObjectPrioritizer();
};
