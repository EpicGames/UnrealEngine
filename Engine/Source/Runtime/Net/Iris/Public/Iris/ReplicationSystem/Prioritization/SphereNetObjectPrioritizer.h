// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Prioritization/LocationBasedNetObjectPrioritizer.h"
#include "Net/Core/NetBitArray.h"
#include "Math/VectorRegister.h"
#include "UObject/StrongObjectPtr.h"
#include "SphereNetObjectPrioritizer.generated.h"

class FMemStackBase;

UCLASS(transient, config=Engine, MinimalAPI)
class USphereNetObjectPrioritizerConfig : public UNetObjectPrioritizerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY(Config)
	float InnerRadius = 2000.0f;

	UPROPERTY(Config)
	float OuterRadius = 10000.0f;

	UPROPERTY(Config)
	/** Priority for objects inside the inner sphere */
	float InnerPriority = 1.0f;

	UPROPERTY(Config)
	/** Priority at the border of the outer sphere */
	float OuterPriority = 0.2f;

	UPROPERTY(Config)
	/** Priority outside the sphere */
	float OutsidePriority = 0.1f;
};

UCLASS(Transient, MinimalAPI)
class USphereNetObjectPrioritizer : public ULocationBasedNetObjectPrioritizer
{
	GENERATED_BODY()

protected:
	// UNetObjectPrioritizer interface
	IRISCORE_API virtual void Init(FNetObjectPrioritizerInitParams& Params) override;
	IRISCORE_API virtual void Deinit() override;
	IRISCORE_API virtual void Prioritize(FNetObjectPrioritizationParams&) override;

protected:
	struct FPriorityCalculationConstants
	{
		VectorRegister InnerRadius;
		VectorRegister OuterRadius;
		// OuterRadius - InnerRadius
		VectorRegister RadiusDiff;
		VectorRegister InvRadiusDiff;
		VectorRegister InnerPriority;
		VectorRegister OuterPriority;
		VectorRegister OutsidePriority;
		// OuterPriority - InnerPriority 
		VectorRegister PriorityDiff;
	};

	struct FBatchParams
	{
		FPriorityCalculationConstants PriorityCalculationConstants;
		UE::Net::FReplicationView View;
		uint32 ConnectionId;

		uint32 ObjectCount;
		VectorRegister* Positions;
		/** 16-byte aligned pointer to priorities. */
		float* Priorities;
	};

protected:
	void PrepareBatch(FBatchParams& BatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset);
	void PrioritizeBatch(FBatchParams& BatchParams);
	void PrioritizeBatchForSingleView(FBatchParams& BatchParams);
	void PrioritizeBatchForDualView(FBatchParams& BatchParams);
	void PrioritizeBatchForMultiView(FBatchParams& BatchParams);
	void FinishBatch(const FBatchParams& BatchParams, FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset);
	void SetupBatchParams(FBatchParams& OutBatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 MaxBatchObjectCount, FMemStackBase& Mem);
	void SetupCalculationConstants(FPriorityCalculationConstants& OutConstants);

	TStrongObjectPtr<USphereNetObjectPrioritizerConfig> Config;
};
