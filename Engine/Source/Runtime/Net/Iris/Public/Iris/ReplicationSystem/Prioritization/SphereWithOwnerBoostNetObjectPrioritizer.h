// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Prioritization/SphereNetObjectPrioritizer.h"
#include "UObject/ObjectPtr.h"
#include "SphereWithOwnerBoostNetObjectPrioritizer.generated.h"

UCLASS(Transient, Config=Engine, MinimalAPI)
class USphereWithOwnerBoostNetObjectPrioritizerConfig : public USphereNetObjectPrioritizerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY(Config)
	/** Priority boost for the owning connection. Added to the priority calculated by the sphere prioritizer. */
	float OwnerPriorityBoost = 2.0f;
};

UCLASS(Transient, MinimalAPI)
class USphereWithOwnerBoostNetObjectPrioritizer : public USphereNetObjectPrioritizer
{
	GENERATED_BODY()

protected:
	// UNetObjectPrioritizer interface
	IRISCORE_API virtual void Init(FNetObjectPrioritizerInitParams& Params) override;
	IRISCORE_API virtual void Deinit() override;
	IRISCORE_API virtual bool AddObject(UE::Net::FInternalNetRefIndex ObjectIndex, FNetObjectPrioritizerAddObjectParams& Params) override;
	IRISCORE_API virtual void RemoveObject(UE::Net::FInternalNetRefIndex ObjectIndex, const FNetObjectPrioritizationInfo& Info) override;
	IRISCORE_API virtual void UpdateObjects(FNetObjectPrioritizerUpdateParams&) override;
	IRISCORE_API virtual void Prioritize(FNetObjectPrioritizationParams&) override;

protected:
	using ConnectionId = uint16;
	enum : unsigned
	{
		/** We're not expecting a huge number of objects being added to this prioritizer. A connection ID is two bytes. */
		OwningConnectionsChunkSize = 2U*1024U,
		InvalidConnectionID = 0U,
	};

	struct FOwnerBoostBatchParams : public Super::FBatchParams
	{
		/** Batch indices for objects owned by the connection being prioritized for. Access other arrays in the batch via  OtherArray[OwnedObjectsLocalIndices[0..OwnedObjectCount-1]]. */
		uint32* OwnedObjectsLocalIndices;
		/** The number of owned objects in this batch. Local indices for them can be found in OwnedObjectsLocalIndices. */
		uint32 OwnedObjectCount;
		/** How much to add to the priority for owned objects. */
		float OwnerPriorityBoost;
	};

	void PrepareBatch(FOwnerBoostBatchParams& BatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset);
	void PrioritizeBatch(FOwnerBoostBatchParams& BatchParams);
	void FinishBatch(const FOwnerBoostBatchParams& BatchParams, FNetObjectPrioritizationParams& PrioritizationParams, uint32 ObjectIndexStartOffset);
	void BoostOwningConnectionPriorities(FOwnerBoostBatchParams& BatchParams) const;
	void SetupBatchParams(FOwnerBoostBatchParams& OutBatchParams, const FNetObjectPrioritizationParams& PrioritizationParams, uint32 MaxBatchObjectCount, FMemStackBase& Mem);

	uint32 AllocOwningConnection();
	void FreeOwningConnection(uint32 Index);

	uint32 GetOwningConnection(const FObjectLocationInfo& Info) const;

	/** The IDs of the objects' owning connection. */
	TChunkedArray<ConnectionId, OwningConnectionsChunkSize> OwningConnections;
	/** Which indices in OwningConnections are in use. */
	UE::Net::FNetBitArray AssignedOwningConnectionIndices;

private:
	TObjectPtr<const UReplicationSystem> ReplicationSystem = nullptr;
};

inline uint32 USphereWithOwnerBoostNetObjectPrioritizer::GetOwningConnection(const FObjectLocationInfo& Info) const
{
	return OwningConnections[Info.GetLocationIndex()];
}
