// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/ReplicationState/IrisFastArraySerializer.h"
#include "Iris/ReplicationState/Private/IrisFastArraySerializerInternal.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Net/Core/NetBitArray.h"
#include "ProfilingDebugging/CsvProfiler.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(NETCORE_API, Networking);

namespace UE::Net
{

namespace FastArrayPollingPolicies
{

struct FPollingState
{
	int32 ArrayReplicationKey = -1;
	struct FEntry
	{
		int32 ReplicationKey = -1;
		int32 ReplicationID = -1;
	};
	TArray<FEntry>  ItemPollData;
};

/*
 * FNeedPollingPolicy, used for Native Iris FastArrays that does not need polling
 */
class FNeedPollingPolicy
{
public:
	FPollingState* GetPollingState() { return &PollingState; }	
private:
	FPollingState PollingState;
};

/*
 * FNeedPollingPolicy, used for Native Iris FastArrays that do not require polling
 */
class FNoPollingPolicy
{
public:
	FPollingState* GetPollingState() { return nullptr; }
};

} // End of namespace FastArrayPollingPolicies

namespace Private
{

/** Utility methods to behave similar to FastArraySerializer */
struct FFastArrayReplicationFragmentHelper
{
	/** Rebuild IndexMap for FastArrraySerializer */
	template <typename FastArrayType, typename ItemArrayType>
	static void ConditionalRebuildItemMap(FastArrayType& ArraySerializer, const ItemArrayType& Items, bool bForceRebuild);

	/** Apply received state and try to behave like current FastArrays */
	template <typename FastArrayType, typename ItemArrayType>
	static void ApplyReplicatedState(FastArrayType* DstFastArray, ItemArrayType* DstWrappedArray, FastArrayType* SrcFastArray, const ItemArrayType* SrcWrappedArray, const FReplicationStateDescriptor* ArrayElementDescriptor, FReplicationStateApplyContext& Context);

	/** Apply array element, only replicated items will be applied, using the serializers' Apply function if present  */
	IRISCORE_API static void InternalApplyArrayElement(const FReplicationStateDescriptor* ArrayElementDescriptor, void* RESTRICT Dst, const void* RESTRICT Src);

	/** Copy array element, only replicated items will be copied */
	IRISCORE_API static void InternalCopyArrayElement(const FReplicationStateDescriptor* ArrayElementDescriptor, void* RESTRICT Dst, const void* RESTRICT Src);

	/** Compare array element, only replicated items will be compared */
	IRISCORE_API static bool InternalCompareArrayElement(const FReplicationStateDescriptor* ArrayElementDescriptor, void* RESTRICT Dst, const void* RESTRICT Src);

	/** Find the member index of the FastArrayIteArray, used to support FastArrayNetSerializers with extra properties */
	IRISCORE_API static uint32 GetFastArrayStructItemArrayMemberIndex(const FReplicationStateDescriptor* StructDescriptor);

	/**
	 * Conditionally invoke PostReplicatedReceive method depending on if it is defined or not
	 * We only want to do this for FastArrays that define PostReplicatedReceive since it might require extra work to calculate the required parameters
	 */
	template<typename FastArrayType>
	static inline typename TEnableIf<TModels_V<FFastArraySerializer::CPostReplicatedReceiveFuncable, FastArrayType, const FFastArraySerializer::FPostReplicatedReceiveParameters&>, void>::Type CallPostReplicatedReceiveOrNot(FastArrayType& ArraySerializer, int32 OldArraySize, bool bHasUnresolvedReferences)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FFastArraySerializer::FPostReplicatedReceiveParameters PostReceivedParameters = { OldArraySize, bHasUnresolvedReferences };
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		ArraySerializer.PostReplicatedReceive(PostReceivedParameters);
	}

	template<typename FastArrayType>
	static inline typename TEnableIf<!TModels_V<FFastArraySerializer::CPostReplicatedReceiveFuncable, FastArrayType, const FFastArraySerializer::FPostReplicatedReceiveParameters&>, void>::Type CallPostReplicatedReceiveOrNot(FastArrayType& ArraySerializer, int32 OldArraySize, bool bHasUnresolvedReferences) {}
};

class FFastArrayReplicationFragmentBase : public FReplicationFragment
{
public:
	IRISCORE_API void Register(FFragmentRegistrationContext& Context, EReplicationFragmentTraits InTraits);

protected:
	IRISCORE_API FFastArrayReplicationFragmentBase(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor, bool bValidateDescriptor = true);
	IRISCORE_API virtual ~FFastArrayReplicationFragmentBase();

	// FReplicationFragment Implementation
	IRISCORE_API virtual void CollectOwner(FReplicationStateOwnerCollector* Owners) const override;

protected:
	// Get the ReplicationStateDescriptor for the FastArraySerializer Struct
	IRISCORE_API const FReplicationStateDescriptor* GetFastArrayPropertyStructDescriptor() const;

	// Get the ReplicationStateDescriptor for the Array Element
	IRISCORE_API const FReplicationStateDescriptor* GetArrayElementDescriptor() const;

	// Copy array element using the descriptor to esure that we only copy replicated data
	IRISCORE_API static void InternalCopyArrayElement(const FReplicationStateDescriptor* ArrayElementDescriptor, void* RESTRICT Dst, const void* RESTRICT Src);

	// Compare an array element using the descriptor to ensure that we only compare replicated data
	IRISCORE_API static bool InternalCompareArrayElement(const FReplicationStateDescriptor* ArrayElementDescriptor, void* RESTRICT Dst, const void* RESTRICT Src);

	// Dequantize state into DstExternalBuffer, Note: it is expected to be initialized
	IRISCORE_API static void InternalDequantizeFastArray(FNetSerializationContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* FastArrayPropertyDescriptor);

	// Partial dequantize state based on changemask into DstExternalBuffer, Note: it is expected to be initialized
	IRISCORE_API static void InternalPartialDequantizeFastArray(FReplicationStateApplyContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* FastArrayPropertyDescriptor);

	// Dequantize additional properties to  DstExternalBuffer, Note: it is expected to be initialized
	IRISCORE_API static void InternalDequantizeExtraProperties(FNetSerializationContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	// Dequantize and output state to string
	IRISCORE_API static void ToString(FStringBuilderBase& StringBuilder, const uint8* ExternalStateBuffer, const FReplicationStateDescriptor* FastArrayPropertyDescriptor);

protected:
	// Replication descriptor built for the specific property
	TRefCountPtr<const FReplicationStateDescriptor> ReplicationStateDescriptor;

	// This is the source state from which we source our state data
	TUniquePtr<FPropertyReplicationState> ReplicationState;

	// Owner
	UObject* Owner;

	// This allows us to quickly find the wrapped array relative to the owner
	SIZE_T WrappedArrayOffsetRelativeFastArraySerializerProperty;
};

// Native FastArray
class FNativeFastArrayReplicationFragmentBase : public FReplicationFragment
{
protected:
	IRISCORE_API FNativeFastArrayReplicationFragmentBase(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor);
	IRISCORE_API const FReplicationStateDescriptor* GetFastArrayPropertyStructDescriptor() const;
	IRISCORE_API const FReplicationStateDescriptor* GetArrayElementDescriptor() const;
	IRISCORE_API virtual void CollectOwner(FReplicationStateOwnerCollector* Owners) const override;

	// Dequantize state into DstExternalBuffer, note it is expected to be initialized
	IRISCORE_API static void InternalDequantizeFastArray(FNetSerializationContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* FastArrayPropertyDescriptor);

	IRISCORE_API static void ToString(FStringBuilderBase& StringBuilder, const uint8* ExternalStateBuffer, const FReplicationStateDescriptor* FastArrayPropertyDescriptor);

protected:
	// Replication descriptor built for the specific property
	TRefCountPtr<const FReplicationStateDescriptor> ReplicationStateDescriptor;

	// Owner
	UObject* Owner;

	// This allows us to quickly find the wrapped array relative to the owner
	SIZE_T WrappedArrayOffsetRelativeFastArraySerializerProperty;
};

template <typename FastArrayType, typename ItemArrayType>
void FFastArrayReplicationFragmentHelper::ConditionalRebuildItemMap(FastArrayType& ArraySerializer, const ItemArrayType& Items, bool bForceRebuild)
{
	typedef typename ItemArrayType::ElementType ItemType;

	if (bForceRebuild || ArraySerializer.ItemMap.Num() != Items.Num())
	{
		UE_LOGF(LogNetFastTArray, Verbose, "FastArrayDeltaSerialize: Recreating Items map. Items.Num: %d Map.Num: %d", Items.Num(), ArraySerializer.ItemMap.Num());

		ArraySerializer.ItemMap.Reset();
			
		const ItemType* SrcItems = Items.GetData();
		for (int32 It = 0, EndIt = Items.Num(); It != EndIt; ++It)
		{
			const ItemType& Item = SrcItems[It];
			if (Item.ReplicationID == INDEX_NONE)
			{
				continue;
			}

			ArraySerializer.ItemMap.Add(Item.ReplicationID, It);
		}
	}
}

template <typename FastArrayType, typename ItemArrayType>
void FFastArrayReplicationFragmentHelper::ApplyReplicatedState(FastArrayType* DstArraySerializer, ItemArrayType* DstWrappedArray, FastArrayType* SrcArraySerializer, const ItemArrayType* SrcWrappedArray, const FReplicationStateDescriptor* ArrayElementDescriptor, FReplicationStateApplyContext& Context)
{
	typedef typename ItemArrayType::ElementType ItemType;

	CSV_SCOPED_TIMING_STAT(Networking, FastArray_Apply);

	UE_LOGF(LogNetFastTArray, Log, "FFastArrayReplicationFragmentHelper::ApplyReplicatedState for %ls", Context.Descriptor->DebugName->Name);

	const uint32* ChangeMaskData = Context.StateBufferData.ChangeMaskData;
	FNetBitArrayView MemberChangeMask = MakeNetBitArrayView(ChangeMaskData, Context.Descriptor->ChangeMaskBitCount);

	// We currently use a simple modulo scheme for bits in the changemask
	// A single bit might represent several entries in the array which all will be considered dirty, it is up to the serializer to handle this
	// The first bit is used by the owning property we need to offset by one and deduct one from the usable bits
	const FReplicationStateMemberChangeMaskDescriptor& MemberChangeMaskDescriptor = Context.Descriptor->MemberChangeMaskDescriptors[0];
	const uint32 ChangeMaskBitOffset = MemberChangeMaskDescriptor.BitOffset + FIrisFastArraySerializer::IrisFastArrayChangeMaskBitOffset;
	const uint32 ChangeMaskBitCount = MemberChangeMaskDescriptor.BitCount - FIrisFastArraySerializer::IrisFastArrayChangeMaskBitOffset;

	// Force rebuild if the array has been modified
	const bool bForceRebuildItemMap = MemberChangeMask.GetBit(0);

	// We need to rebuild our maps for both target array and incoming data
	// Can optimize this later
	ConditionalRebuildItemMap(*DstArraySerializer, *DstWrappedArray, false);
	ConditionalRebuildItemMap(*SrcArraySerializer, *SrcWrappedArray, bForceRebuildItemMap);

	// We need this for callback
	const int32 OriginalSize = DstWrappedArray->Num();

	// Find removed elements in received data, that is elements that exist in old map but not in new map
	TArray<int32> RemovedIndices;
	{
		RemovedIndices.Reserve(DstWrappedArray->Num());
		ItemType* DstItems = DstWrappedArray->GetData();
		for (int32 It=0, EndIt=DstWrappedArray->Num(); It != EndIt; ++It)
		{
			const int32 ReplicationID = DstItems[It].ReplicationID;
			if (ReplicationID != -1 && !SrcArraySerializer->ItemMap.Contains(ReplicationID))
			{
				UE_LOGF(LogNetFastTArray, Log, "   Removed ID: %d local Idx: %d", ReplicationID, It);
				RemovedIndices.Add(It);
			}
		}
	}
	
	// Find new and modified elements in received data, That is elements that do not exist in old map
	TArray<int32> AddedIndices;
	TArray<int32> ModifiedIndices;
	{
		AddedIndices.Reserve(SrcWrappedArray->Num());
		ModifiedIndices.Reserve(SrcWrappedArray->Num());
		const ItemType* SrcItems = SrcWrappedArray->GetData();
		for (int32 It=0, EndIt=SrcWrappedArray->Num(); It != EndIt; ++It)
		{
			const bool bIsDirty = ChangeMaskBitCount == 0U || MemberChangeMask.GetBit((It % ChangeMaskBitCount) + ChangeMaskBitOffset);
			if (!bIsDirty)
			{
				continue;
			}

			ItemType* DstItem = nullptr;
			if (int32* ExistingIndex = DstArraySerializer->ItemMap.Find(SrcItems[It].ReplicationID))
			{
				// Only compare if the changemask indicate that this might be a dirty entry, the compare is required since we do share entries in the changemask.
				if (!InternalCompareArrayElement(ArrayElementDescriptor, &(*DstWrappedArray)[*ExistingIndex], &SrcItems[It]))
				{
					UE_LOGF(LogNetFastTArray, Log, "   Changed. ID: %d -> Idx: %d", SrcItems[It].ReplicationID, *ExistingIndex);

					ModifiedIndices.Add(*ExistingIndex);

					// We use per element apply since we do not want to overwrite data that is not replicated
					DstItem = &(*DstWrappedArray)[*ExistingIndex];
					InternalApplyArrayElement(ArrayElementDescriptor, DstItem, &SrcItems[It]);
				}
			}
			else
			{
				// Since we zero initialize our replicated properties we can end up with ReplicationID == 0 when receiving partial changes which should be ignored.
				if (!(SrcItems[It].ReplicationID == INDEX_NONE || SrcItems[It].ReplicationID == 0))
				{
					int32 AddedIndex = DstWrappedArray->Add(SrcItems[It]);

					UE_LOGF(LogNetFastTArray, Log, "   New. ID: %d. New Element! local Idx: %d", SrcItems[It].ReplicationID, AddedIndex);

					// We need to propagate the ReplicationID in order to find our object
					DstItem = &(*DstWrappedArray)[AddedIndex];
					DstItem->ReplicationID = SrcItems[It].ReplicationID;

					// should we store ids or indices?
					AddedIndices.Add(AddedIndex);
				}
			}

			if (DstItem != nullptr)
			{
				// Update the item's most recent array replication key
				DstItem->MostRecentArrayReplicationKey = DstArraySerializer->ArrayReplicationKey;

				// Update the item's replication key so that a client can re-serialize the array for client replay recording
				DstItem->ReplicationKey++;
			}
		}
	}

	// Increment keys so that a client can re-serialize the array if needed, such as for client replay recording.
	DstArraySerializer->IncrementArrayReplicationKey();

	// Added and changed callbacks to FastArraySerializer
	const int32 PreRemoveSize = DstWrappedArray->Num();
	const int32 FinalSize = PreRemoveSize - RemovedIndices.Num();

	// Remove callback
	for (int32 RemovedIndex : RemovedIndices)
	{
		(*DstWrappedArray)[RemovedIndex].PreReplicatedRemove(*DstArraySerializer);
	}

	// Remove callback to FastArraySerializer - done after adding new elements
	DstArraySerializer->PreReplicatedRemove(MakeArrayView(RemovedIndices), FinalSize);

	if (PreRemoveSize != DstWrappedArray->Num())
	{
		UE_LOGF(LogNetFastTArray, Error, "Item size changed after PreReplicatedRemove! PremoveSize: %d  Item.Num: %d",
			PreRemoveSize, DstWrappedArray->Num());
	}

	// Add callbacks
	for (int32 AddedIndex : AddedIndices)
	{
		(*DstWrappedArray)[AddedIndex].PostReplicatedAdd(*DstArraySerializer);
	}
	DstArraySerializer->PostReplicatedAdd(MakeArrayView(AddedIndices), FinalSize);

	// Change callbacks
	for (int32 ExistingIndex : ModifiedIndices)
	{
		(*DstWrappedArray)[ExistingIndex].PostReplicatedChange(*DstArraySerializer);
	}
	DstArraySerializer->PostReplicatedChange(MakeArrayView(ModifiedIndices), FinalSize);

	if (PreRemoveSize != DstWrappedArray->Num())
	{
		UE_LOGF(LogNetFastTArray, Error, "Item size changed after PostReplicatedAdd/PostReplicatedChange! PreRemoveSize: %d  Item.Num: %d",
			PreRemoveSize, DstWrappedArray->Num());
	}

	// Remove indices
	if (RemovedIndices.Num() > 0)
	{
		RemovedIndices.Sort();
		for (int32 i = RemovedIndices.Num() - 1; i >= 0; --i)
		{
			int32 DeleteIndex = RemovedIndices[i];
			if (DstWrappedArray->IsValidIndex(DeleteIndex))
			{
				DstWrappedArray->RemoveAtSwap(DeleteIndex, EAllowShrinking::No);
			}
		}

		// Clear the map now that the indices are all shifted around. This kind of sucks, we could use slightly better data structures here I think.
		// This will force the ItemMap to be rebuilt for the current Items array
		DstArraySerializer->ItemMap.Empty();
	}

	// Invoke PostReplicatedReceive if is defined by the serializer
	CallPostReplicatedReceiveOrNot(*DstArraySerializer, OriginalSize, Context.bHasUnresolvableReferences);
}

}} // End of namespaces
