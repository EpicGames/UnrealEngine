// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Iris/ReplicationSystem/ReplicationFragment.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"

namespace UE::Net
{
	class FPropertyReplicationState;
}

namespace UE::Net
{

/**
* FPropertyReplicationFragment - used to bind PropertyReplicationStates to their owner
*/
class FPropertyReplicationFragment : public FReplicationFragment
{
public:
	IRISCORE_API FPropertyReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor);

	IRISCORE_API ~FPropertyReplicationFragment();

	/** Allow access to replication state */
	const FPropertyReplicationState* GetPropertyReplicationState() const { return SrcReplicationState.Get(); }

	/**
	 * Register an already existing PropertyReplicationFragment, for example one that is carried around by a base class
	*/
	void Register(FFragmentRegistrationContext& Fragments);
	
	/** 
	* Create and register a PropertyReplicationFragment using the provided descriptor, the lifetime of the fragment will be managed by the ReplicationSystem
	* Lifetime of the created fragment will be managed by the ReplicationSystem
	* returns a pointer to the created fragment
	*/
	IRISCORE_API static FPropertyReplicationFragment* CreateAndRegisterFragment(UObject* InOwner, const FReplicationStateDescriptor* InDescriptor, FFragmentRegistrationContext& Context);

protected:

	/** FReplicationFragment Implementation */
	virtual void ApplyReplicatedState(FReplicationStateApplyContext& Context) const override;

	virtual void CollectOwner(FReplicationStateOwnerCollector* Owners) const override;
	
	virtual void CallRepNotifies(FReplicationStateApplyContext& Context) override;
	
	virtual bool PollReplicatedState(EReplicationFragmentPollFlags PollOption) override;

	virtual void ReplicatedStateToString(FStringBuilderBase& StringBuilder, FReplicationStateApplyContext& Context, EReplicationStateToStringFlags Flags) const override;

private:
	// This is the source state from which we source our state data
	TUniquePtr<FPropertyReplicationState> SrcReplicationState;

	// Previous applied state, only carried around if needed
	TUniquePtr<FPropertyReplicationState> PrevReplicationState;

	TRefCountPtr<const FReplicationStateDescriptor> ReplicationStateDescriptor;

	// Owner
	UObject* Owner;
};

}
