// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// Forward declarations
struct FNetSerializerConfig;
namespace UE::Net
{
	struct FNetSerializer;
	struct FReplicationProtocol;
	struct FReplicationStateDescriptor;
}

namespace UE::Net
{

// Types
typedef uint64 FRepTag;

// Generated code for tags will define UE_REPTAG_<TAGNAME>.
// We need some tags for internal usage so we define them ourselves, compatible with generated code of course.

// This tag was generated using MakeRepTag("WorldLocation")
#ifndef UE_REPTAG_WORLDLOCATION
#define UE_REPTAG_WORLDLOCATION
constexpr FRepTag RepTag_WorldLocation = 0x0719E9E9E02F8B16ULL;
#endif

// This tag was generated using MakeRepTag("NetRole")
#ifndef UE_REPTAG_NETROLE
#define UE_REPTAG_NETROLE
constexpr FRepTag RepTag_NetRole = 0xFFAAB417B1123942ULL;
#endif

// This tag was generated using MakeRepTag("NetRemoteRole")
#ifndef UE_REPTAG_NETREMOTEROLE
#define UE_REPTAG_NETREMOTEROLE
constexpr FRepTag RepTag_NetRemoteRole = 0xF754C2703924C7AAULL;
#endif

// This tag was generated using MakeRepTag("CullDistanceSqr")
#ifndef UE_REPTAG_CULLDISTANCESQR
#define UE_REPTAG_CULLDISTANCESQR
constexpr FRepTag RepTag_CullDistanceSqr = 0x6BB13A5C1A655157ULL;
#endif

/**
 * The invalid RepTag can be used for purposes where one wants to know if there's a valid tag or not.
 * It cannot be a constant like RepTag_Invalid as that would require us to prevent a tag from being
 * called Invalid or cause mismatching values if someone calls a tag Invalid. So it's implemented
 * as a function instead.
 */
constexpr FRepTag GetInvalidRepTag() { return FRepTag(0); }

IRISCORE_API FRepTag MakeRepTag(const char* TagName);

struct FRepTagFindInfo
{
	/** If the tag is found in a protocol this indicates which state the tag was found in, otherwise it will be zero. */
	uint32 StateIndex;
	/** This is the offset into the state indicated by the StateIndex. Absolute offsets cannot beused as each external state may have its own state buffer. */
	uint32 ExternalStateOffset;
	/** The absolute offset into the internal state. One can ignore the StateIndex as the internal state is a single linear allocation for all states. */
	uint32 InternalStateAbsoluteOffset;
	const FNetSerializer* Serializer;
	const FNetSerializerConfig* SerializerConfig;
};

/** Returns true if the RepTag exists */
IRISCORE_API bool HasRepTag(const FReplicationProtocol* Protocol, FRepTag RepTag);

/**
  * Finds the first RepTag in the protocol that matches the supplied RepTag.
  * The offsets written to OutRepTagStateInfo is from the start of a full object state buffer.
  * Returns true if the tag was found, false if not. OutRepTagStateInfo is only valid if the tag is found.
  */
IRISCORE_API bool FindRepTag(const FReplicationProtocol* Protocol, FRepTag RepTag, FRepTagFindInfo& OutRepTagFindInfo);

/**
  * Finds the first RepTag in the replication state that matches the supplied RepTag.
  * The offsets written to OutRepTagStateInfo is from the start of a replication state buffer.
  * Returns true if the tag was found, false if not. OutRepTagStateInfo is only valid if the tag is found.
  */
IRISCORE_API bool FindRepTag(const FReplicationStateDescriptor* Descriptor, FRepTag RepTag, FRepTagFindInfo& OutRepTagFindInfo);

}
