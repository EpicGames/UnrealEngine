// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"

#include "HAL/Platform.h"

#include "Iris/Core/NetObjectReference.h"

#include "Iris/IrisConfig.h"

#include "Iris/ReplicationSystem/NetObjectFactory.h"
#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"
#include "Iris/ReplicationSystem/NetObjectGroupHandle.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/ObjectReferenceTypes.h"
#include "Iris/ReplicationSystem/ReplicationBridgeTypes.h"

#include "Misc/EnumClassFlags.h"

#include "Net/Core/NetHandle/NetHandle.h"

#include "UObject/ObjectKey.h"
#include "UObject/ObjectMacros.h"

#include "ReplicationBridge.generated.h"

class UObjectReplicationBridge;
class UReplicationSystem;
class UNetDriver;

namespace UE::Net
{
	enum class ENetRefHandleError : uint32;

	typedef uint32 FInternalNetRefIndex;

	struct FNetDependencyInfo;
	struct FReplicationInstanceProtocol;
	struct FReplicationProtocol;

	class FNetBitStreamReader;
	class FNetBitStreamWriter;
	class FNetSerializationContext;
	class FNetTokenStoreState;	
	class FReplicationFragment;

	namespace Private
	{
		class FNetRefHandleManager;
		class FNetObjectGroups;
		class FNetPushObjectHandle;
		class FObjectReferenceCache;
		class FReplicationProtocolManager;
		class FReplicationReader;
		class FReplicationStateDescriptorRegistry;
		class FReplicationSystemImpl;
		class FReplicationSystemInternal;
		class FReplicationWriter;
		
		struct FChangeMaskCache;
		struct FCreateNetObjectParams;
		struct FAsyncLoadingSimulator;
	}

	typedef TArray<FNetDependencyInfo, TInlineAllocator<32> > FNetDependencyInfoArray;
}

#define UE_LOG_BRIDGEID(Category, Verbosity, Format, ...)  UE_LOG(Category, Verbosity, TEXT("RepBridge(%u)::") Format, GetReplicationSystem()->GetId(), ##__VA_ARGS__)

struct FReplicationBridgeSerializationContext
{
	FReplicationBridgeSerializationContext(UE::Net::FNetSerializationContext& InSerialiazationContext, uint32 InConnectionId, bool bInIsDestructionInfo = false);

	UE::Net::FNetSerializationContext& SerializationContext;
	uint32 ConnectionId;
	bool bIsDestructionInfo;
};

//------------------------------------------------------------------------

namespace UE::Net
{
	/** The destruction info needed to replicate the destruction event later. */
	struct FDestructionParameters
	{
		/** The location of the object. Used for distance based prioritization. */
		FVector Location;
		/** The container the object is placed in. */
		const UObject* Container = nullptr;
		/** Whether to use distance based priority for the destruction of the object. */
		bool bUseDistanceBasedPrioritization = false;
		/** The NetFactory that the replicated object would be assigned to. */
		UE::Net::FNetObjectFactoryId NetFactoryId = UE::Net::InvalidNetObjectFactoryId;
	};

	enum class ESubObjectInsertionOrder : uint8
	{
		None,
		/** DEPRECATED: Specify ParentSubObject in SubObjectReplicationParams instead. See FSubObjectReplicationParams for details. */
		ReplicateWith UE_DEPRECATED(5.8, "Specify ParentSubObject in SubObjectReplicationParams instead. See FSubObjectReplicationParams for details."),
		/** Insert the subobject at the start of the parents list of subobjeccts so it can be created and replicated first */
		InsertAtStart,
		/** Insert at the start of the parents list of subobjects or right after the relative handle if one is set */
		InsertAtStartOrAfterOther,
	};

	enum class EChildSubObjectsReplicationOrder : uint8
	{
		/** Child subobjects will replicate before the parent */
		BeforeParent,
		/** Child subobjects will replicate after the parent */
		AfterParent,
	};

} // end namespace UE::Net

//------------------------------------------------------------------------
/**
 * Base replication bridge
 * Should only be used by internal Iris classes, public access should be restricted to the UObjectReplicationBridge.
 */
UCLASS(Transient, MinimalAPI)
class UReplicationBridge : public UObject
{
	GENERATED_BODY()

protected:
	using FNetHandle = UE::Net::FNetHandle;
	using FNetRefHandle = UE::Net::FNetRefHandle;
	using FNetDependencyInfoArray = UE::Net::FNetDependencyInfoArray;

public:
	IRISCORE_API UReplicationBridge();
	IRISCORE_API virtual ~UReplicationBridge();


	/**
	* Stop replicating the NetObject associated with the handle and mark the handle to be destroyed.
	* If EEndReplication::TearOff is set the remote instance will be Torn-off rather than being destroyed on the receiving end, after the call, any state changes will not be replicated
	* If EEndReplication::Flush is set all pending states will be delivered before the remote instance is destroyed, final state will be immediately copied so it is safe to remove the object after this call
	* If EEndReplication::Destroy is set the remote instance will be destroyed, if this is set for a static instance and the EndReplicationParameters are set a permanent destruction info will be added
	* Dynamic instances are always destroyed unless the TearOff flag is set.
	*/
	IRISCORE_API void StopReplicatingNetRefHandle(FNetRefHandle Handle, EEndReplicationFlags EndReplicationFlags);

	/** Returns true if the handle is replicated. */
	IRISCORE_API bool IsReplicatedHandle(FNetRefHandle Handle) const;

	/** Get the group associated with the container in order to control connection filtering for it. */
	IRISCORE_API UE::Net::FNetObjectGroupHandle GetContainerGroup(const UObject* Container) const;

	/** Get all container groups. The object key is to a UObject. The TMap must be processed immediately or copied- DO NOT CACHE. */
	IRISCORE_API const TMap<FObjectKey, UE::Net::FNetObjectGroupHandle>& GetAllContainerGroups() const;

	/** Returns true when we are in the middle of processing incoming data. */
	bool IsInReceiveUpdate() const { return bInReceiveUpdate; }

	/** Print common information about this handle and the object it is mapped to */
	[[nodiscard]] IRISCORE_API FString PrintObjectFromNetRefHandle(FNetRefHandle RefHandle) const;

	/** Returns true if object is PendingStopReplication or PendingAsyncStopReplication. */
	IRISCORE_API bool IsPendingStopReplication(FNetRefHandle Handle) const;

protected:
	/** Initializes the bridge. Is called during ReplicationSystem initialization. */
	IRISCORE_API virtual void Initialize(UReplicationSystem* InReplicationSystem);

	/** Deinitializes the bridge. Is called during ReplicationSystem deinitialization. */
	IRISCORE_API virtual void Deinitialize();

	/** Invoked before ReplicationSystem copies dirty state data. */
	virtual void PreSendUpdate() {}

	/** Invoked when the ReplicationSystem starts the PreSendUpdate tick. */
	virtual void OnStartPreSendUpdate() {}
	
	/** Invoked after we sent data to all connections. */
	virtual void OnPostSendUpdate() {}

	/** Invoked after we processed all incoming data */
	virtual void OnPostReceiveUpdate() {}
	
	/** Invoked before ReplicationSystem copies dirty state data for a single replicated object. */
	virtual void PreSendUpdateSingleHandle(FNetRefHandle Handle) {}

	/** Update world locations in FWorldLocations for objects that support it. */
	virtual void UpdateInstancesWorldLocation() {}

	// Remote interface, invoked from Replication code during serialization
	
	/**
	 * Cache info required to allow deferred writing of NetRefHandleCreationInfo
	 * @param Handle The handle of the object to store creation data for.
	 * return whether cached data is stored or not.
	*/
	IRISCORE_API virtual bool CacheNetRefHandleCreationInfo(FNetRefHandle Handle);

	/** Invoked post garbage collect to allow us to detect stale objects */
	IRISCORE_API virtual void PruneStaleObjects();

	/** Invoked when we start to replicate an object for a specific connection to fill in any initial dependencies */
	IRISCORE_API virtual void GetInitialDependencies(FNetRefHandle Handle, FNetDependencyInfoArray& OutDependencies) const;

	/** Returns if the bridge is allowed to create new destruction info at this moment. */
	virtual bool CanCreateDestructionInfo() const { return true; }

	/** Called just prior to garbage collecting/destroying the previous world tied to a NetDriver during seamless travel. Subclasses can implement virtual method OnPreSeamlessTravelGarbageCollect. */
	void PreSeamlessTravelGarbageCollect();

	/** Called just after garbage collecting/destroying the previous world tied to a NetDriver during seamless travel. Subclasses can implement virtual method OnPostSeamlessTravelGarbageCollect. */
	void PostSeamlessTravelGarbageCollect();

protected:

	// Forward calls to internal operations that we allow replication bridges to access

	/** Create a local NetRefHandle / NetObject using the ReplicationProtocol. */
	IRISCORE_API FNetRefHandle InternalCreateNetObject(FNetRefHandle AllocatedHandle, FNetHandle GlobalHandle, const UE::Net::Private::FCreateNetObjectParams& Params);

	/** Create a NetRefHandle / NetObject on request from the authoritative end. */
	IRISCORE_API FNetRefHandle InternalCreateNetObjectFromRemote(FNetRefHandle WantedNetHandle, const UE::Net::Private::FCreateNetObjectParams& Params);

	/** Attach instance to NetRefHandle. */
	IRISCORE_API void InternalAttachInstanceToNetRefHandle(FNetRefHandle RefHandle, bool bBindInstanceProtocol, UE::Net::FReplicationInstanceProtocol* InstanceProtocol, UObject* Instance, FNetHandle NetHandle);

	/** Detach instance from NetRefHandle and destroy the instance protocol. */
	IRISCORE_API void InternalDetachInstanceFromNetRefHandle(UE::Net::FInternalNetRefIndex InternalObjectIndex, bool bUnBindInstanceProtocol = true);

	/** Destroy the handle and all internal book keeping associated with it. */
	IRISCORE_API void InternalDestroyNetObject(UE::Net::FInternalNetRefIndex InternalObjectIndex);
	
	/**
	 * Add SubObjectHandle as SubObject to ParentHandle. If the ParentHandle is a SubObject itself the SubObject will be share the same RootObject but will be subjects to the Parents conditionals.
	 * InsertRelativeToSubObjectHandle and InsertionOrder applies to the order of child subobjects of the parent.
	 * ChildSubObjectsReplicationOrder specified so if a parents childsubobjects replicate before or after the parent itself.
	 */
	IRISCORE_API void InternalAddSubObject(FNetRefHandle ParentHandle, FNetRefHandle SubObjectHandle, FNetRefHandle InsertRelativeToSubObjectHandle, UE::Net::ESubObjectInsertionOrder InsertionOrder, UE::Net::EChildSubObjectsReplicationOrder ChildSubObjectsReplicationOrder);

	/** 
	 * Restart/cancel pending async stop replication.
     * Returns true if the handle was pending async stop replication and was restored to fully replicated status.
	 * NOTE: Always returns true for Handles not marked for stop replication.
	 * Passing in bForceStop as true will issue the pending async stop replication immediately and return false.
	 */
	IRISCORE_API bool RestartOrStopObjectPendingAsyncStopReplication(FNetRefHandle Handle, bool bForceStop);

	inline UE::Net::Private::FReplicationProtocolManager* GetReplicationProtocolManager() const { return ReplicationProtocolManager; }
	inline UReplicationSystem* GetReplicationSystem() const { return ReplicationSystem; }
	inline UE::Net::Private::FReplicationStateDescriptorRegistry* GetReplicationStateDescriptorRegistry() const { return ReplicationStateDescriptorRegistry; }
	inline UE::Net::Private::FObjectReferenceCache* GetObjectReferenceCache() const { return ObjectReferenceCache; }

	/** Return the NetFactoryId assigned to a replicated object. */
	IRISCORE_API UE::Net::FNetObjectFactoryId GetNetObjectFactoryId(FNetRefHandle RefHandle) const;

	/** Creates a group for a container for object filtering purposes. */
	IRISCORE_API UE::Net::FNetObjectGroupHandle CreateContainerGroup(const UObject* Container, FName PackageName);

	/** Destroys the group associated with the container. */
	IRISCORE_API void DestroyContainerGroup(const UObject* Container);

	/** Called when a remote connection detected a protocol mismatch when trying to instantiate the NetRefHandle replicated object. */
	virtual void OnProtocolMismatchReported(FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<uint8>& ClientCDOStateBytes = TArray<uint8>()) {}

	/** Called when a remote connection has a critical error caused by a specific NetRefHandle */
	virtual void OnErrorWithNetRefHandleReported(UE::Net::ENetRefHandleError ErrorType, FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<FNetRefHandle>& ExtraNetRefHandle, const FString& DiagnosticMessage) {}

	/** Called from PreSeamlessTravelGarbageCollect. */
	IRISCORE_API virtual void OnPreSeamlessTravelGarbageCollect();

	/** Called from PostSeamlessTravelGarbageCollect. */
	IRISCORE_API virtual void OnPostSeamlessTravelGarbageCollect();

	/**
	 * Remove destruction infos associated with group
	 * Passing in an invalid group handle indicates that we should remove all destruction infos
	 */
	virtual void RemoveDestructionInfosForGroup(UE::Net::FNetObjectGroupHandle GroupHandle) {}

	/** Remove destruction infos associated to a net object*/
	virtual void RemoveDestructionInfo(UE::Net::FInternalNetRefIndex InternalObjectIndex) {};

private:

	// Internal operations invoked by ReplicationSystem/ReplicationWriter
	
	enum class EPendingEndReplicationImmediate : uint8
	{
		Yes,
		No,
	};

	// Adds the Handle to the list of handles pending deferred EndReplication, if bIsImmediate is true the object will be destroyed after the next update, otherwise
	// it will be kept around until the handle is no longer ref-counted by any connection. It will however be removed from the set of scopeable objects after the first update so new connections will not add it to their scope.
	void AddPendingEndReplication(FNetRefHandle Handle, UE::Net::FInternalNetRefIndex InternalObjectIndex, EEndReplicationFlags DestroyFlags, EPendingEndReplicationImmediate Immediate = EPendingEndReplicationImmediate::No);

	void CallPreSendUpdate(float DeltaSeconds);
	void CallPreSendUpdateSingleHandle(FNetRefHandle Handle);
	void CallUpdateInstancesWorldLocation();
	bool CallCacheNetRefHandleCreationInfo(FNetRefHandle Handle);
	void CallPruneStaleObjects();
	
	void PreReceiveUpdate();
	void PostReceiveUpdate();

private:

	void InternalFlushStateData(UE::Net::FNetSerializationContext& SerializationContext, UE::Net::Private::FChangeMaskCache& ChangeMaskCache, UE::Net::FNetBitStreamWriter& ChangeMaskWriter, UE::Net::FInternalNetRefIndex InternalObjectIndex);
	// Internal method to copy state data for Handle
	void InternalFlushStateData(UE::Net::FInternalNetRefIndex InternalObjectIndex);

	// Internal method to copy state data for Handle and any SubObjects and mark them as being torn-off
	void InternalTearOff(FNetRefHandle OwnerHandle);

	// Internal method to stop replication by filtering out the replicated object. Once it is no longer replicating to any client it will be destroyed.
	void InternalRequestAsyncStopReplication(UE::Net::FInternalNetRefIndex InternalObjectIndex, EEndReplicationFlags Flags);

	/**
	 * Called from ReplicationSystem when a streaming container is about to unload.
	 * Will remove the group associated with the container and remove destruction infos.
	 */
	void NotifyStreamingContainerUnload(const UObject* Container);

	void RemoveFromAllGroups(UE::Net::FInternalNetRefIndex InternalObjectIndex);

	/** Remove mapping between handle and object instance. */
	void UnregisterInstance(UE::Net::FInternalNetRefIndex InternalNetRefIndex);

	// Destroy is split into two phases in order to support replicating data for objects where the external representation might have been destroyed.
	void InternalStartDestroyLocalSubObjects(UE::Net::FInternalNetRefIndex InternalIndex, EEndReplicationFlags Flags);
	void InternalFinishDestroyLocalSubObjects(UE::Net::FInternalNetRefIndex InternalIndex, EEndReplicationFlags Flags);
	void InternalStartDestroyLocalObject(UE::Net::FInternalNetRefIndex InternalIndex, EEndReplicationFlags Flags);
	void InternalFinishDestroyLocalObject(UE::Net::FInternalNetRefIndex InternalIndex, EEndReplicationFlags Flags);

	// Destroy local NetObject associated with handle
	void DestroyLocalNetHandle(FNetRefHandle Handle, EEndReplicationFlags Flags);

	// Update handles pending end replication before we try to anything with them
	// Detach invalid objects, tear-off handle pending tear off that has not yet been torn-off
	void PreUpdateHandlesPendingEndReplication();

	// Update all the handles pending EndReplication
	void UpdateHandlesPendingEndReplication();

	void SetNetPushIdOnFragments(const TArrayView<const UE::Net::FReplicationFragment*const>& Fragments, const UE::Net::Private::FNetPushObjectHandle& PushHandle);
	void ClearNetPushIdOnFragments(const TArrayView<const UE::Net::FReplicationFragment*const>& Fragments);

	void DestroyGlobalNetHandle(UE::Net::FInternalNetRefIndex InternalReplicationIndex);
	void ClearNetPushIds(UE::Net::FInternalNetRefIndex InternalReplicationIndex);

	friend UE::Net::Private::FReplicationSystemImpl;
	friend UE::Net::Private::FReplicationSystemInternal;

	friend UObjectReplicationBridge;
	friend UReplicationSystem;

	UReplicationSystem* ReplicationSystem;
	UE::Net::Private::FReplicationProtocolManager* ReplicationProtocolManager;
	UE::Net::Private::FReplicationStateDescriptorRegistry* ReplicationStateDescriptorRegistry;
	UE::Net::Private::FNetRefHandleManager* NetRefHandleManager;
	UE::Net::Private::FObjectReferenceCache* ObjectReferenceCache;
	UE::Net::Private::FNetObjectGroups* Groups;

	TMap<FObjectKey, UE::Net::FNetObjectGroupHandle> ContainerGroups;

	/** Tracks if we are in the middle of processing incoming data */
	bool bInReceiveUpdate = false;

	/** List of replicated objects that requested to stop replicating while we were in ReceiveUpdate */
	TMap<FNetRefHandle, EEndReplicationFlags> HandlesToStopReplicating;

private:

	struct FPendingEndReplicationInfo
	{
		FPendingEndReplicationInfo(FNetRefHandle InHandle, UE::Net::FInternalNetRefIndex InObjectIndex, EEndReplicationFlags InEndReplicationFlags, EPendingEndReplicationImmediate InImmediate)
		: Handle(InHandle)
		, ObjectIndex(InObjectIndex)
		, EndReplicationFlags(InEndReplicationFlags)
		, Immediate(InImmediate)
		{
		}

		FNetRefHandle Handle;
		UE::Net::FInternalNetRefIndex ObjectIndex;
		EEndReplicationFlags EndReplicationFlags;
		EPendingEndReplicationImmediate Immediate;
	};
	TArray<FPendingEndReplicationInfo> HandlesPendingEndReplication;

};


inline FReplicationBridgeSerializationContext::FReplicationBridgeSerializationContext(UE::Net::FNetSerializationContext& InSerialiazationContext, uint32 InConnectionId, bool bInIsDestructionInfo)
: SerializationContext(InSerialiazationContext)
, ConnectionId(InConnectionId)
, bIsDestructionInfo(bInIsDestructionInfo)
{
}
