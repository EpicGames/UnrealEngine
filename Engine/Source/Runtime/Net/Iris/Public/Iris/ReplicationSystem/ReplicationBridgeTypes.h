// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetRefHandle.h"

#include "Misc/EnumClassFlags.h"

//------------------------------------------------------------------------

/** Flags controlling how the repsystem should deal with the object that will no longer be replicated. */
enum class EEndReplicationFlags : uint32
{
	None								= 0U,
	/** Destroy remote instance. Default for dynamic objects unless they have TearOff flag set. */
	Destroy								= 1U,				
	/** Stop replication object without destroying instance on the remote end. */
	TearOff								= Destroy << 1U,
	/** Complete replication of pending state to all clients before ending replication. */
	Flush								= TearOff << 1U,
	/** Destroy NetHandle if one is associated with the replicated object. This should only be done if the object should not be replicated by any other replication system. */
	DestroyNetHandle					= Flush << 1U,
	/** Clear net push ID to prevent this object and its subobjects from being marked as dirty in the networking system. This should only be done if the object should not be replicated by any other replication system. */
	ClearNetPushId						= DestroyNetHandle << 1U,
	/** Skip bPendingEndReplication Validation, In some cases we want to allow detaching instance from replicated object on clients, such as when shutting down */
	SkipPendingEndReplicationValidation = ClearNetPushId << 1U,
	/** An object is removed from a server but the client should keep it and maintain its internal data, expecting to receive object updates from another server later. Only supported by clients with replication systems configured to use UE::Net::EProxyType::Backend. */
	ProxyReuse							= SkipPendingEndReplicationValidation << 1U,
	/** If true, replication can be resumed effectively canceling the EndReplication by issuing a StartReplication before the StopReplication has been acknowledged by all clients. */
	AllowAsyncStopReplication			= ProxyReuse << 1U,
};
ENUM_CLASS_FLAGS(EEndReplicationFlags);

IRISCORE_API FString LexToString(EEndReplicationFlags EndReplicationFlags);

//------------------------------------------------------------------------

enum class EReplicationBridgeCreateNetRefHandleResultFlags : uint32
{
	None = 0U,
	/** Whether the instance may be destroyed due to the remote peer requesting the object to be destroyed. If not then the object itself must not be destroyed. */
	AllowDestroyInstanceFromRemote = 1U << 0U,
	/** Set this flag if you created a subobject and want the RootObject to be notified of the subobject's creation. */
	ShouldCallSubObjectCreatedFromReplication = AllowDestroyInstanceFromRemote << 1U,
	/** Bind the static netrefhandle to an object. Needed when the object wasn't found via ResolveObjectReference */
	BindStaticObjectInReferenceCache = ShouldCallSubObjectCreatedFromReplication << 1U,
};
ENUM_CLASS_FLAGS(EReplicationBridgeCreateNetRefHandleResultFlags);

//------------------------------------------------------------------------

struct FReplicationBridgeCreateNetRefHandleResult
{
	UE::Net::FNetRefHandle NetRefHandle;
	EReplicationBridgeCreateNetRefHandleResultFlags Flags = EReplicationBridgeCreateNetRefHandleResultFlags::None;
};


//------------------------------------------------------------------------
//TODO: Delete this when deprecating 5.8
enum class EReplicationBridgeDestroyInstanceReason : uint32
{
	DoNotDestroy,
	TearOff,
	Destroy,
	ProxyReuse,
};

UE_DEPRECATED(5.8, "Enum is deprecated")
inline const TCHAR* LexToString(EReplicationBridgeDestroyInstanceReason Reason)
{
	return TEXT("Deprecated");
}

//------------------------------------------------------------------------
//TODO: Delete this when deprecating 5.8
enum class EReplicationBridgeDestroyInstanceFlags : uint32
{
	None = 0U,
	/** Whether the instance may be destroyed when instructed from the remote peer. This flag applies when the destroy reason is TearOff and torn off actors are to be destroyed as well as regular Destroy. */
	AllowDestroyInstanceFromRemote = 1U << 0U,
};
ENUM_CLASS_FLAGS(EReplicationBridgeDestroyInstanceFlags);

UE_DEPRECATED(5.8, "Enum is deprecated")
inline const TCHAR* LexToString(EReplicationBridgeDestroyInstanceFlags DestroyFlags)
{
	return TEXT("Deprecated");
}


