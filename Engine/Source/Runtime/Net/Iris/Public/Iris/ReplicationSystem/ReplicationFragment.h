// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/Map.h"

#include "Misc/CoreMiscDefines.h"
#include "Misc/EnumClassFlags.h"

// Forward declarations

class UReplicationSystem;
namespace UE::Net
{
	class FFragmentRegistrationContext;
	class FNetSerializationContext;
	struct FReplicationStateDescriptor;
	namespace Private
	{
		class FReplicationStateDescriptorRegistry;
		struct FFragmentRegistrationContextPrivateAccessor;
	}
}

namespace UE::Net
{

class FReplicationStateOwnerCollector
{
public:
	FReplicationStateOwnerCollector(UObject** InOwners, uint32 InMaxOwnerCount)
	: Owners(InOwners)
	, OwnerCount(0)
	, MaxOwnerCount(InMaxOwnerCount)
	{
	}

	void AddOwner(UObject* Object)
	{
		if (OwnerCount < MaxOwnerCount)
		{
			if (OwnerCount && Owners[OwnerCount - 1] == Object)
			{
				return;
			}
			Owners[OwnerCount] = Object;
			++OwnerCount;
		}
	}

	UObject** GetOwners() const { return Owners; }
	uint32 GetOwnerCount() const { return OwnerCount; }

	void Reset() { OwnerCount = 0; }

private:
	UObject** Owners;
	uint32 OwnerCount;
	const uint32 MaxOwnerCount;
};

struct FReplicationStateApplyContext
{
	const FReplicationStateDescriptor* Descriptor;
	FNetSerializationContext* NetSerializationContext;

	// We have two different variants of applying state data
	// Either we let the ReplicationSystem create and construct a temporary state OR it is up to the ReplicationFragment to manage the external buffer
	// We might for example want to allow a fragment to keep a persistent buffer and simply write directly into it.
	union FStateBufferData
	{
		uint8* ExternalStateBuffer;
		struct
		{
			const uint32* ChangeMaskData;
			const uint8* RawStateBuffer;
		};
	};
	
	FStateBufferData StateBufferData;
	
	// Indicates that this is the first time we receive this state, (it might also be set if we get this call after resolving references)
	uint32 bIsInit : 1;
	// Set when we are applying state data and a member of the state contains an unresolvable object reference
	uint32 bHasUnresolvableReferences : 1;
	// Set if this fragment is for a init state and the object has unresolved init references
	uint32 bMightHaveUnresolvableInitReferences : 1;
};

enum class EReplicationStateToStringFlags : uint32
{
	None								= 0U,
	OnlyIncludeDirtyMembers				= 1U,
};
ENUM_CLASS_FLAGS(EReplicationStateToStringFlags);

/** Traits describing a ReplicationFragmet */
enum class EReplicationFragmentTraits : uint32
{
	None								= 0,

	// Not implemented
	HasInterpolation					= 1,

	// Fragment has rep notifies
	HasRepNotifies						= HasInterpolation << 1,

	// Save previous state before apply
	KeepPreviousState					= HasRepNotifies << 1,

	// Fragment is owned by instance protocol and will be destroyed when the instance protocol is destroyed
	DeleteWithInstanceProtocol			= KeepPreviousState << 1,

	// Pass raw quantized data when applying state data
	HasPersistentTargetStateBuffer		= DeleteWithInstanceProtocol << 1,

	// Fragment can be used to source replication data
	CanReplicate						= HasPersistentTargetStateBuffer << 1,

	// Fragment can receive replication data
	CanReceive							= CanReplicate << 1,

	// Fragment requires polling to detect dirtiness
	NeedsPoll							= CanReceive << 1,

	// Fragment requires legacy callbacks
	NeedsLegacyCallbacks				= NeedsPoll << 1,

	// Fragment requires PreSendUpdate to be called 
	NeedsPreSendUpdate					= NeedsLegacyCallbacks << 1,

	// Fragment supports push based dirtiness
	HasPushBasedDirtiness				= NeedsPreSendUpdate << 1,

	// Fragment supports full push based dirtiness
	HasFullPushBasedDirtiness			= HasPushBasedDirtiness << 1,

	// Fragment is a PropertyReplication, or is using PropertyReplicationState
	HasPropertyReplicationState			= HasFullPushBasedDirtiness << 1,

	// Fragment has object references
	HasObjectReference					= HasPropertyReplicationState << 1,

	// Fragment supports partial dequantized state in apply
	SupportsPartialDequantizedState		= HasObjectReference << 1,
};
ENUM_CLASS_FLAGS(EReplicationFragmentTraits);

enum class EReplicationFragmentPollFlags : uint32
{
	None = 0,

	/** If set, we need to refresh all cached object references, typically set after a GC to detect stale references in cached data, regardless of if they are dirty or not. */
	ForceRefreshCachedObjectReferencesAfterGC = 1,

	/** Pull members marked for polling or marked as dirty in the changemask, non-push based members will alwasys be considered for polling. Note: if ForceRefreshCachedObjectReferencesAfterGC is set as well, object references must be polled as well. */
	PollDirtyState = ForceRefreshCachedObjectReferencesAfterGC << 1,

	/** Normal full poll. */
	PollAllState = PollDirtyState << 1,

	/** Enable Verbose profiling  */
	EnableVerboseProfiling = PollAllState << 1,
};
ENUM_CLASS_FLAGS(EReplicationFragmentPollFlags);

/** 
* ReplicationFragment
* Binds one or more ReplicationState(s) to the owner and is the key piece to defining the state that makes up a NetObject
* Used to extract and set state data on the game side.
*/
class FReplicationFragment
{
public:
	FReplicationFragment(const FReplicationFragment&) = delete;
	FReplicationFragment& operator=(const FReplicationFragment&) = delete;

	explicit FReplicationFragment(EReplicationFragmentTraits InTraits) : Traits(InTraits) {}
	virtual ~FReplicationFragment() = default;

	/** Traits */
	inline EReplicationFragmentTraits GetTraits() const { return Traits; }

	/**
	* This is called from the ReplicationSystem / ReplicationBridge whenever we have new data
	* Depending on the traits of the fragment we either get pointer to a StateBuffer in the expected external format including changemask information or we
	* get the raw quantized state buffer along with the changemask information for any received states.
	*/
	virtual void ApplyReplicatedState(FReplicationStateApplyContext& Context) const = 0;

	/**
	* Optional method required for backwards compatibility mode which is used to propagate required calls to Pre/PostNetReceive/PostRepNotifies.
	*/
	virtual void CollectOwner(FReplicationStateOwnerCollector* Owners) const {};

	/**
	* Optional method required for backwards compatibility mode which will be invoked for all Fragment with the EReplicationFragmentTraits::HasRepNotifies trait set.
	*/
	virtual void CallRepNotifies(FReplicationStateApplyContext& Context) {};

	/**
	 * Optional Poll method required for backwards compatibility mode which will be invoked for all Fragment with the EReplicationFragmentTraits::NeedsPoll trait set.
	 * @return True if the state is dirty, false if not.
	 */
	virtual bool PollReplicatedState(EReplicationFragmentPollFlags PollOption = EReplicationFragmentPollFlags::PollAllState) { return false; }

	/**
	* Optional method to output state data to StringBuilder.
	*/
	virtual void ReplicatedStateToString(FStringBuilderBase& StringBuilder, FReplicationStateApplyContext& Context, EReplicationStateToStringFlags Flags = EReplicationStateToStringFlags::None) const {};
	
protected:
	EReplicationFragmentTraits Traits;
};

enum class EFragmentRegistrationFlags : uint32
{
	None = 0U,
	
	// Indicates that we only should register RPC:s
	RegisterRPCsOnly = 1U,
	// Indicates that this objects should use the CDO for class defaults instead of the archetype
	InitializeDefaultStateFromClassDefaults = RegisterRPCsOnly << 1U,
	// Allow building descriptors for FastArrays that contain additional properties, NOTE: This should be avoided as fastarrays normally only should contain a single replicated property.
	AllowFastArraysWithAdditionalProperties = InitializeDefaultStateFromClassDefaults << 1U,
};
ENUM_CLASS_FLAGS(EFragmentRegistrationFlags);

/**
* Used when registering ReplicationFragments
*/
struct FReplicationFragmentInfo
{
	const FReplicationStateDescriptor* Descriptor = nullptr;
	void* SrcReplicationStateBuffer = nullptr;
	FReplicationFragment* Fragment = nullptr;
};
typedef TArray<FReplicationFragmentInfo, TInlineAllocator<32>> FReplicationFragments;

class FFragmentRegistrationContext
{
public:
	explicit FFragmentRegistrationContext(Private::FReplicationStateDescriptorRegistry* InReplicationStateRegistry, UReplicationSystem* InReplicationSystem, const EReplicationFragmentTraits InFragmentTraits, UObject* MainObject)
		: ReplicationStateRegistry(InReplicationStateRegistry)
		, ReplicationSystem(InReplicationSystem)
		, MainObjectInstance(MainObject)
		, FragmentTraits(InFragmentTraits)
	{
	}

	/** Returns the traits */
	const EReplicationFragmentTraits GetFragmentTraits() const { return FragmentTraits; }

	/** Register ReplicationFragment */
	void RegisterReplicationFragment(FReplicationFragment* Fragment, const FReplicationStateDescriptor* Descriptor, void* SrcReplicationStateBuffer)
	{ 
		Fragments.Add({ Descriptor, SrcReplicationStateBuffer, Fragment }); 
	}

	/** Call this when you have a netobject that is replicated but will never contain any RPCs or replicated properties. Prevents the registration code from complaining about potential errors. */
	void SetIsFragmentlessNetObject(bool bIsFragmentless) { bIsAFragmentlessNetObject = bIsFragmentless; }

	/** Returns true when the netobject knows it won't contain any replicated properties or RPCs */
	bool IsFragmentlessNetObject() const { return bIsAFragmentlessNetObject; }

	/** Returns true if the fragments (or the lack of) were registered by the instance */
	bool WasRegistered() const
	{
		return Fragments.Num() > 0 || bIsAFragmentlessNetObject;
	}

	/** Returns the number of fragments registered */
	int32 NumFragments() const { return Fragments.Num(); }

	friend Private::FFragmentRegistrationContextPrivateAccessor;

private:

	FReplicationFragments Fragments;
	Private::FReplicationStateDescriptorRegistry* ReplicationStateRegistry;
	UReplicationSystem* ReplicationSystem;

	/** The main object used to pull most fragments from. Used as the basic for the default state source */
	UObject* MainObjectInstance = nullptr;

	/** The externally provided template used to clone the main object instance from */
	const UObject* Template = nullptr;

	/** The final reference we used to build the default state for the main object. Since Template is optional this lets callers know what was used from a template, archetype or cdo. */
	const UObject* MainObjectDefaultStateSource = nullptr;

	const EReplicationFragmentTraits FragmentTraits;

	bool bIsAFragmentlessNetObject = false;

};

}
