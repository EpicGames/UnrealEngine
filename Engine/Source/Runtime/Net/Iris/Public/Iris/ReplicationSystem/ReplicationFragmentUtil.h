// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"

#include "Iris/ReplicationSystem/ReplicationFragment.h"

class UObject;

namespace UE::Net
{

/** FPropertyReplicationFragment - used to bind PropertyReplicationStates to their owner */
class FReplicationFragmentUtil
{
public:

	/** Simple version of CreateAndRegisterFragmentsForObject */
	IRISCORE_API static uint32 CreateAndRegisterFragmentsForObject(UObject* Object, FFragmentRegistrationContext& Context, EFragmentRegistrationFlags RegistrationFlags, TArray<FReplicationFragment*>* OutCreatedFragments = nullptr);


	struct FCreateFragmentParams
	{
		/** The replicated object to create fragments for */
		UObject* ObjectInstance = nullptr;
		/** Flags to control how the fragments should be created */
		EFragmentRegistrationFlags RegistrationFlags = EFragmentRegistrationFlags::None;
	};

	/**
	* Create and register all property replication Fragments for the provided object, Descriptors will be created based on the Class of the object
	* Lifetime of created fragments will be managed by the ReplicationSystem
	* If the OutCreatedFragments are provided pointers to the created fragments will be added to the provided array
	* Returns the number of created fragments
	*/
	IRISCORE_API static uint32 CreateAndRegisterFragmentsForObject(const FCreateFragmentParams& Params, FFragmentRegistrationContext& Context, TArray<FReplicationFragment*>* OutCreatedFragments = nullptr);
};

}
