// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Net/Core/NetBitArray.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"

class FMemStackBase;
namespace UE::Net
{
	class FNetBitStreamWriter;
	class FNetBitStreamReader;
	class FNetSerializationContext;
	struct FReplicationInstanceProtocol;
	struct FReplicationStateDescriptor;
	class FReplicationStateOwnerCollector;
	struct FReplicationProtocol;
	struct FDequantizeAndApplyParameters;
}

namespace UE::Net::Private
{
	/** Structured representation of a single member within a fragment. */
	struct FDebugCDOMemberState
	{
		FString Name;
		FString SerializerName;
		uint64 Hash = 0;
		TArray<FDebugCDOMemberState> SubMembers;

		friend FArchive& operator<<(FArchive& Ar, FDebugCDOMemberState& S)
		{
			return Ar << S.Name << S.SerializerName << S.Hash << S.SubMembers;
		}
	};

	/** Structured representation of a single replicated function within a fragment. */
	struct FDebugCDOFunctionState
	{
		FString Name;
		uint64 DescriptorId = 0;

		friend FArchive& operator<<(FArchive& Ar, FDebugCDOFunctionState& S)
		{
			return Ar << S.Name << S.DescriptorId;
		}
	};

	/** Structured representation of a single fragment from the CDO state. */
	struct FDebugCDOFragmentState
	{
		FString Name;
		uint64 DescriptorId = 0;
		uint64 DefaultStateHash = 0;
		TArray<FDebugCDOMemberState> Members;
		TArray<FDebugCDOFunctionState> Functions;

		friend FArchive& operator<<(FArchive& Ar, FDebugCDOFragmentState& S)
		{
			return Ar << S.Name << S.DescriptorId << S.DefaultStateHash << S.Members << S.Functions;
		}
	};

	/** Structured representation of a full CDO state for protocol mismatch diagnostics. */
	struct FDebugCDOState
	{
		TArray<FDebugCDOFragmentState> Fragments;

		friend FArchive& operator<<(FArchive& Ar, FDebugCDOState& S)
		{
			return Ar << S.Fragments;
		}
	};
}

namespace UE::Net
{

//$IRIS TODO: Consider what methods we should expose here, currently they are all public!

struct FReplicationStateOperations
{
	/** Quantize a Replication state from ExternalBuffer to internal buffer, DstInternalBuffer does not need to be initialized */
	static IRISCORE_API void Quantize(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT SrcExternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Dequantize a Replication state from internal buffer to already constructed ExternalBuffer */
	static IRISCORE_API void Dequantize(FNetSerializationContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Dequantize a Replication state from internal buffer to already constructed ExternalBuffer */
	static IRISCORE_API void DequantizeWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Quantize a Replication state from ExternalBuffer to internal buffer, DstInternalBuffer does not need to be initialized */
	static IRISCORE_API void QuantizeWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT SrcExternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Validate a ReplicationState in external format */
	static IRISCORE_API bool Validate(FNetSerializationContext& Context, const uint8* RESTRICT SrcExternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** FreeDynamicState free all dynamic memory allocated for quantized state data */
	static IRISCORE_API void FreeDynamicState(FNetSerializationContext& Context, uint8* RESTRICT StateInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Compare two quantized states return false if they are different */
	static IRISCORE_API bool IsEqualQuantizedState(FNetSerializationContext& Context, const uint8* RESTRICT Source0, const uint8* RESTRICT Source1, const FReplicationStateDescriptor* Descriptor);

	/** Debug method to output the per member defaultstate hash */
	static IRISCORE_API void OutputDefaultStateMembersHashToString(UReplicationSystem* ReplicationSystem, FStringBuilderBase& StringBuilder, const FReplicationStateDescriptor* Descriptor);

	/** Serialize a Replication state from internal buffer to BitStream */
	static IRISCORE_API void Serialize(FNetSerializationContext& Context, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Deserialize Replication state from BitStream to internal buffer */
	static IRISCORE_API void Deserialize(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Serialize a Replication state from internal buffer to BitStream */
	static IRISCORE_API void SerializeDelta(FNetSerializationContext& Context, const uint8* RESTRICT SrcInternalBuffer, const uint8* RESTRICT PrevInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Deserialize Replication state from BitStream to internal buffer */
	static IRISCORE_API void DeserializeDelta(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT PrevInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Serialize a Replication state from internal buffer to BitStream */
	static IRISCORE_API void SerializeWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Deserialize Replication state from BitStream to internal buffer */
	static IRISCORE_API void DeserializeWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, uint8* RESTRICT DstInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Serialize a Replication state from internal buffer to BitStream */
	static IRISCORE_API void SerializeDeltaWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, const uint8* RESTRICT SrcInternalBuffer, const uint8* RESTRICT PrevInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Deserialize Replication state from BitStream to internal buffer */
	static IRISCORE_API void DeserializeDeltaWithMask(FNetSerializationContext& Context, const FNetBitArrayView& ChangeMask, const uint32 ChangeMaskOffset, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT PrevInternalBuffer, const FReplicationStateDescriptor* Descriptor);

	/** Apply/copy state for a descriptor created for a struct. */
	static IRISCORE_API void ApplyStruct(FNetSerializationContext& Context, uint8* RESTRICT DstExternalBuffer, const uint8* RESTRICT SrcExternalBuffer, const FReplicationStateDescriptor* Descriptor);
};

struct FReplicationInstanceOperations
{
	/** Update all registered Fragments that updates dirtiness by polling, except for those with any of the ExcludeTraits. Returns true if a polled state is dirty. */
	static IRISCORE_API bool PollAndCopyPropertyData(const FReplicationInstanceProtocol* InstanceProtocol, EReplicationFragmentTraits ExcludeTraits, EReplicationFragmentPollFlags PollOptions = EReplicationFragmentPollFlags::PollAllState);

	/** Update all registered Fragments that updates dirtiness by polling.  Returns true if a polled state is dirty. */
	static IRISCORE_API bool PollAndCopyPropertyData(const FReplicationInstanceProtocol* InstanceProtocol, EReplicationFragmentPollFlags PollOptions = EReplicationFragmentPollFlags::PollAllState);

	/** Update object references in fragments that has object references and additional required traits. Returns true if a polled state is dirty. */
	static IRISCORE_API bool PollAndCopyObjectReferences(const FReplicationInstanceProtocol* InstanceProtocol, EReplicationFragmentTraits RequiredTraits, EReplicationFragmentPollFlags PollOptions = EReplicationFragmentPollFlags::None);

	/**
	 * Quantize the state for a replicated object with a given InstanceProtocol using the ReplicationProtocol. 
	 * DstObjectStateBuffer needs to be in a valid state before calling this function.
	 * Changemasks will be written to the ChangeMaskWriter. Dirtiness will not be reset.
	 * @see ResetDirtiness
	 */
	static IRISCORE_API void Quantize(FNetSerializationContext& Context, uint8* DstObjectStateBuffer, FNetBitStreamWriter* ChangeMaskWriter, const FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol);

	/**
	 * Quantize the state for a replicated object with a given InstanceProtocol using the ReplicationProtocol.
	 * DstObjectStateBuffer needs to be in a valid state before calling this function.
	 * Changemasks will be written to the ChangeMaskWriter. Dirtiness will not be reset.
	 * This variant will only Quantize States marked as dirty
	 * @see ResetDirtiness
	 */
	static IRISCORE_API void QuantizeIfDirty(FNetSerializationContext& Context, uint8* DstObjectStateBuffer, FNetBitStreamWriter* ChangeMaskWriter, const FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol);

	/** Resets dirty tracking stored with the protocol, such as changemasks and init state dirtiness. */
	static IRISCORE_API void ResetDirtiness(const FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol);

	/** Dequantize the state for a replicated object with a given protocol. Data will be pushed out by using the ReplicationFragments. */
	static IRISCORE_API void DequantizeAndApply(FNetSerializationContext& Context, const FDequantizeAndApplyParameters& Parameters);

	/** Dequantize the state for a replicated object with a given protocol. Data will be pushed out by using the ReplicationFragments. */
	static IRISCORE_API void DequantizeAndApply(FNetSerializationContext& Context, FMemStackBase& InAllocator, const uint32* ChangeMaskData, const FReplicationInstanceProtocol* InstanceProtocol, const uint8* SrcObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Dequantize the state for a replicated object with a given protocol and output the state to string. */
	static IRISCORE_API void OutputInternalStateToString(FNetSerializationContext& Context, FStringBuilderBase& StringBuilder, const uint32* ChangeMaskData, const uint8* SrcInternalObjectStateBuffer, const FReplicationInstanceProtocol* InstanceProtocol, const FReplicationProtocol* Protocol);

	/** Dequantize the default state for a replicated object with a given protocol and output the state to string. */
	static IRISCORE_API void OutputInternalDefaultStateToString(FNetSerializationContext& NetSerializationContext, FStringBuilderBase& StringBuilder, const FReplicationFragments& Fragments);

	/** Serialize and output per member default state hashes to string. */
	static IRISCORE_API void OutputInternalDefaultStateMemberHashesToString(UReplicationSystem* ReplicationSystem, FStringBuilderBase& StringBuilder, const FReplicationFragments& Fragments);

	/** Build structured per-member default state hashes for all fragments. */
	static IRISCORE_API void BuildInternalDefaultState(UReplicationSystem* ReplicationSystem, const FReplicationFragments& Fragments, Private::FDebugCDOState& OutState);

	/** Serialize a structured CDO state to a byte array. */
	static IRISCORE_API TArray<uint8> SerializeInternalDefaultState(Private::FDebugCDOState& State);

	/** Deserialize a structured CDO state from a byte array. Returns false on error. */
	static IRISCORE_API bool DeserializeInternalDefaultState(const TArray<uint8>& Bytes, Private::FDebugCDOState& OutState);

	/** Diff two structured CDO states and log differences. */
	static IRISCORE_API void DiffAndLogInternalDefaultState(const Private::FDebugCDOState& ClientState, const Private::FDebugCDOState& ServerState);
};

struct FReplicationProtocolOperations
{
	/** Serialize the state for a full NetObject to BitStream */
	static IRISCORE_API void Serialize(FNetSerializationContext& Context, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Deserialize the state of a NetObject from a BitStream to a ObjectStateBuffer large enough to fit all data */
	static IRISCORE_API void Deserialize(FNetSerializationContext& Context, uint8* RESTRICT DstObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Serialize the state of all dirty members a NetObject and changemask to Bitstream */
	static IRISCORE_API void SerializeWithMask(FNetSerializationContext& Context, const uint32* ChangeMaskData, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Deserialize changemask and the state of an object from a BitStream to a ObjectStateBuffer large enough to fit all data */
	static IRISCORE_API void DeserializeWithMask(FNetSerializationContext& Context, uint32* DstChangeMaskData, uint8* RESTRICT DstObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Compare two quantized states and return whether they're equal or not. */
	static IRISCORE_API bool IsEqualQuantizedState(FNetSerializationContext& Context, const uint8* RESTRICT Source0, const uint8* RESTRICT Source1, const FReplicationProtocol* Protocol);

	/** Free dynamic state for the entire protocol. */
	static IRISCORE_API void FreeDynamicState(FNetSerializationContext& Context, uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Initialize from default state */
	static IRISCORE_API void InitializeFromDefaultState(FNetSerializationContext& Context, uint8* RESTRICT StateBuffer, const FReplicationProtocol* Protocol);

	/** Serialize the initial state of all dirty members a NetObject and changemask to a Bitstream, delta compressed against the default state. */
	static IRISCORE_API void SerializeInitialStateWithMask(FNetSerializationContext& Context, const uint32* ChangeMaskData, const uint8* RESTRICT SrcObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Deserialize changemask and the state of an object from a BitStream to an ObjectStateBuffer large enough to fit all data delta compressed against default state. */
	static IRISCORE_API void DeserializeInitialStateWithMask(FNetSerializationContext& Context, uint32* DstChangeMaskData, uint8* RESTRICT DstObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Serialize the state of all dirty members of an object and changemask to a Bitstream, delta compressed against the PrevObjectStateBuffer. */
	static IRISCORE_API void SerializeWithMaskDelta(FNetSerializationContext& Context, const uint32* ChangeMaskData, const uint8* RESTRICT SrcObjectStateBuffer, const uint8* RESTRICT PrevObjectStateBuffer, const FReplicationProtocol* Protocol);

	/** Deserialize changemask and the state of an object from a BitStream to an ObjectStateBuffer large enough to fit all data, delta compressed against the PrevObjectStateBuffer. */
	static IRISCORE_API void DeserializeWithMaskDelta(FNetSerializationContext& Context, uint32* DstChangeMaskData, uint8* RESTRICT DstObjectStateBuffer, const uint8* RESTRICT PrevObjectStateBuffer, const FReplicationProtocol* Protocol);
};

}
