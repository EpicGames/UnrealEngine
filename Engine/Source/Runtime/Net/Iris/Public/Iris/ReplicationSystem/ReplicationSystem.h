// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Templates/PimplPtr.h"
#include "Templates/RefCounting.h"
#include "Iris/IrisConfig.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Iris/ReplicationSystem/NetObjectGroupHandle.h"
#include "Iris/ReplicationSystem/ReplicationSystemTypes.h"
#include "Iris/ReplicationSystem/NetRefHandleGenerator.h"
#include "Net/Core/Connection/NetEnums.h"
#include "Net/Core/Connection/ConnectionHandle.h"
#include "Net/Core/NetHandle/NetHandle.h"

#include "ReplicationSystem.generated.h"

// The number of inline elements the TInlineAllocator for ReplicationSystems can store.
// Projects may define this to a value best suited for their needs.
#ifndef UE_NUM_INLINE_REPLICATIONSYSTEMS
	#define UE_NUM_INLINE_REPLICATIONSYSTEMS 8
#endif

class FRPCDoSDetection;
class UDataStreamManager;
class UDataStream;
class UFunction;
class UNetBlobHandler;
class UNetDriver;
class UNetObjectFilter;
class UNetObjectPrioritizer;
class UObjectReplicationBridge;
class UReplicationSystem;
class UNetRefHandleGenerator;

struct FNetObjectFilterDefinition;

namespace UE::Net
{
	typedef uint32 FNetObjectFilterHandle;
	typedef uint32 FNetObjectPrioritizerHandle;

	class FNetBitArray;
	class FNetCullDistanceOverrides;
	class FNetObjectAttachment;	
	class FNetObjectReference;
	class FReplicationSystemDelegates;
	class FReplicationSystemFactory;
	class FNetTokenStore;
	class FStringTokenStore;
	class FNameTokenStore;
	class FWorldLocations;
	class FNetTokenResolveContext;

	struct FReplicationSystemUtil;
	struct FReplicationView;
	struct FNetDebugName;
	struct FNetMetrics;
	struct FReplicationProtocol;

	enum class ENetConnectionTraits : uint32;
	enum class ENetFilterStatus : uint32;
	enum class ENetObjectDeltaCompressionStatus : unsigned;
	enum class ENetRefHandleError : uint32;
	enum class EReplicationCondition : uint32;

	namespace Private
	{
		class FReplicationSystemImpl;
		class FReplicationSystemInternal;
		class FObjectReferenceCache;
		class FNetStatsContext;
	}
}

UCLASS(transient)
class UReplicationSystem : public UObject
{
	GENERATED_BODY()

	using FNetRefHandle = UE::Net::FNetRefHandle;
	using FNetObjectGroupHandle = UE::Net::FNetObjectGroupHandle;
	using FNetHandle = UE::Net::FNetHandle;

public:

	struct FReplicationSystemParams
	{
		/** The replication bridge that allows communication between the replication system and the game engine  */
		UObjectReplicationBridge* ReplicationBridge = nullptr;

		/** 
		 * The maximum amount of netobjects that can be registered to the replication system 
		 * Note that this variable is automatically rounded up to a multiple of 32 so that all available bits in the NetBitArray storage type are used.
		 */
		uint32 MaxReplicatedObjectCount = 65536U;

		/**
		 * The default allocated size for lists referencing NetObjects by their internal index (NetBitArray or TArray).
		 * Use 0 to preallocate for all possible replicated objects and never reallocate the lists.
		 * Setting a value smaller than Max minimizes the memory footprint of the replication system when few replicated objects are registered.
		 * The downside is you have to pay a CPU hit when the initial list size is met.
		 */
		uint32 InitialNetObjectListCount = 65536U;

		/**
		 * The amount by which we increase the size of every NetObjectList (NetBitArray and TArray) when we hit the initial amount.
		 * Use a small value if you want to keep the memory footprint of the system to a minimum.
		 * But be aware that increasing the NetObjectList's is costly and may increase memory fragmentation so you'll want to do pay the reallocation cost as little as possible.
		 * Note that this variable is automatically rounded up to a multiple of 32 so that all available bits in the NetBitArray storage type are used.
		 */
		uint32 NetObjectListGrowCount = 16384U;

		/**
		 * The amount of netobjects to preallocate internal memory buffers for (NetChunkedArray types).
		 * These arrays hold the biggest memory blocks in the replication system and can grow independently of the NetObjectLists.
		 * Using a large amount of preallocated memory provides faster cache-friendly CPU operations but has the downside of holding into much more memory than might actually be needed. 
		 * Reduce this value if you are operating on a memory constrained platform.
		 */
		uint32 PreAllocatedMemoryBuffersObjectCount = 65536U;

		/**
		 * The maximum amount of netobjects that can replicate properties to remote connection. 
		 * Can be much lower on clients where very few netobjects have authority and support property replication (often just 1 player controller)
		 * When set to 0 it will follow the MaxReplicatedObjectCount and InitialNetObjectListCount limits
		 */
		uint32 MaxReplicationWriterObjectCount = 0;

		/** The maximum amount of netobjects that can be added to the delta compression manager */
		uint32 MaxDeltaCompressedObjectCount = 2048U;

		/** The maximum amount of filter groups that can be created. @see UReplicationSystem::CreateGroup */
		uint32 MaxNetObjectGroupCount = 2048U;

		/** Is this replication system owned by a server or a client. */
		bool bIsServer = false;

		/**
		 * When true enable netobject subsystems like: property replication, filtering, prioritization, deltacompression, dirtytracking, etc.
		 * Generally false on clients or on lightweight RPC-only systems.
		 */
		bool bAllowObjectReplication = false;

		/** When using remote objects with UE_WITH_REMOTE_OBJECT_HANDLE, determines whether or not TObjectPtr or TWeakObjectPtr properties are serialized as FRemoteObjectReferences */
		bool bUseRemoteObjectReferences = false;

		/**
		 * If true, when using remote objects with UE_WITH_REMOTE_OBJECT_HANDLE, runs code that may dereference remote UObjects inside of a transaction.
		 * If the replication system is being used by a NetDriver, the NetDriver should set this flag if the NetDriver is also replicating transactionally.
		 */
		bool bReplicateTransactionally = false;

		/** 
		*   When true, allow subsystems to run parallel workloads, such as the PollAndCopy step running several asynchronous tasks to speed up game thread execution time.
		*   Only supported when bIsServer = true and bAllowObjectReplication = true
		*/
		bool bAllowParallelTasks = false;

		/** If true we will only do a minimal update if there are no valid connections */
		bool bAllowMinimalUpdateIfNoConnections = true;
		
		/** If true, call RepNotifies functions associated with replicated properties. */
		bool bAllowExecuteRepNotifies = true;

		/** If true, call RPC functions associated with replicated objects. */
		bool bAllowExecuteRPC = true;

		/** If true, swap local and remote roles before replicating ENetRole properties to clients. */
		bool bAlwaysSwapRolesOnReplication = false;

		/** Delegate that receives every RPC executed locally. */
		UE::Net::FForwardNetRPCCallDelegate ForwardNetRPCCallDelegate;

		/** NetTokenStore */
		UE::Net::FNetTokenStore* NetTokenStore = nullptr;
		
		/** 
		 * The initial index to use when assigning new NetRefHandle.
		 * Must be greater than 0, because 0 is reserved for an invalid handle.
		 */
		uint32 InitialNetRefHandleIndex = 1;

		/** Configure the replication system as a proxy. */
		UE::Net::EProxyType ProxyType = UE::Net::EProxyType::None;

		/** 
		 * Map a UObject pointer to the unique serial field in a NetRefHandle. 
		 *
		 * This delegate is called when assigning a NetRefHandle to an object being
		 * registered for replication.
		 *
		 * Assumes there's a one-to-one mapping between UObjects registered
		 * for replication and the serial field in NetRefHandle.
		 * 
		 * If left unset, Iris uses an internal incrementing counter to assign
		 * a value to the NetRefHandle serial field.
		 */
		UE::Net::FObjectToNetRefHandleSerial ObjectToNetRefHandleSerial;

		/** 
		 * Map a unique serial field from a NetRefHandle serial to a UObject pointer.
		 *
		 * Will always be passed a serial field that is greater than 0. 
		 *
		 * Assumes there's a one-to-one mapping between UObjects registered
		 * for replication and the serial field in NetRefHandle.
		 */
		UE::Net::FNetRefHandleSerialToObject NetRefHandleSerialToObject;
	};

	/** @return The unique ID of the ReplicationSystem. */
	uint32 GetId() const { return Id; }
	
	/** @return The Max number of connections that is supported. */
	IRISCORE_API uint32 GetMaxConnectionCount() const;

	/** @return Whether the system is run on a server. */
	bool IsServer() const { return bIsServer; }

	bool AllowExecuteRPC() const { return bAllowExecuteRPC; }
	bool AllowExecuteRepNotify() const { return bAllowExecuteRepNotify; };
	bool AlwaysSwapRolesOnReplication() const { return bAlwaysSwapRolesOnReplication; };

	UE::Net::EProxyType GetProxyType() const { return ProxyType; };

	/** @return Is this system configured to replicate object properties. */
	bool AllowObjectReplication() { return bAllowObjectReplication; }

	/** When using remote objects with UE_WITH_REMOTE_OBJECT_HANDLE, whether or not TObjectPtr or TWeakObjectPtr properties should be serialized as FRemoteObjectReferences */
	IRISCORE_API bool IsUsingRemoteObjectReferences() const;

	/**
	 * Update all internal systems, such as filtering, dirty tracking, prioritization, etc.
	 * Also copies all replicated data into the internal Iris protocols so they are ready to be sent to clients
	 */
	IRISCORE_API void NetUpdate(float DeltaSeconds);

	/**
	 * Tick replication system after parsing all received data.
	 * Used to check if immediate RPCs need to be sent before the rest of the engine is ticked
	 */
	IRISCORE_API void TickPostReceive();

	/** Callback triggered before connections start processing their received data */
	IRISCORE_API void PreReceiveUpdate();

	/** Callback triggered after the connections processed all received data */
	IRISCORE_API void PostReceiveUpdate();

	/**
	 * SendUpdate is currently more of a placeholder for a future where the ReplicationSystem itself is responsible for
	 * the low level protocol and sending, rather than having the DataStreamChannel write data when ticked
	 * @see UDataStreamChannel.
	 * @param SendFunction, Function taking an array of ConnectionId`s that has data to send
	 */
	IRISCORE_API void SendUpdate(TFunctionRef<void(TArrayView<uint32>)> SendFunction);

	/**
	 * Cleanup temporaries and prepare for the next send update.
	 */
	IRISCORE_API void PostSendUpdate();

	/**
	 * Notify that a connection was added.
	 * @param ConnectionId The ID of the added connection. Must not collide with an existing ID. Must be <= the maximum number of connections.
	 * @param NetConnectionTraits Traits of the connection being added.
	 */
	IRISCORE_API void AddConnection(uint32 ConnectionId, UE::Net::ENetConnectionTraits NetConnectionTraits = {});

	/**
	 * Notify that a connection was removed.
	 * @param ConnectionId The ID of the removed connection. Must have previously been added.
	 */
	IRISCORE_API void RemoveConnection(uint32 ConnectionId);

	/**
	 * Query whether there's a connection with any of the supplied traits. 
	 * @param NetConnectionTraits The sought after traits of a connection. If UE::Net::ENetConnectionTraits::None is specified it will match all connections.
	 * @return true if a connection exists with the sought after traits.
	 */
	IRISCORE_API bool HasConnectionWithAnyTraits(UE::Net::ENetConnectionTraits NetConnectionTraits) const;

	/**
	 * Verify if a connection is valid, that is has been added to the system.
	 * @param ConnectionId The ID of the connection to check for validness.
	 * @return Whether the connection is valid.
	 */
	IRISCORE_API bool IsValidConnection(uint32 ConnectionId) const;

	/**
	 * Sets a connection as gracefully closing, where it will flush all pending reliable data
	 * before completely shutting down.
	 * @param ConnectionId The ID of the connection that's closing.
	 */
	IRISCORE_API void SetConnectionGracefullyClosing(uint32 ConnectionId) const;

	/**
	 * Enable or disable the ReplicationDataStream to transmit data for a particular connection.
	 * @param ConnectionId The ID of the connection to enable or disable object replication for.
	 * @param bReplicationEnabled Whether to enable or disable object replication.
	 */
	IRISCORE_API void SetReplicationEnabledForConnection(uint32 ConnectionId, bool bReplicationEnabled);

	/**
	 * Check whether object replication is enabled for a particular connection.
	 * @param ConnectionId The ID of the connection to check if object replication is enabled for.
	 * @return Whether object replication is enabled for the connection.
	 */
	IRISCORE_API bool IsReplicationEnabledForConnection(uint32 ConnectionId) const;

	// Prioritization

	/**
	 * Set view information for a connection. The views are used by some prioritizers, typically to make
	 * objects closer to any of the view positions higher priority and thus more likely to replicate
	 * on a given frame. The information will persist until the next call. This is normally handled by
	 * a NetDriver.
	 * @param ConnectionId The ID of the connection to set the view information for.
	 * @param View The view information for the connection and its subconnections.
	 * @see UNetDriver::UpdateReplicationViews
	 */
	IRISCORE_API void SetReplicationView(uint32 ConnectionId, const UE::Net::FReplicationView& View);

	/**
	 * Sets a fixed priority for a replicated object which will be used until the next call to SetStaticPriority or SetPrioritizer.
	 * An object which never gets a call to either SetPrioritizer or SetStaticPriority will have a priority of 1.0.
	 * @param Handle Which object to set the priority for.
	 * @param Priority A value >= 0. 1.0 means the object will be considered for replication every frame, if it has updated replicated properties.
	 * @see SetPrioritizer
	 */
	IRISCORE_API void SetStaticPriority(FNetRefHandle Handle, float Priority);

	/**
	 * Sets a prioritizer for a replicated object which will be used until the next call to SetPrioritizer or SetStaticPriority.
	 * An object which never gets a call to either SetPrioritizer or SetStaticPriority will have a priority of 1.0.
	 * @param Handle A valid handle to an object.
	 * @param PrioritizerHandle A valid handle to a prioritizer, retrieved via a call to GetPrioritizer, or DefaultSpatialNetObjectPrioritizerHandle.
	 * @return true if the prioritizer was successfully set and false if it was not. It can fail for various reasons,
	 * such as the prioritizer not supporting the object in question for implementation defined reasons. If the function fails 
	 * the prioritization of the object is unspecified, it could be using a previous prioritizer or get a default static priority.
	 * @see GetPrioritizerHandle
	 * @see SetStaticPriority
	 */
	IRISCORE_API bool SetPrioritizer(FNetRefHandle Handle, UE::Net::FNetObjectPrioritizerHandle PrioritizerHandle);


	/**
	 * Gets the handle for a prioritizer with a given name. The handle can be used in subsequent calls to SetPrioritizer.
	 * @param PrioritizerName The name of the prioritizer. Names of valid prioritizers are configured in UNetObjectPrioritizerDefinitions.
	 * @return A valid handle if a prioritizer with the given name has been successfully created in this system or InvalidNetObjectPrioritizerHandle
	 * if not.
	 * @see SetPrioritizer
	 * @see UNetObjectPrioritizerDefinitions
	 */
	IRISCORE_API UE::Net::FNetObjectPrioritizerHandle GetPrioritizerHandle(const FName PrioritizerName) const;


	/**
	 * Gets the prioritizer with a given name. Can be useful for special initialization for a custom prioritizer.
	 * @param PrioritizerName The name of the prioritizer. Names of valid filters are configured in UNetObjectPrioritizerDefinitions.
	 * @return A pointer to the filter if it exists, nullptr if not.
	 * @see UNetObjectPrioritizerDefinitions
	 */
	IRISCORE_API UNetObjectPrioritizer* GetPrioritizer(const FName PrioritizerName) const;

	/**
	 * Get DataStream with the provided Name
	 * @param Name The name of the DataStream. Names of valid DataStream are configured in UDataStreamDefinitions.
	 * @return A pointer to the DataStream if it exists
	 * @see UDataStreamDefinitions
	 */
	IRISCORE_API UDataStream* GetDataStream(uint32 ConnectionId, FName Name);

	/** Const version of GetDataStream */
	IRISCORE_API const UDataStream* GetDataStream(uint32 ConnectionId, FName Name) const;

	/** Returns true if there exists a DataStreamDefinition for the provided Name */
	IRISCORE_API bool IsKnownDataStreamDefinition(FName Name) const;

	/**
	 * Request DataStream to be created/opened for the given ConnectionId. NOTE: This only applies to DataStreams that are flagged with bDynamicCreate in the DataStreamDefinition
	 * @param Name The name of the DataStream. Names of valid DataStream are configured in UDataStreamDefinitions.
	 * @return A pointer to the DataStream if successfully opened or already existed.
	 * @see UDataStreamDefinitions
	 */
	IRISCORE_API UDataStream* OpenDataStream(uint32 ConnectionId, FName Name);

	/**
	 * Initiate close of DataStream for the given ConnectionId. NOTE: This only applies to DataStreams that are flagged with bDynamicCreate in the DataStreamDefinition
	 * @param Name The name of the DataStream. Names of valid DataStream are configured in UDataStreamDefinitions.
	 * @see UDataStreamDefinitions
	 */
	IRISCORE_API void CloseDataStream(uint32 ConnectionId, FName Name);
	
	// NetBlob 

	/**
	 * Registers a NetBlobHandler so that its NetBlob type can be sent and received. The user must be sure to keep the handler valid
	 * for the lifetime of the ReplicationSystem. The handler class must have been configured in UNetBlobHandlerDefinitions in order
	 * for the call to succeed. Only one instance per class is allowed to be registered.
	 * @param Handler The UNetBlobHandler to register.
	 * @return Whether the handler was successfully registered or not.
	 */
	IRISCORE_API bool RegisterNetBlobHandler(UNetBlobHandler* Handler);

	/**
	 * Queue an attachment for replication with an object. The attachment will be sent when the target is scheduled for replication.
	 * @param ConnectionId A valid connection ID. Only this connection will receive the attachment.
	 * @param TargetRef A valid FNetObjectReference representing object to replicate the attachment with.
	 * @param Attachment The attachment to replicate. Note that the attachment's NetObjectReference will be modified based on TargetRef.
	 * @return Whether the attachment was properly queued or not.
	 */
	IRISCORE_API bool QueueNetObjectAttachment(uint32 ConnectionId, const UE::Net::FNetObjectReference& TargetRef, const TRefCountPtr<UE::Net::FNetObjectAttachment>& Attachment);

	/**
	 * Multicast an RPC targeting a object/subobject. 
	 * @param RootObject A valid Owner/Actor. If no subobject is specified the function is called on the root object on the remote side.
	 * @param SubObject Optional subobject on whom the function is called on the remote side.
	 * @param Function The function to call.
	 * @param Parameters The function parameters.
	 * 
	 * @return Whether the RPC was successfully queued for replication or not.
	 */
	IRISCORE_API bool SendRPC(const UObject* RootObject, const UObject* SubObject, const UFunction* Function, const void* Parameters);

	/**
	 * Unicast an RPC targeting a object/subobject.
	 * @param ConnectionId A valid connection ID. Only this connection will receive the RPC.
	 * @param RootObject A valid Owner/Actor. If no subobject is specified the function is called on the root object on the remote side.
	 * @param SubObject Optional subobject on whom the function is called on the remote side.
	 * @param Function The function to call.
	 * @param Parameters The function parameters.
	 * 
	 * @return Whether the RPC was successfully queued for replication or not.
	 */
	IRISCORE_API bool SendRPC(uint32 ConnectionId, const UObject* RootObject, const UObject* SubObject, const UFunction* Function, const void* Parameters);

	/**
	 * Set the policy flags for an RPC identified by its function
	 * @param Function a pointer to a valid function identifying the RPC
	 * @param SendFlags the ENetObjectAttachmentSendPolicyFlags to set for the RPC
	 * @return Whether the specified SendFlags is valid for the specific RPC
	 */
	IRISCORE_API bool SetRPCSendPolicyFlags(const UFunction* Function, UE::Net::ENetObjectAttachmentSendPolicyFlags SendFlags);

	/** Resets all set RPCSendPolicy flags */
	IRISCORE_API void ResetRPCSendPolicyFlags();

	/** @return The UObjectReplicationBridge that was passed with the system creation parameters. */
	IRISCORE_API UObjectReplicationBridge* GetReplicationBridge() const;

	/** @return The UReplicationBridge that was passed with the system creation parameters. Will return nullptr if it cannot be cast to the desired type. */
	template<typename T>
	T* GetReplicationBridgeAs() const { return Cast<T>(GetReplicationBridge()); }

	/**
	 * @return The Net token store.
	 * @see UE::Net::FNetTokenStore
	 */
	IRISCORE_API UE::Net::FNetTokenStore* GetNetTokenStore();

	/**
	 * @return A const version of the NetTokenStore.
	 * @see UE::Net::FNetTokenStore
	 */
	IRISCORE_API const UE::Net::FNetTokenStore* GetNetTokenStore() const;

	/**
	 * Get NetTokenResolveContext resolve NetTokens
	 * @see UE::Net::FNetTokenStore
	 */
	IRISCORE_API UE::Net::FNetTokenResolveContext GetNetTokenResolveContext(uint32 ConnectionId) const;

	/**
	 * Check whether a FNetRefHandle is still associated with a replicated object.
	 * An handle can be valid but reference an object that doesn't exist locally.
	 * @param Handle The handle check.
	 * @return true if the handle is assigned, false if not. 
	 */
	IRISCORE_API bool IsNetRefHandleAssigned(FNetRefHandle Handle) const;

	/**
	 * Get the ReplicationProtocol for a handle.
	 * @param Handle The handle to retrieve the protocol for.
	 * @return A valid pointer to the protocol if the handle is valid, nullptr if not.
	 */
	IRISCORE_API const UE::Net::FReplicationProtocol* GetReplicationProtocol(FNetRefHandle Handle) const;

	/**
	 * Get the DebugName associated with a handle.
	 * @param Handle The handle to retrieve the DebugName for
	 * @return DebugName if the handle is valid, otherwise nullptr.
	 */
	IRISCORE_API const UE::Net::FNetDebugName* GetDebugName(FNetRefHandle Handle) const;

	// Groups

	/**
	 * Create a group which can be used to logically group objects together. The group must be
	 * destroyed when it's not needed anymore.
	 * Groups can be used to setup filtering rules on it's members.
	 * @param GroupName An unique name to identify the group. Passing NAME_None will assign it an autogenerated name
	 * @return A handle to the group, or InvalidNetObjectGroupHandle if no more groups could be created.
	 * @see DestroyGroup
	 */
	IRISCORE_API FNetObjectGroupHandle CreateGroup(FName GroupName);

	/**
	 * Destroy a group.
	 * @see CreateGroup
	 */
	IRISCORE_API void DestroyGroup(FNetObjectGroupHandle GroupHandle);

	/**
	 * Return the handle to a group identified by the given name
	 * @return A valid handle if the name is used by an existing group or InvalidNetObjectGroupHandle if no groups is found
	 */
	IRISCORE_API FNetObjectGroupHandle FindGroup(FName GroupName) const;

	/**
	 * Add an object to a group.
	 * @param GroupHandle A valid group handle.
	 * @param Handle A valid handle to an object.
	 */
	IRISCORE_API void AddToGroup(FNetObjectGroupHandle GroupHandle, FNetRefHandle Handle);

	/**
	 * Removes an object from a group.
	 * @param GroupHandle A valid group handle.
	 * @param Handle A valid handle to an object.
	 */
	IRISCORE_API void RemoveFromGroup(FNetObjectGroupHandle GroupHandle, FNetRefHandle Handle);

	/**
	 * Removes an object from all groups it's part of.
	 * @param Handle A valid handle to an object.
	 */
	IRISCORE_API void RemoveFromAllGroups(FNetRefHandle Handle);

	/**
	 * Check whether an objects belongs to a particular group or not.
	 * @param GroupHandle A valid group handle.
	 * @param Handle A valid handle to an object.
	 * @return true if both handles are valid and the object belongs to the group, false otherwise.
	 */
	IRISCORE_API bool IsInGroup(FNetObjectGroupHandle GroupHandle, FNetRefHandle Handle) const;

	/**
	 * Check if a group handle is valid.
	 * @param GroupHandle A group handle.
	 * @return true if the group is valid, false if not.
	 */
	IRISCORE_API bool IsValidGroup(FNetObjectGroupHandle GroupHandle) const;

	/** Special group, root objects assigned to this group will be filtered out for all connections */
	IRISCORE_API FNetObjectGroupHandle GetNotReplicatedNetObjectGroup() const;

	/** Special group, used for async stop replication */
	IRISCORE_API FNetObjectGroupHandle GetAsyncStopReplicationNetObjectGroup() const;

	/** Special group, SubObjects assigned to this group will replicate to owner of RootParent */
	IRISCORE_API FNetObjectGroupHandle GetNetGroupOwnerNetObjectGroup() const;

	/** Special group, SubObjects assigned to this group will replicate if replay netconditions is met  */
	IRISCORE_API FNetObjectGroupHandle GetNetGroupReplayNetObjectGroup() const;

	// Filtering

	/**
	 * Create a filter at runtime.
	 * 
	 * Definition.FilterName must be unique.
	 *
	 * Rather than explicitly creating filters at runtime, they can be specified in config files and instantiated automatically by the network
	 * system. See https://dev.epicgames.com/documentation/en-us/unreal-engine/iris-filtering-in-unreal-engine#dynamic-filter-configuration .
	 *
	 * If .ini configuration applies to the filter or its use, call UObjectReplicationBridge::ReloadConfig() after creating the filter to apply that
	 * config.
	 *
	 * The filter is owned by the replication system, but can be accessed via the returned weak pointer.
	 *
	 * The filter can be destroyed later via DestroyFilter().
	 *
	 * @param Definition Filter definition (class, config).
	 *
	 * @return Weak pointer to filter instance, or nullptr.
	 */
	IRISCORE_API TWeakObjectPtr<UNetObjectFilter> CreateFilter(const FNetObjectFilterDefinition& Definition);

	/**
	 * Destroy a filter.
	 *
	 * Should only be used on filters created at runtime by calls to CreateFilter().
	 *
	 * Any objects still assigned to the filter are unassigned.
	 */
	IRISCORE_API void DestroyFilter(const FName Name);

	/**
	 * Sets the owning connection of an object. This can determine which properties are replicated 
	 * to which connections or affect filtering such that the object is only replicated to the owning connection.
	 * @param Handle A valid handle to an object.
	 * @param ConnectionId A valid connection ID to be set as the owner.
	 */
	IRISCORE_API void SetOwningNetConnection(FNetRefHandle Handle, uint32 ConnectionId);

	/** 
	 * Get the owning net connection for an object.
	 * @see SetOwningNetConnection
	 */
	IRISCORE_API uint32 GetOwningNetConnection(FNetRefHandle Handle) const;

	/**
	 * Sets a filter for a replicated object which will be used until the next call to SetFilter.
	 * Filters are used prevent objects from being replicated to certain connections. An object that is filtered
	 * out will cause the object to be destroyed on the remote side.
	 * 
	 * @param Handle A valid handle to an object.
	 * @param FilterHandle A valid handle to a filter, retrieved via a call to GetFilter or one of the two special handles
	 *		  InvalidNetObjectFilterHandle to clear filtering or ToOwnerFilterHandle for owner filtering.
	 * @param FilterConfigProfile Optional name of a specialized profile to use as the object's configuration. When none filters are expected to use default settings.
	 * 
	 * @return true if the filter was successfully set and false if it was not. It can fail for various reasons,
	 * such as the filter not supporting the object in question for implementation defined reasons. If the function fails
	 * the filter of the object is unspecified, it could be using a previous filter or use no filtering.
	 * 
	 * @see GetFilterHandle
	 */
	IRISCORE_API bool SetFilter(FNetRefHandle Handle, UE::Net::FNetObjectFilterHandle FilterHandle, FName FilterConfigProfile=NAME_None);

	/**
	 * Gets the handle for a filter with a given name. The handle can be used in subsequent calls to SetFilter.
	 * @param Handle A valid handle to an object.
	 * @param FilterName The name of the filter. Names of valid filters are configured in UNetObjectFilterDefinitions.
	 * @return A valid handle if a filter with the given name has been successfully created in this system or InvalidNetObjectFilterHandle if not.
	 * @see SetFilter
	 * @see UNetObjectFilterDefinitions
	 */
	IRISCORE_API UE::Net::FNetObjectFilterHandle GetFilterHandle(const FName FilterName) const;

	/**
	 * Gets the filter with a given name.
	 * @param FilterName The name of the filter. Names of valid filters are configured in UNetObjectFilterDefinitions.
	 * @return A pointer to the filter if it exists, nullptr if not.
	 * @see UNetObjectFilterDefinitions
	 */
	IRISCORE_API UNetObjectFilter* GetFilter(const FName FilterName) const;

	/**
	 * Returns the name of the filter associated to this handle.
	 */
	IRISCORE_API FName GetFilterName(UE::Net::FNetObjectFilterHandle Filter) const;

	// Group Filtering

	/**
	 * Add a group to the filtering system. This group is used only for filtering out objects. Exclusion groups are processed before dynamic filters, those implemented by UNetObjectFilter. 
	 * By default an exclusion group disallows replication for all objects in it. Use SetGroupFilterStatus to change the behavior.
	 * @note A group can only be either an exclusion group or an inclusion group, not both at the same time.
	 * @param GroupHandle A valid handle to a group.
	 * @see CreateGroup
	 * @see AddInclusionFilterGroup
	 * @see SetGroupFilterStatus
	 * @return true if the group was successfully added as an exclusion group, false in all other cases such as being an invalid group, reserved group or used as an inclusion filter.
	 */
	IRISCORE_API bool AddExclusionFilterGroup(FNetObjectGroupHandle GroupHandle);

	/**
	 * Add a group to the filtering system. This group is used only for allowing replication of objects. Inclusion groups are processed after dynamic filters, those implemented by UNetObjectFilter. 
	 * Inclusion groups are used to allow overriding the effect of dynamic filtering which can be useful to always allow replication of team specific objects for example.
	 * By default the group will not override the effects of dynamic filtering. Use SetGroupFilterStatus to set which connection the objects should be allowed to replicate to, overriding the dynamic filtering.
	 * @note A group can only be either an exclusion group or an inclusion group, not both at the same time.
	 * @note Subobjects added to inclusion groups will be ignored during processing. A subobject's filter status is determined by the root object. For subobject filtering one can use SubObjectFilters.
	 * @param GroupHandle A valid handle to a group.
	 * @see CreateGroup
	 * @see AddExclusionFilterGroup
	 * @see SetGroupFilterStatus
	 * @return true if the group was successfully added as an inclusion group, false in all other cases such as being an invalid group, reserved group or used as an exclusion filter.
	 */
	IRISCORE_API bool AddInclusionFilterGroup(FNetObjectGroupHandle GroupHandle);

	/** Remove group from filtering system, canceling all effects of the group. */
	IRISCORE_API void RemoveGroupFilter(FNetObjectGroupHandle GroupHandle);

	/** Set status of GroupFilter for specific connection. */
	IRISCORE_API void SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, uint32 ConnectionId, UE::Net::ENetFilterStatus ReplicationStatus);

	/** Set to true if a connection's remote endpoint is a proxy. */
	IRISCORE_API void SetProxyConnection(uint32 ConnectionId, bool bEnabled);

	/**
	 * Set status of GroupFilter for connection marked in the Connections BitArray to the passed ReplicationStatus,
	 * Connections not marked in the BitArray will be set to the opposite status.
	*/
	IRISCORE_API void SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, const UE::Net::FNetBitArray& Connections, UE::Net::ENetFilterStatus ReplicationStatus);

	/** Set status of GroupFilter for all connections. */
	IRISCORE_API void SetGroupFilterStatus(FNetObjectGroupHandle GroupHandle, UE::Net::ENetFilterStatus ReplicationStatus);


	// SubObject filtering
	/**
	 * Add a group to the filtering system. By default the filter disallows replication for all objects in the group.
	 * @param GroupHandle A valid handle to a group.
	 * @see CreateGroup
	 */
	IRISCORE_API FNetObjectGroupHandle GetOrCreateSubObjectFilter(FName GroupName);

	/** Returns the FNetObjectGroupHandle used by a named SubObjectFilter */
	IRISCORE_API FNetObjectGroupHandle GetSubObjectFilterGroupHandle(FName GroupName) const;

	/** Set status of GroupFilter for specific connection. */
	IRISCORE_API void SetSubObjectFilterStatus(FName GroupName, UE::Net::FConnectionHandle ConnectionHandle, UE::Net::ENetFilterStatus ReplicationStatus);

	/** Remove group from filtering system, will cancel effects of the group. */
	IRISCORE_API void RemoveSubObjectFilter(FName GroupName);

	/**
	 * Enable or disable a replication condition for a single connection and do the inverse for all other connections. This will affect
	 * the replication of properties with conditions that are dependent on this condition.
	 * Calling this function will cancel the effect of previous calls to this function with this condition, i.e. only a single
	 * connection can have RoleAutonomous set.
	 * 
	 * @param Handle A valid handle to a replicated object.
	 * @param Condition The ReplicationCondition to modify. Only EReplicationCondition::RoleAutonomous is supported.
	 * @param ConnectionId The ID of the connection to enable or disable the condition for or 0 to disable the condition.
	 * @param bEnable Whether the the condition should be enabled or disabled for the specified connection.
	 * If ConnectionId is zero this parameter is ignored.
	 * @return true if the condition was successfully set, false if not.
	 * @see SetReplicationCondition
	 */
	IRISCORE_API bool SetReplicationConditionConnectionFilter(FNetRefHandle Handle, UE::Net::EReplicationCondition Condition, uint32 ConnectionId, bool bEnable);

	/**
	 * Enable or disable a replication condition for all connections. This will affect the replication of properties with conditions.
	 *
	 * @param Handle A valid handle to a replicated object.
	 * @param Condition The ReplicationCondition to modify. Only EReplicationCondition::ReplicatePhysics is supported.
	 * @param bEnable Whether the the condition should be enabled or disabled.
	 * @return true if the condition was successfully set, false if not.
	 * @see SetReplicationCondition, SetReplicationConditionConnectionFilter
	 */
	IRISCORE_API bool SetReplicationCondition(FNetRefHandle Handle, UE::Net::EReplicationCondition Condition, bool bEnable);


	/**
	 * Set whether the object allows delta compression when serializing. This does not guarantee that the object
	 * will use delta compression depending on other factors, such as the maximum number of delta compressed objects,
	 * whether the delta compression feature is enabled or not and other reasons.
	 * @param Handle A valid handle to a replicated object.
	 */
	IRISCORE_API void SetDeltaCompressionStatus(FNetRefHandle Handle, UE::Net::ENetObjectDeltaCompressionStatus Status);

	 /**
	  * Mark an object as a net temporary. Such objects only replicate its
	  * initial state and ignores future state changes.
	  */
	IRISCORE_API void SetIsNetTemporary(FNetRefHandle Handle);

	/**
	 * Mark an object to be torn off next update.
	 * @param Handle A valid handle to an object.
	 */
	IRISCORE_API void TearOffNextUpdate(FNetRefHandle Handle);

	/**
	  * Mark an object as a being a debugobject. 
	  * NOTE: Only allowed in non-shipping builds
	  */
	IRISCORE_API void SetIsDebugObject(FNetRefHandle Handle);

	/**
	 * Force the passed object to be considered for replication this frame.
	 * This will also force it's sub objects, root object and any of it's dependents to also be considered for replication.
	 * Normally an object is checked for replication only when it's poll frequency is hit.
	 * @param Handle A valid handle to an object.
	 * @see FObjectReplicationBridgePollConfig
	 */
	IRISCORE_API void ForceNetUpdate(FNetRefHandle Handle);

	/**
	 * Explicitly mark object as having dirty properties.
	 * @param Handle A valid handle to an object.
	 * @see FObjectReplicationBridgePollConfig
	 */
	IRISCORE_API void MarkDirty(FNetRefHandle Handle);

	/**
	 * Retrieve the WorldLocations instance which holds world locations for all objects that support it. 
	 * @return The WorldLocations instance.
	 * @see UE::Net::FWorldLocations
	 */
	IRISCORE_API const UE::Net::FWorldLocations& GetWorldLocations() const;

	/* Init data streams for a connection. For internal use by UDataStreamChannel. */
	IRISCORE_API void InitDataStreamManager(uint32 ConnectionId, UDataStreamManager* DataStreamManager);

	/**
	 * Associate data with a connection. Only a single piece of user data is supported per connection.
	 * The last call determines which data is associated with the connection.
	 * @param ConnectionId A valid connection ID.
	 * @param UserData What data to associate with the connection.
	 * @see GetConnectionUserData
	 */
	IRISCORE_API void SetConnectionUserData(uint32 ConnectionId, UObject* UserData);

	/**
	 * Retrieve the user data associated with a connection.
	 * @param ConnectionId A valid connection ID.
	 * @return The user data associated with the connection.
	 * @see SetConnectionUserData
	 */
	IRISCORE_API UObject* GetConnectionUserData(uint32 ConnectionId) const;

	IRISCORE_API void SetConnectionRPCDoS(uint32 ConnectionId, FRPCDoSDetection* InRPCDoS);

	bool IsReplicatingTransactionally() const { return bReplicateTransactionally; }

	int32 GetPIEInstanceID() const { return PIEInstanceID; }

	/** Set the cull distance for an object temporarily instead of the default cull distance value associated with the object (ex: @see AActor::NetCullDistanceSquared) */
	IRISCORE_API void SetCullDistanceOverride(FNetRefHandle Handle, float Dist);

	/** Clears any previously set cull distance override for an object. This will cause the objects to revert to their internal cull distance property. */
	IRISCORE_API void ClearCullDistanceOverride(FNetRefHandle Handle);

	/** Returns the current cull distance for an object or DefaultValue if it doesn't have a cull distance or is not a replicated object. */
	IRISCORE_API float GetCullDistance(FNetRefHandle Handle, float DefaultValue = -1.0f) const;

	/** Returns elapsed time in seconds since ReplicatonSystem was created */
	double GetElapsedTime() const { return ElapsedTime; }

	/** Called when a connection finds a protocol divergence when instantiating a replicated object. */
	IRISCORE_API void ReportProtocolMismatch(uint64 NetRefHandleId, uint32 ConnectionId, const TArray<uint8>& ClientCDOStateBytes = TArray<uint8>());

	/** Called when a connection reports a critical error with a netrefhandle object */
	IRISCORE_API void ReportErrorWithNetRefHandle(UE::Net::ENetRefHandleError ErrorType, uint64 NetRefHandleId, uint32 ConnectionId, const TArray<uint64>& ExtraNetRefHandles, const FString& DiagnosticMessage = FString());

	/** Collect relevant metrics that could be sent to analytics */
	IRISCORE_API void CollectNetMetrics(UE::Net::FNetMetrics& OutNetMetrics) const;

	/** Reset cached accumulated metrics */
	IRISCORE_API void ResetNetMetrics();

	IRISCORE_API UE::Net::FReplicationSystemDelegates& GetDelegates();

	IRISCORE_API void StartParallelPhase();
	IRISCORE_API void StopParallelPhase();
	IRISCORE_API bool IsInParallelPhase() const;

	// Used to provide a NetStatsContext for the FNetSerializationContext - thread safe
	IRISCORE_API UE::Net::Private::FNetStatsContext* AcquireStatsForSerializationContext();
	// Used to free any aquired NetStatsContext used by this SerializationContext  thread safe
	IRISCORE_API void ReleaseStatsForSerializationContext(UE::Net::Private::FNetStatsContext* NetStatsContext);

	/** Function to query if a particular replicated object is relevant to a connection or not. */
	UE_EXPERIMENTAL(5.7, "IsRelevantForConnection() is experimental and should not be used.")
	IRISCORE_API bool IsRelevantForConnection(FNetRefHandle NetRefHandle, uint32 ConnectionId) const;

public:
	// For internal use and not exported.
	UE::Net::Private::FReplicationSystemInternal* GetReplicationSystemInternal();
	const UE::Net::Private::FReplicationSystemInternal* GetReplicationSystemInternal() const;

private:
	friend UE::Net::FReplicationSystemFactory;

	UReplicationSystem();
	~UReplicationSystem();

	void Init(uint32 InId, const FReplicationSystemParams& Params);
	void Shutdown();

	// UObject interface
	IRISCORE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

private:
	friend UNetDriver;

	IRISCORE_API void NotifyStreamingLevelUnload(const UObject* Level);

	/** Called just prior to garbage collecting/destroying the previous world tied to a NetDriver during seamless travel, but after ResetGameWorldState. */
	IRISCORE_API void PreSeamlessTravelGarbageCollect();

	/** Called just after garbage collecting/destroying the previous world tied to a NetDriver during seamless travel. */
	IRISCORE_API void PostSeamlessTravelGarbageCollect();

	void SetPIEInstanceID(int32 InPIEInstanceID) { PIEInstanceID = InPIEInstanceID; }

	void InitializeNetRefHandleGenerator(UReplicationSystem::FReplicationSystemParams& Params);

private:

	void PostGarbageCollection();
	void CollectGarbage();

	TPimplPtr<UE::Net::Private::FReplicationSystemImpl> Impl;

	FDelegateHandle PostGarbageCollectHandle;

	UPROPERTY(transient)
	TObjectPtr<UObjectReplicationBridge> ReplicationBridge;

	double ElapsedTime = 0;
	uint32 Id;
	int32 PIEInstanceID;
	uint32 bIsServer : 1;
	uint32 bAllowObjectReplication : 1;
	uint32 bAllowExecuteRPC : 1;
	uint32 bAllowExecuteRepNotify : 1;
	uint32 bAlwaysSwapRolesOnReplication : 1;
	uint32 bDoCollectGarbage : 1;
	uint32 bReplicateTransactionally : 1;

	UE::Net::EProxyType ProxyType = UE::Net::EProxyType::None;

	UPROPERTY()
	TObjectPtr<UNetRefHandleGenerator> NetRefHandleGenerator;
};

namespace UE::Net
{

DECLARE_MULTICAST_DELEGATE_OneParam(FReplicationSystemLifeTime, UReplicationSystem*);

using FReplicationSystemCreatedDelegate = FReplicationSystemLifeTime;
using FReplicationSystemDestroyedDelegate = FReplicationSystemLifeTime;

class FReplicationSystemFactory
{
public:
	/**
	 * Creates a new ReplicationSystem.
	 * @param Params The settings for the ReplicationSystem.
	 * @return A pointer to the newly created ReplicationSystem if it was allowed to created, nullptr if not.
	 */
	IRISCORE_API static UReplicationSystem* CreateReplicationSystem(const UReplicationSystem::FReplicationSystemParams& Params);
	/**
	 * Destroys a ReplicationSystem.
	 * @param System A pointer to the system to destroy. Must have been created with CreateReplicationSystem.
	 * @see CreateReplicationSystem
	 */
	IRISCORE_API static void DestroyReplicationSystem(UReplicationSystem* System);

	/** Returns all replication systems. Entries may be null. */
	IRISCORE_API static TArrayView<UReplicationSystem*> GetAllReplicationSystems();

	/** Static delegate that is triggered just after creating and initializing a new replication system. */
	IRISCORE_API static FReplicationSystemCreatedDelegate& GetReplicationSystemCreatedDelegate();

	/** Static delegate that is triggered before we destroy a replication system. */
	IRISCORE_API static FReplicationSystemDestroyedDelegate& GetReplicationSystemDestroyedDelegate();

	static TSharedPtr<UE::Net::Private::FObjectReferenceCache> GetSharedObjectReferenceCache();

private:

	friend UReplicationSystem* GetReplicationSystem(uint32 Id);

	static TSharedPtr<UE::Net::Private::FObjectReferenceCache> SharedObjectReferenceCache;

	using FReplicationSystemArray = TArray<UReplicationSystem*, TInlineAllocator<UE_NUM_INLINE_REPLICATIONSYSTEMS>>;
	IRISCORE_API static FReplicationSystemArray ReplicationSystems;
};

inline UReplicationSystem* GetReplicationSystem(uint32 Id)
{
	return Id >= static_cast<uint32>(FReplicationSystemFactory::ReplicationSystems.Num()) ? nullptr : FReplicationSystemFactory::ReplicationSystems[Id];
}

/**
* A helper RAII class to allow thread-safe access to FNetStatsContext
* If created within a ParallelPhase, the constructor acquires a ChildNetStatsContext,
* otherwise it falls back to the ParentNetStatsContext. This means that we do not
* lock any mutexes for the duration of this object, only briefly inside AcquireChildNetStatsContext
*/
class FScopedNetStatsContext
{
public:
	
	IRISCORE_API explicit FScopedNetStatsContext(UReplicationSystem* const InReplicationSystem);

	IRISCORE_API ~FScopedNetStatsContext();

	IRISCORE_API Private::FNetStatsContext* GetRawPtr();

private:
	class Private::FNetStatsContext* StoredNetStatsContext = nullptr;
	UReplicationSystem* StoredReplicationSystem = nullptr;
	bool bWasAcquiredInParallelPhase = false;
};

} // end namespace UE::Net

