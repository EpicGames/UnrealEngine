// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Delegates/Delegate.h"
#include "Templates/Function.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Core/Connection/ConnectionHandle.h"

namespace UE::Net::Private
{
	class FReplicationSystemImpl;
	class FReplicationFiltering;
}

namespace UE::Net
{

typedef uint32 FInternalNetRefIndex;

class FReplicationSystemDelegates
{
public:
	using FConnectionAddedDelegate = TMulticastDelegate<void(FConnectionHandle ConnectionHandle)>;
	using FConnectionRemovedDelegate = TMulticastDelegate<void(FConnectionHandle ConnectionHandle)>;
	using FResolveObjectFromIndex = TFunctionRef<UObject*(FInternalNetRefIndex)>;
	using FConnectionObjectScopeChangedDelegate = TMulticastDelegate<void(
		FConnectionHandle /*ConnectionHandle*/,
		const FNetBitArrayView& /*NewlyInScope*/,
		const FNetBitArrayView& /*NewlyOutOfScope*/,
		FResolveObjectFromIndex /*ResolveObject*/)>;

	/** 
	 * Returns a delegate registration instance allowing the caller to register their FConnectionAddedDelegate. 
	 * The delegate will be called when a valid and not previously added connection is registered via a FReplicationSystem::AddConnection call.
	 * Currently only parent connections will call the delegates.
	 */
	FConnectionAddedDelegate::RegistrationType& OnConnectionAdded(); 

	/**
	 * Returns a delegate registration instance allowing the caller to register their FConnectionRemovedDelegate.
	 * The delegate will be called when a previously successfully added connection is removed via a FReplicationSystem::RemoveConnection call.
	 * Currently only parent connections will call the delegates.
	 */
	FConnectionAddedDelegate::RegistrationType& OnConnectionRemoved();

	/**
	 * Returns a delegate registration instance allowing the caller to register their FConnectionObjectScopeChangedDelegate.
	 * The delegate will be called once per connection per frame after filtering is complete, with bitarrays
	 * of objects that entered or left scope for that connection since the previous frame.
	 */
	FConnectionObjectScopeChangedDelegate::RegistrationType& OnConnectionObjectScopeChanged();

private:
	friend UE::Net::Private::FReplicationSystemImpl;
	friend class UE::Net::Private::FReplicationFiltering;

	FConnectionAddedDelegate ConnectionAddedDelegate;
	FConnectionRemovedDelegate ConnectionRemovedDelegate;
	FConnectionObjectScopeChangedDelegate ConnectionObjectScopeChangedDelegate;
};

inline FReplicationSystemDelegates::FConnectionAddedDelegate::RegistrationType& FReplicationSystemDelegates::OnConnectionAdded()
{
	return ConnectionAddedDelegate;
}

inline FReplicationSystemDelegates::FConnectionRemovedDelegate::RegistrationType& FReplicationSystemDelegates::OnConnectionRemoved()
{
	return ConnectionRemovedDelegate;
}

inline FReplicationSystemDelegates::FConnectionObjectScopeChangedDelegate::RegistrationType& FReplicationSystemDelegates::OnConnectionObjectScopeChanged()
{
	return ConnectionObjectScopeChangedDelegate;
}

}
