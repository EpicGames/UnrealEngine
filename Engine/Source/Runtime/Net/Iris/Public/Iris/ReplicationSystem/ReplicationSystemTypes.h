// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Net
{

enum class ENetObjectDeltaCompressionStatus : unsigned
{
	Disallow,
	Allow,
};

enum class EGetRefHandleFlags : uint32
{
	None,
	EvenIfGarbage
};
ENUM_CLASS_FLAGS(EGetRefHandleFlags);

enum class EReplicationSystemSendPass : unsigned
{
	Invalid,

	// Sending data directly after PostDispatch, this is a partial update only, not updating scope and filtering and will only process RPC/Attachmeents
	PostTickDispatch,

	// Sending data part of TickFlush, this will do a full update and replicate data to all connections
	TickFlush,
};

enum class EDependentObjectSchedulingHint : uint8
{
	// Default behavior, dependent object will be scheduled to replicate if parent is replicated, if the dependent object has not yet been replicated it will be replicated in the same batch as the parent
	Default = 0,

	// Dependent object will be scheduled to replicate before parent is replicated, if the dependent has data to send and has not yet been replicated the parent will only be scheduled if they both fit in same packet
	ScheduleBeforeParent,

	// Not yet replicated dependent object will behave as ReplicateBeforeParent otherwise it will be scheduled to replicate if the parent is replicated and scheduled after the parent
	ScheduleBeforeParentIfInitialState,
};

using FForwardNetRPCCallDelegate = TDelegate<void(UObject* RootObject,UObject* SubObject, UFunction* Function, void* Params)>;
using FForwardNetRPCCallMulticastDelegate = TMulticastDelegate<typename FForwardNetRPCCallDelegate::TFuncType>;
using FObjectToNetRefHandleSerial = TDelegate<uint64(const TObjectPtr<const UObject>& Object)>;
using FNetRefHandleSerialToObject = TDelegate<TObjectPtr<UObject>(const uint64 Id)>;

enum class ENetObjectAttachmentSendPolicyFlags : uint32
{
	// Default
	None = 0,

	// Schedule attachment to use the Out of bounds channel, essentially schedule the attachment to be sent as early as possible. Note: Only valid for unreliable attachments.
	ScheduleAsOOB = 1U << 0U,

	// Hint that this attachment like to be sent during PostTickDispatch. 
	// If one enqueued attachment has this flag all currently unreliable attachments scheduled to use the OOB channel will be sent during PostTickDispatch.
	SendInPostTickDispatch = ScheduleAsOOB << 1U,	

	// SendImmediate, Attachment should be sent using OOB channel and from PostTickDispatch.
	SendImmediate = ScheduleAsOOB | SendInPostTickDispatch,
};

ENUM_CLASS_FLAGS(ENetObjectAttachmentSendPolicyFlags);

inline const TCHAR* LexToString(ENetObjectAttachmentSendPolicyFlags SendFlags)
{
	switch (SendFlags)
	{
	case ENetObjectAttachmentSendPolicyFlags::None: return TEXT("None");
	case ENetObjectAttachmentSendPolicyFlags::ScheduleAsOOB: return TEXT("ScheduleAsOOB");
	case ENetObjectAttachmentSendPolicyFlags::SendInPostTickDispatch: return TEXT("SendInPostTickDispatch");
	case ENetObjectAttachmentSendPolicyFlags::SendImmediate: return TEXT("SendImmediate");
	default: ensure(false); return TEXT("Missing");
	}
}

inline const TCHAR* LexToString(EDependentObjectSchedulingHint SchedulingHint)
{
	switch (SchedulingHint)
	{
	case EDependentObjectSchedulingHint::Default: return TEXT("Default");
	case EDependentObjectSchedulingHint::ScheduleBeforeParent: return TEXT("BeforeParent");
	case EDependentObjectSchedulingHint::ScheduleBeforeParentIfInitialState: return TEXT("BeforeParentIfInitialState");
	default: ensure(false); return TEXT("Missing");
	}
}

} // end namespace UE::Net
