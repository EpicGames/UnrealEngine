// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Math/Vector.h"
#include "Net/Core/NetHandle/NetHandle.h"

// Define this macro in your game's Target.cs and set it to the maximum clients per connection your game can have (ex: splitscreen).
#ifndef UE_IRIS_INLINE_VIEWS_PER_CONNECTION
	#define UE_IRIS_INLINE_VIEWS_PER_CONNECTION 4
#endif

namespace UE::Net
{

struct FReplicationView
{
	struct FView
	{
		/** The controlling net object associated with this view, typically a player controller. */
		FNetHandle Controller;
		/** The actor that is being directly viewed, usually a pawn. */
		FNetHandle ViewTarget;
		/** Where the viewer is looking from */
		FVector Pos = FVector::ZeroVector;
		/** Direction the viewer is looking */
		FVector Dir = FVector::ForwardVector;
		/** The field of view */
		float FoVRadians = UE_HALF_PI;
	};

	TArray<FView, TInlineAllocator<UE_IRIS_INLINE_VIEWS_PER_CONNECTION>> Views;
};

}
