// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Iris/ReplicationSystem/NetTokenStore.h"
#include "Misc/MemStack.h"

namespace UE::Net
{

// Simple token store used to store string tokens
// When the PackageMapRefactor is complete we will most likely rely on NetTagManager for persistent storage
class FStringTokenStore : public FNetTokenDataStore
{
	UE_NONCOPYABLE(FStringTokenStore);
public:
	IRISCORE_API explicit FStringTokenStore(FNetTokenStore& TokenStore);

	// Create a string token for the provided string
	IRISCORE_API FNetToken GetOrCreateToken(const FString& String);
	IRISCORE_API FNetToken GetOrCreateToken(const TCHAR* Name, uint32 Length);

	// Resolve NetToken, to resolve remote tokens RemoteTokenStoreState must be valid
	IRISCORE_API const TCHAR* ResolveToken(FNetToken Token, const FNetTokenStoreState* RemoteTokenStoreState = nullptr) const;	

	// Resolve a token received from remote
	const TCHAR* ResolveRemoteToken(FNetToken Token, const FNetTokenStoreState& NetTokenStoreState) const
	{ 
		return ResolveToken(Token, &NetTokenStoreState);
	}

	static FName GetTokenStoreName()
	{
		return StringTokenStoreName;
	}

protected:
	// Serialize data for a token, note there is not validation in this function
	virtual void WriteTokenData(FNetSerializationContext& Context, FNetTokenStoreKey TokenStoreKey) const override;
	virtual void WriteTokenData(FArchive& Archive, FNetTokenStoreKey TokenStoreKey, UPackageMap* Map = nullptr) const override;

	// Read data for a token, returns a valid StoreKey if successful read
	virtual FNetTokenStoreKey ReadTokenData(FNetSerializationContext& Context, const FNetToken& NetToken) override;
	virtual FNetTokenStoreKey ReadTokenData(FArchive& Archive, const FNetToken& NetToken, UPackageMap* Map = nullptr) override;

	// Create a persistent string
	FNetTokenStoreKey GetOrCreatePersistentString(const TCHAR* Name, uint32 Length);

private:
	inline static FName StringTokenStoreName = "StringTokenStore";

	TMap<uint64, FNetTokenStoreKey> HashToKey;
	TArray<const TCHAR*> StoredStrings;
	FMemStackBase Allocator;
};

}
