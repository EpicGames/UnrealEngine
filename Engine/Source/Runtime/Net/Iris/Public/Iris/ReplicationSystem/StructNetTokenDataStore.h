// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/ReplicationSystem/NetTokenStore.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/StructNetSerializerUtil.h"
#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisMemoryTracker.h"

namespace UE::Net
{
	/*
	 * This is a simplification of the process of building novel NetToken types and NetTokenDataStores for those types.
	 *
	 * The "General Idea" here is that you can define a USTRUCT that can be easily serialized as a NetToken instead of
	 * serializing the entirety of the data structure every time. This is typically useful for large data structures where the data changes infrequently
	 *
	 * or
	 *
	 * where the data is commonly one of a small-ish set of permutations of that data. There are probably other useful cases as well, but these uses are readily apparent.
	 *
	 * Each NetToken refers to single, immutable, instance of the Struct you pass in/out. Every instance of the Struct that returns the same GetUniqueKey has the same NetToken.
	 * Conceptually this means that you can't store pointers or references in them to sub pieces of data that change.
	 *
	 * You can't store references to other NetToken types, or other reference types... i.e. no properties that also are exported as NetTokens or ObjectReferences (NetGuids/NetRefHandles)
	 * You should generally consider that you can only store primitive types in the NetToken struct.
	 *
	 * This means no: FNames, FGameplayTags or anything with a NetGuid.
	 *
	 * Additionally, this means that the data permutations that you are going to iterate are finite, or effectively finite. This isn't magic and it is not a compression algorithm.
	 * We also aren't doing anything "smart" here about storing off the ShadowCopies of the input Structs. We very naively keep an entire copy of the struct in memory and there is no age out mechanism.
	 * If you try to put "big" pieces of data in here we will just be making copies of that data. It also means complicated hashing mechanics for GetUniqueKey will eat up a lot of time.
	 *
	 * Again, its a simplification, not magic. You still want to be judicious in the data you are choosing to replicate.
	 *
	 * Most/All of the "Hard Bits" are implemented for you in a 'reasonable' default. You will need to implement custom versions if you need more concrete control.
	 * - Replay Compatibility
	 * - Default Serialization/Quantization
	 * - Bookkeeping
	 * - Maintaining Shadow Copies of the required data
	 *
	 * Example: Pretend that StringTokenStore didn't exist and I have a RPC that I want to call with a couple of FStrings that come from a set of known Strings.
	 *
	 *  UPROPERTY(Replicated)
	 *  TArray<FString> KnownStrings =
	 *  {
	 *		TEXT("First string"),
	 *		TEXT("Second string, this string is super long and would take up just a bunch of bytes to transmit. Just pretend its a huge string ok, thanks."),
	 *		....
	 *		TEXT("Last String")
	 *  };
	 *
	 *  When asked to set this up we have the naive implementation:
	 *  USTRUCT()
	 *	struct FTwoStrings
	 *  {
	 *		UPROPERTY()
	 *		FString Left;
	 *		UPROPERTY()
	 *		FString Right;	
	 *  };
	 *
	 *  ...
	 *  UFUCNTION(Server,Reliable)
	 *  void SendSelectionToServer(FTwoStrings Data);
	 *
	 *	...
	 *  void OnTick(double DeltaTime)
	 *  {
	 *		...
	 *		SendSelectionToServer({Left,Right});
	 *	}
	 *
	 *  Given this naive implementation we can (hopefully immediately) see the multitude of problems.
	 *  In this simplistic example we could probably come up with several immediate fixes.
	 *  Let's pretend we have to actually send this data on every tick.
	 *
	 *  We can spend a good deal of time coming up with custom serializer that would use significantly less data or some other representation for the data...
	 *  For example:
	*   USTRUCT()
	 *	struct FTwoStringIdx
	 *  {
	 *		UPROPERTY()
	 *		int32 Left;
	 *		UPROPERTY()
	 *		int32 Right;	
	 *  };
	 *
	 *  But this suffers from a fundamental problem that isn't immediately obvious. KnownStrings is mutable and might be changing...
	 *  And now we have a synchronization problem, where we are trying to maintain the state of KnownStrings, the index and the versioning of both.
	 *  This rapidly grows in complexity beyond a simple "how do we make this use less data". We have to figure out a strategy on determining agreement on state and ordering and more.
	 *  Ex: Which updates first? Do we need to maintain a history of known states in the event of delayed packets,etc,etc...
	 *
	 *  Even though the list of possible strings is known/finite, the mutable nature of the KnownStrings array in conjunction with the indeterminate ordering of network communication makes this a much more difficult problem.
	 *
	 *  This problem can be solved like this.
	 *
	 *	--------------
	 *  TwoStrings.h
	 *  --------------
	 *  USTRUCT()
	 *	struct FTwoStrings : FNetTokenStructBase
	 *  {
	 *      GENERATED_BODY()
	 *      UE_NET_NETTOKEN_GENERATED_BODY(TwoStrings) // Note this is the struct name minus the 'F'
	 *		UPROPERTY()
	 *		FString Left;
	 *		UPROPERTY()
	 *		FString Right;
	 *		uint64 GetUniqueKey()
	 *		{
	 *			FString Combo = Left+Right;
	 *			return CityHash64((const char*)Combo, Combo.Num()*sizeof(TCHAR));
	 *		}
	 *  };
	 *
	 *  UE_NET_DECLARE_NAMED_NETTOKEN_STRUCT_SERIALIZERS(TwoStrings, YOUR_MODULES_API)
	 *	--------------
	 *  TwoStrings.cpp
	 *  --------------
	 *  UE_NET_IMPLEMENT_NAMED_NETTOKEN_STRUCT_SERIALIZERS(TwoStrings)
	 *
	 *	--------------
	 *  DefaultEngine.ini
	 *  --------------
	 *  +ReservedTypeIds=(StoreTypeName="TwoStrings", TypeID=4) # Don't forget to use an unused TypeID.
	 *  
	 *	Having a UPROPERTY() or RPC UFUNCTION() will automatically serialize the type as the NetToken.
	 *
	 *  It is also possible to explicitly declare the DataStore, and use it in a custom NetSerialize.
	 *  class FTwoStringDataStore : public UE::Net::TStructNetTokenDataStore<FTwoStrings>
	 *  {
	 *  public:
	 *		explicit FTwoStringDataStore(UE::Net::FNetTokenStore& InTokenStore)
	 *			: UE::Net::TStructNetTokenDataStore<FTwoStrings>(InTokenStore)
	 *		{
	 *		}
	 *	};
	 *
	 *  It is possible to use these in custom serializers like this:
	 *	FTwoStrings Data{Left,Right};
	 *	FTwoStringDataStore::NetSerializeAndExportToken(Ar,Map,Data);
	 */

template <typename T>
class TStructNetTokenDataStoreHelper;

template <typename T>
class TStructNetTokenDataStore : public FNetTokenDataStore
{
	using FNetTokenStoreKey = FNetTokenDataStore::FNetTokenStoreKey;
	
	UE_NONCOPYABLE(TStructNetTokenDataStore<T>);
public:
	using DataType = T;
	
	explicit TStructNetTokenDataStore<T>(FNetTokenStore& InTokenStore)
	: FNetTokenDataStore(InTokenStore)
	{
	}
	
	// Create a token for input struct
	FNetToken GetOrCreateToken(const T& InData)
	{
		FNetToken Result;
		const FNetTokenStoreKey Key = GetOrCreatePersistentState(InData);
		if (Key.IsValid())
		{
			Result = GetNetTokenFromKey(Key);
			if (!Result.IsValid())
			{
				Result = CreateAndStoreTokenForKey(Key);
				UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::GetOrCreateToken %ls CreatedToken %ls.", *GetTokenStoreName().ToString(), *Result.ToString());
			}
		}
		UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::GetOrCreateToken %ls GetOrCreateToken %ls.", *GetTokenStoreName().ToString(), *Result.ToString());
		return Result;
	}

	// Resolve NetToken, to resolve remote tokens RemoteTokenStoreState must be valid
	const T& ResolveToken(FNetToken Token, const FNetTokenStoreState* RemoteTokenStoreState = nullptr) const
	{
		const FNetTokenStoreState* TokenStoreState = TokenStore.IsLocalToken(Token) ? TokenStore.GetLocalNetTokenStoreState() : RemoteTokenStoreState;
		UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::ResolveToken Starting up %ls - %ls. Local: %d, TokenStoreState: %p", *GetTokenStoreName().ToString(), *Token.ToString(), TokenStore.IsLocalToken(Token), TokenStoreState);
		if (Token.IsValid() && ensureMsgf(TokenStoreState, TEXT("TStructNetTokenDataStore::ResolveToken Needs valid TokenStoreState to resolve %s"), *Token.ToString()))
		{
			const FNetTokenStoreKey StoreKey = GetTokenKey(Token, *TokenStoreState);
			UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::ResolveToken Got Token Key %ls - %ls. StoreKeyIsValid: %d, StoreKeyID: %d, NumStoredStates: %d", *GetTokenStoreName().ToString(), *Token.ToString(), StoreKey.IsValid(), StoreKey.GetKeyIndex(), StoredStates.Num());
			if (StoreKey.IsValid() && StoredStates.Contains(StoreKey.GetKeyIndex()))
			{
				UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::ResolveToken %ls Succeeded %ls.", *GetTokenStoreName().ToString(), *Token.ToString());
				return StoredStates[StoreKey.GetKeyIndex()];
			}
			else
			{
				UE_LOGF(LogNetToken, Error, "TStructNetTokenDataStore::ResolveToken %ls failed to resolve %ls.", *GetTokenStoreName().ToString(), *Token.ToString());
			}
		}
		return InvalidState;
	}

	// Resolve a token received from remote
	const T& ResolveRemoteToken(FNetToken Token, const FNetTokenStoreState& NetTokenStoreState) const
	{ 
		return ResolveToken(Token, &NetTokenStoreState);
	}
	
	static FName GetTokenStoreName()
	{
		return T::GetTokenStoreName();
	}

	static const T& GetInvalidState()
	{
		return InvalidState;
	}

	// Need to declare as a delegate that is bound elsewhere to reduce the includes. We can't eliminate them, since the specialization module needs to include the engine parts necessary to make this work
	// and we can't circular depend on the engine module to implement these details. We could write the helpers somewhere in the engine, though there isn't a great place to do this.
	DECLARE_DELEGATE_ThreeParams(TNetSerializeTokenType, T&, FArchive&, UPackageMap*);
	inline static TNetSerializeTokenType NetSerializeScriptDelegate;
protected:
	friend TStructNetTokenDataStoreHelper<T>;
	
	// Serialize data for a token, note there is not validation in this function
	virtual void WriteTokenData(FNetSerializationContext& Context, FNetTokenStoreKey TokenStoreKey) const override
	{
		if (!TokenStoreKey.IsValid() || !StoredStates.Contains(TokenStoreKey.GetKeyIndex()))
		{
			UE_LOGF(LogNetToken, Error, "TStructNetTokenDataStore::WriteTokenData %ls KeyIndex: %d FAILED",*GetTokenStoreName().ToString(),TokenStoreKey.GetKeyIndex());
			Context.SetError(GNetError_InvalidValue);
			return;
		}
		
		if (!Descriptor.IsValid())
		{
			UE_LOGF(LogNetToken, Error, "TStructNetTokenDataStore::WriteTokenData %ls Failed. Serialization Descriptor Invalid.",*GetTokenStoreName().ToString());
			Context.SetError(GNetError_InternalError);
			return;
		}
		
		UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::WriteTokenData %ls KeyIndex: %d Serializing",*GetTokenStoreName().ToString(),TokenStoreKey.GetKeyIndex());
		T TempValue = StoredStates.FindRef(TokenStoreKey.GetKeyIndex());
		
		WriteStruct(Context, NetSerializerValuePointer(&TempValue), Descriptor);
		
		if (Context.HasError())
		{
			UE_LOGF(LogNetToken, Error, "TStructNetTokenDataStore::WriteTokenData, %ls, FAILED", *GetTokenStoreName().ToString());
		}
	}
	
	virtual void WriteTokenData(FArchive& Ar, FNetTokenStoreKey TokenStoreKey, UPackageMap* Map = nullptr) const override
	{
		if (!TokenStoreKey.IsValid()
				|| !StoredStates.Contains(TokenStoreKey.GetKeyIndex()))
		{
			UE_LOGF(LogNetToken, Error, "TStructNetTokenDataStore::WriteTokenData %ls KeyIndex: %d FAILED",*GetTokenStoreName().ToString(),TokenStoreKey.GetKeyIndex());
			return;
		}
		UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::WriteTokenData %ls KeyIndex: %d Serializing",*GetTokenStoreName().ToString(),TokenStoreKey.GetKeyIndex());
		T TempValue = StoredStates.FindRef(TokenStoreKey.GetKeyIndex());
		NetSerializeScriptDelegate.ExecuteIfBound(TempValue, Ar, Map);
		if (Ar.IsError())
		{
			UE_LOGF(LogNetToken, Error, "TStructNetTokenDataStore::WriteTokenData, %ls, FAILED", *GetTokenStoreName().ToString());
		}	
	}

	// Read data for a token, returns a valid StoreKey if successful read
	virtual FNetTokenStoreKey ReadTokenData(FNetSerializationContext& Context, const FNetToken& NetToken) override
	{
		if (!Descriptor.IsValid())
		{
			UE_LOGF(LogNetToken, Error, "TStructNetTokenDataStore::ReadTokenData %ls Failed. Serialization Descriptor Invalid.",*GetTokenStoreName().ToString());
			return FNetTokenStoreKey();
		}
		
		UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::ReadTokenData %ls", *GetTokenStoreName().ToString());
		T Value;
		ReadStruct( Context, NetSerializerValuePointer(&Value), Descriptor);
		if (!Context.HasErrorOrOverflow())
		{
			UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::ReadTokenData %ls, Succeeded",*GetTokenStoreName().ToString());
			return GetOrCreatePersistentState(Value);
		}
		UE_LOGF(LogNetToken, Error, "TStructNetTokenDataStore::ReadTokenData, %ls, FAILED", *GetTokenStoreName().ToString());
		return FNetTokenStoreKey();
	}

	virtual FNetTokenStoreKey ReadTokenData(FArchive& Ar, const FNetToken& NetToken, UPackageMap* Map = nullptr) override
	{
		T Value;
		UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::ReadTokenData %ls", *GetTokenStoreName().ToString());
		NetSerializeScriptDelegate.ExecuteIfBound(Value, Ar, Map);
		if (!Ar.IsError())
		{
			UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::ReadTokenData %ls, Succeeded",*GetTokenStoreName().ToString());
			return GetOrCreatePersistentState(Value);
		}
		UE_LOGF(LogNetToken, Error, "TStructNetTokenDataStore::ReadTokenData, %ls, FAILED", *GetTokenStoreName().ToString());
		return FNetTokenStoreKey();
	}
	
	// Creates a persistent copy of the input struct 
	FNetTokenStoreKey GetOrCreatePersistentState(const T& InNetTokenData)
	{
		UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess ScopeAccess(&TokenStore));

		LLM_SCOPE_BYTAG(NetTokenStructState);
		const uint64 HashKey = InNetTokenData.GetUniqueKey();
		if (HashToKey.Contains(HashKey))
		{
			UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::GetOrCreatePersistentState %ls FoundToken, Hash: %llu KeyIndex: %d", *GetTokenStoreName().ToString(), HashKey, HashToKey[HashKey].GetKeyIndex());
			return HashToKey[HashKey];
		}

		const FNetTokenStoreKey NewKey = GetNextNetTokenStoreKey();
		if (NewKey.IsValid())
		{
			HashToKey.Add(HashKey, NewKey);
			StoredStates.Add(NewKey.GetKeyIndex(), InNetTokenData);
			UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::GetOrCreatePersistentState %ls Adding New, Hash: %llu KeyIndex: %d", *GetTokenStoreName().ToString(), HashKey, HashToKey[HashKey].GetKeyIndex());
			return NewKey;
		}
		UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::GetOrCreatePersistentState %ls ERROR, Hash: %llu", *GetTokenStoreName().ToString(), HashKey);
		return FNetTokenStoreKey();
	}

private:
	inline static UScriptStruct* Struct;
	inline static TRefCountPtr<const FReplicationStateDescriptor> Descriptor;
	TMap<uint64, FNetTokenStoreKey> HashToKey;
	TMap<uint32, T> StoredStates;
	static inline T InvalidState = T();
	
	// Helper for NetSerializerDescriptor setup
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		bool HasPostFreezeBeenCalled() const { return bPostFreezeHasBeenCalled; }
		
	private:
		bool bPostFreezeHasBeenCalled = false;
		virtual void OnPostFreezeNetSerializerRegistry() override
		{
			bPostFreezeHasBeenCalled = true;
			UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::OnPostFreezeNetSerializerRegistry %ls", *GetTokenStoreName().ToString());
			if (!Struct)
			{
				Struct = StaticStruct<T>();
			}
			UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::OnPostFreezeNetSerializerRegistry %ls Struct: %ls", *GetTokenStoreName().ToString(), *GetNameSafe(Struct));
			
			if (!Descriptor.IsValid())
			{				
				FReplicationStateDescriptorBuilder::FParameters Params;

				// TODO: this need flags to not pickup serializer using NetToken to write the data struct.
				Params.SkipCheckForCustomNetSerializerForStruct = 1U;

				Descriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(Struct, Params);
				UE_LOGF(LogNetToken, VeryVerbose, "TStructNetTokenDataStore::OnPostFreezeNetSerializerRegistry %ls Descriptor: %ls", *GetTokenStoreName().ToString(), Descriptor ? TEXT("Exists") : TEXT("NULL"));
			}
		}
	};
public:
	// Compiler tries to optimize out the NetSerializer since it isn't obvious that anything is referencing it during module compilation.
	UE_DISABLE_OPTIMIZATION_SHIP
	inline static TStructNetTokenDataStore<T>::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	UE_ENABLE_OPTIMIZATION_SHIP
};
	
// Helper to implement a NetSerializer for a struct that should serialize using a NetTokenStore
template <typename T, typename NetTokenDataStoreT = TStructNetTokenDataStore<T>>
class TStructAsNetTokenNetSerializerImpl
{
public:
	// Version
	static const uint32 Version = 0;

	typedef T SourceType;
	typedef FNetToken QuantizedType;
	typedef struct FNetSerializerConfig ConfigType;
	static inline const ConfigType DefaultConfig = ConfigType();

	// Iris
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);
	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);
	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
};

template <typename T, typename NetTokenDataStoreT>
void TStructAsNetTokenNetSerializerImpl<T, NetTokenDataStoreT>::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const FNetToken& NetToken = *reinterpret_cast<FNetToken*>(Args.Source);
	
	// Tokens will differ, so we cannot store them in the default statehash.
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	// Write token without type
	Context.GetNetTokenStore()->WriteNetTokenWithKnownType<NetTokenDataStoreT>(Context, NetToken);

	// Add to pending exports for later export
	FNetTokenStore::AppendExport(Context, NetToken);
}

template <typename T, typename NetTokenDataStoreT>
void TStructAsNetTokenNetSerializerImpl<T, NetTokenDataStoreT>::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetToken& NetToken = *reinterpret_cast<FNetToken*>(Args.Target);
	NetToken = Context.GetNetTokenStore()->ReadNetTokenWithKnownType<NetTokenDataStoreT>(Context);
}

template <typename T, typename NetTokenDataStoreT>
void TStructAsNetTokenNetSerializerImpl<T, NetTokenDataStoreT>::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	FNetToken& TargetValue = *reinterpret_cast<FNetToken*>(Args.Target);

	TargetValue = FNetToken();

	if (NetTokenDataStoreT* NetTokenDataStore = Context.GetNetTokenStore()->GetDataStore<NetTokenDataStoreT>())
	{
		TargetValue = NetTokenDataStore->GetOrCreateToken(SourceValue);
	}
	else
	{
		UE_LOGF(LogIris, Error, "TStructAsNetTokenNetSerializerImpl<T>::Quantize Could not find required FNetTokenDataStore %ls", *NetTokenDataStoreT::GetTokenStoreName().ToString());
		ensure(false);
	}
}

template <typename T, typename NetTokenDataStoreT>
void TStructAsNetTokenNetSerializerImpl<T, NetTokenDataStoreT>::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const FNetToken& Source = *reinterpret_cast<const FNetToken*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	if (NetTokenDataStoreT* NetTokenDataStore = Context.GetNetTokenStore()->GetDataStore<NetTokenDataStoreT>())
	{
		Target = NetTokenDataStore->ResolveToken(Source, Context.GetRemoteNetTokenStoreState());
	}
	else
	{
		UE_LOGF(LogIris, Error, "TStructAsNetTokenNetSerializerImpl<T>::Deqquantize Could not find required FNetTokenDataStore %ls", *NetTokenDataStoreT::GetTokenStoreName().ToString());
		ensure(false);

		Target = SourceType();
	}
}
template <typename T, typename NetTokenDataStoreT>
bool TStructAsNetTokenNetSerializerImpl<T, NetTokenDataStoreT>::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const FNetToken& Value0 = *reinterpret_cast<const FNetToken*>(Args.Source0);
		const FNetToken& Value1 = *reinterpret_cast<const FNetToken*>(Args.Source1);

		// Need to compare actual Tags to properly compare non-auth and auth token
		if (Value0.IsAssignedByAuthority() != Value1.IsAssignedByAuthority())
		{
			NetTokenDataStoreT* NetTokenDataStore = Context.GetNetTokenStore()->GetDataStore<NetTokenDataStoreT>();
			const UE::Net::FNetTokenStoreState* RemoteNetTokenStoreState = Context.GetRemoteNetTokenStoreState();
	
			const SourceType Source0 = NetTokenDataStore->ResolveToken(Value0, RemoteNetTokenStoreState);
			const SourceType Source1 = NetTokenDataStore->ResolveToken(Value1, RemoteNetTokenStoreState);
				
			if (Source0 != Source1)
			{
				return false;
			}
		}
		else if (Value0 != Value1)
		{
			return false;
		}
		return true;
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<SourceType*>(Args.Source1);

		return Value0 == Value1;
	}
}

} // End of namespace







