// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/ReplicationFragment.h"

namespace UE::Net
{

/*
* Typed replication fragments are used when we have generated ReplicationStates
* and will call the SetReplicatedState method on the parent class
* $IRIS TODO: Introduce policies based on ReplicationStateTraits, i.e. should we call other functions on Init etc?
*/
template <typename T, typename ReplicationStateT>
class TReplicationFragment : protected FReplicationFragment
{
public:
	// Receive only
	explicit TReplicationFragment(T& OwnerIn) : FReplicationFragment(EReplicationFragmentTraits::CanReceive), Owner(OwnerIn), SrcState(nullptr) {}

	// Replicate 
	TReplicationFragment(T& OwnerIn, ReplicationStateT& SrcReplicationState) : FReplicationFragment(EReplicationFragmentTraits::CanReplicate), Owner(OwnerIn), SrcState(&SrcReplicationState) {}

	void Register(FFragmentRegistrationContext& Context);

protected:
	// FReplicationFragment
	virtual void ApplyReplicatedState(FReplicationStateApplyContext& Context) const override;


private:
	T& Owner;
	ReplicationStateT* SrcState;
};

template <typename T, typename ReplicationStateT>
void TReplicationFragment<T, ReplicationStateT>::ApplyReplicatedState(FReplicationStateApplyContext& Context) const
{
	// Cast to the expected type
	const ReplicationStateT* ReplicationState  = reinterpret_cast<const ReplicationStateT*>(Context.StateBufferData.ExternalStateBuffer);

	// $TODO: We need a state management system here so that we can keep states around until update time
	// either we implement this at this level or we make this part of the replication system, maybe a linear allocator during receive that we will free after we have applied the states
	Owner.ApplyReplicationState(*ReplicationState, Context);
}

template <typename T, typename ReplicationStateT>
void TReplicationFragment<T, ReplicationStateT>::Register(FFragmentRegistrationContext& Context)
{
	Context.RegisterReplicationFragment(this, ReplicationStateT::GetReplicationStateDescriptor(), SrcState);
}

}
