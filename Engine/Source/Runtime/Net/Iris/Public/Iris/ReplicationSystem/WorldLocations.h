// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ChunkedArray.h"
#include "Iris/Core/NetChunkedArray.h"
#include "Net/Core/NetBitArray.h"
#include "Math/Vector.h"
#include "Misc/Optional.h"
#include "WorldLocations.generated.h"

class UReplicationSystem;

namespace UE::Net
{
	typedef uint32 FInternalNetRefIndex;

	namespace Private
	{
		class FNetRefHandleManager;
	}
}

/**
* Common settings used to configure how the GridFilter behaves
*/
UCLASS(Config=Engine)
class UWorldLocationsConfig : public UObject
{
	GENERATED_BODY()

public:
	/** All world positions will be clamped to MinPos and MaxPos. */
	UPROPERTY(Config)
	FVector MinPos = { -0.5f * 2097152.0f, -0.5f * 2097152.0f, -0.5f * 2097152.0f };

	/** All world positions will be clamped to MinPos and MaxPos. */
	UPROPERTY(Config)
	FVector MaxPos = { +0.5f * 2097152.0f, +0.5f * 2097152.0f, +0.5f * 2097152.0f };

	/** We will issue a warning if user sets a higher NetCullDistance or NetCullDistanceOverride than the MaxNetCullDistance. */
	UPROPERTY(Config)
	float MaxNetCullDistance = 150000.f;
};

namespace UE::Net
{

struct FWorldLocationsInitParams
{
	TObjectPtr<UReplicationSystem> ReplicationSystem = nullptr;

	FInternalNetRefIndex MaxInternalNetRefIndex = 0;

	/** How many world info storage slots to preallocate. */
	uint32 PreallocatedStorageCount = 0;
};

class FWorldLocations
{
public: 

	/** Publically available information of a replicated root object */
	struct FWorldInfo
	{
		/** Absolute coordinate of the object */
		FVector WorldLocation = FVector::ZeroVector;

		/** The current network cull distance of the object */
		float CullDistance = 0.0f;
	};

public:

	void Init(const FWorldLocationsInitParams& InitParams);
	void PostSendUpdate();

	/** Returns whether the object has a valid cached data or not. */
	bool HasInfoForObject(FInternalNetRefIndex ObjectIndex) const
	{
		return ValidInfoIndexes.IsBitSet(ObjectIndex);
	}

	/** Returns the object's world location if it's valid or a zero vector if it's not. */
	inline FVector GetWorldLocation(FInternalNetRefIndex ObjectIndex) const;

	/** Get the object's current cull distance. */
	inline float GetCullDistance(FInternalNetRefIndex ObjectIndex) const;

	/** Return the current stored world information of the given object, or NullOpt if the object did not register in the world location cache. */
	inline TOptional<FWorldLocations::FWorldInfo> GetWorldInfo(FInternalNetRefIndex ObjectIndex) const;

	/** Set the mandatory info of a replicated root object*/
	void SetObjectInfo(FInternalNetRefIndex ObjectIndex, const FVector& Location, float NetCullDistance);
	
	/** Assign a world information cache to the replicated object */
	void InitObjectInfoCache(FInternalNetRefIndex ObjectIndex);

	/** Remove the world information cache of the replicated object */
	void RemoveObjectInfoCache(FInternalNetRefIndex ObjectIndex);

	/**
	 * Objects are not necessarily marked as dirty just because they're moving, such as objects attached to other objects. 
	 * If such objects are spatially filtered they need to update their world locations in order for replication to work as expected.
	 * Use SetObjectRequiresFrequentWorldLocationUpdate to force frequent world location update on an object.
	 */
	void SetObjectRequiresFrequentWorldLocationUpdate(FInternalNetRefIndex ObjectIndex, bool bRequiresFrequentUpdate)
	{
		ObjectsRequiringFrequentWorldLocationUpdate.SetBitValue(ObjectIndex, ValidInfoIndexes.GetBit(ObjectIndex) && bRequiresFrequentUpdate);
	}

	/** Returns whether an object requires frequent world location updates. */
	bool GetObjectRequiresFrequentWorldLocationUpdate(FInternalNetRefIndex ObjectIndex) const
	{
		return ObjectsRequiringFrequentWorldLocationUpdate.GetBit(ObjectIndex);
	}

	/** 
	 * Add a temporary net cull distance that will have priority over the regular net cull distance.
	 * Returns true if the object had registered to use the world location cache and can store the override.
	 */
	bool SetCullDistanceOverride(FInternalNetRefIndex ObjectIndex, float CullDistance);

	/** 
	 * Remove the temporary net cull distance override and instead use the regular net cull distance 
	 * Returns true if the object had registered to use the world location cache and had an override value previously set
	 */
	bool ClearCullDistanceOverride(FInternalNetRefIndex ObjectIndex);

	/** Returns true if the object was set a cull distance override and is using it instead of his default cull distance value */
	bool HasCullDistanceOverride(FInternalNetRefIndex ObjectIndex) const
	{
		return ValidInfoIndexes.IsBitSet(ObjectIndex) ? GetObjectInfo(ObjectIndex).CullDistanceOverride != FLT_MAX : false;
	}

	/** Returns the list of objects that need to check for a location change every frame */
	const FNetBitArrayView GetObjectsRequiringFrequentWorldLocationUpdate() const { return MakeNetBitArrayView(ObjectsRequiringFrequentWorldLocationUpdate); }

	/** Returns the list of objects that changed world location or cull distance this frame */
	const FNetBitArrayView GetObjectsWithDirtyInfo() const { return MakeNetBitArrayView(ObjectsWithDirtyInfo); }

	/** Returns the list of objects that registered world location information */
	const FNetBitArrayView GetObjectsWithWorldInfo() const { return MakeNetBitArrayView(ValidInfoIndexes); }

	/** Reset the list of objects that changed location or cull distance */
	void ResetObjectsWithDirtyInfo();

	/** Debug tool to track when its legal to modify the DirtyInfo list. */
	void LockDirtyInfoList(bool bLock);

	/** Return the world boundaries (min and max position). */
	const FVector& GetWorldMinPos() const { return MinWorldPos; };
	const FVector& GetWorldMaxPos() const { return MaxWorldPos; };
	
	/** Return a position clamped to the configured world boundary. */
	FVector ClampPositionToBoundary(const FVector& Position) const
	{
		return Position.BoundToBox(GetWorldMinPos(), GetWorldMaxPos());
	}

	/** Is the location without the configured Min/Max WorldPos*/
	bool IsValidLocation(const FVector& Location) const
	{
		return (Location.X >= MinWorldPos.X && Location.Y >= MinWorldPos.Y && Location.Z >= MinWorldPos.Z &&
				Location.X <= MaxWorldPos.X && Location.Y <= MaxWorldPos.Y && Location.Z <= MaxWorldPos.Z);
	}

	void OnMaxInternalNetRefIndexIncreased(FInternalNetRefIndex NewMaxInternalIndex);

private:

	/** Contains the cached object data we are storing. */
	struct FObjectInfo
	{
		/** Absolute coordinate of the object */
		FVector WorldLocation = FVector::ZeroVector;

		/** The default network cull distance of the object */
		float CullDistance = 0.0f;

		/** The optional temporary cull distance override. Max means it is not used */
		float CullDistanceOverride = FLT_MAX;
	};

	enum : uint32
	{
		StorageElementsPerChunk = 65536U / sizeof(FObjectInfo),
	};

private:

	const int32 GetStorageIndex(FInternalNetRefIndex ObjectIndex) const
	{
		return StorageIndexes[ObjectIndex];
	}

	const FObjectInfo& GetObjectInfo(FInternalNetRefIndex ObjectIndex) const
	{
		return StoredObjectInfo[GetStorageIndex(ObjectIndex)];
	}

	FObjectInfo& GetObjectInfo(FInternalNetRefIndex ObjectIndex)
	{
		return StoredObjectInfo[GetStorageIndex(ObjectIndex)];
	}

private:

	/** Set bits indicate that we have stored information for this internal object index */
	FNetBitArray ValidInfoIndexes;
	
	/** Set bits indicate that the world location or net cull distance has changed since last update */
	FNetBitArray ObjectsWithDirtyInfo;
	
	/** Set bits indicate that the object requires frequent world location updates */
	FNetBitArray ObjectsRequiringFrequentWorldLocationUpdate;

	/** Map that returns the storage index for the world info of a registered object. */
	TArray<int32> StorageIndexes;

	/** List detailing if a given index slot is free to be used to store world info. */
	FNetBitArray ReservedStorageSlot;

	TNetChunkedArray<FObjectInfo, StorageElementsPerChunk> StoredObjectInfo;

	const UE::Net::Private::FNetRefHandleManager* NetRefHandleManager = nullptr;

	/** World boundaries (min and max position). */
	FVector MinWorldPos;
	FVector MaxWorldPos;
	float MaxNetCullDistance;

	/** Controls if the dirty list can be modified */
	bool bLockdownDirtyList = false;
};

inline FVector FWorldLocations::GetWorldLocation(FInternalNetRefIndex ObjectIndex) const
{
	return ValidInfoIndexes.IsBitSet(ObjectIndex) ? GetObjectInfo(ObjectIndex).WorldLocation : FVector::Zero();
}

inline float FWorldLocations::GetCullDistance(FInternalNetRefIndex ObjectIndex) const
{
	if (ValidInfoIndexes.IsBitSet(ObjectIndex))
	{
		const FObjectInfo& Info = GetObjectInfo(ObjectIndex);
		return Info.CullDistanceOverride == FLT_MAX ? Info.CullDistance : Info.CullDistanceOverride;
	}

	return 0.0f;
}

inline TOptional<FWorldLocations::FWorldInfo> FWorldLocations::GetWorldInfo(FInternalNetRefIndex ObjectIndex) const
{
	if (ValidInfoIndexes.IsBitSet(ObjectIndex))
	{
		const FObjectInfo& Info = GetObjectInfo(ObjectIndex);
		return FWorldInfo
		{ 
			.WorldLocation = Info.WorldLocation, 
			.CullDistance = Info.CullDistanceOverride == FLT_MAX ? Info.CullDistance : Info.CullDistanceOverride,
		};
	}

	return NullOpt;
}

} // end namespace UE::Net
