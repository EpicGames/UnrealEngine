// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Templates/EnableIf.h"
#include "Templates/IsIntegral.h"
#include "Templates/IsSigned.h"

namespace UE::Net
{

/**
 * All the SerializeIntDelta functions assume that the Value and PrevValue are representable using LargeBitCount. Negative values are expected to be properly represented
 * with set bits from LargeBitCount and up. The SmallBitCountTable should be kept relatively small as the index into it must be replicated. The SmallBitCountTableEntryCount
 * needs to be a power of two minus one, i.e. of the form (2^N) - 1. The LargeBitCount is treated as the last entry in the table and is used when the delta between the 
 * Value and PrevValue cannot be represented with any of the bit counts in the table. 0 is a valid bit count in the first entry but is only used when the Value and PrevValue
 * are equal.
 */
IRISCORE_API void SerializeIntDelta(FNetBitStreamWriter& Writer, const int32 Value, const int32 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount);
/**
 * All the DeserializeIntDelta functions assume that the PrevValue is representable using LargeBitCount. Negative values are expected to be properly represented with set bits from LargeBitCount and up.
 * The OutValue is guaranteed to also be representable using LargeBitCount. An incorrectly replicated delta will never be able to cause on overflow on the receiving side. 
 * This is ensured via masking and sign-bit propagation.
 */
IRISCORE_API void DeserializeIntDelta(FNetBitStreamReader& Reader, int32& OutValue, const int32 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount);

/*
 * All the SerializeUintDelta functions assume that the Value and PrevValue are representable using LargeBitCount. For SmallBitCountTable information see SerializeIntDelta.
 * @see SerializeIntDelta
 */
IRISCORE_API void SerializeUintDelta(FNetBitStreamWriter& Writer, const uint32 Value, const uint32 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount);
/**
 * All the DeserializeIntDelta functions assume that the PrevValue is representable using LargeBitCount. The OutValue is guaranteed to also be representable using LargeBitCount. 
 * An incorrectly replicated delta will never be able to cause on overflow on the receiving side. This is ensured via masking.
 */
IRISCORE_API void DeserializeUintDelta(FNetBitStreamReader& Reader, uint32& OutValue, const uint32 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount);

IRISCORE_API void SerializeIntDelta(FNetBitStreamWriter& Writer, const int64 Value, const int64 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount);
IRISCORE_API void DeserializeIntDelta(FNetBitStreamReader& Reader, int64& OutValue, const int64 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount);

IRISCORE_API void SerializeUintDelta(FNetBitStreamWriter& Writer, const uint64 Value, const uint64 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount);
IRISCORE_API void DeserializeUintDelta(FNetBitStreamReader& Reader, uint64& OutValue, const uint64 PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount);

/**
 * Converts a unit float to a quantized representation using a specified bit count.
 * @param Value A float in range [-1.0f, 1.0f]. No clamping is performed.
 * @param BitCount The desired bit count of the quantized value.
 * @return The quantized representation. Unused top bits will be zero.
 * @note For bit counts > 23 the return value will use 25 bits, otherwise exactly as many bits as requested.
 */
IRISCORE_API uint32 QuantizeSignedUnitFloat(float Value, uint32 BitCount);

/**
 * Dequantizes the value returned from QuantizeSignedUnitFloat or DeserializeSignedUnitFloat called with the same BitCount.
 * @param Value The quantized value, as returned by QuantizeSignedUnitFloat.
 * @return The float in range [-1.0f, 1.0f] corresponding to the quantized value.
 * @see QuantizeSignedUnitFloat
 */
IRISCORE_API float DequantizeSignedUnitFloat(uint32 Value, uint32 BitCount);

/**
 * Serializes a signed unit float quantized with QuantizeSignedUnitFloat with the same BitCount.
 * @see QuantizeSignedUnitFloat
 */
void SerializeSignedUnitFloat(FNetBitStreamWriter& Writer, uint32 Value, uint32 BitCount);

/**
 * Deserializes a signed unit float that was serialized using SerializeSignedUnitFloat with the same BitCount.
 * @see SerializeSignedUnitFloat.
 * @see DequantizeSignedUnitFloat
 */
uint32 DeserializeSignedUnitFloat(FNetBitStreamReader& Reader, uint32 BitCount);


/**
 * Serializes a single bit indicating whether the value is equal to another value or not.
 * @return true if the Value was equal to OtherValue, false if not.
 * @note The actual value is never written.
 * @see DeserializeSameValue
 */
template<typename T>
inline bool SerializeSameValue(FNetBitStreamWriter& Writer, const T Value, const T OtherValue)
{
	const uint32 IsSameValue = (Value == OtherValue ? 1U : 0U);
	Writer.WriteBits(IsSameValue, 1U);
	return IsSameValue & 1;
}

/**
 * Deserializes what was serialized using SerializeSameValue.
 * @param Reader The bitstream to read from.
 * @param OutValue Will be set to OtherValue if the read bit was 1, unset otherwise.
 * @param OtherValue The value to set to OutValue if the read bit was 1.
 * @return true if OutValue was set to OtherValue, false if not.
 * @see SerializeSameValue
 */
template<typename T>
inline bool DeserializeSameValue(FNetBitStreamReader& Reader, T& OutValue, const T OtherValue)
{
	const uint32 IsSameValue = Reader.ReadBits(1U);
	if (IsSameValue)
	{
		OutValue = OtherValue;
		return true;
	}

	return false;
}

template<typename T, typename TEnableIf<TIsSigned<T>::Value && sizeof(T) <= 8U, int32>::Type U = -1>
inline void SerializeIntDelta(FNetBitStreamWriter& Writer, const T Value, const T PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	// Careful casting to properly represent negative numbers in the larger type as we want the delta between small negative numbers and small positive numbers to be small.
	using SignedType = std::conditional_t<sizeof(T) == 8, int64, int32>;
	return SerializeIntDelta(Writer, SignedType(Value), SignedType(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
}

template<typename T, typename TEnableIf<TIsSigned<T>::Value && sizeof(T) <= 8U, int32>::Type U = -1>
inline void DeserializeIntDelta(FNetBitStreamReader& Reader, T& OutValue, const T PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	using SignedType = std::conditional_t<sizeof(T) == 8, int64, int32>;

	SignedType Value;
	DeserializeIntDelta(Reader, Value, SignedType(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
	OutValue = T(Value);
}

template<typename T, typename TEnableIf<!TIsSigned<T>::Value && TIsIntegral<T>::Value && sizeof(T) <= 8U, uint32>::Type U = 1U>
inline void SerializeUintDelta(FNetBitStreamWriter& Writer, const T Value, const T PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	// Careful casting to allow the delta between small positive numbers and large positive numbers to be small.
	using UnsignedType = std::conditional_t<sizeof(T) == 8, uint64, uint32>;
	return SerializeUintDelta(Writer, UnsignedType(Value), UnsignedType(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
}

template<typename T, typename TEnableIf<!TIsSigned<T>::Value && TIsIntegral<T>::Value && sizeof(T) <= 8U, uint32>::Type U = 1U>
inline void DeserializeUintDelta(FNetBitStreamReader& Reader, T& OutValue, const T PrevValue, const uint8* SmallBitCountTable, const uint32 SmallBitCountTableEntryCount, uint8 LargeBitCount)
{
	using UnsignedType = std::conditional_t<sizeof(T) == 8, uint64, uint32>;

	UnsignedType Value;
	DeserializeUintDelta(Reader, Value, UnsignedType(PrevValue), SmallBitCountTable, SmallBitCountTableEntryCount, LargeBitCount);
	OutValue = T(Value);
}

inline void SerializeSignedUnitFloat(FNetBitStreamWriter& Writer, uint32 Value, uint32 BitCount)
{
	if (BitCount < 24U)
	{
		Writer.WriteBits(Value, BitCount);
	}
	else
	{
		const uint32 SignAndIsNotZero = Value >> 23U;
		Writer.WriteBits(SignAndIsNotZero, 2U);
		if (SignAndIsNotZero & 1U)
		{
			Writer.WriteBits(Value, 23U);
		}
	}
}

inline uint32 DeserializeSignedUnitFloat(FNetBitStreamReader& Reader, uint32 BitCount)
{
	if (BitCount < 24U)
	{
		uint32 Value = Reader.ReadBits(BitCount);
		// Sign-extend
		const uint32 Mask = 1U << (BitCount - 1U);
		Value = (Value ^ Mask) - Mask;
		return Value;
	}
	else
	{
		const uint32 SignAndIsNotZero = Reader.ReadBits(2U);
		uint32 Value = SignAndIsNotZero << 23U;
		if (SignAndIsNotZero & 1U)
		{
			Value |= Reader.ReadBits(23U);
		}
		return Value;
	}
}

}
