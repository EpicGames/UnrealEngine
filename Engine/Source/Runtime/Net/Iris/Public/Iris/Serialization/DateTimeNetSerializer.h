// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "NetSerializerConfig.h"
#include "UObject/ObjectMacros.h"
#include "DateTimeNetSerializer.generated.h"

USTRUCT()
struct FDateTimeNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FDateTimeNetSerializer, IRISCORE_API);

}
