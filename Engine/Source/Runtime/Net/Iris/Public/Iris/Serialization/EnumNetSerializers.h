// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "NetSerializerConfig.h"
#include "UObject/ObjectMacros.h"
#include "EnumNetSerializers.generated.h"

USTRUCT()
struct FEnumInt8NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int8 LowerBound = 0;
	UPROPERTY()
	int8 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
	const UEnum* Enum = nullptr;
};

USTRUCT()
struct FEnumInt16NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int16 LowerBound = 0;
	UPROPERTY()
	int16 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
	const UEnum* Enum = nullptr;
};

USTRUCT()
struct FEnumInt32NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 LowerBound = 0;
	UPROPERTY()
	int32 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
	const UEnum* Enum = nullptr;
};

USTRUCT()
struct FEnumInt64NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int64 LowerBound = 0;
	UPROPERTY()
	int64 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
	const UEnum* Enum = nullptr;
};

USTRUCT()
struct FEnumUint8NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint8 LowerBound = 0;
	UPROPERTY()
	uint8 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
	const UEnum* Enum = nullptr;
};

USTRUCT()
struct FEnumUint16NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint16 LowerBound = 0;
	UPROPERTY()
	uint16 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
	const UEnum* Enum = nullptr;
};

USTRUCT()
struct FEnumUint32NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint32 LowerBound = 0;
	UPROPERTY()
	uint32 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
	const UEnum* Enum = nullptr;
};

USTRUCT()
struct FEnumUint64NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint64 LowerBound = 0;
	UPROPERTY()
	uint64 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
	const UEnum* Enum = nullptr;
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FEnumInt8NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FEnumInt16NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FEnumInt32NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FEnumInt64NetSerializer, IRISCORE_API);

UE_NET_DECLARE_SERIALIZER(FEnumUint8NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FEnumUint16NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FEnumUint32NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FEnumUint64NetSerializer, IRISCORE_API);

}
