// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "FloatNetSerializers.generated.h"

USTRUCT()
struct FFloatNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FDoubleNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FFloatNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FDoubleNetSerializer, IRISCORE_API);

}
