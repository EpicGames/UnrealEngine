// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Containers/LruCache.h"
#include "UObject/ObjectMacros.h"
#include "Misc/TransactionallySafeCriticalSection.h"

#include "InstancedStructNetSerializer.generated.h"

class UScriptStruct;

namespace UE::Net::Private
{

class FInstancedStructDescriptorCache
{
public:
	FInstancedStructDescriptorCache();
	~FInstancedStructDescriptorCache();

	// Name for debugging purposes
	void SetDebugName(const FString& DebugName);

	// Set max cached descriptor count. The most recently used descriptors will be kept. MaxCount <= 0 means no limit which is the default.
	void SetMaxCachedDescriptorCount(int32 MaxCount);

	// 
	void AddSupportedTypes(const TConstArrayView<TSoftObjectPtr<UScriptStruct>>& SupportedTypes);

	bool IsSupportedType(const UScriptStruct* Struct) const;

	// Find descriptor for struct with fully qualified name.
	TRefCountPtr<const FReplicationStateDescriptor> FindDescriptor(FName StructPath);

	// Find descriptor for struct.
	TRefCountPtr<const FReplicationStateDescriptor> FindDescriptor(const UScriptStruct* Struct);

	// Find or create descriptor for struct with fully qualified name.
	TRefCountPtr<const FReplicationStateDescriptor> FindOrAddDescriptor(FName StructPath);

	// Find or create descriptor for struct.
	TRefCountPtr<const FReplicationStateDescriptor> FindOrAddDescriptor(const UScriptStruct* Struct);

private:
	TRefCountPtr<const FReplicationStateDescriptor> CreateAndCacheDescriptor(const UScriptStruct* Struct, FName StructPath);

	FTransactionallySafeCriticalSection Mutex;
	// LRU cache for descriptors for limited descriptor counts.
	TLruCache<FName, TRefCountPtr<const FReplicationStateDescriptor>> DescriptorLruCache;
	// Map struct name -> FReplicationStateDescriptor for unlimited descriptor counts. 
	TMap<FName, TRefCountPtr<const FReplicationStateDescriptor>> DescriptorMap;
	// Supported types. An empty set indicates all UScriptStructs are supported.
	TSet<TSoftObjectPtr<UScriptStruct>> SupportedTypes;
	FString DebugName;
	int32 MaxCachedDescriptorCount = 0;
};

}

USTRUCT()
struct FInstancedStructNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

	FInstancedStructNetSerializerConfig();
	~FInstancedStructNetSerializerConfig();

	FInstancedStructNetSerializerConfig(const FInstancedStructNetSerializerConfig&) = delete;
	FInstancedStructNetSerializerConfig& operator=(const FInstancedStructNetSerializerConfig&) = delete;

	// The property is for serialization support. We store the supported types differently in the descriptor cache.
	UPROPERTY()
	TArray<TSoftObjectPtr<UScriptStruct>> SupportedTypes;

	UE::Net::Private::FInstancedStructDescriptorCache DescriptorCache;
};

template<>
struct TStructOpsTypeTraits<FInstancedStructNetSerializerConfig> : public TStructOpsTypeTraitsBase2<FInstancedStructNetSerializerConfig>
{
	enum
	{
		WithCopy = false
	};
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FInstancedStructNetSerializer, IRISCORE_API);

}
