// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "IntNetSerializers.generated.h"

// Integer serializers
USTRUCT()
struct FIntNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	explicit inline FIntNetSerializerConfig(uint8 InBitCount) : BitCount(InBitCount) {}
	inline FIntNetSerializerConfig() : BitCount(0) {}

	UPROPERTY()
	uint8 BitCount;
};

typedef FIntNetSerializerConfig FInt64NetSerializerConfig;
typedef FIntNetSerializerConfig FInt32NetSerializerConfig;
typedef FIntNetSerializerConfig FInt16NetSerializerConfig;
typedef FIntNetSerializerConfig FInt8NetSerializerConfig;

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FInt64NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FInt32NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FInt16NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FInt8NetSerializer, IRISCORE_API);

}