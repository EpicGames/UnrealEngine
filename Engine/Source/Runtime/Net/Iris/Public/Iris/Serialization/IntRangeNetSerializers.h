// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "IntRangeNetSerializers.generated.h"

// Integer range serializers
USTRUCT()
struct FInt8RangeNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int8 LowerBound = 0;
	UPROPERTY()
	int8 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
};

USTRUCT()
struct FInt16RangeNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int16 LowerBound = 0;
	UPROPERTY()
	int16 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
};

USTRUCT()
struct FInt32RangeNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int32 LowerBound = 0;
	UPROPERTY()
	int32 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
};

USTRUCT()
struct FInt64RangeNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	int64 LowerBound = 0;
	UPROPERTY()
	int64 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FInt8RangeNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FInt16RangeNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FInt32RangeNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FInt64RangeNetSerializer, IRISCORE_API);

}
