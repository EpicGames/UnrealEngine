// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/CoreNet.h"
#include "Net/Core/NetToken/NetToken.h"
#include "IrisObjectReferencePackageMap.generated.h"

// Forward declarations
class FNetworkGUID;
class UIrisObjectReferencePackageMap;

namespace UE::Net
{
	class FIrisObjectReferencePackageMapWriteScope;
	class FIrisObjectReferencePackageMapReadScope;

	// In order to properly capture exported data when calling in to old style NetSerialize methods
	// we need to capture and inject certain types.
	struct FIrisPackageMapExports
	{
		typedef TArray<TObjectPtr<UObject>, TInlineAllocator<4>> FObjectReferenceArray;
		typedef TArray<FName, TInlineAllocator<4>> FNameArray;

		bool IsEmpty() const
		{
			return References.IsEmpty() && Names.IsEmpty();
		}

		void Reset()
		{
			References.Reset();
			Names.Reset();
		}

		FObjectReferenceArray References;
		FNameArray Names;
	};

	// Scope that calls InitForRead on target PackageMap and invalidates set PackageMapExports on scope exit.
	class FIrisObjectReferencePackageMapReadScope
	{
	public:
		IRISCORE_API FIrisObjectReferencePackageMapReadScope(UIrisObjectReferencePackageMap* PackageMap, const UE::Net::FIrisPackageMapExports* PackageMapExports, const UE::Net::FNetTokenResolveContext* NetTokenResolveContext);
		IRISCORE_API ~FIrisObjectReferencePackageMapReadScope();
		UIrisObjectReferencePackageMap* GetPackageMap() { return PackageMap; }

	private:
		UIrisObjectReferencePackageMap* PackageMap = nullptr;
	};

	// Scope that calls InitForWrite on target PackageMap and invalidates set PackageMapExports on scope exit.
	class FIrisObjectReferencePackageMapWriteScope
	{
	public:
		IRISCORE_API FIrisObjectReferencePackageMapWriteScope(UIrisObjectReferencePackageMap* PackageMap, UE::Net::FIrisPackageMapExports* PackageMapExports);
		IRISCORE_API ~FIrisObjectReferencePackageMapWriteScope();
		UIrisObjectReferencePackageMap* GetPackageMap() { return PackageMap; }

	private:
		UIrisObjectReferencePackageMap* PackageMap = nullptr;
	};
}

/**
 * Custom packagemap implementation used to be able to capture exports such as UObject* references, names and NetTokens from external serialization.
 * Exports written when using this packagemap will be captured in an array and serialized as an index.
 * When reading using this packagemap exports will be read as an index and resolved by picking the corresponding entry from the provided array containing the data associated with the export.
 */
UCLASS(transient, MinimalAPI)
class UIrisObjectReferencePackageMap : public UPackageMap
{
public:
	GENERATED_BODY()

	// We override SerializeObject in order to be able to capture object references
	virtual bool SerializeObject(FArchive& Ar, UClass* InClass, UObject*& Obj, FNetworkGUID* OutNetGUID) override;

	// Override SerializeName in order to be able to capture name and serialize them with iris instead.
	virtual bool SerializeName(FArchive& Ar, FName& InName);

	virtual const UE::Net::FNetTokenResolveContext* GetNetTokenResolveContext() const  override { return &NetTokenResolveContext; }

	// Init for read, we need to set the exports from which we are going to read our data.
	IRISCORE_API void InitForRead(const UE::Net::FIrisPackageMapExports* PackageMapExports, const UE::Net::FNetTokenResolveContext& InNetTokenResolveContext);

	// Init for write, all captured exports will be serialized as in index and added to the PackageMapExports for later export using iris.
	IRISCORE_API void InitForWrite(UE::Net::FIrisPackageMapExports* PackageMapExports);

private:
	friend UE::Net::FIrisObjectReferencePackageMapReadScope;
	friend UE::Net::FIrisObjectReferencePackageMapWriteScope;

	UE::Net::FIrisPackageMapExports* PackageMapExportsForReading = nullptr;
	UE::Net::FIrisPackageMapExports* PackageMapExportsForWriting = nullptr;
	UE::Net::FNetTokenResolveContext NetTokenResolveContext;
};
