// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Note: This util is intended for internal use and should not be included or used outside of NetSerializers having to deal with calling into existing NetSerialzie functions.
 
#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "Iris/Core/NetObjectReference.h"
#include "Net/Core/NetToken/NetToken.h"
#include "Iris/Serialization/NetSerializerArrayStorage.h"
#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Iris/Serialization/StringNetSerializers.h"

namespace UE::Net
{
struct FIrisPackageMapExports;
class FNetReferenceCollector;

struct FIrisPackageMapExportsQuantizedType
{
	static constexpr uint32 MaxInlinedObjectRefs = 4;
	typedef FNetSerializerArrayStorage<FObjectNetSerializerQuantizedReferenceStorage, AllocationPolicies::TInlinedElementAllocationPolicy<MaxInlinedObjectRefs>> FObjectReferenceStorage;

	struct FQuantizedName
	{
		alignas(8) uint8 Name[GetNameNetSerializerSafeQuantizedSize()];
	};

	static constexpr uint32 MaxInlinedNames = 4;
	typedef FNetSerializerArrayStorage<FQuantizedName, AllocationPolicies::TInlinedElementAllocationPolicy<MaxInlinedNames>> FNamesStorage;

	static constexpr uint32 MaxInlinedNetTokens = 4;
	typedef FNetSerializerArrayStorage<FNetToken, AllocationPolicies::TInlinedElementAllocationPolicy<MaxInlinedNetTokens>> FNetTokenStorage;

	FObjectReferenceStorage ObjectReferenceStorage;
	FNamesStorage NameStorage;
	FNetTokenStorage NetTokenStorage;
};

}

template <> struct TIsPODType<UE::Net::FIrisPackageMapExportsQuantizedType> { enum { Value = true }; };

namespace UE::Net
{

// Util to facilitate capture and export of supported types when calling into old NetSerialize() methods.
struct FIrisPackageMapExportsUtil
{
	typedef FIrisPackageMapExportsQuantizedType QuantizedType;

	// We need some sort of limit here.
	static constexpr uint32 MaxExports = 65536U;

	// Matches NetSerializer functions
	IRISCORE_API static void Serialize(FNetSerializationContext& Context, const QuantizedType& Value);
	IRISCORE_API static void Deserialize(FNetSerializationContext& Context, QuantizedType& Value);
	IRISCORE_API static void Quantize(FNetSerializationContext& Context, const UE::Net::FIrisPackageMapExports& PackageMapExports, TArrayView<const UE::Net::FNetToken> NetTokensPendingExport, QuantizedType& Target);
	IRISCORE_API static void Dequantize(FNetSerializationContext& Context, const QuantizedType& Source, UE::Net::FIrisPackageMapExports& PackageMapExports);
	IRISCORE_API static bool IsEqual(FNetSerializationContext& Context, const QuantizedType& Value0, const QuantizedType& Value1);
	IRISCORE_API static void CloneDynamicState(FNetSerializationContext& Context, QuantizedType& Target, const QuantizedType& Source);
	IRISCORE_API static void FreeDynamicState(FNetSerializationContext& Context, QuantizedType& Value);
	IRISCORE_API static void FreeDynamicState(QuantizedType& Value);
	IRISCORE_API static void CollectNetReferences(FNetSerializationContext& Context, const QuantizedType& Value, const FNetSerializerChangeMaskParam& ChangeMaskInfo, FNetReferenceCollector& Collector);
	IRISCORE_API static bool Validate(FNetSerializationContext&, const QuantizedType& Value);

private:

	static const FNetSerializer* ObjectNetSerializer;
	static const FNetSerializer* NameNetSerializer;
};

}
