// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Net
{

class FNetBitStreamReader
{
public:
	IRISCORE_API FNetBitStreamReader();
	IRISCORE_API ~FNetBitStreamReader();

	/**
	 * InitBits must be called before reading from the stream.
	 * @param Buffer The buffer must be at least 4-byte aligned.
	 * @param BitCount The number of bits that is allowed to be read from the buffer.
	 */
	IRISCORE_API void InitBits(const void* Buffer, uint32 BitCount);

	/**
	 * Reads BitCount bits that are stored in the least significant bits in the return value. Other bits
	 * will be set to zero. If the BitCount exceeds the remaining space the function will return zero 
	 * and the stream will be marked as overflown.
	 */
	IRISCORE_API uint32 ReadBits(uint32 BitCount);

	/**
	 * Reads a bool from the stream and returns the value,
	 * A failed read will always return false and stream will be marked as overflown
	 */
	bool ReadBool() { return ReadBits(1) & 1U; }

	/**
	 * Reads BitCount bits and stores them in Dst, starting from bit offset 0. The bits will be stored
	 * as they are stored internally in this class, i.e. bits will be written from lower to higher
	 * memory addresses.
	 * If the BitCount exceeds the remaining space no bits will be written to Dst and the stream will be
	 * marked as overflown. It's up to the user to check for overflow.
	 */
	IRISCORE_API void ReadBitStream(uint32* Dst, uint32 BitCount);

	/**
	 * Seek to a specific position from the start of the stream or substream. If the stream is overflown and you seek back to a position
	 * where you can still read bits the stream will no longer be considered overflown.
	 */
	IRISCORE_API void Seek(uint32 BitPosition);

	/** Returns the the current byte position. */
	inline uint32 GetPosBytes() const { return (BufferBitPosition - BufferBitStartOffset + 7) >> 3U; }

	/** Returns the current bit position */
	inline uint32 GetPosBits() const { return BufferBitPosition - BufferBitStartOffset; }

	/** Returns the absolute bit position */
	inline uint32 GetAbsolutePosBits() const { return BufferBitPosition; }

	/** Returns the number of bits that can be read before overflowing. */
	inline uint32 GetBitsLeft() const { return (OverflowBitCount ? 0U : (BufferBitCapacity - BufferBitPosition)); }

	/** Force an overflow. */
	IRISCORE_API void DoOverflow();
	
	/** Returns whether the stream is overflown or not. */
	inline bool IsOverflown() const { return OverflowBitCount != 0; }

	/** 
	 * Creates a substream at the current bit position. The substream must be committed or discarded. Only one active substream at a time is allowed,
	 * but a substream can have an active substream as well. Once the substream has been commited or discarded a new substream may be created. No
	 * reads may be performed on this stream until the substream has been committed or discarded.  
	 *
	 * The returned FNetBitStreamReader will have similar behavior to a newly constructed regular FNetBitStreamWriter. 
	 * 
	 * @param MaxBitCount The maximum allowed bits that may be read. The value will be clamped to the number of bits left in this stream/substream. If it's a requirement a specific size is supported you can verify it with GetBitsLeft().
	 * @return A FNetBitStreamReader.
	 */
	IRISCORE_API FNetBitStreamReader CreateSubstream(uint32 MaxBitCount = ~0U);

	/**
	 * Commits a substream to this stream. Substreams that are overflown or do not belong to this stream will be ignored. 
	 * If the substream is valid then this stream's bit position will be updated.
	 */
	IRISCORE_API void CommitSubstream(FNetBitStreamReader& Substream);

	/** Discards a substream of this stream. This stream's bit position will remain intact. */
	IRISCORE_API void DiscardSubstream(FNetBitStreamReader& Substream);

private:
	const uint32* Buffer;
	// The BufferBitCapacity is an absolute bit position indicating the bit after the last valid bit position to read.
	uint32 BufferBitCapacity;
	// For substreams this indicate the absolute bit position in the buffer where it will start reading
	uint32 BufferBitStartOffset;
	uint32 BufferBitPosition;
	uint32 PendingWord;
	uint32 OverflowBitCount;

	uint32 bHasSubstream : 1;
	uint32 bIsSubstream : 1;
	uint32 bIsInvalid : 1;
};

}

// Always report the actual bitstream position, even on overflow. This normally allows for better comparisons between sending and receiving side when bitstream errors occur.
inline uint32 GetBitStreamPositionForNetTrace(const UE::Net::FNetBitStreamReader& Stream) { return Stream.GetAbsolutePosBits(); }
