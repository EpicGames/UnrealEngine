// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringView.h"
#include "Misc/AssertionMacros.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/IrisConfig.h"

namespace UE::Net
{

/** Simple helper to a temporary write at a specific offset in the stream and return to the original position when exiting the scope. */
class FNetBitStreamWriteScope
{
public:
	FNetBitStreamWriteScope(FNetBitStreamWriter& InWriter, uint32 WritePos)
	: Writer(InWriter)
	, OriginalPos(InWriter.GetPosBits())
	{
#if UE_NETBITSTREAMWRITER_VALIDATE
		check(!Writer.IsOverflown());
#endif

		Writer.Seek(WritePos);
	}

	~FNetBitStreamWriteScope()
	{
#if UE_NETBITSTREAMWRITER_VALIDATE
		check(Writer.GetPosBits() <= OriginalPos);
#endif
		Writer.Seek(OriginalPos);
	}

private:
	FNetBitStreamWriter& Writer;
	const uint32 OriginalPos;
};


/**
 * RollbackScope, if the provided BitWriter is in an invalid state when exiting the scope, the BitWriter will be restored to the state it had when entering the FNetBitStreamRollbackScope
 */
class FNetBitStreamRollbackScope
{
public:
	explicit FNetBitStreamRollbackScope(FNetBitStreamWriter& InWriter)
	: Writer(InWriter)
	, StartPos(InWriter.GetPosBits())
	{
	}

	~FNetBitStreamRollbackScope()
	{
		if (Writer.IsOverflown())
		{
			Writer.Seek(StartPos);
		}
	}

	void Rollback()
	{
		Writer.Seek(StartPos);
	}

	uint32 GetStartPos() const { return StartPos; }

private:
	FNetBitStreamWriter& Writer;
	const uint32 StartPos;
};

/**
 * Write a full uint64 to the provided FNetBitStreamWriter
 */
inline void WriteUint64(FNetBitStreamWriter* Writer, uint64 Value)
{
	Writer->WriteBits((uint32)Value, 32);
	Writer->WriteBits((uint32)(Value >> 32), 32);
}

/**
 * Read a full uint64 to the provided FNetBitStreamReader
 */
inline uint64 ReadUint64(FNetBitStreamReader* Reader)
{
	uint64 Value = Reader->ReadBits(32);
	uint64 HighValue = Reader->ReadBits(32);
	Value |= HighValue << 32;

	return Value;
}

/**
 * Write a full uint64 to the provided FNetBitStreamWriter
 */
inline void WriteInt64(FNetBitStreamWriter* Writer, int64 Value)
{
	WriteUint64(Writer, static_cast<uint64>(Value));
}

/**
 * Read a full uint64 to the provided FNetBitStreamReader
 */
inline int64 ReadInt64(FNetBitStreamReader* Reader)
{
	uint64 Value = ReadUint64(Reader);
	return static_cast<int64>(Value);
}

/** Write a uint64 using as few bytes as possible */
IRISCORE_API void WritePackedUint64(FNetBitStreamWriter* Writer, uint64 Value);

/** Read a uint64 that was written using WritePackedUint64 */
IRISCORE_API uint64 ReadPackedUint64(FNetBitStreamReader* Reader);

/** Write an int64 using as few bytes as possible */
IRISCORE_API void WritePackedInt64(FNetBitStreamWriter* Writer, int64 Value);

/** Read an int64 that was written using WritePackedInt64 */
IRISCORE_API int64 ReadPackedInt64(FNetBitStreamReader* Reader);

/** Write a uint32 using as few bytes as possible */
IRISCORE_API void WritePackedUint32(FNetBitStreamWriter* Writer, uint32 Value);

/** Read a uint32 that was written using WritePackedUint32 */
IRISCORE_API uint32 ReadPackedUint32(FNetBitStreamReader* Reader);

/** Write an int32 using as few bytes as possible */
IRISCORE_API void WritePackedInt32(FNetBitStreamWriter* Writer, int32 Value);

/** Read an int32 that was written using WritePackedInt32 */
IRISCORE_API int32 ReadPackedInt32(FNetBitStreamReader* Reader);

/** Write a uint16 using as few bits as possible */
IRISCORE_API void WritePackedUint16(FNetBitStreamWriter* Writer, uint16 Value);

/** Read a uint16 that was written using WritePackedUint16 */
IRISCORE_API uint16 ReadPackedUint16(FNetBitStreamReader* Reader);

/** Read an UTF8-like encoded string into a FString. It must've been written by WriteString(). */
IRISCORE_API void ReadString(FNetBitStreamReader* Reader, FString& OutString);

/** Write an FString as a UTF8-like encoded stream. */
IRISCORE_API void WriteString(FNetBitStreamWriter* Writer, const FString& String);

/** Write an FStringView as a UTF8-like encoded stream. It can be read by ReadString. */
IRISCORE_API void WriteString(FNetBitStreamWriter* Writer, FStringView String);

/** Write full Vector */
IRISCORE_API void WriteVector(FNetBitStreamWriter* Writer, const FVector& Vector);

/** Read full Vector */
IRISCORE_API void ReadVector(FNetBitStreamReader* Reader, FVector& Vector);

/** Write vector using default value compression */
IRISCORE_API void WriteVector(FNetBitStreamWriter* Writer, const FVector& Vector, const FVector& DefaultValue, float Epsilon);

/** Read default value compressed vector */
IRISCORE_API void ReadVector(FNetBitStreamReader* Reader, FVector& OutVector, const FVector& DefaultValue);

/** Write full Rotator */
IRISCORE_API void WriteRotator(FNetBitStreamWriter* Writer, const FRotator& Vector);

/** Read full Rotator */
IRISCORE_API void ReadRotator(FNetBitStreamReader* Reader, FRotator& Vector);

/** 
 * Write rotator using default value compression
 * i.e. Only write the Vector if it differs from the provided DefaultValue, if the the diff is within the provided epsilon a single bit will be written
 * Note: The provided DefaultValue must match be the same on Server and Client;
*/
IRISCORE_API void WriteRotator(FNetBitStreamWriter* Writer, const FRotator& Rotator, const FRotator& DefaultValue, float Epsilon);

/** Read default value compressed rotator, See WriteRotator.*/
IRISCORE_API void ReadRotator(FNetBitStreamReader* Reader, FRotator& OutRotator, const FRotator& DefaultValue);

enum class ESparseBitArraySerializationHint : uint8
{
	None,
	ContainsMostlyOnes,
};
/** 
 * Write sparse BitArray which is expected to contain mostly 0`s, if the array contains mostly 1`s passing the hint ESparseBitArraySerializationHint::ContainsMostlyOnes can be provided to flip the data before writing 
 * Note: BitCount and hint is not Written so the write must be matched by a corresponding Read
 */
IRISCORE_API void WriteSparseBitArray(FNetBitStreamWriter* Writer, const uint32* Data, uint32 BitCount, ESparseBitArraySerializationHint Hint = ESparseBitArraySerializationHint::None);

/** 
 * Read sparse BitArray which is expected to contain mostly 0`s, if the array contains mostly 1`s passing the hint ESparseBitArraySerializationHint::ContainsMostlyOnes can be provided to flip the read data
 */
IRISCORE_API void ReadSparseBitArray(FNetBitStreamReader* Reader,  uint32* OutData, uint32 BitCount, ESparseBitArraySerializationHint Hint = ESparseBitArraySerializationHint::None);

/** 
 * Write sparse BitArray which is expected to contain mostly 0`s, if the array contains mostly 1`s passing the hint ESparseBitArraySerializationHint::ContainsMostlyOnes can be provided to flip the data before writing 
 * Note: BitCount and hint is not Written so the write must be matched by a corresponding Read
 */
IRISCORE_API void WriteSparseBitArrayDelta(FNetBitStreamWriter* Writer, const uint32* Data, const uint32* OldData, uint32 BitCount);

/** 
 * Read sparse BitArray which is expected to contain mostly 0`s, if the array contains mostly 1`s passing the hint ESparseBitArraySerializationHint::ContainsMostlyOnes can be provided to flip the read data
 */
IRISCORE_API void ReadSparseBitArrayDelta(FNetBitStreamReader* Reader,  uint32* OutData, const uint32* OldData, uint32 BitCount);

/**
 * Write sentinelBits from predefined sentinel value
 */
IRISCORE_API void WriteSentinelBits(FNetBitStreamWriter* Writer, uint32 BitCount = 32U);

/**
 * Read and verify sentinelBits against predefined sentinel value, will return false if we have a bitstream overflow or the read sentinel bits mismatch
 */

IRISCORE_API bool ReadAndVerifySentinelBits(FNetBitStreamReader* Reader, const TCHAR* ErrorString = TEXT("Sentinel"), uint32 BitCount = 32U);

/** 
* Write bytes
* Note: The purpose of this method is for cases when we cannot guarantee that the buffer is word aligned. If buffer is known to be word aligned, it is faster to use NetBitStreamReader::WriteBitStream directly.
*       Does not write the size.
*/
IRISCORE_API void WriteBytes(FNetBitStreamWriter* Writer, const uint8* Src, uint32 BytesToWrite);

/** 
 * Read bytes into preallocated buffer
 * Note: The purpose of this method is for cases when we cannot guarantee that the buffer is word aligned. If buffer is known to be word aligned, it is faster to use NetBitStreamReader::ReadBitStream directly.
 */
IRISCORE_API void ReadBytes(FNetBitStreamReader* Reader, uint8* Destination, uint32 BytesToRead);

/** Send 1 bit if the vector equals the DefaultValue, otherwise write a compressed vector using Quantize10 */
IRISCORE_API void WriteConditionallyQuantizedVector(FNetBitStreamWriter* Writer, const FVector& Vector, const FVector& DefaultValue, bool bQuantize);

/** Read a vector serialized using WriteConditionallyQuantizedVector. DefaultValue must be identical in both Read and Write calls. */
IRISCORE_API void ReadConditionallyQuantizedVector(FNetBitStreamReader* Reader, FVector& OutVector, const FVector& DefaultValue);

} // end namespace UE::Net
