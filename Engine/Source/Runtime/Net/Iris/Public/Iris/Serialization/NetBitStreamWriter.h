// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Net
{

class FNetBitStreamWriter
{
public:
	IRISCORE_API FNetBitStreamWriter();
	IRISCORE_API ~FNetBitStreamWriter();

	/**
	 * InitBytes must be called before writing to the stream. 
	 * @param Buffer The buffer must be at least 4-byte aligned.
	 * @param ByteCount The number of bytes that is allowed to be written. Must be a multiple of 4.
	 */
	IRISCORE_API void InitBytes(void* Buffer, uint32 ByteCount);

	/**
	 * Writes the BitCount least significant bits from Value. There's no validation of the Value, meaning
	 * it is allowed to contain garbage in the bits that are not going to be written to the buffer.
	 * If the BitCount exceeds the remaining space no bits will be written and the stream will be 
	 * marked as overflown.
	 */
	IRISCORE_API void WriteBits(uint32 Value, uint32 BitCount);

	/**
	 * Writes a bool to the stream and returns the value of the bool
	 */
	inline bool WriteBool(bool Value);

	/**
	 * Writes BitCount bits from Src starting at SrcBitOffset. Assumes that Src was written to via 
	 * this class or that bits were written in order from lowest to highest memory address. 
	 * If the BitCount exceeds the remaining space no bits will be written and the stream will be
	 * marked as overflown.
	 */
	IRISCORE_API void WriteBitStream(const uint32* Src, uint32 SrcBitOffset, uint32 BitCount);

	/**
	 * Commits pending writes to the buffer. Before this call, or the destruction of this instance, the buffer 
	 * may not be up to date. Typically called after all WriteBits() calls have been made.
	 */
	IRISCORE_API void CommitWrites();

	/** 
	 * Seek to a specific BitPosition. If the stream is overflown and you seek back to a position
	 * where you can still write bits the stream will no longer be considered overflown.
	 */
	IRISCORE_API void Seek(uint32 BitPosition);

	/** Returns the the current byte position. */
	inline uint32 GetPosBytes() const { return (BufferBitPosition - BufferBitStartOffset + 7) >> 3U; }

	/** Returns the current bit position */
	inline uint32 GetPosBits() const { return BufferBitPosition - BufferBitStartOffset; }

	/** Returns the absolute bit position */
	inline uint32 GetAbsolutePosBits() const { return BufferBitPosition; }

	/** Returns the number of bits that can be written before overflowing. */
	inline uint32 GetBitsLeft() const { return (OverflowBitCount ? 0U : (BufferBitCapacity - BufferBitPosition)); }

	/** Force an overflow. */
	IRISCORE_API void DoOverflow();

	/** Returns whether the stream is overflown or not. */
	inline bool IsOverflown() const { return OverflowBitCount != 0; }

	/** 
	 * Creates a substream at the current bit position. The substream must be committed or discarded. Only one active substream at a time is allowed,
	 * but a substream can have an active substream as well. Once the substream has been commited or discarded a new substream may be created. No
	 * writes may be performed to this stream until the substream has been committed or discarded. Any write to a substream that occurs before overflow
	 * can modify the stream buffer contents regardless of whether the stream is committed or discarded. 
	 *
	 * The returned BitStreamWriter will have similar behavior to a newly constructed regular FNetBitStreamWriter. 
	 * 
	 * @param MaxBitCount The maximum allowed bits that may be written. The value will be clamped to the number of bits left in this stream/substream.
	 * @return A BitStreamWriter that can be written to just like a regular BitStreamWriter.
	 */
	IRISCORE_API FNetBitStreamWriter CreateSubstream(uint32 MaxBitCount = ~0U);

	/**
	 * Commits a substream to this stream. Substreams that are overflown or do not belong to this stream will be ignored. 
	 * If the substream is valid then this stream's bit position will be updated. CommitWrites() is not called by this method.
	 */
	IRISCORE_API void CommitSubstream(FNetBitStreamWriter& Substream);

	/** Discards a substream of this stream. This stream's bit position will remain intact. The buffer may be modified anyway as mentioned in CreateSubStream(). */
	IRISCORE_API void DiscardSubstream(FNetBitStreamWriter& Substream);

private:
	uint32* Buffer;
	uint32 BufferBitCapacity;
	// For substreams this indicate the bit position in the buffer where it may start writing
	uint32 BufferBitStartOffset;
	uint32 BufferBitPosition;
	uint32 PendingWord;
	uint32 OverflowBitCount;

	uint32 bHasSubstream : 1;
	uint32 bIsSubstream : 1;
	uint32 bIsInvalid : 1;
};

bool FNetBitStreamWriter::WriteBool(bool Value)
{
	// This is to support a Value other than 0 or 1.
	volatile int8 ValueAsInt8 = Value;
	WriteBits(ValueAsInt8 ? 1U : 0U, 1U);
	return ValueAsInt8 ? true : false;
}

}

inline uint32 GetBitStreamPositionForNetTrace(const UE::Net::FNetBitStreamWriter& Stream)
{
	return (uint32(Stream.IsOverflown()) - 1U) & Stream.GetAbsolutePosBits();
}
