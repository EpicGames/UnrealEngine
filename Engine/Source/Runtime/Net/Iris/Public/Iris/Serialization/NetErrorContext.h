// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

#include "Iris/ReplicationSystem/NetRefHandle.h"

namespace UE::Net
{

typedef uint32 FReplicationProtocolIdentifier;

// Some common errors. Licensees should add their own headers if they want to easily share errors between files.
IRISCORE_API extern const FName GNetError_BitStreamOverflow;
IRISCORE_API extern const FName GNetError_BitStreamError;
IRISCORE_API extern const FName GNetError_ArraySizeTooLarge;
IRISCORE_API extern const FName GNetError_InvalidNetHandle;
IRISCORE_API extern const FName GNetError_BrokenNetHandle;
IRISCORE_API extern const FName GNetError_InvalidValue;
IRISCORE_API extern const FName GNetError_InternalError;

class FNetErrorContext
{
public:

	bool HasError() const;

	/** If an error has already been set calling this function again will be a no-op. */
	IRISCORE_API void SetError(const FName Error);

	FName GetError() const { return Error; }

	void SetErrorDiagnostic(FString InDiagnostic) { ErrorDiagnostic = MoveTemp(InDiagnostic); }

	const FString& GetErrorDiagnostic() const { return ErrorDiagnostic; }

	void SetObjectHandle(const FNetRefHandle& InObjectHandle) { ObjectHandle = InObjectHandle; }

	const FNetRefHandle& GetObjectHandle() const { return ObjectHandle; }

	void SetErrorProtocolId(FReplicationProtocolIdentifier InProtocolId) { ErrorProtocolId = InProtocolId; }

	FReplicationProtocolIdentifier GetErrorProtocolId() const { return ErrorProtocolId; }

private:

	FNetRefHandle ObjectHandle;
	FName Error;
	FString ErrorDiagnostic;
	FReplicationProtocolIdentifier ErrorProtocolId = 0;
};

inline bool FNetErrorContext::HasError() const
{
	return !Error.IsNone();
}

}
