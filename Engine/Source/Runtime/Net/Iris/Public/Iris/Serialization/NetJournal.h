// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Net/Core/Trace/NetDebugName.h"

class UReplicationSystem;

#ifndef UE_NET_ENABLE_READ_JOURNAL
#if (UE_BUILD_SHIPPING)
#	define UE_NET_ENABLE_READ_JOURNAL 1
#else
#	define UE_NET_ENABLE_READ_JOURNAL 1
#endif 
#endif

#if UE_NET_ENABLE_READ_JOURNAL
	/** Add entry to the read journal. The Name must be a static/permanently allocated string as the raw pointer will be stored. */
	#define UE_ADD_READ_JOURNAL_ENTRY(SerializationContext, Name) SerializationContext.AddReadJournalEntry(Name);
	#define UE_RESET_READ_JOURNAL(SerializationContext) SerializationContext.ResetReadJournal();
#else
	#define UE_ADD_READ_JOURNAL_ENTRY(...)
#define UE_RESET_READ_JOURNAL(...)
#endif

namespace UE::Net
{
// Simple journal to track last few entries of read data
class FNetJournal
{
	static constexpr uint32 JournalSize = 32U;
	static constexpr uint32 JournalMask = JournalSize - 1U;

public:
	void Reset() { NumEntries = 0U; }
	/** Add entry to the journal. Only the last 32 entries are stored. The Name must be a static/permanently allocated string as the raw pointer will be stored. */
	void AddEntry(const TCHAR* Name, uint32 BitOffset, FNetRefHandle NetRefHandle);
	FString Print(const UReplicationSystem* ReplicationSystem) const;

private:
	struct FJournalEntry
	{
		const TCHAR* Name;
		FNetRefHandle NetRefHandle;
		uint32 BitOffset;
	};
	FJournalEntry Entries[JournalSize];
	uint32 NumEntries = 0U;
};

inline void FNetJournal::AddEntry(const TCHAR* Name, uint32 BitOffset, FNetRefHandle NetRefHandle)
{ 
	Entries[NumEntries & JournalMask] = FJournalEntry({Name, NetRefHandle, BitOffset});
	++NumEntries;
}

}
