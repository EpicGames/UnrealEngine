// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/Serialization/NetSerializer.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"

namespace UE::Net
{

struct FQuantizedObjectReference;

enum class ENetReferenceCollectorTraits : uint32
{
	None								= 0U,
	IncludeInvalidReferences			= 1U,
	OnlyCollectReferencesThatCanBeExported = IncludeInvalidReferences << 1U,
};
ENUM_CLASS_FLAGS(ENetReferenceCollectorTraits);

class FNetReferenceCollector
{
public:
	struct FReferenceInfo
	{
		FNetReferenceInfo Info;
		FNetObjectReference Reference;
		FNetSerializerChangeMaskParam ChangeMaskInfo;
	};

	typedef TArray<FReferenceInfo, TInlineAllocator<32>> FReferenceInfoArray;

public:
	FNetReferenceCollector() : Traits(ENetReferenceCollectorTraits::None) {}
	explicit FNetReferenceCollector(ENetReferenceCollectorTraits InTraits) : Traits(InTraits) {}
	
	IRISCORE_API void Add(const FNetReferenceInfo& ReferenceInfo, const FQuantizedObjectReference& Reference, const FNetSerializerChangeMaskParam& ChangeMaskInfo);

	const FReferenceInfoArray& GetCollectedReferences() const { return ReferenceInfos; }

	void Reset() { ReferenceInfos.Reset(); }

private:
	FReferenceInfoArray ReferenceInfos;	
	const ENetReferenceCollectorTraits Traits;
};

}
