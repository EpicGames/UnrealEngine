// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetErrorContext.h"
#include "Iris/Serialization/NetJournal.h"

class FName;
class INetBlobReceiver;
class FNetTraceCollector;
namespace UE::Net
{
	class FNetBitArrayView;
	class FNetBitStreamReader;
	class FNetBitStreamWriter;
	class FNetTokenStore;
	class FNetTokenStoreState;

	namespace Private
	{
		class FInternalNetSerializationContext;
		class FNetExportContext;
		class FNetStatsContext;
	}
}

namespace UE::Net
{

class FNetSerializationContext
{
public:
	FNetSerializationContext();
	FNetSerializationContext(FNetBitStreamReader*, FNetBitStreamWriter*);
	explicit FNetSerializationContext(FNetBitStreamReader*);
	explicit FNetSerializationContext(FNetBitStreamWriter*);

	FNetBitStreamReader* GetBitStreamReader() { return BitStreamReader; }
	FNetBitStreamWriter* GetBitStreamWriter() { return BitStreamWriter; }

	FNetSerializationContext MakeSubContext(FNetBitStreamWriter*) const;
	FNetSerializationContext MakeSubContext(FNetBitStreamReader*) const;

	void SetTraceCollector(FNetTraceCollector* InTraceCollector) { TraceCollector = InTraceCollector; }
	FNetTraceCollector* GetTraceCollector() { return TraceCollector; }

	bool HasError() const { return ErrorContext.HasError(); }
	bool HasErrorOrOverflow() const;
	
	/** If an error has already been set calling this function again will be a no-op, if bDoOverFlow is true, the function will also mark the current bitstream as overflown */
	void SetError(const FName Error, bool bDoOverFlow = true);
	FName GetError() const { return ErrorContext.GetError(); }

	void SetErrorDiagnostic(FString InDiagnostic) { ErrorContext.SetErrorDiagnostic(MoveTemp(InDiagnostic)); }
	const FString& GetErrorDiagnostic() const { return ErrorContext.GetErrorDiagnostic(); }

	/** Store extra information regarding the object that triggered an error. */
	void SetErrorHandleContext(const FNetRefHandle& HandleContext);
	const FNetRefHandle& GetErrorHandleContext() const { return ErrorContext.GetObjectHandle(); }

	/** Store the replication protocol id associated with an error. Used to dedupe diagnostic messages per protocol. */
	void SetErrorProtocolId(FReplicationProtocolIdentifier InProtocolId) { ErrorContext.SetErrorProtocolId(InProtocolId); }
	FReplicationProtocolIdentifier GetErrorProtocolId() const { return ErrorContext.GetErrorProtocolId(); }

	/** Direct access to the underlying error context for callers that need to forward both the diagnostic and the protocol id. */
	const FNetErrorContext& GetErrorContext() const { return ErrorContext; }

	/** There are cases where an error is handled and reported where we want to stay calm, reset the error context and carry on */
	void ResetErrorContext() { ErrorContext = FNetErrorContext(); }

	/** Add entry into read journal, Name must be a static string as the pointer will be stored */
	void AddReadJournalEntry(const TCHAR* Name);

	/** Add entry in to error context, Name must be a static string as the pointer will be stored */
	void AddReadJournalEntry(const FNetDebugName* DebugName);

	/** Print the ReadJournal */
	FString PrintReadJournal();

	void ResetReadJournal() { ReadJournal.Reset(); }

	void SetIsInitState(bool bInIsInitState) { bIsInitState = bInIsInitState; }
	bool IsInitState() const { return bIsInitState; }

	// If set, this is the changemask for the entire protocol
	void SetChangeMask(const FNetBitArrayView* InChangeMask) { ChangeMask = InChangeMask; }
	const FNetBitArrayView* GetChangeMask() const { return ChangeMask; }

	INetBlobReceiver* GetNetBlobReceiver() { return NetBlobReceiver; }
	void SetNetBlobReceiver(INetBlobReceiver* InNetBlobReceiver) { NetBlobReceiver = InNetBlobReceiver; }

	void SetLocalConnectionId(uint32 InLocalConnectionId) { LocalConnectionId = InLocalConnectionId; }
	uint32 GetLocalConnectionId() const { return LocalConnectionId; }

	/**
	 * Retrieves the user data object associated with the local connection.
	 *
	 * @param ConnectionId Local connection ID.
	 * @return The user data object associated with the connection.
	 */
	IRISCORE_API UObject* GetLocalConnectionUserData(uint32 ConnectionId) const;

	IRISCORE_API const UE::Net::FNetTokenStore* GetNetTokenStore() const;
	IRISCORE_API UE::Net::FNetTokenStore* GetNetTokenStore();
	IRISCORE_API const UE::Net::FNetTokenStoreState* GetRemoteNetTokenStoreState() const;

	void SetInternalContext(Private::FInternalNetSerializationContext* InInternalContext) { InternalContext = InInternalContext; }
	Private::FInternalNetSerializationContext* GetInternalContext() { return InternalContext; }

	void SetExportContext(Private::FNetExportContext* InExportContext) { ExportContext = InExportContext; }
	Private::FNetExportContext* GetExportContext() { return ExportContext; }

	void SetNetStatsContext(Private::FNetStatsContext* InNetStatsContext) { NetStatsContext = InNetStatsContext; }
	Private::FNetStatsContext* GetNetStatsContext() { return NetStatsContext; }

	void SetIsInitializingDefaultState(bool bInIsInitializingDefaultState) { bIsInitializingDefaultState = bInIsInitializingDefaultState; }
	bool IsInitializingDefaultState() const { return bIsInitializingDefaultState; }

	void SetPacketId(int32 InPacketId) { PacketId = InPacketId; }
	int32 GetPacketId() const { return PacketId; }

private:
	IRISCORE_API bool IsBitStreamOverflown() const;
	IRISCORE_API void SetBitStreamOverflow();

	FNetErrorContext ErrorContext;
	FNetJournal ReadJournal;

	FNetBitStreamReader* BitStreamReader = nullptr;
	FNetBitStreamWriter* BitStreamWriter = nullptr;
	FNetTraceCollector* TraceCollector = nullptr;

	Private::FInternalNetSerializationContext* InternalContext = nullptr;
	Private::FNetExportContext* ExportContext = nullptr;
	Private::FNetStatsContext* NetStatsContext = nullptr;

	const FNetBitArrayView* ChangeMask = nullptr;
	INetBlobReceiver* NetBlobReceiver = nullptr ;

	uint32 LocalConnectionId = 0;
	int32 PacketId = -1;

	/** Set when replicated objects send their very first state. */
	uint32 bIsInitState : 1;
	/** Set only when dealing with a default state. */
	uint32 bIsInitializingDefaultState : 1;
};

// Implementation
inline FNetSerializationContext::FNetSerializationContext(FNetBitStreamReader* InBitStreamReader, FNetBitStreamWriter* InBitStreamWriter)
: BitStreamReader(InBitStreamReader)
, BitStreamWriter(InBitStreamWriter)
, bIsInitState(0)
, bIsInitializingDefaultState(0)
{
}

inline FNetSerializationContext::FNetSerializationContext()
: FNetSerializationContext(static_cast<FNetBitStreamReader*>(nullptr), static_cast<FNetBitStreamWriter*>(nullptr))
{
}

inline FNetSerializationContext::FNetSerializationContext(FNetBitStreamReader* InBitStreamReader)
: FNetSerializationContext(InBitStreamReader, static_cast<FNetBitStreamWriter*>(nullptr))
{
}

inline FNetSerializationContext::FNetSerializationContext(FNetBitStreamWriter* InBitStreamWriter)
: FNetSerializationContext(static_cast<FNetBitStreamReader*>(nullptr), InBitStreamWriter)
{
}

inline FNetSerializationContext FNetSerializationContext::MakeSubContext(FNetBitStreamWriter* InBitStreamWriter) const
{
	FNetSerializationContext SubContext(*this);

	SubContext.BitStreamReader = nullptr;
	SubContext.BitStreamWriter = InBitStreamWriter;

	return SubContext;
}

inline FNetSerializationContext FNetSerializationContext::MakeSubContext(FNetBitStreamReader* InBitStreamReader) const
{
	FNetSerializationContext SubContext(*this);

	SubContext.BitStreamReader = InBitStreamReader;
	SubContext.BitStreamWriter = nullptr;

	return SubContext;
}

inline void FNetSerializationContext::SetError(const FName Error, bool bDoOverFlow)
{
	if (bDoOverFlow)
	{
		SetBitStreamOverflow();
	}
	ErrorContext.SetError(Error);
}

inline void FNetSerializationContext::SetErrorHandleContext(const FNetRefHandle& HandleContext)
{
	ErrorContext.SetObjectHandle(HandleContext);
}

inline bool FNetSerializationContext::HasErrorOrOverflow() const
{
	return ErrorContext.HasError() || IsBitStreamOverflown();
}

}
