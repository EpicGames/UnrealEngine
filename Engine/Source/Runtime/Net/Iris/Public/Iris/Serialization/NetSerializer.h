// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/NetSerializerConfig.h"
#include <limits>

/**
 * A FNetSerializer is needed for replication support of a certain type. Most types that can be 
 * marked as UPROPERTY are already supported. Types that aren't supported or need special support
 * will emit warnings when descriptors are built for a UCLASS, USTRUCT or UFUNCTION.
 * Find below how to implement one.
 */

#if 0
// Example.h
/** Always declare a serializer specific config for versioning reasons. */
USTRUCT()
struct FExampleNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

UE_NET_DECLARE_SERIALIZER(FFullExampleNetSerializer, EXAMPLE_API);

// Example.cpp
struct FExampleNetSerializer
{
	/** Version is required. */
	static constexpr uint32 Version = 0;

	// The various traits are optional and need only be specified if different from the defaults listed below.

	/** Specify when you want to make sure you implement all possible functions as you intend to forward calls to at least one other serializer. */
	static constexpr bool bIsForwardingSerializer = false;
	/**
	 * Specify when connection specific serialization is needed. Avoid it!
	 * @see ENetSerializerTraits
	 */
	static constexpr bool bHasConnectionSpecificSerialization = false;
	/** Specify when a CollectNetReferences implementation is needed. */
	static constexpr bool bHasCustomNetReference = false;
	/** Specify when the serializer requires dynamic state. Requires implementing CloneDynamicState and FreeDynamicState. */
	static constexpr bool bHasDynamicState = false;
	/** Set to false when a same value delta compression method is undesirable, for example when the serializer only writes a single bit for the state. */
	static constexpr bool bUseDefaultDelta = true;

	/**
	 * A typedef for the SourceType is required. Needed in order to calculate external state
	 * size and alignment and provide default implementations of some functions.
	 */
	typedef FSomeSourceType SourceType;

	/**
	 * A typedef for the QuantizedType is optional unless the SourceType isn't POD. Assumed to be SourceType if not specified.
	 * The QuantizedType needs to be POD.
	 */
	typedef FSomePodType QuantizedType;

	/** A typedef for the ConfigType is required. */
	typedef FNetSerializerConfig ConfigType;

	/** DefaultConfig is optional but highly recommended as the serializer can then be used without requiring special configuration setup. */
	inline static const ConfigType DefaultConfig;


	/** Required. Serialize is responsible for writing the quantized data to a bit stream provided by the serialization context. */
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	/** Required. Deserialize is responsible for reading the quantized data from a bit stream provided by the serialization context. */
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	/**
	 * Optional. Same as Serialize but where an acked previous state is provided for bitpacking purposes.
	 * This is implemented by default to do same value optimization, at the cost of a bit. If implemented
	 * then DeserializeDelta is required.
	 */
	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);

	/**
	 * Optional. Same as Deserialize but where an acked previous state is provided for bitpacking purposes.
	 * This is implemented by default to do same value optimization, at the cost of a bit. If implemented
	 * then SerializeDelta is required.
	 */
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);

	/**
	 * Optional, unless there's a QuantizedType typedef or Dequantize implementation.
	 * Transforms potentially non-POD source data to POD form which can be serialized quickly.
	 * Quantization is only performed at most once per object and frame and is an excellent opportunity to perform slightly heavier computations in order
	 * to make the serialization quicker.
	 */
	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);

	/**
	 * Optional, unless there's a QuantizedType typedef or Quantize implementation. 
	 * The dequantize function is responsible for transforming the quantized state to a 
	 * valid source data form of the state, approximately equal to the original source data that was quantized.
	 */
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	/** Optional. Determine whether data is equal to other data or not. The default implementation will use the equality operator. */
	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);

	/** Optional. Validate that the data fulfills serializer specific requirements. The default implementation will return true. */
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

	/** Required if bHasDynamicState is true. Clones the quantized data from one state buffer to another. */
	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);

	/** Required if bHasDynamicState is true. Frees dynamic allocations in the state buffer and resets the data to default state. */
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	/** Required if bHasCustomNetReference is true. Add object references to a collector. */
	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

	/** Serializers that want to be selective about which members to modify in the target instance when applying state should implement Apply where the serializer is responsible for setting the members of the target instance. The function operates on non-quantized state. */
	static void Apply(FNetSerializationContext&, const FNetApplyArgs&);
};
UE_NET_IMPLEMENT_SERIALIZER(FExampleNetSerializer);

/**
 * How to properly implement the actual functions can be seen in various NetSerializer implementations,
 * such as FFloatNetSerializer to name one.
 */

#endif

namespace UE::Net
{
	class FNetBitArrayView;
	class FNetMemoryContext;
}

namespace UE::Net
{

typedef UPTRINT NetSerializerValuePointer;
typedef const FNetSerializerConfig* NetSerializerConfigParam;

struct FNetSerializerChangeMaskParam
{
	/** Offset in the change mask where we store the bits for the member. */
	uint32 BitOffset = 0;
	/** Number of bits used for the member. This is typically 1, except for special cases like arrays. */
	uint32 BitCount = 0;
};

/**
 * Things that need to be passed to most functions that are part of a NetSerializer.
 * Along with the function arguments there's often a FNetSerializationContext as well.
 */
struct FNetSerializerBaseArgs
{
	/** The serializer's config. */
	NetSerializerConfigParam NetSerializerConfig = 0;
	/** Change mask info if available. */
	FNetSerializerChangeMaskParam ChangeMaskInfo;
	/** The Version of the NetSerializer. Currently not properly propagated. */
	uint32 Version = 0;
};

/**
 * Parameters passed to a NetSerializer's CollectNetReferences function.
 * CollectNetReferences is typically called to determine whether all references
 * can be properly resolved or not.
 */
struct FNetCollectReferencesArgs : FNetSerializerBaseArgs
{
	/** A pointer to the quantized data. */
	NetSerializerValuePointer Source;
	/** A pointer to a FNetReferenceCollector. */
	NetSerializerValuePointer Collector;
};
typedef void(*NetCollectNetReferencesFunction)(FNetSerializationContext&, const FNetCollectReferencesArgs&);

/**
 * Parameters passed to a NetSerializer's Serialize function.
 * Serialize is called to write quantized data to a bit stream.
 */
struct FNetSerializeArgs : FNetSerializerBaseArgs
{
	/** A pointer to the quantized data. */
	NetSerializerValuePointer Source;
};
typedef void(*NetSerializeFunction)(FNetSerializationContext&, const FNetSerializeArgs&);

/**
 * Parameters passed to a NetSerializer's Deserialize function.
 * Deserialize is called to read quantized data from a bit stream.
 */
struct FNetDeserializeArgs : FNetSerializerBaseArgs
{
	NetSerializerValuePointer Target;
};
typedef void(*NetDeserializeFunction)(FNetSerializationContext&, const FNetDeserializeArgs&);

/**
 * Parameters passed to a NetSerializer's SerializeDelta function.
 * SerializeDelta is called to write quantized data to a bit stream.
 * As a pointer to a previously acked quantized state is provided the
 * serializer can use bit packing for example to reduce the number of
 * bits to be serialized.
 */
struct FNetSerializeDeltaArgs : FNetSerializeArgs
{
	/** A pointer to acked quantized data, which can be used for bit packing. */
	NetSerializerValuePointer Prev;
};
typedef void(*NetSerializeDeltaFunction)(FNetSerializationContext&, const FNetSerializeDeltaArgs&);

/**
 * Parameters passed to a NetSerializer's DeserializeDelta function.
 * DeserializeDelta is responsible to read the data produced by SerializeDelta.
 * @see FNetSerializeDeltaArgs
 * @note DeserializeDelta must always store the deserialized delta in the target memory,
 * even if it's determined that the data is the same as the quantized data passed
 * in the Prev pointer.
 */
struct FNetDeserializeDeltaArgs : FNetDeserializeArgs
{
	/** A pointer to quantized data which was used by the SerializeDelta call on the sending side. */
	NetSerializerValuePointer Prev;
};
typedef void(*NetDeserializeDeltaFunction)(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);

/**
 * Parameters passed to a NetSerializer's Quantize function.
 * The purpose of the Quantize function is to transform the original source data
 * to a POD state. Quantized state buffers are passed to memcpy and similar functions. Apart from
 * required to being POD it must also be bit deterministic. 
 * As state buffers are initialized to zero before first use a quantized state of zero
 * should represent a valid state which the serializer's other functions can operate on.
 * There is an option to allow dynamic state which can be used by serializers operating on
 * container types to minimize the footprint of the quantized state, rather than always having 
 * a fixed buffer that can handle the maximum number of elements in the container for example.
 * For dynamic state additional functions need to be implemented; CloneDynamicState and FreeDynamicState.
 * @warning Beware of padding in your quantized state as creating it on the stack for example
 *          may cause non-determinstic state.
 */
struct FNetQuantizeArgs : FNetSerializerBaseArgs
{
	/** A pointer to the non-quantized source data. */
	NetSerializerValuePointer Source;
	/** A pointer to the quantized state buffer which contains valid, but unknown, quantized state. */
	NetSerializerValuePointer Target;
};
typedef void(*NetQuantizeFunction)(FNetSerializationContext&, const FNetQuantizeArgs&);

/**
 * Parameters passed to a NetSerializer's Dequantize function.
 * A Dequantize function must be provided if there's a Quantize function.
 * The dequantize function is responsible for transforming the quantized state
 * to a valid source data form of the state, approximately equal to the original
 * source data that was quantized.
 */
struct FNetDequantizeArgs : FNetSerializerBaseArgs
{
	/** A pointer to a valid quantized state buffer. */
	NetSerializerValuePointer Source;
	/** A pointer to the source data buffer which contains valid, but unknown, source data. */
	NetSerializerValuePointer Target;
};
typedef void(*NetDequantizeFunction)(FNetSerializationContext&, const FNetDequantizeArgs&);

/**
 * Parameters passed to a NetSerializer's IsEqual function.
 * IsEqual is used to check whether data is network equal, that is if
 * the quantized forms of the data are equal. IsEqual needs to work for both
 * source data and quantized data.
 */
struct FNetIsEqualArgs : FNetSerializerBaseArgs
{
	/** Source data or quantized data. */
	NetSerializerValuePointer Source0;
	/** Source data or quantized data to compare with. */
	NetSerializerValuePointer Source1;
	/** Whether the data pointed to is source or quantized form. */
	bool bStateIsQuantized;
};
typedef bool(*NetIsEqualFunction)(FNetSerializationContext&, const FNetIsEqualArgs&);

/**
 * Parameters passed to a NetSerializer's Validate function.
 * Validate is used to determine whether the source data is correct
 * or not. An enum serializer could validate the the value is support by the enum
 * for example. An array serializer could validate that the array doesn't have more
 * number of elements than some limit.
 */
struct FNetValidateArgs : FNetSerializerBaseArgs
{
	/** A pointer to the non-quantized source data. */
	NetSerializerValuePointer Source;
};
typedef bool(*NetValidateFunction)(FNetSerializationContext&, const FNetValidateArgs&);

/**
 * Forwarding serializers and serializers in need of dynamic state must implement
 * CloneDynamicState and FreeDynamicState. Forwarding serializers should only forward
 * calls to forwarding members and members with dynamic state.
 * CloneDynamicState will get undefined memory contents in the target state. It's up
 * to the clone function to deep copy the source quantized state and overwrite the
 * target state. Allocation of memory needs to be done via FNetSerializationContext.
 * @see FNetSerializationContext
*/
struct FNetCloneDynamicStateArgs : FNetSerializerBaseArgs
{
	/** A pointer to the quantized source data. */
	NetSerializerValuePointer Source;
	/** A pointer to the target data which should be overwritten with a deep copy of the quantized source data. */
	NetSerializerValuePointer Target;
};
typedef void(*NetCloneDynamicStateFunction)(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);

/**
 * Forwarding serializers and serializers in need of dynamic state must implement
 * CloneDynamicState and FreeDynamicState. Forwarding serializers should only forward
 * calls to forwarding members and members with dynamic state.
 * FreeDynamicState must be re-entrant. To achieve this it's recommended to clear the quantized state
 * after freeing dynamically allocated memory. Freeing of memory needs to be done via FNetSerializationContext.
 * @see FNetSerializationContext
 */
struct FNetFreeDynamicStateArgs : FNetSerializerBaseArgs
{
	/** A pointer to the quantized source data. */
	NetSerializerValuePointer Source;
};
typedef void(*NetFreeDynamicStateFunction)(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

/** Serializers that want to be selective about which members to modify in the target instance when applying state should implement Apply where the serializer is responsible for setting the members of the target instance. The function operates on non-quantized state. */
struct FNetApplyArgs : FNetSerializerBaseArgs
{
	/** A pointer to the non-quantized source data. */
	NetSerializerValuePointer Source;
	/** A pointer to the non-quantized target data. */
	NetSerializerValuePointer Target;
};
typedef void(*NetApplyFunction)(FNetSerializationContext&, const FNetApplyArgs&);

/**
 * Various traits that can be set for a FNetSerializer.
 * These traits are typically set via constexpr bool in the declaration of the serializer.
 * Only publicly visible traits are part of the enum.
 */
enum class ENetSerializerTraits : uint32
{
	None = 0U,
	/** Forwarding serializers need to implement all functions that a serializer may have. */
	IsForwardingSerializer = 1U << 0U,
	/** Serializers in need of dynamic state must implement CloneDynamicState and FreeDynamicState. */
	HasDynamicState = IsForwardingSerializer << 1U,
	/**
	 * Connection specific serialization is sometimes required but should be avoided by all means necessary
	 * as it prevents sharing of serialized state. For example when multicasting RPCs and one of the arguments
	 * is using connection specific serialization it requires serializing the RPC for each connection and possibly
	 * allocating memory for duplicating the data for each connection. It wastes both CPU and memory.
	 */
	HasConnectionSpecificSerialization = HasDynamicState << 1U,
	/** There are net references that need to be gathered via calls to CollectNetReferences. */
	HasCustomNetReference = HasConnectionSpecificSerialization << 1U,

	/** Data replicated using this serializer should use the IsEqual implementation in order to determine whether the data is dirty or not. */
	UseSerializerIsEqual = HasCustomNetReference << 1U,

	/** Has an Apply function which should be used when applying its dequantized data to another instance. Useful for custom struct serializers where not all of the struct properties are replicated. Without a custom Apply all values will be overwritten. */
	HasApply = UseSerializerIsEqual << 1U,
};
ENUM_CLASS_FLAGS(ENetSerializerTraits);

/**
 * The end result of a UE_NET_IMPLEMENT_SERIALIZER call on a struct adhering to 
 * the conventions of the TNetSerializerBuilder. If direct access to a serializer
 * is needed for some reason use UE_NET_GET_SERIALIZER.
 */
struct FNetSerializer
{
	uint32 Version;
	ENetSerializerTraits Traits;

	NetSerializeFunction Serialize;
	NetDeserializeFunction Deserialize;
	NetSerializeDeltaFunction SerializeDelta;
	NetDeserializeDeltaFunction DeserializeDelta;
	NetQuantizeFunction Quantize;
	NetDequantizeFunction Dequantize;
	NetIsEqualFunction IsEqual;
	NetValidateFunction Validate;
	NetCloneDynamicStateFunction CloneDynamicState;
	NetFreeDynamicStateFunction FreeDynamicState;
	NetCollectNetReferencesFunction CollectNetReferences;
	NetApplyFunction Apply;
	const FNetSerializerConfig* DefaultConfig;
	uint16 QuantizedTypeSize;
	uint16 QuantizedTypeAlignment;
	uint16 ConfigTypeSize;
	uint16 ConfigTypeAlignment;

	const TCHAR* Name;
};

}

#include "NetSerializerBuilder.inl"

namespace UE::Net
{

template<typename NetSerializerImpl>
class TNetSerializer
{
public:
	static constexpr FNetSerializer ConstructNetSerializer(const TCHAR* Name)
	{
		TNetSerializerBuilder<NetSerializerImpl> Builder;
		Builder.Validate();

		FNetSerializer Serializer = {};
		Serializer.Version = Builder.GetVersion();
		Serializer.Traits = Builder.GetTraits();

		Serializer.Serialize = Builder.GetSerializeFunction();
		Serializer.Deserialize = Builder.GetDeserializeFunction();
		Serializer.SerializeDelta = Builder.GetSerializeDeltaFunction();
		Serializer.DeserializeDelta = Builder.GetDeserializeDeltaFunction();
		Serializer.Quantize = Builder.GetQuantizeFunction();
		Serializer.Dequantize = Builder.GetDequantizeFunction();
		Serializer.IsEqual = Builder.GetIsEqualFunction();
		Serializer.Validate = Builder.GetValidateFunction();
		Serializer.CloneDynamicState = Builder.GetCloneDynamicStateFunction();
		Serializer.FreeDynamicState = Builder.GetFreeDynamicStateFunction();
		Serializer.CollectNetReferences = Builder.GetCollectNetReferencesFunction();
		Serializer.Apply = Builder.GetApplyFunction();

		Serializer.DefaultConfig = Builder.GetDefaultConfig();

		static_assert(Builder.GetQuantizedTypeSize() <= std::numeric_limits<uint16>::max(), "");
		Serializer.QuantizedTypeSize = static_cast<uint16>(Builder.GetQuantizedTypeSize());
		static_assert(Builder.GetQuantizedTypeAlignment() <= std::numeric_limits<uint16>::max(), "");
		Serializer.QuantizedTypeAlignment = static_cast<uint16>(Builder.GetQuantizedTypeAlignment());

		static_assert(Builder.GetConfigTypeSize() <= std::numeric_limits<uint16>::max(), "");
		Serializer.ConfigTypeSize = static_cast<uint16>(Builder.GetConfigTypeSize());
		static_assert(Builder.GetConfigTypeAlignment() <= std::numeric_limits<uint16>::max(), "");
		Serializer.ConfigTypeAlignment = static_cast<uint16>(Builder.GetConfigTypeAlignment());

		Serializer.Name = Name;
		return Serializer;
	}
};

}

/** Declare a serializer. */
#define UE_NET_DECLARE_SERIALIZER(SerializerName, Api) struct Api SerializerName ## NetSerializerInfo  \
{ \
	static const UE::Net::FNetSerializer Serializer; \
	static uint32 GetQuantizedTypeSize(); \
	static uint32 GetQuantizedTypeAlignment(); \
	static const FNetSerializerConfig* GetDefaultConfig(); \
};

/** Implement a serializer using the struct named SerializerName. */
#define UE_NET_IMPLEMENT_SERIALIZER(SerializerName) const UE::Net::FNetSerializer SerializerName ## NetSerializerInfo::Serializer = UE::Net::TNetSerializer<SerializerName>::ConstructNetSerializer(TEXT(#SerializerName)); \
	uint32 SerializerName ## NetSerializerInfo::GetQuantizedTypeSize() { return UE::Net::TNetSerializerBuilder<SerializerName>::GetQuantizedTypeSize(); }; \
	uint32 SerializerName ## NetSerializerInfo::GetQuantizedTypeAlignment() { return UE::Net::TNetSerializerBuilder<SerializerName>::GetQuantizedTypeAlignment(); }; \
	const FNetSerializerConfig* SerializerName ## NetSerializerInfo::GetDefaultConfig() { return UE::Net::TNetSerializerBuilder<SerializerName>::GetDefaultConfig(); };

/** Retrieve a const reference to a named serializer. */
#define UE_NET_GET_SERIALIZER(SerializerName) static_cast<const UE::Net::FNetSerializer&>(SerializerName ## NetSerializerInfo::Serializer)
/** Retrieve the quantized state size for a serializer. */
#define UE_NET_GET_SERIALIZER_INTERNAL_TYPE_SIZE(SerializerName) SerializerName ## NetSerializerInfo::GetQuantizedTypeSize()
/** Retrieve the quantized state alignment for a serializer. */
#define UE_NET_GET_SERIALIZER_INTERNAL_TYPE_ALIGNMENT(SerializerName) SerializerName ## NetSerializerInfo::GetQuantizedTypeAlignment()
/** Retrieve the default config, if present, for a serializer. */
#define UE_NET_GET_SERIALIZER_DEFAULT_CONFIG(SerializerName) SerializerName ## NetSerializerInfo::GetDefaultConfig()
