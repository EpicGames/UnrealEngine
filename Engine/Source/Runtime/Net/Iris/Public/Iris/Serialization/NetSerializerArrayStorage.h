// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Iris/IrisConfig.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/IsPODType.h"

namespace UE::Net
{

namespace AllocationPolicies
{

/**
 * Default AllocationPolicy for FNetSerializerArrayStorage helper class
 */
class FElementAllocationPolicy
{
public:
	using SizeType = uint32;

	template<typename ElementType>
	class ForElementType
	{
	public:
		ForElementType() : Data(nullptr) {}

		~ForElementType() {	checkf(Data == nullptr, TEXT("Data allocated for dynamic states must be explictly freed")); }

		ElementType* GetAllocation() const { return Data; }

		void Initialize() { Data = nullptr; }
	
		void ResizeAllocation(FNetSerializationContext& Context, SizeType PreviousNumElements, SizeType NumElements);

		SizeType CalculateNewCapacity(SizeType NumElements) const { return NumElements; }

		bool HasAllocation() const { return !!Data; }

		SizeType GetInitialCapacity() const { return 0; }

	private:

		ForElementType(const ForElementType&);
		ForElementType& operator=(const ForElementType&);

		ElementType* Data;
	};

private:

	static IRISCORE_API void Free(FNetSerializationContext& Context, void* Ptr);
	static IRISCORE_API void* Realloc(FNetSerializationContext& Context, void* Original, SIZE_T Size, uint32 Alignment);
};

/**
 * The InlinedElementAllocationPolicy allocates up to a specified number of elements in the same allocation as the container.
 * Any allocation needed beyond that causes all data to be moved into an indirect allocation.
 */
template <uint32 NumInlineElements, typename SecondaryAllocator = FElementAllocationPolicy>
class TInlinedElementAllocationPolicy
{
public:
	using SizeType = typename SecondaryAllocator::SizeType;

	template<typename ElementType>
	class ForElementType
	{
	public:

		ForElementType() {}

		ElementType* GetAllocation() const
		{
			if (ElementType* Result = SecondaryData.GetAllocation())
			{
				return Result;
			}
			return GetInlineElements();
		}

		void Initialize() { SecondaryData.Initialize(); }

		void ResizeAllocation(FNetSerializationContext& Context, SizeType PreviousNumElements, SizeType NumElements);

		SizeType CalculateNewCapacity(SizeType NumElements) const;

		bool HasAllocation() const { return SecondaryData.HasAllocation(); }

		SizeType GetInitialCapacity() const { return NumInlineElements; }

	private:
		ForElementType(const ForElementType&);
		ForElementType& operator=(const ForElementType&);

		/** The data is stored in this array if less than NumInlineElements is needed. */
		TTypeCompatibleBytes<ElementType> InlineData[NumInlineElements];

		/** The data is allocated through the indirect allocation policy if more than NumInlineElements is needed. */
		typename SecondaryAllocator::template ForElementType<ElementType> SecondaryData;

		/** @return the base of the aligned inline element data */
		ElementType* GetInlineElements() const { return (ElementType*)InlineData; }
	};
};

}

/** 
 * Helper class to manage storage of dynamic arrays in quantized data
 * FNetSerializerArrayStorage is only intended to be used from within Quantized replication states that requires dynamic storage as it has very specific expectations and limitations. 
 * NOTE: A NetSerializer using FNetSerializerArrayStorage MUST specify the trait bHasDynamicState
 * A zero constructed state is considered valid
 */
template <typename QuantizedElementType, typename AllocationPolicy = AllocationPolicies::FElementAllocationPolicy>
class FNetSerializerArrayStorage
{
public:
	// Verify that type is pod and trivially copyable
	static_assert(TIsPODType<QuantizedElementType>::Value, "Only pod types are supported by FNetSerializerArrayHelper");
	static_assert(TIsTriviallyCopyAssignable<QuantizedElementType>::Value, "Only types are triviallyCopyAssignable is supported by FNetSerializerArrayHelper");

	typedef FNetSerializerArrayStorage<QuantizedElementType, AllocationPolicy> ArrayType;
	typedef QuantizedElementType ElementType;

	typedef typename AllocationPolicy::template ForElementType<ElementType> ElementAllocatorType;
	typedef typename AllocationPolicy::SizeType SizeType;

public:

	/**
	 * Constructor - Typically not invoked as states containing FNetSerializerArrayStorage typically are zero initialized
	 */
	inline FNetSerializerArrayStorage();

	/**
	 * AdjustSize - Adjust the size of FNetSerializerArrayStorage as needed
	 * The state storage in which the storage is located is expected to be in a valid state which can either be zero initialized or a previous valid state
	 */
	inline void AdjustSize(FNetSerializationContext& Context, SizeType InNum);
	
	/**
	 * Free - Free allocated memory and reset state. it is valid to pass in a zero-initialized state 
	 * The state storage in which the storage is located is expected to be in a valid state which can either be zero initialized or a previous valid state.	
	 * Should typically only be called from within a the implemntation of NetSerializer::FreeDynamicState()
	 */
	inline void Free(FNetSerializationContext& Context);

	/**
	 * Clone - Clone dynamic storage from Source
	 * No assumptions of the validity of the target is made, as this is typically called AFTER a memcopy is made to the target state
	 * which invalidates all dynamic data
	 * Should typically only be called from within a the implemntation of NetSerializer::CloneDynamicState()
	*/
	inline void Clone(FNetSerializationContext& Context, const ArrayType& Source);

	const ElementType* GetData() const { return AllocatorInstance.GetAllocation(); }
	ElementType* GetData() { return AllocatorInstance.GetAllocation(); }
	SizeType Num() const { return ArrayNum; }

private:
	ElementAllocatorType AllocatorInstance;
	SizeType ArrayNum;
	SizeType ArrayMaxCapacity;
};

/**
 * Helper class to manage storage of untyped dynamic storage with arbitrary alignment in quantized data.
 * If storage is used to store elements that themselves uses dynamic allocations the user must take care to properly free and clone such elements properly prior to and/or after changing sizes.
 * FNetSerializerAlignedStorage is only intended to be used from within Quantized replication states that requires dynamic storage as it has very specific expectations and limitations.
 * NOTE: A NetSerializer using FNetSerializerAlignedStorage MUST specify the trait bHasDynamicState and forward appropriate calls to Clone and Free to this class.
 * A zero constructed state is considered valid.
 */
class FNetSerializerAlignedStorage
{
public:
	// Use whatever SizeType that is typically used by FNetSerializerArrayStorage
	typedef typename AllocationPolicies::FElementAllocationPolicy::SizeType SizeType;

public:
	/**
	 * AdjustSize - Adjust the size of FNetSerializerArrayStorage as needed
	 * The state storage in which the storage is located is expected to be in a valid state which can either be zero initialized or a previous valid state
	 */
	IRISCORE_API void AdjustSize(FNetSerializationContext& Context, SizeType InNum, SizeType InAlignment);

	/**
	 * Free - Free allocated memory and reset state. it is valid to pass in a zero-initialized state
	 * The state storage in which the storage is located is expected to be in a valid state which can either be zero initialized or a previous valid state.
	 * Should typically only be called from within a the implemntation of NetSerializer::FreeDynamicState()
	 */
	IRISCORE_API void Free(FNetSerializationContext& Context);

	/**
	 * Clone - Clone dynamic storage from Source
	 * No assumptions of the validity of the target is made, as this is typically called AFTER a memcopy is made to the target state
	 * which invalidates all dynamic data
	 * Should typically only be called from within a the implemntation of NetSerializer::CloneDynamicState()
	*/
	IRISCORE_API void Clone(FNetSerializationContext& Context, const FNetSerializerAlignedStorage& Source);

	inline const uint8* GetData() const;
	inline uint8* GetData();
	inline SizeType Num() const;
	inline SizeType GetAlignment() const;

private:
	uint8* Data = nullptr;
	SizeType StorageNum = 0;
	SizeType StorageMaxCapacity = 0;
	// An alignment of 0 isn't valid but the value is only used if there's actual data involved.
	SizeType StorageAlignment = 0;
};

// FNetSerializerArrayStorage Implementation
template <typename QuantizedElementType, typename AllocationPolicy>
FNetSerializerArrayStorage<QuantizedElementType, AllocationPolicy>::FNetSerializerArrayStorage()
: ArrayNum(0)
, ArrayMaxCapacity(AllocatorInstance.GetInitialCapacity())
{
}

template <typename QuantizedElementType, typename AllocationPolicy>
void FNetSerializerArrayStorage<QuantizedElementType, AllocationPolicy>::AdjustSize(FNetSerializationContext& Context, SizeType InNum)
{
	const SizeType NewCapacity = AllocatorInstance.CalculateNewCapacity(InNum);
	AllocatorInstance.ResizeAllocation(Context, ArrayNum, NewCapacity);
	ArrayMaxCapacity = NewCapacity;

	if (NewCapacity > ArrayNum)
	{
		// To avoid issues with bad data in padding we always zero initialize new memory.
		FMemory::Memzero(GetData() + ArrayNum, (NewCapacity - ArrayNum) * sizeof(ElementType));
	}

	ArrayNum = FMath::Min(InNum, NewCapacity);
}

template <typename QuantizedElementType, typename AllocationPolicy>
void FNetSerializerArrayStorage<QuantizedElementType, AllocationPolicy>::Free(FNetSerializationContext& Context)
{
	AllocatorInstance.ResizeAllocation(Context, ArrayNum, 0);
	ArrayMaxCapacity = AllocatorInstance.GetInitialCapacity();
	ArrayNum = 0;
}

template <typename QuantizedElementType, typename AllocationPolicy>
void FNetSerializerArrayStorage<QuantizedElementType, AllocationPolicy>::Clone(FNetSerializationContext& Context, const ArrayType& Source)
{
	AllocatorInstance.Initialize();

	const SizeType SourceNum = Source.Num();
	const SizeType NewCapacity = AllocatorInstance.CalculateNewCapacity(SourceNum);

	AllocatorInstance.ResizeAllocation(Context, 0, NewCapacity);
	CopyAssignItems(GetData(), Source.GetData(), SourceNum);

	if (NewCapacity > SourceNum)
	{
		// To avoid issues with bad data in padding we always zero initialize new memory.
		FMemory::Memzero(GetData() + SourceNum, (NewCapacity - SourceNum) * sizeof(ElementType));
	}

	ArrayMaxCapacity = NewCapacity;
	ArrayNum = SourceNum;
}

// FNetSerializerAlignedStorage implementation

inline const uint8* FNetSerializerAlignedStorage::GetData() const
{
	return Data;
}

inline uint8* FNetSerializerAlignedStorage::GetData()
{
	return Data;
}

inline FNetSerializerAlignedStorage::SizeType FNetSerializerAlignedStorage::Num() const
{
	return StorageNum;
}

inline FNetSerializerAlignedStorage::SizeType FNetSerializerAlignedStorage::GetAlignment() const
{
	return StorageAlignment;
}

namespace AllocationPolicies
{

// FElementAllocationPolicy implementation

template<typename ElementType>
void FElementAllocationPolicy::ForElementType<ElementType>::ResizeAllocation(FNetSerializationContext& Context, SizeType PreviousNumElements, SizeType NumElements)
{
	if (NumElements == 0)
	{
		if (Data)
		{
			Free(Context, Data);
			Data = nullptr;
		}
	}
	else
	{
		constexpr SIZE_T ElementSize = sizeof(ElementType);
		constexpr SIZE_T ElementAlignment = alignof(ElementType);

		Data = (ElementType*)Realloc(Context, Data, NumElements*ElementSize, ElementAlignment);
	}
}

// TInlinedElementAllocationPolicy implementation

template<uint32 NumInlineElements, typename SecondaryAllocator>
template<typename ElementType>
void TInlinedElementAllocationPolicy<NumInlineElements, SecondaryAllocator>::ForElementType<ElementType>::ResizeAllocation(FNetSerializationContext& Context, SizeType PreviousNumElements, SizeType NumElements)
{
	// Check if the new allocation will fit in the inline data area.
	if (NumElements <= NumInlineElements)
	{
		// If the old allocation wasn't in the inline data area, relocate it into the inline data area.
		if (SecondaryData.GetAllocation())
		{
			RelocateConstructItems<ElementType>((void*)InlineData, (ElementType*)SecondaryData.GetAllocation(), FMath::Min(PreviousNumElements, NumInlineElements));

			// Free the old indirect allocation.
			SecondaryData.ResizeAllocation(Context, 0, 0);
		}
	}
	else
	{
		if (!SecondaryData.GetAllocation())
		{
			// Allocate new indirect memory for the data.
			SecondaryData.ResizeAllocation(Context, 0, NumElements);

			// Move the data out of the inline data area into the new allocation.
			RelocateConstructItems<ElementType>((void*)SecondaryData.GetAllocation(), GetInlineElements(), PreviousNumElements);
		}
		else
		{
			// Reallocate the indirect data for the new size.
			SecondaryData.ResizeAllocation(Context, PreviousNumElements, NumElements);
		}
	}
}

template<uint32 NumInlineElements, typename SecondaryAllocator>
template<typename ElementType>
typename TInlinedElementAllocationPolicy<NumInlineElements, SecondaryAllocator>::SizeType TInlinedElementAllocationPolicy<NumInlineElements, SecondaryAllocator>::ForElementType<ElementType>::CalculateNewCapacity(SizeType NumElements) const
{
	// If the elements use less space than the inline allocation, only use the inline allocation as slack.
	return NumElements <= NumInlineElements ? NumInlineElements : SecondaryData.CalculateNewCapacity(NumElements);
}

}

}
