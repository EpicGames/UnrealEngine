// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/EnableIf.h"
#include "Templates/IsPODType.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"

namespace UE::Net
{

template<NetSerializeFunction Serialize>
void
NetSerializeDeltaDefault(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	Serialize(Context, Args);
};

template<NetDeserializeFunction Deserialize>
void
NetDeserializeDeltaDefault(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	Deserialize(Context, Args);
}

template<NetSerializeFunction Serialize, NetIsEqualFunction IsEqual>
void
NetSerializeDeltaDefault(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	FNetIsEqualArgs EqualArgs;
	EqualArgs.Version = 0;
	EqualArgs.NetSerializerConfig = Args.NetSerializerConfig;
	EqualArgs.Source0 = Args.Source;
	EqualArgs.Source1 = Args.Prev;
	EqualArgs.bStateIsQuantized = true;

	if (Context.GetBitStreamWriter()->WriteBool(IsEqual(Context, EqualArgs)))
	{
		return;
	}

	Serialize(Context, Args);
};

template<uint32 QuantizedTypeSize, NetDeserializeFunction Deserialize, NetFreeDynamicStateFunction FreeDynamicState, NetCloneDynamicStateFunction CloneDynamicState>
void
NetDeserializeDeltaDefault(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	if (Context.GetBitStreamReader()->ReadBool())
	{
		// Clone from prev. Need to free target first.
		if (FreeDynamicState != NetFreeDynamicStateFunction(nullptr))
		{
			FNetFreeDynamicStateArgs FreeArgs;
			FreeArgs.Version = 0;
			FreeArgs.NetSerializerConfig = Args.NetSerializerConfig;
			FreeArgs.Source = Args.Target;

			FreeDynamicState(Context, FreeArgs);
		}

		FMemory::Memcpy(reinterpret_cast<uint8*>(Args.Target), reinterpret_cast<uint8*>(Args.Prev), QuantizedTypeSize);

		if (CloneDynamicState != NetCloneDynamicStateFunction(nullptr))
		{
			FNetCloneDynamicStateArgs CloneArgs;
			CloneArgs.Version = 0;
			CloneArgs.NetSerializerConfig = Args.NetSerializerConfig;
			CloneArgs.Source = Args.Prev;
			CloneArgs.Target = Args.Target;

			CloneDynamicState(Context, CloneArgs);
		}

		return;
	}

	Deserialize(Context, Args);
}

template<typename T>
void
NetQuantizeDefault(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	*reinterpret_cast<T*>(Args.Target) = *reinterpret_cast<const T*>(Args.Source);
}

template<typename T>
void
NetDequantizeDefault(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	*reinterpret_cast<T*>(Args.Target) = *reinterpret_cast<const T*>(Args.Source);
}

template<typename T>
bool
NetIsEqualDefault(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	return *reinterpret_cast<const T*>(Args.Source0) == *reinterpret_cast<const T*>(Args.Source1);
}

template<typename T = void>
bool
NetValidateDefault(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	return true;
}

template<typename NetSerializerImpl>
class TNetSerializerBuilder
{
private:
	enum class ETrueType : unsigned
	{
		Value = 1
	};

	enum class EFalseType : unsigned
	{
		Value = 0
	};

	struct FVersion
	{
		static constexpr uint32 Version = 0;
	};

	struct FTraits
	{
		static constexpr bool bIsForwardingSerializer = false;
		static constexpr bool bHasConnectionSpecificSerialization = false;
		static constexpr bool bHasCustomNetReference = false;
		static constexpr bool bHasDynamicState = false;
		static constexpr bool bUseDefaultDelta = true;
		static constexpr bool bUseSerializerIsEqual = false;
	};

	template<typename U, U> struct FSignatureCheck;
	template<typename> struct FTypeCheck;

	// Version check
	template<typename U> static ETrueType TestHasVersion(typename TEnableIf<std::is_same_v<decltype(&FVersion::Version), decltype(&U::Version)>>::Type*);
	template<typename> static EFalseType TestHasVersion(...);

	// Traits
	template<typename U> static ETrueType TestHasCustomNetReferenceIsPresent(FTypeCheck<decltype(&U::bHasCustomNetReference)>*);
	template<typename> static EFalseType TestHasCustomNetReferenceIsPresent(...);

	template<typename U> static ETrueType TestHasCustomNetReferenceIsBool(typename TEnableIf<std::is_same_v<decltype(&FTraits::bHasCustomNetReference), decltype(&U::bHasCustomNetReference)>>::Type*);
	template<typename> static EFalseType TestHasCustomNetReferenceIsBool(...);

	template<typename U> static ETrueType TestUseSerializerIsEqualIsPresent(FTypeCheck<decltype(&U::bUseSerializerIsEqual)>*);
	template<typename> static EFalseType TestUseSerializerIsEqualIsPresent(...);

	template<typename U> static ETrueType TestUseSerializerIsEqualIsBool(typename TEnableIf<std::is_same_v<decltype(&FTraits::bUseSerializerIsEqual), decltype(&U::bUseSerializerIsEqual)>>::Type*);
	template<typename> static EFalseType TestUseSerializerIsEqualIsBool(...);

	template<typename U> static ETrueType TestIsForwardingSerializerIsPresent(FTypeCheck<decltype(&U::bIsForwardingSerializer)>*);
	template<typename> static EFalseType TestIsForwardingSerializerIsPresent(...);

	template<typename U> static ETrueType TestIsForwardingSerializerIsBool(typename TEnableIf<std::is_same_v<decltype(&FTraits::bIsForwardingSerializer), decltype(&U::bIsForwardingSerializer)>>::Type*);
	template<typename> static EFalseType TestIsForwardingSerializerIsBool(...);

	template<typename U> static ETrueType TestHasConnectionSpecificSerializationIsPresent(FTypeCheck<decltype(&U::bHasConnectionSpecificSerialization)>*);
	template<typename> static EFalseType TestHasConnectionSpecificSerializationIsPresent(...);

	template<typename U> static ETrueType TestHasConnectionSpecificSerializationIsBool(typename TEnableIf<std::is_same_v<decltype(&FTraits::bHasConnectionSpecificSerialization), decltype(&U::bHasConnectionSpecificSerialization)>>::Type*);
	template<typename> static EFalseType TestHasConnectionSpecificSerializationIsBool(...);

	template<typename U> static ETrueType TestHasDynamicStateIsPresent(FTypeCheck<decltype(&U::bHasDynamicState)>*);
	template<typename> static EFalseType TestHasDynamicStateIsPresent(...);

	template<typename U> static ETrueType TestHasDynamicStateIsBool(typename TEnableIf<std::is_same_v<decltype(&FTraits::bHasDynamicState), decltype(&U::bHasDynamicState)>>::Type*);
	template<typename> static EFalseType TestHasDynamicStateIsBool(...);

	template<typename U> static ETrueType TestUseDefaultDeltaIsPresent(FTypeCheck<decltype(&U::bUseDefaultDelta)>*);
	template<typename> static EFalseType TestUseDefaultDeltaIsPresent(...);

	template<typename U> static ETrueType TestUseDefaultDeltaIsBool(typename TEnableIf<std::is_same_v<decltype(&FTraits::bUseDefaultDelta), decltype(&U::bUseDefaultDelta)>>::Type*);
	template<typename> static EFalseType TestUseDefaultDeltaIsBool(...);

	// Type checks
	template<typename U> static ETrueType TestHasConfigType(FTypeCheck<typename U::ConfigType>*);
	template<typename> static EFalseType TestHasConfigType(...);

	template<typename U> static ETrueType TestHasSourceType(FTypeCheck<typename U::SourceType>*);
	template<typename> static EFalseType TestHasSourceType(...);

	template<typename U> static ETrueType TestHasQuantizedType(FTypeCheck<typename U::QuantizedType>*);
	template<typename> static EFalseType TestHasQuantizedType(...);

	template<typename U> static constexpr bool IsSourceTypePod(FTypeCheck<typename U::SourceType>*) { return TIsPODType<typename U::SourceType>::Value; }
	template<typename> static constexpr bool IsSourceTypePod(...) { return false; }

	template<typename U> static constexpr bool IsQuantizedTypePod(FTypeCheck<typename U::QuantizedType>*) { return TIsPODType<typename U::QuantizedType>::Value; }
	template<typename> static constexpr bool IsQuantizedTypePod(...) { return false; }

	// Default config check
	template<typename U> static constexpr bool TestHasDefaultConfig(FTypeCheck<decltype(&U::DefaultConfig)>*) { return true; }
	template<typename> static constexpr bool TestHasDefaultConfig(...) { return false; }

	// Function checks
	template<typename U> static ETrueType TestHasSerialize(FSignatureCheck<NetSerializeFunction, &U::Serialize>*);
	template<typename> static EFalseType TestHasSerialize(...);

	template<typename U> static ETrueType TestHasDeserialize(FSignatureCheck<NetDeserializeFunction, &U::Deserialize>*);
	template<typename> static EFalseType TestHasDeserialize(...);

	template<typename U> static ETrueType TestHasSerializeDelta(FSignatureCheck<NetSerializeDeltaFunction, &U::SerializeDelta>*);
	template<typename> static EFalseType TestHasSerializeDelta(...);

	template<typename U> static ETrueType TestHasDeserializeDelta(FSignatureCheck<NetDeserializeDeltaFunction, &U::DeserializeDelta>*);
	template<typename> static EFalseType TestHasDeserializeDelta(...);

	template<typename U> static ETrueType TestHasQuantize(FSignatureCheck<NetQuantizeFunction, &U::Quantize>*);
	template<typename> static EFalseType TestHasQuantize(...);

	template<typename U> static ETrueType TestHasDequantize(FSignatureCheck<NetDequantizeFunction, &U::Dequantize>*);
	template<typename> static EFalseType TestHasDequantize(...);

	template<typename U> static ETrueType TestHasIsEqual(FSignatureCheck<NetIsEqualFunction, &U::IsEqual>*);
	template<typename> static EFalseType TestHasIsEqual(...);

	template<typename U> static ETrueType TestHasValidate(FSignatureCheck<NetValidateFunction, &U::Validate>*);
	template<typename> static EFalseType TestHasValidate(...);

	template<typename U> static ETrueType TestHasFreeDynamicState(FSignatureCheck<NetFreeDynamicStateFunction, &U::FreeDynamicState>*);
	template<typename> static EFalseType TestHasFreeDynamicState(...);

	template<typename U> static ETrueType TestHasCloneDynamicState(FSignatureCheck<NetCloneDynamicStateFunction, &U::CloneDynamicState>*);
	template<typename> static EFalseType TestHasCloneDynamicState(...);

	template<typename U> static ETrueType TestHasCollectNetReferences(FSignatureCheck<NetCollectNetReferencesFunction, &U::CollectNetReferences>*);
	template<typename> static EFalseType TestHasCollectNetReferences(...);

	template<typename U> static ETrueType TestHasApply(FSignatureCheck<NetApplyFunction, &U::Apply>*);
	template<typename> static EFalseType TestHasApply(...);

	enum ETraits : unsigned
	{
		HasVersion = unsigned(decltype(TestHasVersion<NetSerializerImpl>(nullptr))::Value),

		HasCustomNetReferenceIsPresent = unsigned(decltype(TestHasCustomNetReferenceIsPresent<NetSerializerImpl>(nullptr))::Value),
		HasCustomNetReferenceIsBool = unsigned(decltype(TestHasCustomNetReferenceIsBool<NetSerializerImpl>(nullptr))::Value),

		UseSerializerIsEqualIsPresent = unsigned(decltype(TestUseSerializerIsEqualIsPresent<NetSerializerImpl>(nullptr))::Value),
		UseSerializerIsEqualIsBool = unsigned(decltype(TestUseSerializerIsEqualIsBool<NetSerializerImpl>(nullptr))::Value),

		IsForwardingSerializerIsPresent = unsigned(decltype(TestIsForwardingSerializerIsPresent<NetSerializerImpl>(nullptr))::Value),
		IsForwardingSerializerIsBool = unsigned(decltype(TestIsForwardingSerializerIsBool<NetSerializerImpl>(nullptr))::Value),
		HasConnectionSpecificSerializationIsPresent = unsigned(decltype(TestHasConnectionSpecificSerializationIsPresent<NetSerializerImpl>(nullptr))::Value),
		HasConnectionSpecificSerializationIsBool = unsigned(decltype(TestHasConnectionSpecificSerializationIsBool<NetSerializerImpl>(nullptr))::Value),

		HasDynamicStateIsPresent = unsigned(decltype(TestHasDynamicStateIsPresent<NetSerializerImpl>(nullptr))::Value),
		HasDynamicStateIsBool = unsigned(decltype(TestHasDynamicStateIsBool<NetSerializerImpl>(nullptr))::Value),

		UseDefaultDeltaIsPresent = unsigned(decltype(TestUseDefaultDeltaIsPresent<NetSerializerImpl>(nullptr))::Value),
		UseDefaultDeltaIsBool = unsigned(decltype(TestUseDefaultDeltaIsBool<NetSerializerImpl>(nullptr))::Value),

		HasConfigType = unsigned(decltype(TestHasConfigType<NetSerializerImpl>(nullptr))::Value),
		HasSourceType = unsigned(decltype(TestHasSourceType<NetSerializerImpl>(nullptr))::Value),
		HasQuantizedType = unsigned(decltype(TestHasQuantizedType<NetSerializerImpl>(nullptr))::Value),
		SourceTypeIsPod = IsSourceTypePod<NetSerializerImpl>(nullptr),
		QuantizedTypeIsPod = IsQuantizedTypePod<NetSerializerImpl>(nullptr),

		HasDefaultConfig = TestHasDefaultConfig<NetSerializerImpl>(nullptr),

		HasSerialize = unsigned(decltype(TestHasSerialize<NetSerializerImpl>(nullptr))::Value),
		HasDeserialize = unsigned(decltype(TestHasDeserialize<NetSerializerImpl>(nullptr))::Value),
		HasSerializeDelta = unsigned(decltype(TestHasSerializeDelta<NetSerializerImpl>(nullptr))::Value),
		HasDeserializeDelta = unsigned(decltype(TestHasDeserializeDelta<NetSerializerImpl>(nullptr))::Value),
		HasQuantize = unsigned(decltype(TestHasQuantize<NetSerializerImpl>(nullptr))::Value),
		HasDequantize = unsigned(decltype(TestHasDequantize<NetSerializerImpl>(nullptr))::Value),
		HasIsEqual = unsigned(decltype(TestHasIsEqual<NetSerializerImpl>(nullptr))::Value),
		HasValidate = unsigned(decltype(TestHasValidate<NetSerializerImpl>(nullptr))::Value),
		HasFreeDynamicState = unsigned(decltype(TestHasFreeDynamicState<NetSerializerImpl>(nullptr))::Value),
		HasCloneDynamicState = unsigned(decltype(TestHasCloneDynamicState<NetSerializerImpl>(nullptr))::Value),
		HasCollectNetReferences = unsigned(decltype(TestHasCollectNetReferences<NetSerializerImpl>(nullptr))::Value),
		HasApply = unsigned(decltype(TestHasApply<NetSerializerImpl>(nullptr))::Value),
	};

public:
	template<typename T = void, typename U = typename TEnableIf<HasVersion, T>::Type, bool V = true>
	static constexpr uint32 GetVersion() { return NetSerializerImpl::Version; }

	template<typename T = void, typename U = typename TEnableIf<!HasVersion, T>::Type, char V = 0>
	static constexpr uint32 GetVersion() { return ~0U; }

	template<typename T = void, typename U = typename TEnableIf<HasCustomNetReferenceIsBool, T>::Type, bool V = true>
	static constexpr bool HasCustomNetReference() { return NetSerializerImpl::bHasCustomNetReference; }

	template<typename T = void, typename U = typename TEnableIf<!HasCustomNetReferenceIsBool, T>::Type, char V = 0>
	static constexpr bool HasCustomNetReference() { return false; }

	template<typename T = void, typename U = typename TEnableIf<UseSerializerIsEqualIsBool && HasIsEqual, T>::Type, bool V = true>
	static constexpr bool UseSerializerIsEqual() { return NetSerializerImpl::bUseSerializerIsEqual; }

	template<typename T = void, typename U = typename TEnableIf<!(UseSerializerIsEqualIsBool && HasIsEqual), T>::Type, char V = 0>
	static constexpr bool UseSerializerIsEqual() { return false; }

	template<typename T = void, typename U = typename TEnableIf<IsForwardingSerializerIsBool, T>::Type, bool V = true>
	static constexpr bool IsForwardingSerializer() { return NetSerializerImpl::bIsForwardingSerializer; }

	template<typename T = void, typename U = typename TEnableIf<!IsForwardingSerializerIsBool, T>::Type, char V = 0>
	static constexpr bool IsForwardingSerializer() { return false; }

	template<typename T = void, typename U = typename TEnableIf<HasConnectionSpecificSerializationIsBool, T>::Type, bool V = true>
	static constexpr bool HasConnectionSpecificSerialization() { return NetSerializerImpl::bHasConnectionSpecificSerialization; }

	template<typename T = void, typename U = typename TEnableIf<!HasConnectionSpecificSerializationIsBool, T>::Type, char V = 0>
	static constexpr bool HasConnectionSpecificSerialization() { return false; }

	template<typename T = void, typename U = typename TEnableIf<HasDynamicStateIsBool, T>::Type, bool V = true>
	static constexpr bool HasDynamicState() { return NetSerializerImpl::bHasDynamicState; }

	template<typename T = void, typename U = typename TEnableIf<!HasDynamicStateIsBool, T>::Type, char V = 0>
	static constexpr bool HasDynamicState() { return false; }

	template<typename T = void, typename U = typename TEnableIf<UseDefaultDeltaIsBool, T>::Type, bool V = true>
	static constexpr bool ShouldUseDefaultDelta() { return NetSerializerImpl::bUseDefaultDelta; }

	template<typename T = void, typename U = typename TEnableIf<!UseDefaultDeltaIsBool, T>::Type, char V = 0>
	static constexpr bool ShouldUseDefaultDelta() { return true; }

	template<typename T = void, typename U = typename TEnableIf<HasSerialize, T>::Type, bool V = true>
	static NetSerializeFunction GetSerializeFunction() { return NetSerializerImpl::Serialize; }

	template<typename T = void, typename U = typename TEnableIf<!HasSerialize, T>::Type, char V = 0>
	static NetSerializeFunction GetSerializeFunction() { return NetSerializeFunction(0); }

	template<typename T = void, typename U = typename TEnableIf<HasDeserialize, T>::Type, bool V = true>
	static NetDeserializeFunction GetDeserializeFunction() { return NetSerializerImpl::Deserialize; }

	template<typename T = void, typename U = typename TEnableIf<!HasDeserialize, T>::Type, char V = 0>
	static NetDeserializeFunction GetDeserializeFunction() { return NetDeserializeFunction(0); }

	// Provide a default SerializeDelta implementation if needed. The default will compare the value with previous value and write an extra bit and forward to Serialize if the value differs
	// It is possible to opt out by adding static constexpr bool bUseDefaultDelta = false; in the serializer declaration
	template<typename T = void, typename U = typename TEnableIf<HasSerializeDelta, T>::Type, bool V = true>
	static NetSerializeDeltaFunction GetSerializeDeltaFunction() { return NetSerializerImpl::SerializeDelta; }

	template<typename T = void, typename U = typename TEnableIf<!HasSerializeDelta && !ShouldUseDefaultDelta(), T>::Type, int V = 0>
	static NetSerializeDeltaFunction GetSerializeDeltaFunction() { return NetSerializeDeltaDefault<NetSerializerImpl::Serialize>; }

	template<typename T = void, typename U = typename TEnableIf<!HasSerializeDelta && ShouldUseDefaultDelta() && HasIsEqual, T>::Type, char V = 0>
	static NetSerializeDeltaFunction GetSerializeDeltaFunction() { return NetSerializeDeltaDefault<NetSerializerImpl::Serialize, NetSerializerImpl::IsEqual>; }

	template<typename T = void, typename U = typename TEnableIf<!HasSerializeDelta && ShouldUseDefaultDelta() && !HasIsEqual, T>::Type, unsigned char V = 0>
	static NetSerializeDeltaFunction GetSerializeDeltaFunction() { return NetSerializeDeltaDefault<NetSerializerImpl::Serialize, NetIsEqualDefault<typename NetSerializerImpl::SourceType> >; }

	// Provide a default DeserializeDelta implementation if needed. The default will call Deserialize.
	template<typename T = void, typename U = typename TEnableIf<HasDeserializeDelta, T>::Type, bool V = true>
	static NetDeserializeDeltaFunction GetDeserializeDeltaFunction(const T* = nullptr) { return NetSerializerImpl::DeserializeDelta; }

	template<typename T = void, typename U = typename TEnableIf<!HasDeserializeDelta && !ShouldUseDefaultDelta(), T>::Type, int V = 0>
	static NetDeserializeDeltaFunction GetDeserializeDeltaFunction(const void* = nullptr) { return NetDeserializeDeltaDefault<NetSerializerImpl::Deserialize>; }

	template<typename T = void, typename U = typename TEnableIf<!HasDeserializeDelta && ShouldUseDefaultDelta() && (HasCloneDynamicState && HasFreeDynamicState), T>::Type, char V = 0>
	static NetDeserializeDeltaFunction GetDeserializeDeltaFunction(const void* = nullptr) { return NetDeserializeDeltaDefault<GetQuantizedTypeSize(), NetSerializerImpl::Deserialize, NetSerializerImpl::FreeDynamicState, NetSerializerImpl::CloneDynamicState>; }

	template<typename T = void, typename U = typename TEnableIf<!HasDeserializeDelta && ShouldUseDefaultDelta() && !(HasCloneDynamicState && HasFreeDynamicState), T>::Type, unsigned char V = 0>
	static NetDeserializeDeltaFunction GetDeserializeDeltaFunction(const void* = nullptr) { return NetDeserializeDeltaDefault<GetQuantizedTypeSize(), NetSerializerImpl::Deserialize, NetFreeDynamicStateFunction(nullptr), NetCloneDynamicStateFunction(nullptr)>; }

	// Provide a default Quantize implementation if needed. The default will copy the value.
	template<typename T = void, typename U = typename TEnableIf<HasQuantize, T>::Type, bool V = true>
	static NetQuantizeFunction GetQuantizeFunction() { return NetSerializerImpl::Quantize; }

	template<typename T = void, typename U = typename TEnableIf<!HasQuantize, T>::Type, char V = 0>
	static NetQuantizeFunction GetQuantizeFunction() { return NetQuantizeDefault<typename NetSerializerImpl::SourceType>; }

	// Provide a default Dequantize implementation if needed. The default will copy the value.
	template<typename T = void, typename U = typename TEnableIf<HasDequantize, T>::Type, bool V = true>
	static NetDequantizeFunction GetDequantizeFunction() { return NetSerializerImpl::Dequantize; }

	template<typename T = void, typename U = typename TEnableIf<!HasDequantize, T>::Type, char V = 0>
	static NetDequantizeFunction GetDequantizeFunction() { return NetDequantizeDefault<typename NetSerializerImpl::SourceType>; }

	// Provide a default IsEqual implementation if needed. The default will call the equality operator.
	template<typename T = void, typename U = typename TEnableIf<HasIsEqual, T>::Type, bool V = true>
	static NetIsEqualFunction GetIsEqualFunction() { return NetSerializerImpl::IsEqual; }

	template<typename T = void, typename U = typename TEnableIf<!HasIsEqual, T>::Type, char V = 0>
	static NetIsEqualFunction GetIsEqualFunction() { return NetIsEqualDefault<typename NetSerializerImpl::SourceType>; }

	// Provide a default Validate implementation if needed. The default will not perform any validation.
	template<typename T = void, typename U = typename TEnableIf<HasValidate, T>::Type, bool V = true>
	static NetValidateFunction GetValidateFunction() { return NetSerializerImpl::Validate; }

	template<typename T = void, typename U = typename TEnableIf<!HasValidate, T>::Type, char V = 0>
	static NetValidateFunction GetValidateFunction() { return NetValidateDefault<>; }

	template<typename T = void, typename U = typename TEnableIf<HasCollectNetReferences, T>::Type, bool V = true>
	static NetCollectNetReferencesFunction GetCollectNetReferencesFunction() { return NetSerializerImpl::CollectNetReferences; }

	template<typename T = void, typename U = typename TEnableIf<!HasCollectNetReferences, T>::Type, char V = 0>
	static NetCollectNetReferencesFunction GetCollectNetReferencesFunction() { return NetCollectNetReferencesFunction(nullptr); }

	template<typename T = void, typename U = typename TEnableIf<HasApply, T>::Type, bool V = true>
	static NetApplyFunction GetApplyFunction() { return NetSerializerImpl::Apply; }

	template<typename T = void, typename U = typename TEnableIf<!HasApply, T>::Type, char V = 0>
	static NetApplyFunction GetApplyFunction() { return NetApplyFunction(nullptr); }

	// CloneDynamicState
	template<typename T = void, typename U = typename TEnableIf<HasCloneDynamicState && (IsForwardingSerializer() || HasDynamicState()), T>::Type, bool V = true>
	static NetCloneDynamicStateFunction GetCloneDynamicStateFunction() { return NetSerializerImpl::CloneDynamicState; }

	template<typename T = void, typename U = typename TEnableIf<!(HasCloneDynamicState && (IsForwardingSerializer() || HasDynamicState())), T>::Type, char V = 0>
	static NetCloneDynamicStateFunction GetCloneDynamicStateFunction() { return NetCloneDynamicStateFunction(nullptr); }

	// FreeDynamicState
	template<typename T = void, typename U = typename TEnableIf<HasFreeDynamicState && (IsForwardingSerializer() || HasDynamicState()), T>::Type, bool V = true>
	static NetFreeDynamicStateFunction GetFreeDynamicStateFunction() { return NetSerializerImpl::FreeDynamicState; }

	template<typename T = void, typename U = typename TEnableIf<!(HasFreeDynamicState && (IsForwardingSerializer() || HasDynamicState())), T>::Type, char V = 0>
	static NetFreeDynamicStateFunction GetFreeDynamicStateFunction() { return NetFreeDynamicStateFunction(nullptr); }

	// DefaultConfig
	template<typename T = void, typename U = typename TEnableIf<HasConfigType && HasDefaultConfig, T>::Type, bool V = true>
	static const FNetSerializerConfig* GetDefaultConfig() { return &NetSerializerImpl::DefaultConfig; }

	template<typename T = void, typename U = typename TEnableIf<!(HasConfigType && HasDefaultConfig), T>::Type, char V = 0>
	static constexpr FNetSerializerConfig* GetDefaultConfig() { return nullptr; }

	// Type sizes and alignments
	template<typename T = void, typename U = typename TEnableIf<HasConfigType, T>::Type, bool V = true>
	static constexpr uint32 GetConfigTypeSize() { return sizeof(typename NetSerializerImpl::ConfigType); }

	template<typename T = void, typename U = typename TEnableIf<!HasConfigType, T>::Type, char V = 0>
	static constexpr uint32 GetConfigTypeSize() { return 0; }

	template<typename T = void, typename U = typename TEnableIf<HasConfigType, T>::Type, bool V = true>
	static constexpr uint32 GetConfigTypeAlignment() { return alignof(typename NetSerializerImpl::ConfigType); }

	template<typename T = void, typename U = typename TEnableIf<!HasConfigType, T>::Type, char V = 0>
	static constexpr uint32 GetConfigTypeAlignment() { return 1; }

	template<typename T = void, typename U = typename TEnableIf<HasQuantizedType, T>::Type, bool V = true, bool W = true>
	static constexpr uint32 GetQuantizedTypeSize() { return sizeof(typename NetSerializerImpl::QuantizedType); }

	template<typename T = void, typename U = typename TEnableIf<!HasQuantizedType && HasSourceType, T>::Type, bool V = true, char W = 0>
	static constexpr uint32 GetQuantizedTypeSize() { return std::is_same_v<void, typename NetSerializerImpl::SourceType> ? uint32(0) : sizeof(std::conditional_t<std::is_same_v<void, typename NetSerializerImpl::SourceType>, uint8, typename NetSerializerImpl::SourceType>); }

	template<typename T = void, typename U = typename TEnableIf<!(HasSourceType || HasQuantizedType), T>::Type, char V = 0>
	static constexpr uint32 GetQuantizedTypeSize() { return 0; }

	template<typename T = void, typename U = typename TEnableIf<HasQuantizedType, T>::Type, bool V = true, bool W = true>
	static constexpr uint32 GetQuantizedTypeAlignment() { return alignof(typename NetSerializerImpl::QuantizedType); }

	template<typename T = void, typename U = typename TEnableIf<!HasQuantizedType && HasSourceType, T>::Type, bool V = true, char W = 0>
	static constexpr uint32 GetQuantizedTypeAlignment() { return alignof(std::conditional_t<std::is_same_v<void, typename NetSerializerImpl::SourceType>, uint8, typename NetSerializerImpl::SourceType>); }

	template<typename T = void, typename U = typename TEnableIf<!(HasSourceType || HasQuantizedType), T>::Type, char V = 0>
	static constexpr uint32 GetQuantizedTypeAlignment() { return 1; }

	static constexpr ENetSerializerTraits GetTraits()
	{ 
		ENetSerializerTraits Traits = ENetSerializerTraits::None;
		Traits |= (IsForwardingSerializer() ? ENetSerializerTraits::IsForwardingSerializer : ENetSerializerTraits::None);
		Traits |= (HasConnectionSpecificSerialization() ? ENetSerializerTraits::HasConnectionSpecificSerialization : ENetSerializerTraits::None);
		Traits |= (HasCustomNetReference() ? ENetSerializerTraits::HasCustomNetReference : ENetSerializerTraits::None);
		Traits |= (HasDynamicState() ? ENetSerializerTraits::HasDynamicState : ENetSerializerTraits::None);
		Traits |= (UseSerializerIsEqual() ? ENetSerializerTraits::UseSerializerIsEqual : ENetSerializerTraits::None);
		Traits |= (HasApply ? ENetSerializerTraits::HasApply : ENetSerializerTraits::None);

		return Traits;
	}

	static void Validate()
	{
		static_assert(HasVersion, "FNetSerializer must have a 'static constexpr uint32 Version' member.");

		static_assert(!IsForwardingSerializerIsPresent || IsForwardingSerializerIsBool, "FNetSerializer bIsForwardingSerializer member should be declared as 'static constexpr bool bIsForwardingSerializer'.");
		static_assert(!HasConnectionSpecificSerializationIsPresent || HasConnectionSpecificSerializationIsBool, "FNetSerializer bHasConnectionSpecificSerialization member should be declared as 'static constexpr bool bHasConnectionSpecificSerialization'.");
		static_assert(!HasCustomNetReferenceIsPresent || HasCustomNetReferenceIsBool, "FNetSerializer bHasCustomNetReference member should be declared as 'static constexpr bool bHasCustomNetReference'.");
		static_assert(!HasDynamicStateIsPresent || HasDynamicStateIsBool, "FNetSerializer bHasDynamicState member should be declared as 'static constexpr bool bHasDynamicState'.");

		static_assert(HasConfigType, "FNetSerializer must have a ConfigType.");
		static_assert(GetConfigTypeSize() <= TNumericLimits<decltype(FNetSerializer::ConfigTypeSize)>::Max() , "FNetSerializer NetSerializerConfig type is too large.");
		static_assert(GetConfigTypeAlignment() <= TNumericLimits<decltype(FNetSerializer::ConfigTypeAlignment)>::Max() , "FNetSerializer NetSerializerConfig type has too large alignment requirements.");

		static_assert(HasSourceType, "FNetSerializer must have a SourceType.");
		static_assert(!HasQuantizedType || QuantizedTypeIsPod, "QuantizedType in FNetSerializer must be POD.");
		static_assert(GetQuantizedTypeSize() <= TNumericLimits<decltype(FNetSerializer::QuantizedTypeSize)>::Max() , "FNetSerializer quantized type is too large.");
		static_assert(GetQuantizedTypeAlignment() <= TNumericLimits<decltype(FNetSerializer::QuantizedTypeAlignment)>::Max() , "FNetSerializer quantized type has too large alignment requirements.");

		static_assert(HasSerialize, "FNetSerializer must implement Serialize.");
		static_assert(HasDeserialize, "FNetSerializer must implement Deserialize.");

		static_assert(HasSerializeDelta == HasDeserializeDelta, "FNetSerializer should implement both SerializeDelta and DeserializeDelta or none of them.");

		static_assert(HasQuantize || SourceTypeIsPod, "FNetSerializer must implement Quantize and Dequantize when SourceType isn't POD.");
		static_assert(!HasQuantizedType || (HasQuantize && HasDequantize), "FNetSerializer must implement Quantize and Dequantize when it has a QuantizedType.");
		static_assert(HasQuantize == HasDequantize, "FNetSerializer must implement both Quantize and Dequantize or none of them.");
		static_assert(!HasQuantize || HasIsEqual, "FNetSerializer must implement IsEqual when it has Quantize.");

		static_assert(!HasCustomNetReference() || (HasCustomNetReference() && HasCollectNetReferences), "FNetSerializer with bHasCustomNetReference = true must implement CollectNetReferences method.");

		static_assert(!HasDynamicStateIsBool || (HasFreeDynamicState && HasCloneDynamicState), "FNetSerializer must implement CloneDynamicState and FreeDynamicState when it has dynamic state.");

		static_assert(!UseDefaultDeltaIsPresent || UseDefaultDeltaIsBool, "FNetSerializer bUseDefaultDelta member should be declared as 'static constexpr bool bUseDefaultDelta'.");

		ValidateForwardingSerializer();
	}

private:
	template<typename T = void, typename U = typename TEnableIf<IsForwardingSerializer(), T>::Type, bool V = true>
	static void ValidateForwardingSerializer()
	{
		static_assert(HasSerialize, "Forwarding FNetSerializer must implement Serialize.");
		static_assert(HasDeserialize, "Forwarding FNetSerializer must implement Deserialize.");
		static_assert(HasSerializeDelta, "Forwarding FNetSerializer must implement SerializeDelta.");
		static_assert(HasDeserializeDelta, "Forwarding FNetSerializer must implement DeserializeDelta.");
		static_assert(HasQuantize, "Forwarding FNetSerializer must implement Quantize.");
		static_assert(HasDequantize, "Forwarding FNetSerializer must implement Dequantize.");
		static_assert(HasIsEqual, "Forwarding FNetSerializer must implement IsEqual.");
		static_assert(HasValidate, "Forwarding FNetSerializer must implement Validate.");
		static_assert(HasCloneDynamicState, "Forwarding FNetSerializer must implement CloneDynamicState.");
		static_assert(HasFreeDynamicState, "Forwarding FNetSerializer must implement FreeDynamicState.");
		static_assert(HasCollectNetReferences, "Forwarding FNetSerializer must implement CollectNetReferences.");
	}

	template<typename T = void, typename U = typename TEnableIf<!IsForwardingSerializer(), T>::Type, char V = 0>
	static void ValidateForwardingSerializer()
	{
	}
};

}
