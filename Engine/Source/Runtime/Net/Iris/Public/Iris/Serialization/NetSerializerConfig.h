// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "NetSerializerConfig.generated.h"

enum class ENetSerializerConfigTraits : uint32
{
	None = 0,
	NeedDestruction = 1 << 0,
};
ENUM_CLASS_FLAGS(ENetSerializerConfigTraits);

USTRUCT()
struct FNetSerializerConfig
{
	GENERATED_BODY()

	FNetSerializerConfig()
	: ConfigTraits(ENetSerializerConfigTraits::None)
	{}

	IRISCORE_API virtual ~FNetSerializerConfig();

	ENetSerializerConfigTraits ConfigTraits;
};
