// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

namespace UE::Net
{

/**
 * Helper class for registering NetSerializers.
 * Override OnPreFreezeNetSerializerRegistry() to register your serializer
 * and OnPostFreezeNetSerializerRegistry() to perform any additional fixup
 * needed, such as creating a descriptor for a struct that you forward your
 * NetSerializer's calls to.
 * The virtual functions OnPreFreezeNetSerializerRegistry and OnPostFreezeNetSerializerRegistry will be called zero or one times each.
 */
class FNetSerializerRegistryDelegates
{
public:

	enum EFlags : uint32
	{
		None = 0U,
		ShouldBindLoadedModulesUpdatedDelegate = 1U,
	};

	IRISCORE_API FNetSerializerRegistryDelegates();

	explicit IRISCORE_API FNetSerializerRegistryDelegates(EFlags Flags);

	/** Implement a destructor to unregister your NetSerializers. */
	IRISCORE_API virtual ~FNetSerializerRegistryDelegates();

protected:
	/**
	 * Pre freeze can be called before there are any serializers registered.
	 * At this point it's fine to register custom serializers for structs, but if you
	 * need a descriptor for a struct as a helper for your serializer you need to wait
	 * until post freeze.
	 */
	IRISCORE_API virtual void OnPreFreezeNetSerializerRegistry();

	/**
	 * Post freeze is called after all loaded modules, including this one, has
	 * registered their serializers. At this point you should be able to get
	 * a descriptor for a struct that contains types that your module depends on.
	 */
	IRISCORE_API virtual void OnPostFreezeNetSerializerRegistry();

	/**
	 * OnLoadedModulesUpdated delegate will be called every time a module has been loaded or unloaded
	 * It gives the serializer an opportunity to update cached data that could be affected by loading or unloading modules.
	 * NOTE: It will only be called if the EFlags::ShouldBindLoadedModulesUpdatedDelegate is set
	 */
	IRISCORE_API virtual void OnLoadedModulesUpdated();


private:

	void PreFreezeNetSerializerRegistry();
	void PostFreezeNetSerializerRegistry();
	void LoadedModulesUpdated();

	FDelegateHandle PreFreezeDelegate;
	FDelegateHandle PostFreezeDelegate;
	FDelegateHandle LoadedModulesUpdatedDelegate;
};

}
