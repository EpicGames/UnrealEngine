// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "EnumNetSerializers.h"
#include "FloatNetSerializers.h"
#include "GuidNetSerializer.h"
#include "InstancedStructNetSerializer.h"
#include "IntNetSerializers.h"
#include "IntRangeNetSerializers.h"
#include "IntNetSerializers.h"
#include "ObjectNetSerializer.h"
#include "PackedIntNetSerializers.h"
#include "PackedVectorNetSerializers.h"
#include "QuatNetSerializers.h"
#include "RotatorNetSerializers.h"
#include "SoftObjectNetSerializers.h"
#include "StringNetSerializers.h"
#include "UintNetSerializers.h"
#include "UintRangeNetSerializers.h"
#include "VectorNetSerializers.h"
#include "NetSerializers.generated.h"

namespace UE::Net
{
	struct FReplicationStateDescriptor;
}

USTRUCT()
struct FBoolNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FStructNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

	IRISCORE_API FStructNetSerializerConfig();
	IRISCORE_API ~FStructNetSerializerConfig();

	FStructNetSerializerConfig(const FStructNetSerializerConfig&) = delete;
	FStructNetSerializerConfig(FStructNetSerializerConfig&&) = delete;

	FStructNetSerializerConfig& operator=(const FStructNetSerializerConfig&) = delete;
	FStructNetSerializerConfig& operator=(FStructNetSerializerConfig&&) = delete;

	TRefCountPtr<const UE::Net::FReplicationStateDescriptor> StateDescriptor;
};

template<>
struct TStructOpsTypeTraits<FStructNetSerializerConfig> : public TStructOpsTypeTraitsBase2<FStructNetSerializerConfig>
{
	enum
	{
		WithCopy = false
	};
};

/**
 * The NopNetSerializer have all the serializer functions implemented
 * as no-ops. The main purpose of this serializer is to allow adding
 * a non-replicated member as part of a ReplicationStateDescriptor
 * without incurring a bandwidth cost. This allows systems, such as 
 * prioritization and filtering, to access the source data given an
 * instance protocol and information regarding the member, for example
 * by looking for a particular RepTag.
 */
USTRUCT()
struct FNopNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FBoolNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FStructNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FNopNetSerializer, IRISCORE_API);

}
