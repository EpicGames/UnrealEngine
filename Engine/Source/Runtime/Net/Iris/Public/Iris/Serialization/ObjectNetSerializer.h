// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "UObject/ObjectMacros.h"

#include "ObjectNetSerializer.generated.h"

namespace UE::Net
{
	class FNetObjectReference;
}

USTRUCT()
struct FObjectNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UClass> PropertyClass = nullptr;
};

USTRUCT()
struct FObjectPtrNetSerializerConfig : public FObjectNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FWeakObjectNetSerializerConfig : public FObjectNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FScriptInterfaceNetSerializerConfig : public FObjectNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

IRISCORE_API FNetRefHandle ReadNetRefHandle(FNetSerializationContext& Context);
IRISCORE_API void WriteNetRefHandle(FNetSerializationContext& Context, const FNetRefHandle Handle);

IRISCORE_API void ReadFullNetObjectReference(FNetSerializationContext& Context, FNetObjectReference& Reference);
IRISCORE_API void WriteFullNetObjectReference(FNetSerializationContext& Context, const FNetObjectReference& Reference);

UE_NET_DECLARE_SERIALIZER(FObjectNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FObjectPtrNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FWeakObjectNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FScriptInterfaceNetSerializer, IRISCORE_API);

// Public type that can be used when storage for a quantized reference is needed. This should only be needed
// in specific cases of forwarding net serializers.
struct FObjectNetSerializerQuantizedReferenceStorage
{
	alignas(8) uint8 Storage[32];
};

}
