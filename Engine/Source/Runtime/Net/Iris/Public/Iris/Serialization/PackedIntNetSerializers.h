// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "PackedIntNetSerializers.generated.h"

USTRUCT()
struct FPackedInt64NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FPackedUint64NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FPackedInt32NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FPackedUint32NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FPackedInt64NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FPackedUint64NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FPackedInt32NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FPackedUint32NetSerializer, IRISCORE_API);

}
