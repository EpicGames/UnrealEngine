// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "PackedVectorNetSerializers.generated.h"

USTRUCT()
struct FVectorNetQuantizeNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FVectorNetQuantize10NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FVectorNetQuantize100NetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FVectorNetQuantizeNormalNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

// For FVector_NetQuantize. Replicates the components as rounded integers. No clamping- if the values are very large they will be sent as full floats.
UE_NET_DECLARE_SERIALIZER(FVectorNetQuantizeNetSerializer, IRISCORE_API);

// For FVector_NetQuantize10. Replicates components scaled by 8 as rounded integers. No clamping- if the values are very large they will be sent as full floats.
UE_NET_DECLARE_SERIALIZER(FVectorNetQuantize10NetSerializer, IRISCORE_API);

// For FVector_NetQuantize100. Replicates components scaled by 128 as rounded integers. No clamping- if the values are very large they will be sent as full floats.
UE_NET_DECLARE_SERIALIZER(FVectorNetQuantize100NetSerializer, IRISCORE_API);

// For FVector_NetQuantizeNormal. Replicates components with 16 bits per component.
UE_NET_DECLARE_SERIALIZER(FVectorNetQuantizeNormalNetSerializer, IRISCORE_API);

}
