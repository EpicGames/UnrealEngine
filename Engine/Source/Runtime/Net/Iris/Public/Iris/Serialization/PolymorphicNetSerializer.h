// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Containers/Map.h"
#include "UObject/NameTypes.h"

#include "PolymorphicNetSerializer.generated.h"

class UScriptStruct;
namespace UE::Net
{
	struct FReplicationStateDescriptor;
}

namespace UE::Net::Private
{
	class FTestPolymorphicArrayStructNetSerializerFixture;
}

namespace UE::Net
{

struct FPolymorphicNetSerializerScriptStructCache
{
	struct FTypeInfo
	{
		FName StructName;
		const UScriptStruct* ScriptStruct = nullptr;
		TRefCountPtr<const FReplicationStateDescriptor> Descriptor;
	};

	IRISCORE_API void InitForType(const UScriptStruct* InScriptStruct);

	inline const FTypeInfo* GetTypeInfo(const UScriptStruct* ScriptStruct) const;
	inline const FTypeInfo* GetTypeInfo(FName StructName) const;
	inline bool CanHaveNetReferences() const { return EnumHasAnyFlags(CommonTraits, EReplicationStateTraits::HasObjectReference); }

	/** Identity name used on the wire for the given ScriptStruct. Uses GetPathName() so it stays unique across modules. */
	IRISCORE_API static FName MakeStructName(const UScriptStruct* ScriptStruct);

private:
	friend class UE::Net::Private::FTestPolymorphicArrayStructNetSerializerFixture;

	TMap<FName, FTypeInfo> RegisteredTypes;
	EReplicationStateTraits CommonTraits = EReplicationStateTraits::None;
	int32 EstimatedScriptStructCount = 4096;
};

}

USTRUCT()
struct FPolymorphicStructNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

	UE::Net::FPolymorphicNetSerializerScriptStructCache RegisteredTypes;
};

USTRUCT()
struct FPolymorphicArrayStructNetSerializerConfig : public FPolymorphicStructNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

inline const FPolymorphicNetSerializerScriptStructCache::FTypeInfo* FPolymorphicNetSerializerScriptStructCache::GetTypeInfo(FName StructName) const
{
	if (StructName.IsNone())
	{
		return nullptr;
	}
	return RegisteredTypes.Find(StructName);
}

inline const FPolymorphicNetSerializerScriptStructCache::FTypeInfo* FPolymorphicNetSerializerScriptStructCache::GetTypeInfo(const UScriptStruct* ScriptStruct) const
{
	return GetTypeInfo(MakeStructName(ScriptStruct));
}

}
