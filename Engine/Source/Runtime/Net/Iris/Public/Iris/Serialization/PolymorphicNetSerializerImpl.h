// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NetSerializer.h"
#include "PolymorphicNetSerializer.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/ReplicationState/ReplicationStateDescriptor.h"
#include "Iris/ReplicationSystem/NameTokenStore.h"
#include "Iris/ReplicationSystem/NetTokenStore.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Net/Core/Trace/NetTrace.h"
#include "UObject/NameTypes.h"

// NOTE: This file should not be included in a header

namespace UE::Net
{

class FNetReferenceCollector;
extern IRISCORE_API const FName NetError_PolymorphicStructNetSerializer_InvalidStructType;

}

namespace UE::Net::Private
{

// Wrapper to allow access to the internal context
struct FPolymorphicStructNetSerializerInternal
{
protected:
	IRISCORE_API static void* Alloc(FNetSerializationContext& Context, SIZE_T Size, SIZE_T Alignment);
	IRISCORE_API static void Free(FNetSerializationContext& Context, void* Ptr);
	IRISCORE_API static void CollectReferences(FNetSerializationContext& Context, UE::Net::FNetReferenceCollector& Collector, const FNetSerializerChangeMaskParam& OuterChangeMaskInfo, const uint8* RESTRICT SrcInternalBuffer,  const FReplicationStateDescriptor* Descriptor);
	IRISCORE_API static void CloneQuantizedState(FNetSerializationContext& Context, uint8* RESTRICT DstInternalBuffer, const uint8* RESTRICT SrcInternalBuffer, const FReplicationStateDescriptor* Descriptor);
};

struct FPolymorphicStructQuantizedData
{
	void* StructData = nullptr;
	FName StructName;
};

struct FPolymorphicStructQuantizedItem
{
	void* StructData = nullptr;
	FName StructName;
};

}

template <> struct TIsPODType<UE::Net::Private::FPolymorphicStructQuantizedData> { enum { Value = true }; };
template <> struct TIsPODType<UE::Net::Private::FPolymorphicStructQuantizedItem> { enum { Value = true }; };

namespace UE::Net
{

/**
  * TPolymorphicStructNetSerializerImpl
  *
  * Helper to implement serializers that requires dynamic polymorphism.
  * It can either be used to declare a typed serializer or be used as an internal helper.
  * ExternalSourceType is the class/struct that has the TSharedPtr<ExternalSourceItemType> data.
  * ExternalSourceItemType is the polymorphic struct type
  * GetItem is a function that will return a reference to the TSharedPtr<ExternalSourceItemType>
  *
  * !BIG DISCLAIMER:!
  *
  * This serializer was written to mimic the behavior seen in FGameplayAbilityTargetDataHandle and FGameplayEffectContextHandle 
  * which both are written with the intent of being used for RPCs and not being used for replicated properties and uses a TSharedPointer to hold the polymorphic struct
  *
  * That said, IF the serializer is used for replicated properties it has very specific requirements on the implementation of the SourceType to work correctly.
  *
  * 1. The sourcetype MUST provide a custom assignment operator performing a deep-copy/clone
  * 2. The sourcetype MUST define a comparison operator that compares the instance data of the stored ExternalSourceItemType
  * 3. TStructOpsTypeTraits::WithCopy and TStructOpsTypeTraits::WithIdenticalViaEquality must be specified
  *
  */
template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
struct TPolymorphicStructNetSerializerImpl : protected Private::FPolymorphicStructNetSerializerInternal
{
	using FQuantizedData = Private::FPolymorphicStructQuantizedData;

	// Traits
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasCustomNetReference = true; // We must support this as we do not know the type

	typedef ExternalSourceType SourceType;
	typedef FQuantizedData QuantizedType;
	typedef FPolymorphicStructNetSerializerConfig ConfigType;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

protected:
	typedef TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem> ThisType;

	typedef ExternalSourceItemType SourceItemType;
	typedef FPolymorphicNetSerializerScriptStructCache::FTypeInfo FTypeInfo;

	struct FSourceItemTypeDeleter
	{
		void operator()(SourceItemType* Object) const
		{
			check(Object);
			UScriptStruct* ScriptStruct = Object->GetScriptStruct();
			check(ScriptStruct);
			ScriptStruct->DestroyStruct(Object);
			FMemory::Free(Object);
		}
	};

	template <typename SerializerType>
	static void InitTypeCache()
	{
		FPolymorphicNetSerializerScriptStructCache* Cache = const_cast<FPolymorphicNetSerializerScriptStructCache*>(&SerializerType::DefaultConfig.RegisteredTypes);
		Cache->InitForType(SerializerType::SourceItemType::StaticStruct());
	}

private:
	static void InternalFreeItem(FNetSerializationContext& Context, const ConfigType& Config, QuantizedType& Value);
};


/**
  * TPolymorphicArrayStructNetSerializerImpl
  *
  * Helper to implement array serializers that requires dynamic polymorphism.
  * It can either be used to declare a typed serializer or be used as an internal helper.
  *
  * @See: TPolymorphicStructNetSerializerImpl for requirements on external data
  */
template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
struct TPolymorphicArrayStructNetSerializerImpl : protected Private::FPolymorphicStructNetSerializerInternal
{
	using FQuantizedItem = Private::FPolymorphicStructQuantizedItem;

	struct FQuantizedArray
	{
		FQuantizedItem* Items;
		uint32 NumItems;
	};

	// Traits
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasCustomNetReference = true; // We must support this as we do not know the type

	typedef ExternalSourceType SourceType;
	typedef FQuantizedArray QuantizedType;
	typedef ExternalSourceArrayItemType SourceArrayItemType;
	typedef FPolymorphicArrayStructNetSerializerConfig ConfigType;

	typedef FPolymorphicNetSerializerScriptStructCache::FTypeInfo FTypeInfo;

	static const uint32 ArrayItemBits = 8U;
	static const uint32 MaxArrayItems = (1U << ArrayItemBits) - 1U;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

	struct FSourceArrayItemTypeDeleter
	{
		inline void operator()(SourceArrayItemType* Object) const
		{
			check(Object);
			UScriptStruct* ScriptStruct = Object->GetScriptStruct();
			check(ScriptStruct);
			ScriptStruct->DestroyStruct(Object);
			FMemory::Free(Object);
		}
	};

	template <typename SerializerType>
	static void InitTypeCache()
	{
		FPolymorphicNetSerializerScriptStructCache* Cache = const_cast<FPolymorphicNetSerializerScriptStructCache*>(&SerializerType::DefaultConfig.RegisteredTypes);
		Cache->InitForType(SerializerType::SourceArrayItemType::StaticStruct());
	}

private:
	using FItemNetSerializer = TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, nullptr>;

	// Allocate storage for the item array
	static void InternalAllocateItemArray(FNetSerializationContext& Context, QuantizedType& Value, uint32 NumItems);

	// Free allocated storage for the item array, including allocated struct data
	static void InternalFreeItemArray(FNetSerializationContext& Context, QuantizedType& Value, const FPolymorphicArrayStructNetSerializerConfig& Config);
};

/** TPolymorphicStructNetSerializerImpl */
template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
void TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(Value.StructName);
	if (Writer.WriteBool(TypeInfo != nullptr))
	{
		CA_ASSUME(TypeInfo != nullptr);
		FNetTokenStore* TokenStore = Context.GetNetTokenStore();
		FNameTokenStore* NameStore = TokenStore->GetDataStore<FNameTokenStore>();
		const FNetToken NameToken = NameStore->GetOrCreateToken(Value.StructName);
		TokenStore->WriteNetTokenWithKnownType<FNameTokenStore>(Context, NameToken);
		FNetTokenStore::AppendExport(Context, NameToken);
		UE::Net::FReplicationStateOperations::Serialize(Context, static_cast<const uint8*>(Value.StructData), TypeInfo->Descriptor);
	}
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
void TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	InternalFreeItem(Context, Config, Target);

	QuantizedType TempValue = {};
	if (const bool bIsValidType = Reader.ReadBool())
	{
		FNetTokenStore* TokenStore = Context.GetNetTokenStore();
		const FNetToken NameToken = TokenStore->ReadNetTokenWithKnownType<FNameTokenStore>(Context);
		FNameTokenStore* NameStore = TokenStore->GetDataStore<FNameTokenStore>();
		const FName StructName = NameStore->ResolveToken(NameToken, Context.GetRemoteNetTokenStoreState());
		if (const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(StructName))
		{
			const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;

			// Allocate storage and read struct data
			TempValue.StructData = FPolymorphicStructNetSerializerInternal::Alloc(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
			TempValue.StructName = StructName;

			FMemory::Memzero(TempValue.StructData, Descriptor->InternalSize);
			FReplicationStateOperations::Deserialize(Context, static_cast<uint8*>(TempValue.StructData), Descriptor);
		}
		else
		{
			Context.SetError(NetError_PolymorphicStructNetSerializer_InvalidStructType);
			// Fall through to clear the target
		}
	}

	Target = TempValue;
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
void TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	// If prev has the same type we can delta-compress
	if (Writer.WriteBool(Value.StructName == PrevValue.StructName))
	{
		const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(Value.StructName);
		if (TypeInfo != nullptr)
		{
			UE::Net::FReplicationStateOperations::SerializeDelta(Context, static_cast<const uint8*>(Value.StructData), static_cast<const uint8*>(PrevValue.StructData), TypeInfo->Descriptor);
		}
	}
	else
	{
		const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(Value.StructName);
		if (Writer.WriteBool(TypeInfo != nullptr))
		{
			CA_ASSUME(TypeInfo != nullptr);
			FNetTokenStore* TokenStore = Context.GetNetTokenStore();
			FNameTokenStore* NameStore = TokenStore->GetDataStore<FNameTokenStore>();
			const FNetToken NameToken = NameStore->GetOrCreateToken(Value.StructName);
			TokenStore->WriteNetTokenWithKnownType<FNameTokenStore>(Context, NameToken);
			FNetTokenStore::AppendExport(Context, NameToken);
			UE::Net::FReplicationStateOperations::Serialize(Context, static_cast<const uint8*>(Value.StructData), TypeInfo->Descriptor);
		}
	}
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
void TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	InternalFreeItem(Context, Config, Target);

	QuantizedType TempValue = {};

	// If prev has the same type we can delta-compress
	if (const bool bIsSameType = Reader.ReadBool())
	{
		const FName StructName = PrevValue.StructName;
		if (const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(StructName))
		{
			const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;

			// Allocate storage and read struct data
			TempValue.StructData = FPolymorphicStructNetSerializerInternal::Alloc(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
			TempValue.StructName = StructName;

			FMemory::Memzero(TempValue.StructData, Descriptor->InternalSize);
			FReplicationStateOperations::DeserializeDelta(Context, static_cast<uint8*>(TempValue.StructData), static_cast<uint8*>(PrevValue.StructData), Descriptor);
		}
	}
	else
	{
		if (const bool bIsValidType = Reader.ReadBool())
		{
			FNetTokenStore* TokenStore = Context.GetNetTokenStore();
			const FNetToken NameToken = TokenStore->ReadNetTokenWithKnownType<FNameTokenStore>(Context);
			FNameTokenStore* NameStore = TokenStore->GetDataStore<FNameTokenStore>();
			const FName StructName = NameStore->ResolveToken(NameToken, Context.GetRemoteNetTokenStoreState());
			if (const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(StructName))
			{
				const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;

				// Allocate storage and read struct data
				TempValue.StructData = FPolymorphicStructNetSerializerInternal::Alloc(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
				TempValue.StructName = StructName;

				FMemory::Memzero(TempValue.StructData, Descriptor->InternalSize);
				FReplicationStateOperations::Deserialize(Context, static_cast<uint8*>(TempValue.StructData), Descriptor);
			}
			else
			{
				Context.SetError(NetError_PolymorphicStructNetSerializer_InvalidStructType);
				// Fall through to clear the target
			}
		}
	}

	Target = TempValue;
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
void TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	SourceType& SourceValue = *reinterpret_cast<SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	InternalFreeItem(Context, Config, TargetValue);

	const TSharedPtr<SourceItemType>& Item = GetItem(SourceValue);
	const UScriptStruct* ScriptStruct = Item.IsValid() ? Item->GetScriptStruct() : nullptr;
	const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(ScriptStruct);

	// Quantize polymorphic data
	QuantizedType TempValue = {};
	if (TypeInfo != nullptr)
	{
		const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;

		TempValue.StructName = TypeInfo->StructName;
		TempValue.StructData = FPolymorphicStructNetSerializerInternal::Alloc(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
		FMemory::Memzero(TempValue.StructData, Descriptor->InternalSize);

		FReplicationStateOperations::Quantize(Context, static_cast<uint8*>(TempValue.StructData), reinterpret_cast<const uint8*>(Item.Get()), Descriptor);
	}
	else
	{
		if (ScriptStruct)
		{
			Context.SetError(NetError_PolymorphicStructNetSerializer_InvalidStructType);
			UE_LOGF(LogIris, Warning, "TPolymorphicStructNetSerializerImpl::Quantize Trying to quantize unregistered ScriptStruct type %ls.", ToCStr(ScriptStruct->GetName()));
		}
	}

	TargetValue = TempValue;
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
void TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	// Dequantize polymorphic data
	if (const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(SourceValue.StructName))
	{
		const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;
		const UScriptStruct* ScriptStruct = TypeInfo->ScriptStruct;

		// NOTE: We always allocate new memory in order to behave like the code we are trying to mimic expects, see GameplayEffectContextHandle
		// this should really be a policy of this class as it is far from optimal.

		// Allocate external struct, owned by external state therefore using global allocator
		SourceItemType* NewData = static_cast<SourceItemType*>(FMemory::Malloc(ScriptStruct->GetStructureSize(), ScriptStruct->GetMinAlignment()));
		ScriptStruct->InitializeStruct(NewData);

		FReplicationStateOperations::Dequantize(Context, reinterpret_cast<uint8*>(NewData), static_cast<const uint8*>(SourceValue.StructData), Descriptor);

		GetItem(TargetValue) = MakeShareable(NewData, FSourceItemTypeDeleter());
	}
	else
	{
		GetItem(TargetValue).Reset();
	}
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
bool TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const QuantizedType& ValueA = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& ValueB = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (ValueA.StructName != ValueB.StructName)
		{
			return false;
		}

		const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(ValueA.StructName);
		if (TypeInfo != nullptr && !UE::Net::FReplicationStateOperations::IsEqualQuantizedState(Context, static_cast<const uint8*>(ValueA.StructData), static_cast<const uint8*>(ValueB.StructData), TypeInfo->Descriptor))
		{
			return false;
		}
	}
	else
	{
		const SourceType& ValueA = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& ValueB = *reinterpret_cast<const SourceType*>(Args.Source1);

		// Assuming there's a custom operator== because if there's not we would be hitting TSharedRef== which checks if the reference is identical, not the instance data.
		return ValueA == ValueB;
	}

	return true;
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
bool TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	SourceType& SourceValue = *reinterpret_cast<SourceType*>(Args.Source);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	const TSharedPtr<SourceItemType>& Item = GetItem(SourceValue);
	const UScriptStruct* ScriptStruct = Item.IsValid() ? Item->GetScriptStruct() : nullptr;
	const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(ScriptStruct);

	if (ScriptStruct != nullptr && TypeInfo == nullptr)
	{
		return false;
	}

	if (TypeInfo != nullptr && !UE::Net::FReplicationStateOperations::Validate(Context, reinterpret_cast<const uint8*>(Item.Get()), TypeInfo->Descriptor))
	{
		return false;
	}

	return true;
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
void TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	// Clone polymorphic data
	const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(SourceValue.StructName);
	TargetValue.StructName = SourceValue.StructName;

	if (TypeInfo)
	{
		const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;

		// We need some memory to store the state for the polymorphic struct
		TargetValue.StructData = FPolymorphicStructNetSerializerInternal::Alloc(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
		FPolymorphicStructNetSerializerInternal::CloneQuantizedState(Context, static_cast<uint8*>(TargetValue.StructData), static_cast<const uint8*>(SourceValue.StructData), Descriptor);
	}
	else
	{
		TargetValue.StructData = nullptr;
	}
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
void TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& SourceValue = *reinterpret_cast<QuantizedType*>(Args.Source);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	InternalFreeItem(Context, Config, SourceValue);
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
void TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FNetReferenceCollector& Collector = *reinterpret_cast<UE::Net::FNetReferenceCollector*>(Args.Collector);

	// No references nothing to do
	if (Value.StructName.IsNone() || !Config.RegisteredTypes.CanHaveNetReferences())
	{
		return;
	}

	const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(Value.StructName);
	const FReplicationStateDescriptor* Descriptor = TypeInfo ? TypeInfo->Descriptor.GetReference() : nullptr;
	if (Descriptor != nullptr && EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasObjectReference))
	{
		FPolymorphicStructNetSerializerInternal::CollectReferences(Context, Collector, Args.ChangeMaskInfo, static_cast<const uint8*>(Value.StructData), Descriptor);
	}
}

template <typename ExternalSourceType, typename ExternalSourceItemType, TSharedPtr<ExternalSourceItemType>&(*GetItem)(ExternalSourceType&)>
void TPolymorphicStructNetSerializerImpl<ExternalSourceType, ExternalSourceItemType, GetItem>::InternalFreeItem(FNetSerializationContext& Context, const ConfigType& Config, QuantizedType& Value)
{
	const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(Value.StructName);
	const FReplicationStateDescriptor* Descriptor = TypeInfo ? TypeInfo->Descriptor.GetReference() : nullptr;

	if (Value.StructData != nullptr)
	{
		if (Descriptor && EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState))
		{
			FReplicationStateOperations::FreeDynamicState(Context, static_cast<uint8*>(Value.StructData), Descriptor);
		}
		FPolymorphicStructNetSerializerInternal::Free(Context, Value.StructData);
		Value.StructData = nullptr;
		Value.StructName = FName();
	}
}

/** TPolymorphicArrayStructNetSerializerImpl */
template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	// Serialize quantized data
	Writer.WriteBits(SourceValue.NumItems, ArrayItemBits);	
	for (uint32 It = 0, EndIt = SourceValue.NumItems; It != EndIt; ++It)
	{
		UE_NET_TRACE_SCOPE(Element, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		const FQuantizedItem& Item = SourceValue.Items[It];
		const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(Item.StructName);
		if (Writer.WriteBool(TypeInfo != nullptr))
		{
			CA_ASSUME(TypeInfo != nullptr);
			FNetTokenStore* TokenStore = Context.GetNetTokenStore();
			FNameTokenStore* NameStore = TokenStore->GetDataStore<FNameTokenStore>();
			const FNetToken NameToken = NameStore->GetOrCreateToken(Item.StructName);
			TokenStore->WriteNetTokenWithKnownType<FNameTokenStore>(Context, NameToken);
			FNetTokenStore::AppendExport(Context, NameToken);
			FReplicationStateOperations::Serialize(Context, static_cast<const uint8*>(Item.StructData), TypeInfo->Descriptor);
		}
	}
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	// Deserialize quantized data
	const uint32 NumItems = Reader.ReadBits(ArrayItemBits); 

	if (Reader.IsOverflown())
	{
		Context.SetError(GNetError_BitStreamOverflow);
		return;
	}

	// Currently we always free all memory even though we in theory could keep it if all types are matching
	InternalFreeItemArray(Context, TargetValue, Config);

	// Allocate space for the ItemArray
	InternalAllocateItemArray(Context, TargetValue, NumItems);

	// Read polymorphic state data
	for (uint32 It = 0, EndIt = TargetValue.NumItems; It != EndIt; ++It)
	{
		UE_NET_TRACE_SCOPE(Element, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

		FQuantizedItem& Item = TargetValue.Items[It];
		if (Reader.ReadBool())
		{
			FNetTokenStore* TokenStore = Context.GetNetTokenStore();
			const FNetToken NameToken = TokenStore->ReadNetTokenWithKnownType<FNameTokenStore>(Context);
			FNameTokenStore* NameStore = TokenStore->GetDataStore<FNameTokenStore>();
			const FName StructName = NameStore->ResolveToken(NameToken, Context.GetRemoteNetTokenStoreState());
			if (const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(StructName))
			{
				const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;

				// Allocate storage and read struct data
				Item.StructData = FPolymorphicStructNetSerializerInternal::Alloc(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
				Item.StructName = StructName;

				FMemory::Memzero(Item.StructData, Descriptor->InternalSize);
				UE::Net::FReplicationStateOperations::Deserialize(Context, static_cast<uint8*>(Item.StructData), Descriptor);
			}
			else
			{
				InternalFreeItemArray(Context, TargetValue, Config);
				Context.SetError(NetError_PolymorphicStructNetSerializer_InvalidStructType);
				return;
			}
		}
	}
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& Array = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& PrevArray = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	const uint32 NumItems = Array.NumItems;
	const uint32 PrevNumItems = PrevArray.NumItems;
	const bool bSameSizeArray = (NumItems == PrevNumItems);
	Writer.WriteBits(bSameSizeArray, 1U);
	if (!bSameSizeArray)
	{
		Writer.WriteBits(NumItems, ArrayItemBits);
	}

	// Use delta serialization for elements available in both the previous and current state.
	if (PrevNumItems)
	{
		FNetSerializeDeltaArgs ElementArgs = Args;

		for (uint32 ElementIt = 0, ElementEndIt = FPlatformMath::Min(NumItems, PrevNumItems); ElementIt != ElementEndIt; ++ElementIt)
		{
			UE_NET_TRACE_SCOPE(Element, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

			ElementArgs.Source = NetSerializerValuePointer(&Array.Items[ElementIt]);
			ElementArgs.Prev = NetSerializerValuePointer(&PrevArray.Items[ElementIt]);

			FItemNetSerializer::SerializeDelta(Context, ElementArgs);
		}
	}

	// Serialize additional items with standard serialization.
	if (NumItems > PrevNumItems)
	{
		for (uint32 ElementIt = PrevNumItems, ElementEndIt = NumItems; ElementIt < ElementEndIt; ++ElementIt)
		{
			UE_NET_TRACE_SCOPE(Element, Writer, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
			const FQuantizedItem& Item = Array.Items[ElementIt];
			const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(Item.StructName);
			if (Writer.WriteBool(TypeInfo != nullptr))
			{
				CA_ASSUME(TypeInfo != nullptr);
				FNetTokenStore* TokenStore = Context.GetNetTokenStore();
				FNameTokenStore* NameStore = TokenStore->GetDataStore<FNameTokenStore>();
				const FNetToken NameToken = NameStore->GetOrCreateToken(Item.StructName);
				TokenStore->WriteNetTokenWithKnownType<FNameTokenStore>(Context, NameToken);
				FNetTokenStore::AppendExport(Context, NameToken);
				FReplicationStateOperations::Serialize(Context, static_cast<const uint8*>(Item.StructData), TypeInfo->Descriptor);
			}
		}
	}
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);
	QuantizedType& Array = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& PrevArray = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	const uint32 PrevNumItems = PrevArray.NumItems;
	const bool bSameSizeArray = !!Reader.ReadBits(1U);
	const uint32 NumItems = (bSameSizeArray ? PrevNumItems : Reader.ReadBits(ArrayItemBits));

	if (Reader.IsOverflown())
	{
		Context.SetError(GNetError_BitStreamOverflow);
		return;
	}

	// Currently we always free all memory even though we in theory could keep it if all types are matching
	InternalFreeItemArray(Context, Array, Config);

	// Allocate space for the ItemArray
	InternalAllocateItemArray(Context, Array, NumItems);

	// Elements in the current array up to the previous size can use delta serialization.
	if (PrevNumItems)
	{
		FNetDeserializeDeltaArgs ElementArgs = Args;

		for (uint32 ElementIt = 0, ElementEndIt = FPlatformMath::Min(NumItems, PrevNumItems); ElementIt != ElementEndIt; ++ElementIt)
		{
			UE_NET_TRACE_SCOPE(Element, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);

			ElementArgs.Target = NetSerializerValuePointer(&Array.Items[ElementIt]);
			ElementArgs.Prev = NetSerializerValuePointer(&PrevArray.Items[ElementIt]);

			FItemNetSerializer::DeserializeDelta(Context, ElementArgs);
		}
	}

	if (Context.HasError())
	{
		InternalFreeItemArray(Context, Array, Config);
		return;
	}

	if (Reader.IsOverflown())
	{
		Context.SetError(GNetError_BitStreamOverflow);
		InternalFreeItemArray(Context, Array, Config);
		return;
	}

	// Deserialize additional items with standard deserialization.
	if (NumItems > PrevNumItems)
	{
		for (uint32 ElementIt = PrevNumItems, ElementEndIt = NumItems; ElementIt < ElementEndIt; ++ElementIt)
		{
			UE_NET_TRACE_SCOPE(Element, Reader, Context.GetTraceCollector(), ENetTraceVerbosity::VeryVerbose);
			FQuantizedItem& Item = Array.Items[ElementIt];
			if (Reader.ReadBool())
			{
				FNetTokenStore* TokenStore = Context.GetNetTokenStore();
				const FNetToken NameToken = TokenStore->ReadNetTokenWithKnownType<FNameTokenStore>(Context);
				FNameTokenStore* NameStore = TokenStore->GetDataStore<FNameTokenStore>();
				const FName StructName = NameStore->ResolveToken(NameToken, Context.GetRemoteNetTokenStoreState());
				if (const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(StructName))
				{
					const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;

					// Allocate storage and read struct data
					Item.StructData = FPolymorphicStructNetSerializerInternal::Alloc(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
					Item.StructName = StructName;

					FMemory::Memzero(Item.StructData, Descriptor->InternalSize);
					UE::Net::FReplicationStateOperations::Deserialize(Context, static_cast<uint8*>(Item.StructData), Descriptor);
				}
				else
				{
					InternalFreeItemArray(Context, Array, Config);
					Context.SetError(NetError_PolymorphicStructNetSerializer_InvalidStructType);
					return;
				}
			}
		}
	}

	if (Reader.IsOverflown())
	{
		Context.SetError(GNetError_BitStreamOverflow);
		InternalFreeItemArray(Context, Array, Config);
		return;
	}
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	SourceType& SourceValue = *reinterpret_cast<SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	// Make sure we can properly serialize the array.
	TArrayView<const TSharedPtr<SourceArrayItemType>> ItemArray = GetArray(SourceValue);
	const uint32 NumItems = static_cast<uint32>(ItemArray.Num());
	if (NumItems > MaxArrayItems)
	{
		Context.SetError(GNetError_ArraySizeTooLarge);
		return;
	}

	// First we need to free previously allocated data
	InternalFreeItemArray(Context, TargetValue, Config);

	// Allocate new array
	InternalAllocateItemArray(Context, TargetValue, NumItems);

	// Copy polymorphic struct data
	const TSharedPtr<SourceArrayItemType>* ItemArrayData = ItemArray.GetData();
	for (const TSharedPtr<SourceArrayItemType>& SourceItem : ItemArray)
	{
		const SIZE_T ArrayIndex = &SourceItem - ItemArrayData;
		const UScriptStruct* ScriptStruct = SourceItem.IsValid() ? SourceItem->GetScriptStruct() : nullptr;
		FQuantizedItem& TargetItem = TargetValue.Items[ArrayIndex];
		const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(ScriptStruct);

		if (TypeInfo != nullptr)
		{
			const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;

			TargetItem.StructName = TypeInfo->StructName;
			TargetItem.StructData = FPolymorphicStructNetSerializerInternal::Alloc(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
			FMemory::Memzero(TargetItem.StructData, Descriptor->InternalSize);

			// Quantize data
			FReplicationStateOperations::Quantize(Context, static_cast<uint8*>(TargetItem.StructData), reinterpret_cast<const uint8*>(SourceItem.Get()), Descriptor);
		}
		else
		{
			if (ScriptStruct)
			{
				Context.SetError(NetError_PolymorphicStructNetSerializer_InvalidStructType);
				UE_LOGF(LogIris, Warning,  "TPolymorphicArrayStructNetSerializerImpl::Quantize Trying to quantize unregistered ScriptStruct type %ls.", ToCStr(ScriptStruct->GetName()));
			}
		}
	}
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	SourceType& TargetValue = *reinterpret_cast<SourceType*>(Args.Target);
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	SetArrayNum(TargetValue, SourceValue.NumItems);
	TArrayView<TSharedPtr<SourceArrayItemType>> TargetArray = GetArray(TargetValue);
	// Dequantize polymorphic data
	for (uint32 It = 0, EndIt = SourceValue.NumItems; It != EndIt; ++It)
	{
		const FQuantizedItem& Item = SourceValue.Items[It];

		if (const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(Item.StructName))
		{
			const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;
			const UScriptStruct* ScriptStruct = TypeInfo->ScriptStruct;

			// NOTE: We always allocate new memory in order to behave like the code we are trying to mimic expects, see GameplayEffectContextHandle
			// this should really be a policy of this class as it is far from optimal.

			// Allocate external struct, owned by external state therefore using global allocator
			SourceArrayItemType* NewData = static_cast<SourceArrayItemType*>(FMemory::Malloc(ScriptStruct->GetStructureSize(), ScriptStruct->GetMinAlignment()));
			ScriptStruct->InitializeStruct(NewData);

			// Dequantize data
			FReplicationStateOperations::Dequantize(Context, reinterpret_cast<uint8*>(NewData), static_cast<const uint8*>(Item.StructData), Descriptor);

			TargetArray[It] = MakeShareable(NewData, FSourceArrayItemTypeDeleter());
		}
		else
		{
			TargetArray[It].Reset();
		}
	}
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
bool TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const QuantizedType& ValueA = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& ValueB = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (ValueA.NumItems != ValueB.NumItems)
		{
			return false;
		}

		for (uint32 It = 0, EndIt = ValueA.NumItems; It != EndIt; ++It)
		{
			const FQuantizedItem& ItemA = ValueA.Items[It];
			const FQuantizedItem& ItemB = ValueB.Items[It];

			if (ItemA.StructName != ItemB.StructName)
			{
				return false;
			}

			const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(ItemA.StructName);
			if (TypeInfo && !UE::Net::FReplicationStateOperations::IsEqualQuantizedState(Context, static_cast<const uint8*>(ItemA.StructData), static_cast<const uint8*>(ItemB.StructData), TypeInfo->Descriptor))
			{
				return false;
			}
		}
	}
	else
	{
		const SourceType& ValueA = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& ValueB = *reinterpret_cast<const SourceType*>(Args.Source1);

		return ValueA == ValueB;
	}

	return true;
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
bool TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	SourceType& SourceValue = *reinterpret_cast<SourceType*>(Args.Source);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	TArrayView<const TSharedPtr<SourceArrayItemType>> ItemArray = GetArray(SourceValue);
	const uint32 NumItems = (uint32)ItemArray.Num();
	if (NumItems > MaxArrayItems)
	{
		return false;
	}

	for (const TSharedPtr<SourceArrayItemType>& Item : ItemArray)
	{
		const UScriptStruct* ScriptStruct = Item.IsValid() ? Item->GetScriptStruct() : nullptr;
		const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(ScriptStruct);

		if (ScriptStruct && TypeInfo == nullptr)
		{
			return false;
		}

		if (TypeInfo && !FReplicationStateOperations::Validate(Context, reinterpret_cast<const uint8*>(Item.Get()), TypeInfo->Descriptor))
		{
			return false;
		}
	}

	return true;
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FNetReferenceCollector& Collector = *reinterpret_cast<FNetReferenceCollector*>(Args.Collector);

	// No references nothing to do
	if (SourceValue.NumItems == 0U || !Config.RegisteredTypes.CanHaveNetReferences())
	{
		return;
	}

	for (uint32 It = 0, EndIt = SourceValue.NumItems; It != EndIt; ++It)
	{
		const FQuantizedItem& Item = SourceValue.Items[It];
		const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(Item.StructName);
		const FReplicationStateDescriptor* Descriptor = TypeInfo ? TypeInfo->Descriptor.GetReference() : nullptr;

		if (Descriptor && EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasObjectReference))
		{
			FPolymorphicStructNetSerializerInternal::CollectReferences(Context, Collector, Args.ChangeMaskInfo, static_cast<const uint8*>(Item.StructData), Descriptor);
		}
	}
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	InternalAllocateItemArray(Context, TargetValue, SourceValue.NumItems);

	// Clone polymorphic data
	for (uint32 It = 0, EndIt = SourceValue.NumItems; It != EndIt; ++It)
	{
		const FQuantizedItem& SourceItem = SourceValue.Items[It];
		FQuantizedItem& TargetItem = TargetValue.Items[It];

		const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(SourceItem.StructName);
		TargetItem.StructName = SourceItem.StructName;

		if (TypeInfo)
		{
			const FReplicationStateDescriptor* Descriptor = TypeInfo->Descriptor;

			// We need some memory to store the state for the polymorphic struct
			TargetItem.StructData = FPolymorphicStructNetSerializerInternal::Alloc(Context, Descriptor->InternalSize, Descriptor->InternalAlignment);
			FPolymorphicStructNetSerializerInternal::CloneQuantizedState(Context, static_cast<uint8*>(TargetItem.StructData), static_cast<const uint8*>(SourceItem.StructData), Descriptor);
		}
		else
		{
			TargetItem.StructData = nullptr;
		}	
	}
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& SourceValue = *reinterpret_cast<QuantizedType*>(Args.Source);
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);

	InternalFreeItemArray(Context, SourceValue, Config);
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::InternalAllocateItemArray(FNetSerializationContext& Context, QuantizedType& Value, uint32 NumItems)
{
	const SIZE_T ElementSize = sizeof(FQuantizedItem);
	const SIZE_T Alignment = alignof(FQuantizedItem);

	if (NumItems > 0)
	{
		Value.Items = static_cast<FQuantizedItem*>(FPolymorphicStructNetSerializerInternal::Alloc(Context, ElementSize*NumItems, Alignment));
		Value.NumItems = NumItems;
		FMemory::Memzero(Value.Items, ElementSize*NumItems);
	}
	else
	{
		Value.Items = nullptr;
		Value.NumItems = 0;
	}
}

template <typename ExternalSourceType, typename ExternalSourceArrayItemType, TArrayView<TSharedPtr<ExternalSourceArrayItemType>>(*GetArray)(ExternalSourceType& Source), void(*SetArrayNum)(ExternalSourceType& Source, SIZE_T Num)>
void TPolymorphicArrayStructNetSerializerImpl<ExternalSourceType, ExternalSourceArrayItemType, GetArray, SetArrayNum>::InternalFreeItemArray(FNetSerializationContext& Context, QuantizedType& Value, const FPolymorphicArrayStructNetSerializerConfig& Config)
{
	for (uint32 It = 0, EndIt = Value.NumItems; It != EndIt; ++It)
	{
		FQuantizedItem& Item = Value.Items[It];

		const FTypeInfo* TypeInfo = Config.RegisteredTypes.GetTypeInfo(Item.StructName);
		const FReplicationStateDescriptor* Descriptor = TypeInfo ? TypeInfo->Descriptor.GetReference() : nullptr;

		if (Item.StructData)
		{
			if (Descriptor && EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasDynamicState))
			{
				FReplicationStateOperations::FreeDynamicState(Context, static_cast<uint8*>(Item.StructData), Descriptor);
			}
			FPolymorphicStructNetSerializerInternal::Free(Context, Value.Items[It].StructData);
		}
		Item.StructName = FName();
	}
	FPolymorphicStructNetSerializerInternal::Free(Context, Value.Items);

	// Reset the value
	Value.Items = nullptr;
	Value.NumItems = 0U;
}

}
