// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "QuatNetSerializers.generated.h"

USTRUCT()
struct FUnitQuatNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FUnitQuat4fNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FUnitQuat4dNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FUnitQuatNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FUnitQuat4fNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FUnitQuat4dNetSerializer, IRISCORE_API);

}
