// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "RotatorNetSerializers.generated.h"

USTRUCT()
struct FRotatorNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FRotatorAsByteNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FRotatorAsShortNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FRotator3fNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FRotator3dNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FRotatorNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FRotatorAsByteNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FRotatorAsShortNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FRotator3fNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FRotator3dNetSerializer, IRISCORE_API);

}
