// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "SoftObjectNetSerializers.generated.h"

USTRUCT()
struct FSoftObjectNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FSoftObjectPathNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FSoftClassPathNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FSoftObjectNetSerializer, IRISCORE_API)
UE_NET_DECLARE_SERIALIZER(FSoftObjectPathNetSerializer, IRISCORE_API)
UE_NET_DECLARE_SERIALIZER(FSoftClassPathNetSerializer, IRISCORE_API)

}
