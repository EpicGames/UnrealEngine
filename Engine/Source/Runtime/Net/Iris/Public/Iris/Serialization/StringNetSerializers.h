// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "StringNetSerializers.generated.h"

USTRUCT()
struct FNameNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FNameAsNetTokenNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FStringNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FNameNetSerializer, IRISCORE_API)
UE_NET_DECLARE_SERIALIZER(FNameAsNetTokenNetSerializer, IRISCORE_API)
UE_NET_DECLARE_SERIALIZER(FStringNetSerializer, IRISCORE_API)

constexpr SIZE_T GetNameNetSerializerSafeQuantizedSize() { return 24; }
}

