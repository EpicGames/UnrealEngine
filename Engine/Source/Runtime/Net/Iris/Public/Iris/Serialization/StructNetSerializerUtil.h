// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"

namespace UE::Net
{
	struct FReplicationStateDescriptor;
	// Write a Struct value to the provided context using the provided Descriptor.
	IRISCORE_API void WriteStruct(FNetSerializationContext& Context, NetSerializerValuePointer InValue, const FReplicationStateDescriptor* Descriptor);
	// Read a Struct value to OutValue from the provided context using the provided Descriptor.
	IRISCORE_API void ReadStruct(FNetSerializationContext& Context, NetSerializerValuePointer OutValue, const FReplicationStateDescriptor* Descriptor);
}
