// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "UintNetSerializers.generated.h"

// Unsigned integer serializers
USTRUCT()
struct FUintNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	explicit inline FUintNetSerializerConfig(uint8 InBitCount) : BitCount(InBitCount) {}
	inline FUintNetSerializerConfig() : BitCount(0) {}

	UPROPERTY()
	uint8 BitCount;
};

typedef FUintNetSerializerConfig FUint64NetSerializerConfig;
typedef FUintNetSerializerConfig FUint32NetSerializerConfig;
typedef FUintNetSerializerConfig FUint16NetSerializerConfig;
typedef FUintNetSerializerConfig FUint8NetSerializerConfig;

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FUint64NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FUint32NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FUint16NetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FUint8NetSerializer, IRISCORE_API);

}
