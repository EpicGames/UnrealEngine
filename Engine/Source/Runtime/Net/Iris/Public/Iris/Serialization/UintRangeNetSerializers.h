// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "UintRangeNetSerializers.generated.h"

USTRUCT()
struct FUint8RangeNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint8 LowerBound = 0;
	UPROPERTY()
	uint8 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
};

USTRUCT()
struct FUint16RangeNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint16 LowerBound = 0;
	UPROPERTY()
	uint16 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
};

USTRUCT()
struct FUint32RangeNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint32 LowerBound = 0;
	UPROPERTY()
	uint32 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
};

USTRUCT()
struct FUint64RangeNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint64 LowerBound = 0;
	UPROPERTY()
	uint64 UpperBound = 0;
	UPROPERTY()
	uint8 BitCount = 0;
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FUint8RangeNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FUint16RangeNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FUint32RangeNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FUint64RangeNetSerializer, IRISCORE_API);

}
