// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "VectorNetSerializers.generated.h"

USTRUCT()
struct FVectorNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FVector3fNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

USTRUCT()
struct FVector3dNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FVectorNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FVector3fNetSerializer, IRISCORE_API);
UE_NET_DECLARE_SERIALIZER(FVector3dNetSerializer, IRISCORE_API);

}
